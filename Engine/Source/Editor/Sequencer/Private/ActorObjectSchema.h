// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ISequencerObjectSchema.h"

class AActor;
class ISequencer;
class FMenuBuilder;

namespace UE::Sequencer
{

class FActorSchema : public IObjectSchema
{
	FText GetPrettyName(const UObject* Object) const override;
	UObject* GetParentObject(UObject* Object) const override;
	FObjectSchemaRelevancy GetRelevancy(const UObject* InObject) const override;
	TSharedPtr<FExtender> ExtendObjectBindingMenu(TSharedRef<FUICommandList> CommandList, TWeakPtr<ISequencer> WeakSequencer, TArrayView<UObject* const> ContextSensitiveObjects) const override;
private:

	void HandleTrackMenuExtensionAddTrack(FMenuBuilder& AddTrackMenuBuilder, TWeakPtr<ISequencer> WeakSequencer, TArray<AActor*> Actors) const;
	void HandleAddComponentActionExecute(FText ComponentName, TWeakPtr<ISequencer> WeakSequencer, TArray<AActor*> Actors) const;
};

} // namespace UE::Sequencer