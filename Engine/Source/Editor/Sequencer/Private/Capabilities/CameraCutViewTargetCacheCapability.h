// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/IMovieScenePlaybackCapability.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UCameraComponent;

namespace UE::MovieScene
{
	struct FCameraCutViewTargetCacheCapability
	{
		UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY(FCameraCutViewTargetCacheCapability)
		
		/** The last evaluated view target. */
		TWeakObjectPtr<UCameraComponent> LastViewTargetCamera;
	};
}


