// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorCommandBinder.h"

#include "CurveEditor.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "Filters/Linking/FilterAreaManager.h"
#include "Filters/Linking/Mode/LinkedFilterViewModel.h"
#include "ISequencer.h"
#include "Sequencer.h"
#include "SequencerCommands.h"

namespace UE::Sequencer
{
FCurveCommandBinder::FCurveCommandBinder(
	const TSharedRef<FSequencer>& InSequencer,
	const TSharedRef<FCurveEditor>& InCurveEditor,
	const FFilterAreaManager& InFilterArea
	)
	: WeakSequencer(InSequencer)
	, CurveEditor(InCurveEditor)
	, FilterArea(InFilterArea)
{
	FilterArea.GetFilterModel()->OnFilterModeChanged().AddRaw(this, &FCurveCommandBinder::OnFilterModeChanged);
}

FCurveCommandBinder::~FCurveCommandBinder()
{
	FilterArea.GetFilterModel()->OnFilterModeChanged().RemoveAll(this);
}

void FCurveCommandBinder::InitSequencerCommandBindings()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}
	
	const TSharedPtr<FUICommandList> CurveEditorSharedBindings = GetCurveEditorCommands();
	const TSharedPtr<FUICommandList> SequencerCommandBindings = Sequencer->GetCommandBindings(ESequencerCommandBindings::Sequencer);
	
	const FSequencerCommands& Commands = FSequencerCommands::Get();
	CurveEditorSharedBindings->MapAction(Commands.TogglePlay, *SequencerCommandBindings->GetActionForCommand(Commands.TogglePlay));
	CurveEditorSharedBindings->MapAction(Commands.TogglePlayViewport, *SequencerCommandBindings->GetActionForCommand(Commands.TogglePlayViewport));
	CurveEditorSharedBindings->MapAction(Commands.PlayForward, *SequencerCommandBindings->GetActionForCommand(Commands.PlayForward));
	CurveEditorSharedBindings->MapAction(Commands.JumpToStart, *SequencerCommandBindings->GetActionForCommand(Commands.JumpToStart));
	CurveEditorSharedBindings->MapAction(Commands.JumpToEnd, *SequencerCommandBindings->GetActionForCommand(Commands.JumpToEnd));
	CurveEditorSharedBindings->MapAction(Commands.JumpToStartViewport, *SequencerCommandBindings->GetActionForCommand(Commands.JumpToStartViewport));
	CurveEditorSharedBindings->MapAction(Commands.JumpToEndViewport, *SequencerCommandBindings->GetActionForCommand(Commands.JumpToEndViewport));
	CurveEditorSharedBindings->MapAction(Commands.ShuttleBackward, *SequencerCommandBindings->GetActionForCommand(Commands.ShuttleBackward));
	CurveEditorSharedBindings->MapAction(Commands.ShuttleForward, *SequencerCommandBindings->GetActionForCommand(Commands.ShuttleForward));
	CurveEditorSharedBindings->MapAction(Commands.Pause, *SequencerCommandBindings->GetActionForCommand(Commands.Pause));
	CurveEditorSharedBindings->MapAction(Commands.StepForward, *SequencerCommandBindings->GetActionForCommand(Commands.StepForward));
	CurveEditorSharedBindings->MapAction(Commands.StepBackward, *SequencerCommandBindings->GetActionForCommand(Commands.StepBackward));
	CurveEditorSharedBindings->MapAction(Commands.StepForwardViewport, *SequencerCommandBindings->GetActionForCommand(Commands.StepForwardViewport));
	CurveEditorSharedBindings->MapAction(Commands.StepBackwardViewport, *SequencerCommandBindings->GetActionForCommand(Commands.StepBackwardViewport));
	CurveEditorSharedBindings->MapAction(Commands.JumpForward, *SequencerCommandBindings->GetActionForCommand(Commands.JumpForward));
	CurveEditorSharedBindings->MapAction(Commands.JumpBackward, *SequencerCommandBindings->GetActionForCommand(Commands.JumpBackward));
	CurveEditorSharedBindings->MapAction(Commands.StepToNextKey, *SequencerCommandBindings->GetActionForCommand(Commands.StepToNextKey));
	CurveEditorSharedBindings->MapAction(Commands.StepToPreviousKey, *SequencerCommandBindings->GetActionForCommand(Commands.StepToPreviousKey));
	CurveEditorSharedBindings->MapAction(Commands.StepToNextMark, *SequencerCommandBindings->GetActionForCommand(Commands.StepToNextMark));
	CurveEditorSharedBindings->MapAction(Commands.StepToPreviousMark, *SequencerCommandBindings->GetActionForCommand(Commands.StepToPreviousMark));
	CurveEditorSharedBindings->MapAction(Commands.ToggleMarkAtPlayPosition, *SequencerCommandBindings->GetActionForCommand(Commands.ToggleMarkAtPlayPosition));
	CurveEditorSharedBindings->MapAction(Commands.SetStartPlaybackRange, *SequencerCommandBindings->GetActionForCommand(Commands.SetStartPlaybackRange));
	CurveEditorSharedBindings->MapAction(Commands.SetEndPlaybackRange, *SequencerCommandBindings->GetActionForCommand(Commands.SetEndPlaybackRange));
	CurveEditorSharedBindings->MapAction(Commands.SetSelectionRangeStart, *SequencerCommandBindings->GetActionForCommand(Commands.SetSelectionRangeStart));
	CurveEditorSharedBindings->MapAction(Commands.SetSelectionRangeEnd, *SequencerCommandBindings->GetActionForCommand(Commands.SetSelectionRangeEnd));
	CurveEditorSharedBindings->MapAction(Commands.ClearSelectionRange, *SequencerCommandBindings->GetActionForCommand(Commands.ClearSelectionRange));

	CurveEditorSharedBindings->MapAction(Commands.AddTransformKey, *SequencerCommandBindings->GetActionForCommand(Commands.AddTransformKey));
	CurveEditorSharedBindings->MapAction(Commands.AddTranslationKey, *SequencerCommandBindings->GetActionForCommand(Commands.AddTranslationKey));
	CurveEditorSharedBindings->MapAction(Commands.AddRotationKey, *SequencerCommandBindings->GetActionForCommand(Commands.AddRotationKey));
	CurveEditorSharedBindings->MapAction(Commands.AddScaleKey, *SequencerCommandBindings->GetActionForCommand(Commands.AddScaleKey));
	
	CurveEditorSharedBindings->MapAction(Commands.ToggleAutoExpandCurveEditorOnSelection, *SequencerCommandBindings->GetActionForCommand(Commands.ToggleAutoExpandCurveEditorOnSelection));
	
	// The shared bindings should include the commands related to filtering.
	CurveEditorSharedBindings.ToSharedRef()->Append(FilterArea.GetFilterAreaCommandList());
	// Curve Editor's command list should execute whatever we bind in Sequencer.
	CurveEditor->GetCommands()->Append(CurveEditorSharedBindings.ToSharedRef());
	
	BindCurveEditorToActiveFilterBar();
}

TSharedPtr<FUICommandList> FCurveCommandBinder::GetCurveEditorCommands() const
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	return Sequencer ? Sequencer->GetCommandBindings(ESequencerCommandBindings::CurveEditor) : nullptr;
}

void FCurveCommandBinder::BindCurveEditorToActiveFilterBar() const
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	const TSharedPtr<FUICommandList> CurveEditorBindings = Sequencer ? Sequencer->GetCommandBindings(ESequencerCommandBindings::CurveEditor) : nullptr;
	if (!ensure(CurveEditorBindings))
	{
		return;
	}
	
	const TSharedPtr<FSequencerFilterBar> FilterBar = FilterArea.GetFilterModel()->GetActiveFilterBarImpl();
	const TSharedPtr<FUICommandList> FilterBarCommandList = FilterBar->GetCommandList();
	
	// Add the general track filter commands
	FSequencerTrackFilterCommands& TrackFilterCommands = FSequencerTrackFilterCommands::Get();
	for (const TSharedPtr<FUICommandInfo>& Command : TrackFilterCommands.GetAllCommands())
	{
		if (Command.IsValid() && FilterBarCommandList->IsActionMapped(Command))
		{
			CurveEditorBindings->MapAction(Command, *FilterBarCommandList->GetActionForCommand(Command));
		}
	}

	// Add the specific track filter toggle commands
	const TArray<TSharedRef<FSequencerTrackFilter>> AllFilters = FilterBar->GetFilterList(true);
	for (const TSharedRef<FSequencerTrackFilter>& Filter : AllFilters)
	{
		const TSharedPtr<FUICommandList>& FilterCommandList = Filter->GetFilterInterface().GetCommandList();
		const TSharedPtr<FUICommandInfo>& FilterCommand = Filter->GetToggleCommand();

		if (FilterCommand.IsValid() && FilterCommandList->IsActionMapped(FilterCommand))
		{
			CurveEditorBindings->MapAction(FilterCommand, *FilterCommandList->GetActionForCommand(FilterCommand));
		}
	}
}
}
