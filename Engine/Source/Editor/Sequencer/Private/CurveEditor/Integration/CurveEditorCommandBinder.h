// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/Linking/FilterAreaManager.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FUICommandList;
class FCurveEditor;
class FSequencer;

namespace UE::Sequencer
{
class FLinkedFilterViewModel;

/** 
 * Injects some Sequencer commands into Curve Editor such that when Curve Editor UI is focused, the user can continue to use some of the
 * Sequencer commands, and they'll take affects as if they were issued while Sequencer UI was focused.
 */
class FCurveCommandBinder : public FNoncopyable
{
public:
	
	explicit FCurveCommandBinder(
		const TSharedRef<FSequencer>& InSequencer, const TSharedRef<FCurveEditor>& InCurveEditor, 
		const FFilterAreaManager& InFilterArea UE_LIFETIMEBOUND
		);
	~FCurveCommandBinder();
	
	/** Binds Sequencer commands that should be exposed in Curve Editor. */
	void InitSequencerCommandBindings();
	
	/** @return The command list that is used in the UI of Sequencer's curve editor. */
	TSharedPtr<FUICommandList> GetCurveEditorCommands() const;
	
private:
	
	/** The owning Sequencer whose commands are being rebound and that owns the FCurveEditor. */
	const TWeakPtr<FSequencer> WeakSequencer;
	
	/** The curve editor to which we're binding the commands */
	const TSharedRef<FCurveEditor> CurveEditor;
	
	/** When the ELinkedFilterMode changes, the bindings in the active filter bar must be reapplied to Curve Editor.*/
	const FFilterAreaManager& FilterArea;
	
	/** Handles any command rebinding that must take place when the active filter bar changes. */
	void OnFilterModeChanged() const { BindCurveEditorToActiveFilterBar(); }
	/** Remaps all curve editor bindings to invoke the command bindings of the active filter bar. */
	void BindCurveEditorToActiveFilterBar() const;
};
} // namespace UE::Sequencer
