// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveModelSyncer.h"

#include "CurveEditor.h"
#include "CurveEditor/Views/CurveEditorWidgetOwner.h"
#include "Filters/Linking/Mode/ILinkedFilterViewModel.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Modification/Utils/ScopedSelectionChange.h"
#include "Sequencer.h"
#include "Tree/CurveEditorTree.h"

namespace UE::Sequencer
{
FCurveModelSyncer::FCurveModelSyncer(
	const TSharedRef<FSequencer>& InSequencer,
	const TSharedRef<FCurveEditor>& InCurveEditor, const TSharedRef<ILinkedFilterViewModel>& InLinkedFilterViewModel
	)
	: WeakSequencer(InSequencer)
	, CurveEditor(InCurveEditor)
	, LinkedFilterViewModel(InLinkedFilterViewModel)
{
	const TSharedPtr<FSequenceModel> OwnerModel = InSequencer->GetViewModel()->GetRootSequenceModel();
	FSimpleMulticastDelegate& HierarchyChanged = OwnerModel->GetSharedData()->SubscribeToHierarchyChanged(OwnerModel);
	HierarchyChanged.AddRaw(this, &FCurveModelSyncer::OnHierarchyChanged);
	
	InSequencer->GetNodeTree()->OnUpdated().AddRaw(this, &FCurveModelSyncer::OnNodeTreeUpdated);
	InSequencer->GetViewModel()->GetSelection()->Outliner.OnChanged.AddRaw(this, &FCurveModelSyncer::OnOutlinerSelectionChanged);
}

FCurveModelSyncer::~FCurveModelSyncer()
{
	if (const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin())
	{
		const TSharedPtr<FSequenceModel> OwnerModel = Sequencer->GetViewModel()->GetRootSequenceModel();
		OwnerModel->GetSharedData()->UnsubscribeFromHierarchyChanged(OwnerModel, this);
	
		Sequencer->GetNodeTree()->OnUpdated().RemoveAll(this);
		Sequencer->GetViewModel()->GetSelection()->Outliner.OnChanged.RemoveAll(this);
	}
	
	// Prevent crash on shutdown. RequestSyncSelection sets a timer that keeps a FScopedTransaction. 
	// ~FScopedTransaction expects GEditor to be valid, but it is null at shutdown.
	CleanUpPendingSyncSelection();
}

static bool bSyncSelectionRequested = false;
void FCurveModelSyncer::RequestSyncSelection()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (bSyncSelectionRequested || !Sequencer || SelectionSyncSuspension.IsSuspended())
	{
		return;
	}
	bSyncSelectionRequested = true;

	// We schedule selection syncing to the next editor tick because we might want to select items that
	// have just been added to the curve editor tree this tick. If it happened after the Slate update,
	// these items don't yet have a UI widget, and so selecting them doesn't do anything.
	//
	// Note that we capture a weak pointer of our owner model because selection changes can happen
	// right around the time when we want to unload everything (such as when loading a new map in the
	// editor). We don't want to extend the lifetime of our stuff in that case.
	TWeakPtr<FSequencerEditorViewModel> WeakRootViewModel = Sequencer->GetViewModel();

	// Key selection supports undo. If RequestSyncSelection is called as part of an ongoing transaction, record the key selection change for undo.
	const bool bShouldRecord = GUndo != nullptr;
	// The current transaction needs to be extended until next tick...
	TSharedPtr<FScopedTransaction> Transaction = bShouldRecord
		? MakeShared<FScopedTransaction>(FText::GetEmpty(), bShouldRecord).ToSharedPtr() : nullptr;
	// ... at which point we'll diff the changes.
	TSharedPtr<CurveEditor::FScopedSelectionChange> SelectionChange = bShouldRecord
		? MakeShared<CurveEditor::FScopedSelectionChange>(CurveEditor).ToSharedPtr() : nullptr;
	
	SyncSelectionHandle = GEditor->GetTimerManager()->SetTimerForNextTick(
	[this, WeakRootViewModel, Transaction = MoveTemp(Transaction), SelectionChange = MoveTemp(SelectionChange)]() mutable
		{
			bSyncSelectionRequested = false;
		
			const TSharedPtr<FSequencerEditorViewModel> RootViewModel = WeakRootViewModel.Pin();
			if (!RootViewModel.IsValid())
			{
				return;
			}
			const TSharedPtr<ISequencer> Sequencer = RootViewModel->GetSequencer();
			if (!Sequencer)
			{
				return;
			}
				
			SyncSelectionHandle.Invalidate();
			// Notify UI to refresh the selection to that of Sequencer.
			OnRequestSyncCurveEditorUIDelegate.Broadcast();

			// Order matters: this appends a sub-transaction for the selection change...
			SelectionChange.Reset();
			// ... and only then can the parent transaction be closed. 
			Transaction.Reset();
		});
}

void FCurveModelSyncer::ResetCurveEditor()
{
	// The RemoveTreeItem calls below would trigger follow-up events, such as selection change events.
	// Each RemoveTreeItem would cause UpdateCurveEditor to be called: that's bad for performance but also causes incorrect behavior because 
	// UpdateCurveEditor would be called with a half intact hierarchy. Just process everything once at the end.
	const FScopedCurveEditorTreeEventGuard Guard(CurveEditor->GetTree());
	
	for (const TPair<TWeakPtr<FViewModel>, FCurveEditorTreeItemID>& Pair : ViewModelToTreeItemIDMap)
	{
		const FCurveEditorTreeItemID ItemID(Pair.Value);
		const TViewModelPtr<ICurveEditorTreeItemExtension> ViewModel = CastViewModel<ICurveEditorTreeItemExtension>(Pair.Key.Pin());
		if (ViewModel)
		{
			ViewModel->OnRemovedFromCurveEditor(CurveEditor);
		}
		CurveEditor->RemoveTreeItem(ItemID);
	}

	ViewModelToTreeItemIDMap.Reset();
	TreeItemIDToViewModelMap.Reset();
}

TWeakPtr<FViewModel> FCurveModelSyncer::FindViewModelById(const FCurveEditorTreeItemID& InTreeId) const
{
	const TWeakPtr<FViewModel>* ViewModel = TreeItemIDToViewModelMap.Find(InTreeId);
	return ViewModel ? *ViewModel : nullptr;
}

FCurveEditorTreeItemID FCurveModelSyncer::FindIdByViewModel(const TWeakPtr<FViewModel>& InViewModel) const
{
	const FCurveEditorTreeItemID* TreeItemID = ViewModelToTreeItemIDMap.Find(InViewModel);
	return TreeItemID ? *TreeItemID : FCurveEditorTreeItemID::Invalid();
}

void FCurveModelSyncer::UpdateCurveEditor()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	FCurveEditorTree* CurveEditorTree = CurveEditor->GetTree();
	if (!Sequencer || !CurveEditorTree || SelectionSyncSuspension.IsSuspended())
	{
		return;
	}
	
	OnStartUpdateCurveEditorDelegate.Broadcast();
	SyncSequencerToCurveEditor(*Sequencer, *CurveEditorTree);
	OnStopUpdateCurveEditorDelegate.Broadcast();
}

FCurveEditorTreeItemID FCurveModelSyncer::AddToCurveEditor(
	TViewModelPtr<ICurveEditorTreeItemExtension> InViewModel,
	TSharedPtr<FCurveEditor> InCurveEditor
	)
{
	// If the view model doesn't want to be in the curve editor, bail out
	// Note that this means we will create curve editor items for each parent in the hierarchy up
	// until the first parent that doesn't implement ICurveEditorTreeItemExtension
	// That is: we don't create "dummy" entries when there's a "gap" in the hierarchy
	if (!InViewModel)
	{
		return FCurveEditorTreeItemID::Invalid();
	}

	// Check if we already have a valid curve editor ID
	if (FCurveEditorTreeItemID* Existing = ViewModelToTreeItemIDMap.Find(InViewModel.AsModel()))
	{
		if (InCurveEditor->GetTree()->FindItem(*Existing) != nullptr)
		{
			return *Existing;
		}
	}

	// Recursively create any needed parent curve editor items
	TViewModelPtr<ICurveEditorTreeItemExtension> Parent = InViewModel.AsModel()->CastParent<ICurveEditorTreeItemExtension>();

	FCurveEditorTreeItemID ParentID = Parent ?
		AddToCurveEditor(Parent, InCurveEditor) :
		FCurveEditorTreeItemID::Invalid();

	// Create the new curve editor item
	FCurveEditorTreeItem* NewItem = InCurveEditor->AddTreeItem(ParentID);
	TSharedPtr<ICurveEditorTreeItem> CurveEditorTreeItem = InViewModel->GetCurveEditorTreeItem();
	NewItem->SetWeakItem(CurveEditorTreeItem);
	TOptional<FString> UniquePathName =  InViewModel->GetUniquePathName();
	NewItem->SetUniquePathName(UniquePathName);
	
	// Register the new ID in our map and notify the view model
	ViewModelToTreeItemIDMap.Add(InViewModel.AsModel(), NewItem->GetID());
	TreeItemIDToViewModelMap.Add(NewItem->GetID(), InViewModel.AsModel());
	
	InViewModel->OnAddedToCurveEditor(NewItem->GetID(), InCurveEditor);

	return NewItem->GetID();
}

void FCurveModelSyncer::OnOutlinerSelectionChanged()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	USequencerSettings* Settings = Sequencer ? Sequencer->GetSequencerSettings() : nullptr;
	if (Settings && Settings->ShouldSyncCurveEditorSelection())
	{
		// We schedule a selection synchronization for the next update. This synchronization must happen
		// after all filters have been applied, because the items we want to select in the curve editor
		// might be currently filtered out, but will be visible when filters are re-evaluated. This is
		// why curve editor integration runs in FSequencerNodeTree after filtering.
		RequestSyncSelection();
	}
}

void FCurveModelSyncer::SyncSequencerToCurveEditor(FSequencer& InSequencer, FCurveEditorTree& InTree)
{
	// Guard against multiple broadcasts here and defer them until the end of this function
	const FScopedCurveEditorTreeEventGuard ScopedEventGuard = InTree.ScopedEventGuard();
	
	const auto IsFilteredOut = [this](const TViewModelPtr<IOutlinerExtension>& InItem)
	{
		return LinkedFilterViewModel->IsFilteredOut(InItem);
	};
	
	for (auto It = ViewModelToTreeItemIDMap.CreateIterator(); It; ++It)
	{
		bool bIsRelevant = false;
		const FCurveEditorTreeItemID ItemID(It.Value());
		TViewModelPtr<ICurveEditorTreeItemExtension> ViewModel = CastViewModel<ICurveEditorTreeItemExtension>(It.Key().Pin());
		if (ViewModel)
		{
			TViewModelPtr<IOutlinerExtension> OutlinerModel = ViewModel.ImplicitCast();
			const bool bIsVisible = !IsFilteredOut(OutlinerModel);
			
			const FCurveEditorTreeItem* TreeItem = InTree.FindItem(ItemID);

			FCurveEditorTreeItemID ParentID;
			if (auto Parent = ViewModel.AsModel()->FindAncestorOfType<ICurveEditorTreeItemExtension>())
			{
				ParentID = ViewModelToTreeItemIDMap.FindRef(Parent.AsModel());
			}
			
			bIsRelevant = bIsVisible && TreeItem && TreeItem->GetParentID() == ParentID;
		}
		
		if (!bIsRelevant)
		{
			if (ViewModel)
			{
				ViewModel->OnRemovedFromCurveEditor(CurveEditor);
			}
			CurveEditor->RemoveTreeItem(ItemID);
			It.RemoveCurrent();
		}
	}

	// Do a second pass to remove any items that were removed recursively above
	for (auto It = ViewModelToTreeItemIDMap.CreateIterator(); It; ++It)
	{
		if (InTree.FindItem(It->Value) == nullptr)
		{
			TreeItemIDToViewModelMap.Remove(It->Value);
			It.RemoveCurrent();
		}
	}

	// Iterate all non-filtered out outliners and check for curve editor tree extensions
	const bool bIncludeRootNode = false;
	bool bItemsAdded = false;
	const TSharedPtr<FSequenceModel> OwnerModel = InSequencer.GetViewModel()->GetRootSequenceModel();
	for (TParentFirstChildIterator<IOutlinerExtension> It(OwnerModel, bIncludeRootNode); It; ++It)
	{
		if (IsFilteredOut(*It))
		{
			It.IgnoreCurrentChildren();
			continue;
		}

		if (TViewModelPtr<ICurveEditorTreeItemExtension> ChildViewModel = (*It).ImplicitCast())
		{
			const FCurveEditorTreeItemID ItemID = ViewModelToTreeItemIDMap.FindRef(ChildViewModel.AsModel());
			if (!ItemID.IsValid())
			{
				bItemsAdded = true;
				AddToCurveEditor(ChildViewModel, CurveEditor);
			}
		}
	}

	// If new items have been added, synchronize selection after the view models have been added to the curve editor. Synchronization depends on curve model tree item IDs
	if (bItemsAdded)
	{
		RequestSyncSelection();
	}
}

void FCurveModelSyncer::CleanUpPendingSyncSelection()
{
	bSyncSelectionRequested = false;
	
	if (GEditor)
	{
		GEditor->GetTimerManager()->ClearTimer(SyncSelectionHandle);
	}
}
} // namespace UE::Sequencer
