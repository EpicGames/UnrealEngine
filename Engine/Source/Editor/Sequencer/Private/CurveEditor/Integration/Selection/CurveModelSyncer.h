// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Engine/TimerHandle.h"
#include "Misc/Counter/ScopedSuspension.h"
#include "MVVM/Extensions/ICurveEditorTreeItemExtension.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "Templates/UnrealTemplate.h"

class FSequencer;
class FCurveEditorTree;

namespace UE::Sequencer
{
class ILinkedFilterViewModel;

using FScopedSuspendSequencerToCurveEditorSync = FScopedSuspension;

/** 
 * Makes sure that Curve Editor has the same selection as that in Sequencer.
 * Consequently, it knows which FCurveEditorTreeItemID and FViewModel correspond to each other.
 * 
 * Responsible for creating FCurveModel instances in FCurveEditor based on ICurveEditorTreeItemExtension in the Sequencer hierarchy.
 * This relies on outliner items implementing ICurveEditorTreeItemExtension (or its default shim) if they want to show up in the curve editor.
 */
class FCurveModelSyncer : public FNoncopyable
{
public:
	
	explicit FCurveModelSyncer(
		const TSharedRef<FSequencer>& InSequencer,
		const TSharedRef<FCurveEditor>& InCurveEditor, 
		const TSharedRef<ILinkedFilterViewModel>& InLinkedFilterViewModel
		);
	~FCurveModelSyncer();
	
	/** Synchronize curve editor selection with sequencer outliner selection on the next update. */
	void RequestSyncSelection();
	
	/** Clears the curve editor of all contents. */
	void ResetCurveEditor();
	
	/** @return The view model associated with the tree item. */
	TWeakPtr<FViewModel> FindViewModelById(const FCurveEditorTreeItemID& InTreeId) const;
	/** @return The tree item associated with the view model. */
	FCurveEditorTreeItemID FindIdByViewModel(const TWeakPtr<FViewModel>&) const;
	
	/** Keeps the curve editor items up-to-date with the sequencer outliner by adding/removing  entries as needed. */
	void UpdateCurveEditor();
	
	/** Suspends sync to Curve Editor for the lifetime of the scoped object. */
	FScopedSuspendSequencerToCurveEditorSync SuspendSelectionSync() { return FScopedSuspendSequencerToCurveEditorSync(SelectionSyncSuspension); }
	
	/** Invoked to request that the UI be updated with the current selection. */
	FSimpleMulticastDelegate& OnRequestSyncCurveEditorUI() { return OnRequestSyncCurveEditorUIDelegate; }
	/** Invoked before the curve editor is updated by UpdateCurveEditor. */
	FSimpleMulticastDelegate& OnStartUpdateCurveEditor() { return OnStartUpdateCurveEditorDelegate; }
	/** Invoked after the curve editor is updated by UpdateCurveEditor. */
	FSimpleMulticastDelegate& OnStopUpdateCurveEditor() { return OnStopUpdateCurveEditorDelegate; }
	
private:
	
	/** The owning Sequencer instance. Used to get settings. */
	const TWeakPtr<FSequencer> WeakSequencer;
	
	/** The curve editor we are synchronizing to. */
	const TSharedRef<FCurveEditor> CurveEditor;
	/** Curve Editor selection must be refreshed when we switch filtering modes. */
	const TSharedRef<ILinkedFilterViewModel> LinkedFilterViewModel;
	
	TMap<TWeakPtr<FViewModel>, FCurveEditorTreeItemID> ViewModelToTreeItemIDMap;
	TMap<FCurveEditorTreeItemID, TWeakPtr<FViewModel>> TreeItemIDToViewModelMap;
	
	/** Handle to latent refresh of selection. */
	FTimerHandle SyncSelectionHandle;
	/** Invoked to request that the UI be updated with the current selection. */
	FSimpleMulticastDelegate OnRequestSyncCurveEditorUIDelegate;
	/** Invoked before the curve editor is updated by UpdateCurveEditor. */
	FSimpleMulticastDelegate OnStartUpdateCurveEditorDelegate;
	/** Invoked after the curve editor is updated by UpdateCurveEditor. */
	FSimpleMulticastDelegate OnStopUpdateCurveEditorDelegate;
	
	/** Suspends syncing selection to Curve Editor. */
	FSuspendCounter SelectionSyncSuspension;
	
	/** Adds the given view-model to the curve editor */
	FCurveEditorTreeItemID AddToCurveEditor(TViewModelPtr<ICurveEditorTreeItemExtension> InViewModel, TSharedPtr<FCurveEditor> InCurveEditor);
	
	/** Update curve editor items when sequencer outliner items change */
	void OnHierarchyChanged() { UpdateCurveEditor(); }
	/** Updates the curve editor as result of the node tree hierarchy changing. */
	void OnNodeTreeUpdated() { UpdateCurveEditor(); }
	/** Syncs Curve Editor's selection if selection syncing is enabled. */
	void OnOutlinerSelectionChanged();
	
	/** Applies Sequencer's selection to the curve editor tree. */
	void SyncSequencerToCurveEditor(FSequencer& InSequencer, FCurveEditorTree& InTree);
	
	/** Cancels any pending transactions. */
	void CleanUpPendingSyncSelection();
};
} // namespace UE::Sequencer



