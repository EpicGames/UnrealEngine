// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionSyncRootLogic.h"

#include "CurveEditor.h"
#include "Sequencer.h"
#include "SequencerSelectionCurveFilter.h"
#include "MVVM/Selection/Selection.h"

namespace UE::Sequencer
{
FSelectionSyncRootLogic::FSelectionSyncRootLogic(
	const TSharedRef<FSequencer>& InSequencer, const TSharedRef<FCurveEditor>& InCurveEditor, 
	const TSharedRef<ILinkedFilterViewModel> InLinkedFilteringViewModel
	)
	: WeakSequencer(InSequencer)
	, CurveEditor(InCurveEditor)
	, SequencerToCurveEditorSyncer(InSequencer, InCurveEditor, InLinkedFilteringViewModel)
	, CurveEditorToSequencerSyncer(SequencerToCurveEditorSyncer, InCurveEditor, InSequencer)
{
	// If Sequencer initiates a sync from Sequencer to Curve Editor, prevent Curve Editor from syncing its select back to Sequencer.
	SequencerToCurveEditorSyncer.OnRequestSyncCurveEditorUI().AddLambda([this]
	{
		const FScopedSuspendCurveEditorToSequencerSync Suspend = CurveEditorToSequencerSyncer.SuspendSync();
		OnRequestRefreshCurveEditorUIDelegate.Broadcast();
	});
	SequencerToCurveEditorSyncer.OnStartUpdateCurveEditor().AddLambda([this]
	{
		SuspendDuringUpdateCurveEditor = CurveEditorToSequencerSyncer.SuspendSync();
	});
	SequencerToCurveEditorSyncer.OnStopUpdateCurveEditor().AddLambda([this]
	{
		SuspendDuringUpdateCurveEditor.Reset();
	});
	
	CurveEditorToSequencerSyncer.OnStartSync().AddRaw(this, &FSelectionSyncRootLogic::OnStartCurveEditorToSequencerSync);
	CurveEditorToSequencerSyncer.OnEndSync().AddRaw(this, &FSelectionSyncRootLogic::OnEndCurveEditorToSequencerSync);
	
	InSequencer->GetViewModel()->GetSelection()->Outliner.OnChanged.AddRaw(this, &FSelectionSyncRootLogic::OnOutlinerSelectionChanged);
}

FSelectionSyncRootLogic::~FSelectionSyncRootLogic()
{
	if (const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin())
	{
		Sequencer->GetViewModel()->GetSelection()->Outliner.OnChanged.RemoveAll(this);
	}
}

void FSelectionSyncRootLogic::UpdateIsolationFilter()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin(); 
	const USequencerSettings* SequencerSettings = Sequencer ? Sequencer->GetSequencerSettings() : nullptr;
	if (!SequencerSettings || bSkipIsolation)
	{
		return;
	}
	
	// If we're isolating to the selection and there is one, add the filter
	if (SequencerSettings->ShouldIsolateToCurveEditorSelection() && Sequencer->GetViewModel()->GetSelection()->Outliner.Num() != 0)
	{
		if (!SelectionCurveEditorFilter)
		{
			SelectionCurveEditorFilter = MakeShared<FSequencerSelectionCurveFilter>();
		}
		SelectionCurveEditorFilter->Update(Sequencer->GetViewModel()->GetSelection(), SequencerSettings->GetAutoExpandNodesOnSelection());
		CurveEditor->GetTree()->AddFilter(SelectionCurveEditorFilter);
	}
	// If we're not isolating to the selection (or there is no selection) remove the filter
	else if (SelectionCurveEditorFilter)
	{
		CurveEditor->GetTree()->RemoveFilter(SelectionCurveEditorFilter);
		SelectionCurveEditorFilter.Reset();
	}
}

void FSelectionSyncRootLogic::OnStartCurveEditorToSequencerSync()
{
	SuspendDuringRequestUpdateUI.Emplace(SequencerToCurveEditorSyncer.SuspendSelectionSync());
			
	// We don't want Curve Editor items to disappear as result of the user selecting something in the Curve Editor tree.
	bSkipIsolation = true;
}

void FSelectionSyncRootLogic::OnEndCurveEditorToSequencerSync()
{
	SuspendDuringRequestUpdateUI.Reset();
	bSkipIsolation = false;
	
	// Applying Curve Editor selection to the Outliner may cause items to become hidden in the Outliner, so refresh so the items are up to date. 
	// Specifically,the "Selected" filter can hide items after changing the selection. 
	// Example: Suppose that before you had a control rig selected: it would show all the controls, like chest, foot, torso, etc.
	// Then, you select the chest control rig handle actor. The "Selected" filter hides foot, torso, etc. because they are no longer selected.
	SequencerToCurveEditorSyncer.UpdateCurveEditor();
}
} // namespace UE::Sequencer
