// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveModelSyncer.h"
#include "Misc/Optional.h"
#include "SequencerSelectionSyncer.h"
#include "Templates/UnrealTemplate.h"

class FCurveEditor;
class FSequencer;

namespace UE::Sequencer
{
struct FSequencerSelectionCurveFilter;
class ILinkedFilterViewModel;

/** Instantiates and mediates between all subfeatures to do with syncing selection between Sequencer's Outliner and Curve Editor. */
class FSelectionSyncRootLogic : public FNoncopyable
{
public:
	
	explicit FSelectionSyncRootLogic(
		const TSharedRef<FSequencer>& InSequencer,
		const TSharedRef<FCurveEditor>& InCurveEditor, 
		const TSharedRef<ILinkedFilterViewModel> InLinkedFilteringViewModel
		);
	~FSelectionSyncRootLogic();
	
	/** Synchronize curve editor selection with sequencer outliner selection on the next update. */
	void RequestSyncSelection() { SequencerToCurveEditorSyncer.RequestSyncSelection(); }
	/** Keeps the curve editor items up-to-date with the sequencer outliner by adding/removing  entries as needed. */
	void UpdateCurveEditor() { SequencerToCurveEditorSyncer.UpdateCurveEditor(); }
	/** Clears the curve editor of all contents. */
	void ResetCurveEditor() { SequencerToCurveEditorSyncer.ResetCurveEditor(); }
	
	FCurveModelSyncer& GetSequencerToCurveEditorSyncer() { return SequencerToCurveEditorSyncer; }
	
	/** Invoked to request that the Curve Editor tree view UI updates its items. */
	FSimpleMulticastDelegate& OnRequestRefreshCurveEditorUI() { return OnRequestRefreshCurveEditorUIDelegate; }
	
private:
	
	/** The owning Sequencer instance. */
	const TWeakPtr<FSequencer> WeakSequencer;
	/** The curve editor being synced. */
	const TSharedRef<FCurveEditor> CurveEditor;
	
	/** Handles that Curve Editor has the same selection as that of Sequencer. */
	FCurveModelSyncer SequencerToCurveEditorSyncer;
	
	/** Handles that Sequencer has the same selection as that of Curve Editor. */
	FSequencerSelectionSyncer CurveEditorToSequencerSyncer;
	/** Set while CurveEditorToSequencerSyncer is syncing due to OnRequestSyncCurveEditorUI. */
	TOptional<FScopedSuspendSequencerToCurveEditorSync> SuspendDuringRequestUpdateUI;
	/** Set while CurveEditorToSequencerSyncer is syncing due to a UpdateCurveEditor call. */
	TOptional<FScopedSuspendSequencerToCurveEditorSync> SuspendDuringUpdateCurveEditor;
	
	/** Curve editor filter that shows only the nodes selected in the Outliner. Can be null. */
	TSharedPtr<FSequencerSelectionCurveFilter> SelectionCurveEditorFilter;
	/** 
	 * Whether to skip isolating. 
	 * 
	 * Set when selection is being pushed from Curve Editor to Sequencer: 
	 * We don't want Curve Editor items to disappear as result of the user selecting something in the Curve Editor tree.
	 */
	bool bSkipIsolation = false;
	
	/** Invoked to request that the Curve Editor tree view UI updates its items. */
	FSimpleMulticastDelegate OnRequestRefreshCurveEditorUIDelegate;
	
	void OnOutlinerSelectionChanged() { UpdateIsolationFilter(); }
	void UpdateIsolationFilter();
	
	void OnStartCurveEditorToSequencerSync();
	void OnEndCurveEditorToSequencerSync();
};
} // namespace UE::Sequencer
