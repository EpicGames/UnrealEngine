// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerSelectionSyncer.h"

#include "CurveEditor.h"
#include "CurveEditor/Views/SelectionSyncUtils.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/Views/SOutlinerView.h"
#include "Misc/ScopeExit.h"
#include "Sequencer.h"

namespace UE::Sequencer
{
FSequencerSelectionSyncer::FSequencerSelectionSyncer(
	const FCurveModelSyncer& InCurveModelSyncer,
	const TSharedRef<FCurveEditor>& InCurveEditor,
	TWeakPtr<FSequencer> InWeakSequencer
	)
	: CurveModelSyncer(InCurveModelSyncer)
	, CurveEditor(InCurveEditor)
	, WeakSequencer(MoveTemp(InWeakSequencer))
{
	if (FCurveEditorTree* Tree = CurveEditor->GetTree(); ensure(Tree))
	{
		Tree->Events.OnSelectionChanged.AddRaw(this, &FSequencerSelectionSyncer::OnCurveEditorSelectionChanged);
	}
}

FSequencerSelectionSyncer::~FSequencerSelectionSyncer()
{
	if (FCurveEditorTree* Tree = CurveEditor->GetTree())
	{
		Tree->Events.OnSelectionChanged.RemoveAll(this);
	}
}

static TAutoConsoleVariable<bool> CVarSelectionExpandsSequencer(TEXT("Sequencer.CurveEditor.SelectionExpandsSequencer"), false, TEXT(""));

void FSequencerSelectionSyncer::OnCurveEditorSelectionChanged()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	FCurveEditorTree* Tree = CurveEditor->GetTree();
	
	USequencerSettings* Settings = Sequencer ? Sequencer->GetSequencerSettings() : nullptr;
	const bool bIsEnabled = Settings && Settings->ShouldSyncOutlinerSelectionToCurveEditor();
	
	if (SyncSuspension.IsSuspended() || !bIsEnabled || !ensure(Tree) || !ensure(Sequencer))
	{
		return;
	}
	
	OnStartSyncDelegate.Broadcast();
	ON_SCOPE_EXIT{ OnEndSyncDelegate.Broadcast(); };
	
	FSequencerSelection& TargetSelection = Sequencer->GetSelection();
	ApplyCurveEditorToSequencer(TargetSelection, *Tree, CurveModelSyncer);
	
	// E.g. in Curve Editor click Cube.Transform.Location.X but Cube is collapsed in Sequencer, so the user wouldn't see its now selected in Sequencer.
	// So we expand to that item in Sequencer's Outliner if configured to do so by the user.
	if (Settings->GetAutoExpandNodesOnSelection())
	{
		ExpandSelectedSequencerItems(TargetSelection);
		// This is needed to apply the expansion state by causing SOutlinerView::Refresh.
		Sequencer->RefreshTree();
	}
	
	// If the user clicked a single item, show it in the UI
	const TSet<TWeakViewModelPtr<IOutlinerExtension>>& Selected = TargetSelection.Outliner.GetSelected();
	if (Selected.Num() == 1)
	{
		const TWeakViewModelPtr<IOutlinerExtension> FirstSelected = *Selected.CreateConstIterator();
		Sequencer->GetOutlinerViewWidget()->RequestScrollIntoView(FirstSelected);
	}
}
} // namespace UE::Sequencer
