// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Misc/Counter/ScopedSuspension.h"
#include "Misc/Counter/SuspendCounter.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FSequencer;
class FCurveEditor;
struct FCurveModelID;

namespace UE::Sequencer
{
class FCurveModelSyncer;
class FSequencerSelection;

using FScopedSuspendCurveEditorToSequencerSync = FScopedSuspension;

/** Syncs selection changes from Curve Editor to Sequencer. */
class FSequencerSelectionSyncer : public FNoncopyable
{
public:
	
	explicit FSequencerSelectionSyncer(
		const FCurveModelSyncer& InCurveModelSyncer UE_LIFETIMEBOUND,
		const TSharedRef<FCurveEditor>& InCurveEditor, TWeakPtr<FSequencer> InWeakSequencer
		);
	~FSequencerSelectionSyncer();
	
	/** Suspends sync to Sequencer for the lifetime of the scoped object. */
	[[nodiscard]] FScopedSuspendCurveEditorToSequencerSync SuspendSync() { return FScopedSuspendCurveEditorToSequencerSync(SyncSuspension); }
	
	/** Invoked before curve editor selection is synced to Sequencer's outliner. */
	FSimpleMulticastDelegate& OnStartSync() { return OnStartSyncDelegate; }
	/** Invoked after curve editor selection is synced to Sequencer's outliner. */
	FSimpleMulticastDelegate& OnEndSync() { return OnEndSyncDelegate; }

private:
	
	/** Used to translate curve model IDs to IOutlinerExtensions. */
	const FCurveModelSyncer& CurveModelSyncer;
	
	/** Acts as source. Whenever selection changes in the tree, the selection is synced to Sequencer. */
	const TSharedRef<FCurveEditor> CurveEditor;
	
	/** Used to get the selection and refresh the tree hierarchy when the expansion state changes. */
	const TWeakPtr<FSequencer> WeakSequencer;
	
	/** Keeps track whether syncing is enabled. */
	FSuspendCounter SyncSuspension;
	
	/** Invoked before curve editor selection is synced to Sequencer's outliner. */
	FSimpleMulticastDelegate OnStartSyncDelegate;
	/** Invoked after curve editor selection is synced to Sequencer's outliner. */
	FSimpleMulticastDelegate OnEndSyncDelegate;
	
	/** Handles the selection changing in curve editor by applying it to Sequencer's outliner. */
	void OnCurveEditorSelectionChanged();
};
} // namespace UE::Sequencer
