// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerCurveEditorApp.h"

#include "Filters/FilterConfigIdentifiers.h"
#include "Filters/Linking/FilterAreaManager.h"
#include "Filters/Linking/Mode/LinkedFilterFactoryViewModel.h"
#include "Filters/Linking/Mode/LinkedFilterViewModel.h"
#include "Filters/Linking/Mode/MakeLinkedFilterViewModelArgs.h"
#include "ICurveEditorModule.h"
#include "MVVM/CurveEditorExtension.h"
#include "MVVM/ViewModels/SequencerEditorViewModelImpl.h"
#include "MVVM/ViewModels/SequencerModelUtils.h"
#include "Specializations/SequencerCurveEditor.h"
#include "Specializations/SequencerCurveEditorBounds.h"
#include "Specializations/SequencerCurveEditorToolbarExtender.h"
#include "Specializations/SequencerCurveModelUtils.h"
#include "TimeSliderArgs.h"
#include "Views/CurveEditorWidgetOwnerArgs.h"
#include "Widgets/SequencerCurveEditorTimeSliderController.h"

namespace UE::Sequencer
{
class FCurveEditorExtension;

namespace CurveEditorDetail
{
static FKeyAttributes GetDefaultKeyAttributes(USequencerSettings* InSettings)
{
	switch (InSettings->GetKeyInterpolation())
	{
	case EMovieSceneKeyInterpolation::User:     return FKeyAttributes().SetInterpMode(RCIM_Cubic).SetTangentMode(RCTM_User);
	case EMovieSceneKeyInterpolation::Break:    return FKeyAttributes().SetInterpMode(RCIM_Cubic).SetTangentMode(RCTM_Break);
	case EMovieSceneKeyInterpolation::Linear:   return FKeyAttributes().SetInterpMode(RCIM_Linear).SetTangentMode(RCTM_Auto);
	case EMovieSceneKeyInterpolation::Constant: return FKeyAttributes().SetInterpMode(RCIM_Constant).SetTangentMode(RCTM_Auto);
	case EMovieSceneKeyInterpolation::Auto:     return FKeyAttributes().SetInterpMode(RCIM_Cubic).SetTangentMode(RCTM_Auto);
	case EMovieSceneKeyInterpolation::SmartAuto:
	default:                                    return FKeyAttributes().SetInterpMode(RCIM_Cubic).SetTangentMode(RCTM_SmartAuto);
	}
}

static TSharedRef<FCurveEditor> MakeCurveEditor(const TSharedRef<FSequencer>& InSequencer, const FTimeSliderArgs& InTimeSliderArgs)
{
	USequencerSettings* const SequencerSettings = InSequencer->GetSequencerSettings();
	check(SequencerSettings);
	
	FCurveEditorInitParams CurveEditorInitParams;
	CurveEditorInitParams.AdditionalEditorExtensions = { MakeShared<FSequencerCurveEditorToolbarExtender>(InSequencer.ToWeakPtr()) };
	CurveEditorInitParams.ZoomScalingAttr.BindLambda([SequencerSettings]
	{
		return &SequencerSettings->GetCurveEditorZoomScaling();
	});
	CurveEditorInitParams.ResolveCurveModelDelegate.BindStatic(&ResolveCurveEditorModel, InSequencer.ToWeakPtr());
		
	const TSharedRef<FSequencerCurveEditor> CurveEditorModel = MakeShared<FSequencerCurveEditor>(InSequencer, InTimeSliderArgs.NumericTypeInterface);
	CurveEditorModel->SetBounds(MakeUnique<FSequencerCurveEditorBounds>(InSequencer));
	CurveEditorModel->InitCurveEditor(CurveEditorInitParams);

	CurveEditorModel->InputSnapEnabledAttribute = MakeAttributeLambda([SequencerSettings] { return SequencerSettings->GetIsSnapEnabled(); });
	CurveEditorModel->OnInputSnapEnabledChanged = FOnSetBoolean::CreateLambda([SequencerSettings](bool NewValue) { SequencerSettings->SetIsSnapEnabled(NewValue); });

	CurveEditorModel->OutputSnapEnabledAttribute = MakeAttributeLambda([SequencerSettings] { return SequencerSettings->GetSnapCurveValueToInterval(); });
	CurveEditorModel->OnOutputSnapEnabledChanged = FOnSetBoolean::CreateLambda([SequencerSettings](bool NewValue) { SequencerSettings->SetSnapCurveValueToInterval(NewValue); });

	CurveEditorModel->FixedGridSpacingAttribute = MakeAttributeLambda([SequencerSettings]() -> TOptional<float> { return SequencerSettings->GetGridSpacing(); });
	CurveEditorModel->InputSnapRateAttribute = MakeAttributeSP(InSequencer, &FSequencer::GetFocusedDisplayRate);

	CurveEditorModel->DefaultKeyAttributes = MakeAttributeLambda([SequencerSettings]() { return GetDefaultKeyAttributes(SequencerSettings); });
	
	return CurveEditorModel;
}

static TSharedRef<FLinkedFilterViewModel> MakeFilteringModel(const TSharedRef<FSequencer>& InSequencer)
{
	const TSharedPtr<FLinkedFilterFactoryViewModel> FilterFactory = InSequencer->GetViewModelImpl()->GetLinkedFilterViewModelFactoryImpl();
	FMakeLinkedFilterViewModelArgs CurveEditorInstanceArgs(ConfigIds::CurveEditor_InstancedConfigId);
	CurveEditorInstanceArgs.FilterAreaConfigId = ConfigIds::FilterArea_CurveEditor;
	const TSharedRef<FLinkedFilterViewModel> ViewModelImpl = FilterFactory->MakeFilteringModelImpl(CurveEditorInstanceArgs);
	
	return ViewModelImpl;
}
}

FSequencerCurveEditorApp::FSequencerCurveEditorApp(const TSharedRef<FSequencer>& InSequencer, const FTimeSliderArgs& InTimeSliderArgs)
	: CurveEditor(CurveEditorDetail::MakeCurveEditor(InSequencer, InTimeSliderArgs))
	, FilterAreaClient(
		ConfigIds::FilterArea_CurveEditor,
		InSequencer,
		CurveEditorDetail::MakeFilteringModel(InSequencer)
	)
	, CommandBinder(InSequencer, CurveEditor, FilterAreaClient)
	, SelectionSyncLogic(InSequencer, CurveEditor, FilterAreaClient.GetFilterModel())
	, WidgetOwner(FCurveEditorWidgetOwnerArgs(
		InSequencer, CurveEditor, 
		MakeShared<FSequencerCurveEditorTimeSliderController>(InTimeSliderArgs, InSequencer, CurveEditor), 
		FilterAreaClient.GetFilterModel(), CommandBinder.GetCurveEditorCommands().ToSharedRef(),
		SelectionSyncLogic.GetSequencerToCurveEditorSyncer())
		)
{
	SelectionSyncLogic.OnRequestRefreshCurveEditorUI().AddLambda([this]
	{
		WidgetOwner.SyncSelection();
	});
	
	// The items in curve editor are supposed to have the same order as in Sequencer's Outliner.
	CurveEditor->GetTree()->SetSortPredicate(FCurveEditorTree::FCurveTreeItemSortPredicate::CreateLambda(
		[this](const FCurveEditorTreeItem& ItemA, const FCurveEditorTreeItem& ItemB)
		{
			FCurveModelSyncer& Syncer = SelectionSyncLogic.GetSequencerToCurveEditorSyncer();
			const TSharedPtr<FViewModel> ViewModelA = Syncer.FindViewModelById(ItemA.GetID()).Pin();
			const TSharedPtr<FViewModel> ViewModelB = Syncer.FindViewModelById(ItemB.GetID()).Pin();
			return ensure(ViewModelA && ViewModelB)
				? ComesFirstInHierarchy(FViewModelPtr(ViewModelA.Get()), FViewModelPtr(ViewModelB.Get()))
				// Sorting algorithms require consistent < implementation so fallback to pointer comparision.
				: ViewModelA.Get() < ViewModelB.Get();
		}));
	
	SelectionSyncLogic.UpdateCurveEditor();
}

FSequencerCurveEditorApp* FSequencerCurveEditorApp::Get(const ISequencer& InSequencer)
{
	const FCurveEditorExtension* CurveEditorExtension = InSequencer.GetViewModel()->CastDynamic<FCurveEditorExtension>();
	return CurveEditorExtension ? CurveEditorExtension->CurveEditorApp.Get() : nullptr;
}
} // namespace UE::Sequencer
