// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditor/Views/CurveEditorWidgetOwner.h"
#include "Filters/Linking/FilterAreaManager.h"
#include "Integration/CurveEditorCommandBinder.h"
#include "Integration/Selection/SelectionSyncRootLogic.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class ISequencer;

namespace UE::Sequencer
{
class FLinkedFilterViewModel;

/** 
 * Integrates the Curve Editor into Sequencer.
 * 
 * Acts as the composition root for the Curve Editor within Sequencer:
 * constructs all required subsystems, wires their dependencies,
 * and coordinates their interaction with Sequencer.
 * 
 * FCurveEditorExtension provides the public API and forwards calls to this class.
 * @see FCurveEditorExtension
 */
class FSequencerCurveEditorApp : public FNoncopyable
{
public:
	
	explicit FSequencerCurveEditorApp(
		const TSharedRef<FSequencer>& InSequencer, const FTimeSliderArgs& InTimeSliderArgs
		);
	
	/** 
	 * @return The FSequencerCurveEditorApp used in the Sequencer instance. Returns nullptr, if curve editor integration is not enabled. 
	 * @note This function exists as compromise / for convenience. Avoid introducing direct dependencies from core Sequencer code to the Curve Editor,
	 * to ensure Sequencer follows the open-closed principle. For example, when adding new functionality, consider e.g. exposing a delegate in the
	 * core Sequencer API and handling it within the Curve Editor integration layer. 
	 */
	static FSequencerCurveEditorApp* Get(const ISequencer& InSequencer);
	
	/** Called when FSequencer::BindCommands() initializes its commands. */
	void BindCommands() { CommandBinder.InitSequencerCommandBindings(); }
	
	/** Gets the curve editor view-model */
	TSharedPtr<FCurveEditor> GetCurveEditor() const { return CurveEditor; }

	/** Opens the curve editor */
	void OpenCurveEditor() { WidgetOwner.OpenCurveEditor(); }
	/** Returns whether the curve editor is open */
	bool IsCurveEditorOpen() const { return WidgetOwner.IsCurveEditorOpen(); }
	/** Closes the curve editor */
	void CloseCurveEditor() { WidgetOwner.CloseCurveEditor(); }

	/** Curve editor tree widget */
	TSharedPtr<SCurveEditorTree> GetCurveEditorTreeView() const { return WidgetOwner.GetTreeView(); }

	/** Synchronize curve editor selection with sequencer outliner selection on the next update. */
	void RequestSyncSelection() { SelectionSyncLogic.RequestSyncSelection(); }
	/** Keeps the curve editor items up-to-date with the sequencer outliner by adding/removing  entries as needed. */
	void UpdateCurveEditor() { SelectionSyncLogic.UpdateCurveEditor(); }
	/** Clears the curve editor of all contents. */
	void ResetCurveEditor() { SelectionSyncLogic.ResetCurveEditor(); }
	
private:
	
	// @note Member order matters. C++ destruction order is reverse declaration order, so the dependencies must point upwards:
	// Lower members can only depend on members declared after. Higher members cannot depend on members declared after.
	
	/** The Curve Editor instance that Sequencer uses. */
	TSharedRef<FCurveEditor> CurveEditor;
	
	/** Manages linked filtering for curve editor. Owns the view model for linked filtering. */
	FFilterAreaManager FilterAreaClient;
	
	/** Allows users to use issue some Sequencer commands when the Curve Editor is UI is focused. */
	FCurveCommandBinder CommandBinder;
	
	/** Instantiates and mediates between all subfeatures to do with syncing selection between Sequencer's Outliner and Curve Editor. */
	FSelectionSyncRootLogic SelectionSyncLogic;
	
	/** Manages lifecycle of views exposed in Sequencer. */
	FCurveEditorWidgetOwner WidgetOwner;
};
}


