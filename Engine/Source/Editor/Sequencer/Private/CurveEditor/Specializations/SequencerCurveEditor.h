// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CurveEditor.h"
#include "CurveEditorAxis.h"
#include "Sequencer.h"
#include "SequencerTimeCurveEditorAxis.h"

namespace UE::Sequencer
{
class FSequencerCurveEditor : public FCurveEditor
{
	TWeakPtr<FSequencer> WeakSequencer;
	TSharedPtr<FLinearCurveEditorAxis> FocusedTimeAxis;
	
public:
	
	explicit FSequencerCurveEditor(TWeakPtr<FSequencer> InSequencer, TSharedPtr<INumericTypeInterface<double>> InNumericTypeInterface)
		: WeakSequencer(InSequencer)
	{
		FocusedTimeAxis = MakeShared<FSequencerTimeCurveEditorAxis>(InSequencer);
		FocusedTimeAxis->NumericTypeInterface = InNumericTypeInterface;

		InSequencer.Pin()->OnActivateSequence().AddRaw(this, &FSequencerCurveEditor::HandleSequenceActivated);

		AddAxis("FocusedSequenceTime", FocusedTimeAxis);
	}

	virtual ~FSequencerCurveEditor() override
	{
		if (TSharedPtr< FSequencer> Sequencer = WeakSequencer.Pin())
		{
			Sequencer->OnActivateSequence().RemoveAll(this);
		}
	}

	virtual void GetGridLinesX(TArray<float>& MajorGridLines, TArray<float>& MinorGridLines, TArray<FText>* MajorGridLabels) const override
	{
		TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
		if (!Sequencer)
		{
			return;
		}
		FCurveEditorScreenSpaceH PanelInputSpace = GetPanelInputSpace();

		double MajorGridStep = 0.0;
		int32  MinorDivisions = 0;
		const bool bGotDimensions = Sequencer->GetGridMetrics(
			PanelInputSpace.GetPhysicalWidth(), PanelInputSpace.GetInputMin(), PanelInputSpace.GetInputMax(), MajorGridStep, MinorDivisions
			);

		if (bGotDimensions)
		{
			const double FirstMajorLine = FMath::FloorToDouble(PanelInputSpace.GetInputMin() / MajorGridStep) * MajorGridStep;
			const double LastMajorLine = FMath::CeilToDouble(PanelInputSpace.GetInputMax() / MajorGridStep) * MajorGridStep;

			for (double CurrentMajorLine = FirstMajorLine; CurrentMajorLine < LastMajorLine; CurrentMajorLine += MajorGridStep)
			{
				MajorGridLines.Add(PanelInputSpace.SecondsToScreen(CurrentMajorLine));

				for (int32 Step = 1; Step < MinorDivisions; ++Step)
				{
					MinorGridLines.Add(PanelInputSpace.SecondsToScreen(CurrentMajorLine + Step * MajorGridStep / MinorDivisions));
				}
			}
		}
	}

	virtual int32 GetSupportedTangentTypes() override
	{
		return ((int32)ECurveEditorTangentTypes::InterpolationConstant	|
			(int32)ECurveEditorTangentTypes::InterpolationLinear		|
			(int32)ECurveEditorTangentTypes::InterpolationCubicAuto		|
			(int32)ECurveEditorTangentTypes::InterpolationCubicUser		|
			(int32)ECurveEditorTangentTypes::InterpolationCubicBreak	|
			(int32)ECurveEditorTangentTypes::InterpolationCubicWeighted |
			(int32)ECurveEditorTangentTypes::InterpolationCubicSmartAuto);
	}

	void HandleSequenceActivated(FMovieSceneSequenceIDRef) const
	{
		FocusedTimeAxis->NumericTypeInterface = WeakSequencer.Pin()->GetNumericTypeInterface();
	}
};
} // namespace UE::Sequencer

