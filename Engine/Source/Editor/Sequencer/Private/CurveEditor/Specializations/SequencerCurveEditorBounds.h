// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimatedRange.h"
#include "ICurveEditorBounds.h"
#include "Sequencer.h"

namespace UE::Sequencer
{
struct FSequencerCurveEditorBounds : ICurveEditorBounds
{
	FSequencerCurveEditorBounds(TSharedRef<FSequencer> InSequencer)
		: WeakSequencer(InSequencer)
	{
		TRange<double> Bounds = InSequencer->GetViewRange();
		InputMin = Bounds.GetLowerBoundValue();
		InputMax = Bounds.GetUpperBoundValue();
	}

	virtual void GetInputBounds(double& OutMin, double& OutMax) const override
	{
		TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
		if (Sequencer.IsValid())
		{
			const bool bLinkTimeRange = Sequencer->GetSequencerSettings()->GetLinkCurveEditorTimeRange();
			if (bLinkTimeRange)
			{
				TRange<double> Bounds = Sequencer->GetViewRange();
				OutMin = Bounds.GetLowerBoundValue();
				OutMax = Bounds.GetUpperBoundValue();
			}
			else
			{
				// If they don't want to link the time range with Sequencer we return the cached value.
				OutMin = InputMin;
				OutMax = InputMax;
			}
		}
	}

	virtual void SetInputBounds(double InMin, double InMax) override
	{
		TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
		if (Sequencer.IsValid())
		{
			const bool bLinkTimeRange = Sequencer->GetSequencerSettings()->GetLinkCurveEditorTimeRange();
			if (bLinkTimeRange)
			{
				FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();

				if (InMin * TickResolution > TNumericLimits<int32>::Lowest() && InMax * TickResolution < TNumericLimits<int32>::Max())
				{
					Sequencer->SetViewRange(TRange<double>(InMin, InMax), EViewRangeInterpolation::Immediate);
				}
			}

			// We update these even if you are linked to the Sequencer Timeline so that when you turn off the link setting
			// you don't pop to your last values, instead your view stays as is and just stops moving when Sequencer moves.
			InputMin = InMin;
			InputMax = InMax;
		}
	}

	/** The min/max values for the viewing range. Only used if Curve Editor/Sequencer aren't linked ranges. */
	double InputMin, InputMax;
	TWeakPtr<FSequencer> WeakSequencer;
};
} // namespace UE::Sequencer
