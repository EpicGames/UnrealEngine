// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ICurveEditorExtension.h"
#include "Menus/SequencerToolbarUtils.h"
#include "Sequencer.h"

namespace UE::Sequencer
{
class FSequencerCurveEditorToolbarExtender : public ICurveEditorExtension
{
	TWeakPtr<FSequencer> WeakSequencer;
public:

	explicit FSequencerCurveEditorToolbarExtender(TWeakPtr<FSequencer> InWeakSequencer) : WeakSequencer(MoveTemp(InWeakSequencer)) {}
	
	virtual void BindCommands(TSharedRef<FUICommandList> CommandBindings) override {}
	virtual TSharedPtr<FExtender> MakeToolbarExtender(const TSharedRef<FUICommandList>& InCommandList) override
	{
		TSharedRef<FExtender> Extender = MakeShared<FExtender>();
		Extender->AddToolBarExtension("Adjustment", EExtensionHook::After, InCommandList,
			FToolBarExtensionDelegate::CreateLambda([this](FToolBarBuilder& ToolbarBuilder)
			{
				TSharedPtr<FSequencer> SequenerPin = WeakSequencer.Pin();
				
				ToolbarBuilder.BeginSection("Keying");
				ToolbarBuilder.PushCommandList(SequenerPin->GetCommandBindings().ToSharedRef());
				EToolbarItemFlags Flags = EToolbarItemFlags::KeyGroup | EToolbarItemFlags::AutoKey;
				AppendSequencerToolbarEntries(SequenerPin, ToolbarBuilder, Flags);
				ToolbarBuilder.PopCommandList();
				ToolbarBuilder.EndSection();
			}));
		return Extender;
	}
};
} // namespace UE::Sequencer
