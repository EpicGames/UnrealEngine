// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerCurveModelUtils.h"

#include "Channels/MovieSceneChannelProxy.h"
#include "ISequencerModule.h"
#include "Modification/Resolution/CurveMetaDataIdentifiers.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneSection.h"
#include "Sequencer.h"

namespace UE::Sequencer
{
TUniquePtr<FCurveModel> ResolveCurveEditorModel(const CurveEditor::FCurveModelLookUpArgs& InArgs, TWeakPtr<FSequencer> OwningSequencer)
{
	const TSharedPtr<ISequencer> SequencerPin = OwningSequencer.Pin();
	if (!SequencerPin)
	{
		return nullptr;
	}
	
	const CurveEditor::FCurveMetaDataIdentifiers* CurveMetadata = InArgs.GetCurveMetaData();
	if (!CurveMetadata)
	{
		return nullptr;
	}

	UMovieSceneSection* OwningSection = Cast<UMovieSceneSection>(CurveMetadata->Owner.Get());
	if (!OwningSection)
	{
		return nullptr;
	}

	const FMovieSceneChannelHandle ChannelHandle = OwningSection->GetChannelProxy().GetChannelByName(CurveMetadata->ChannelName);
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	ISequencerChannelInterface* EditorInterface = SequencerModule.FindChannelEditorInterface(ChannelHandle.GetChannelTypeName());
	if (!EditorInterface)
	{
		return nullptr;
	}
	
	const FMovieSceneChannelMetaData* ChannelMetadata = ChannelHandle.GetMetaData();
	FCreateCurveEditorModelParams Params = {
		OwningSection,
		ChannelMetadata->WeakOwningObject.Get(),
		SequencerPin.ToSharedRef()
	};

	return EditorInterface->CreateCurveEditorModel_Raw(ChannelHandle, Params);
}
} // namespace UE::Sequencer