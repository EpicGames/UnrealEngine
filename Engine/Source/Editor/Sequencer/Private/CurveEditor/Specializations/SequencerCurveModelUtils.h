// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modification/Resolution/CurveModelLookUpInfo.h"
#include "Templates/UniquePtr.h"

class FCurveModel;
class FSequencer;
namespace UE::CurveEditor { struct FCurveModelLookUpArgs; }

namespace UE::Sequencer
{
/** Constructs a FCurveModel by looking up the channel from the owning UMovieSceneSection and ChannelName. */
TUniquePtr<FCurveModel> ResolveCurveEditorModel(const CurveEditor::FCurveModelLookUpArgs& InArgs, TWeakPtr<FSequencer> OwningSequencer);
} // namespace UE::Sequencer
