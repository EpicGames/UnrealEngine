// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorAxis.h"
#include "SCurveEditorView.h"
#include "Sequencer.h"

namespace UE::Sequencer
{
/** Custom curve editor axis that displays the 'current time' in display rate */
class FSequencerTimeCurveEditorAxis : public FLinearCurveEditorAxis
{
	TWeakPtr<FSequencer> WeakSequencer;
public:

	explicit FSequencerTimeCurveEditorAxis(TWeakPtr<FSequencer> InWeakSequencer)
		: WeakSequencer(InWeakSequencer)
	{}

	virtual void GetGridLines(
		const FCurveEditor& CurveEditor, const SCurveEditorView& View, FCurveEditorViewAxisID AxisID, 
		TArray<double>& OutMajorGridLines, TArray<double>& OutMinorGridLines, ECurveEditorAxisOrientation Axis
		) const override
	{
		TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();

		if (!Sequencer.IsValid())
		{
			return;
		}

		double MajorGridStep = 0.0;
		int32  MinorDivisions = 0;
		
		float Size = 1.0;
		float Min  = 0.0;
		float Max  = 1.0;

		if (Axis == ECurveEditorAxisOrientation::Horizontal)
		{
			FCurveEditorScreenSpaceH AxisSpace = View.GetHorizontalAxisSpace(AxisID);
			Size = AxisSpace.GetPhysicalWidth();
			Min  = AxisSpace.GetInputMin();
			Max  = AxisSpace.GetInputMax();
		}
		else
		{
			FCurveEditorScreenSpaceV AxisSpace = View.GetVerticalAxisSpace(AxisID);
			Size = AxisSpace.GetPhysicalHeight();
			Min = AxisSpace.GetOutputMin();
			Max = AxisSpace.GetOutputMax();
		}

		if (Sequencer.IsValid() && Sequencer->GetGridMetrics(Size, Min, Max, MajorGridStep, MinorDivisions))
		{
			double FirstMajorLine = FMath::FloorToDouble(Min / MajorGridStep) * MajorGridStep;
			double LastMajorLine = FMath::CeilToDouble(Max / MajorGridStep) * MajorGridStep;

			for (double CurrentMajorLine = FirstMajorLine; CurrentMajorLine < LastMajorLine; CurrentMajorLine += MajorGridStep)
			{
				OutMajorGridLines.Add(CurrentMajorLine);

				for (int32 Step = 1; Step < MinorDivisions; ++Step)
				{
					double MinorLine = CurrentMajorLine + Step * MajorGridStep / MinorDivisions;
					OutMinorGridLines.Add(MinorLine);
				}
			}
		}
	}
};
} // namespace UE::Sequencer
