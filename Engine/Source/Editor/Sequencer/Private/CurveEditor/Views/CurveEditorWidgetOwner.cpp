// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorWidgetOwner.h"

#include "AnimatedRange.h"
#include "CurveEditorWidgetOwnerArgs.h"
#include "CurveEditor/Widgets/SSequencerCurveEditor.h"
#include "CurveEditor/Widgets/SCurveTreeContent.h"
#include "Filters/SCurveEditorFilterPanel.h"
#include "Filters/Linking/Mode/LinkedFilterViewModel.h"
#include "Filters/Widgets/Linking/SFilterBarSwitcher.h"
#include "HAL/IConsoleManager.h"
#include "IPropertyRowGenerator.h"
#include "IStructureDetailsView.h"
#include "MVVM/CurveEditorExtension.h"
#include "SCurveEditorPanel.h"
#include "SCurveEditorToolProperties.h"
#include "SCurveKeyDetailPanel.h"
#include "Sequencer.h"
#include "SequencerCommands.h"
#include "Filters/FilterConfigIdentifiers.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"

struct FTimeSliderArgs;

#define LOCTEXT_NAMESPACE "FSequencerCurveEditorPresenter"

namespace UE::Sequencer
{
namespace PresenterDetail
{
static FCurveEditorWidgets MakePresenterWidgets(
	const FCurveEditorWidgetOwnerArgs& InArgs,
	const TSharedPtr<FTabManager> InTabManager
	)
{
	const TSharedRef<SCurveTreeContent> TreeContent = SNew(SCurveTreeContent)
		.TabManager(InTabManager)
		.Sequencer(InArgs.Sequencer)
		.CurveEditor(InArgs.CurveEditor)
		.FilterViewModel(InArgs.FilteringViewModel)
		.CommandList(InArgs.CommandList)
		.CurveModelSyncer(&InArgs.CurveModelSyncer)
		.FilterPills()
		[
			SNew(SFilterBarSwitcher)
			.Sequencer(InArgs.Sequencer)
			.FilterAreaConfigId(ConfigIds::FilterArea_CurveEditor)
			.LinkedFilterViewModel(InArgs.FilteringViewModel)
		];
	
	const TSharedRef<SCurveEditorPanel> CurveEditorPanel = SNew(SCurveEditorPanel, InArgs.CurveEditor)
		// Grid lines match the color specified in FSequencerTimeSliderController::OnPaintViewArea
		.GridLineTint(FLinearColor(0.f, 0.f, 0.f, 0.3f))
		.ExternalTimeSliderController(InArgs.TimeSliderController)
		.MinimumViewPanelHeight(0.f)
		.TabManager(InTabManager)
		.DisabledTimeSnapTooltip(LOCTEXT("CurveEditorTimeSnapDisabledTooltip", "Time Snapping is currently driven by Sequencer."))
		.TreeContent()
		[
			TreeContent
		];
	
	const TSharedRef<SSequencerCurveEditor> RootWidget = SNew(SSequencerCurveEditor, CurveEditorPanel, InArgs.Sequencer);
	return FCurveEditorWidgets(RootWidget, CurveEditorPanel, TreeContent);
}
}

FCurveEditorWidgetOwner::FCurveEditorWidgetOwner(const FCurveEditorWidgetOwnerArgs& InArgs)
	: WeakSequencer(InArgs.Sequencer)
	, CurveEditor(InArgs.CurveEditor)
	, Widgets(PresenterDetail::MakePresenterWidgets(InArgs, GetTabManager()))
{
	BindCommands();
	InitFrameNumberPropertyLayout(InArgs.Sequencer);
	
	const TSharedPtr<FTabManager> TabManager = GetTabManager();
	if (const TSharedPtr<SDockTab> ExistingCurveEditorTab = TabManager->FindExistingLiveTab(FCurveEditorExtension::CurveEditorTabName))
	{
		ExistingCurveEditorTab->SetContent(Widgets.RootWidget);
	}
}

FCurveEditorWidgetOwner::~FCurveEditorWidgetOwner()
{
	// It should not happen defensively prevent crash in case the widget outlives FSequencerCurveEditorPresenter.
	Widgets.CurveEditorPanel->OnFilterClassChanged.Unbind();
}

bool FCurveEditorWidgetOwner::IsCurveEditorOpen() const
{
	const TSharedPtr<FTabManager> TabManager = GetTabManager();
	return TabManager && TabManager->FindExistingLiveTab(FCurveEditorExtension::CurveEditorTabName).IsValid();
}

void FCurveEditorWidgetOwner::OpenCurveEditor()
{
	const TSharedPtr<FTabManager> TabManager = GetTabManager();
	if (!ensure(TabManager))
	{
		return;
	}

	// Do this on first open so we capture Sequencer's current view range, not the stale range from construction
	if (!bHasOpenedCurveEditor)
	{
		if (const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin())
		{
			const FAnimatedRange ViewRange = Sequencer->GetViewRange();
			CurveEditor->GetBounds().SetInputBounds(ViewRange.GetLowerBoundValue(), ViewRange.GetUpperBoundValue());
		}
	}

	// Request the Tab Manager invoke the tab. This will spawn the tab if needed, otherwise pull it to focus. This assumes
	// that the Toolkit Host's Tab Manager has already registered a tab with a NullWidget for content.
	if (const TSharedPtr<SDockTab> CurveEditorTab = TabManager->TryInvokeTab(FCurveEditorExtension::CurveEditorTabName))
	{
		CurveEditorTab->SetContent(Widgets.RootWidget);

		const FSlateIcon SequencerGraphIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCurveEditor.TabIcon");
		CurveEditorTab->SetTabIcon(SequencerGraphIcon.GetIcon());
		CurveEditorTab->SetLabel(LOCTEXT("SequencerMainGraphEditorTitle", "Sequencer Curves"));

		bHasOpenedCurveEditor = true;
	}
}

void FCurveEditorWidgetOwner::CloseCurveEditor()
{
	const TSharedPtr<FTabManager> TabManager = GetTabManager();
	if (!TabManager.IsValid())
	{
		return;
	}

	if (const TSharedPtr<SDockTab> CurveEditorTab = TabManager->FindExistingLiveTab(FCurveEditorExtension::CurveEditorTabName))
	{
		CurveEditorTab->RequestCloseTab();
	}
}

void FCurveEditorWidgetOwner::SyncSelection() const
{
	Widgets.TreeContent->SyncSelection();
}

TSharedPtr<SCurveEditorTree> FCurveEditorWidgetOwner::GetTreeView() const
{
	return Widgets.TreeContent->GetCurveEditorTreeView();
}

USequencerSettings* FCurveEditorWidgetOwner::GetSequencerSettings() const
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	return Sequencer ? Sequencer->GetSequencerSettings() : nullptr;
}

TSharedPtr<FTabManager> FCurveEditorWidgetOwner::GetTabManager() const
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	const TSharedPtr<IToolkitHost> ToolkitHost = Sequencer ? Sequencer->GetToolkitHost() : nullptr;
	return ToolkitHost.IsValid() ? ToolkitHost->GetTabManager() : nullptr;
}

void FCurveEditorWidgetOwner::BindCommands()
{
	const TSharedPtr<FUICommandList> CurveEditorCommandList = CurveEditor->GetCommands();
	if (!ensure(CurveEditorCommandList))
	{
		return;
	}
	
	CurveEditorCommandList->MapAction(
		FSequencerCommands::Get().QuickTreeSearch,
		FExecuteAction::CreateRaw(this, &FCurveEditorWidgetOwner::HandleQuickTreeSearch)
	);
	CurveEditorCommandList->MapAction(
		FSequencerCommands::Get().ToggleShowGotoBox,
		FExecuteAction::CreateRaw(this, &FCurveEditorWidgetOwner::HandleToggleShowGotoBox)
	);
}

void FCurveEditorWidgetOwner::HandleQuickTreeSearch()
{
	Widgets.TreeContent->FocusSearchBox();
}

void FCurveEditorWidgetOwner::HandleToggleShowGotoBox()
{
	Widgets.TreeContent->FocusPlayTimeDisplay();
}

void FCurveEditorWidgetOwner::InitFrameNumberPropertyLayout(const TSharedRef<FSequencer>& InSequencer)
{
	const TSharedRef<SCurveEditorPanel>& CurveEditorPanel = Widgets.CurveEditorPanel;
	
	CurveEditorPanel->GetKeyDetailsView()->GetPropertyRowGenerator()->RegisterInstancedCustomPropertyTypeLayout(
		"FrameNumber", 
		FOnGetPropertyTypeCustomizationInstance::CreateSP(InSequencer, &FSequencer::MakeFrameNumberDetailsCustomization));
	CurveEditorPanel->GetToolPropertiesPanel()->GetStructureDetailsView()->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout(
		"FrameNumber",
		FOnGetPropertyTypeCustomizationInstance::CreateSP(InSequencer, &FSequencer::MakeFrameNumberDetailsCustomization));
	
	Widgets.CurveEditorPanel->OnFilterClassChanged.BindRaw(this, &FCurveEditorWidgetOwner::OnFilterClassChanged);
}

void FCurveEditorWidgetOwner::OnFilterClassChanged()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	const TSharedPtr<SCurveEditorFilterPanel> FilterPanel = Widgets.CurveEditorPanel->GetFilterPanel();
	if (Sequencer && FilterPanel)
	{
		FilterPanel->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout(
			TEXT("FrameNumber"),
			FOnGetPropertyTypeCustomizationInstance::CreateSP(Sequencer.ToSharedRef(), &FSequencer::MakeFrameNumberDetailsCustomization));
	}
}
} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
