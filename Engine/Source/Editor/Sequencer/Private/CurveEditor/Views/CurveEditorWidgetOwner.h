// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorWidgets.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

class FUICommandList;
class FCurveEditor;
class FSequencer;
class FTabManager;
class ITimeSliderController;
class SCurveEditorPanel;
class SCurveEditorTree;
class SSequencerCurveEditor;
class USequencerSettings;
struct FTimeSliderArgs;

namespace UE::Sequencer
{
class FCurveModelSyncer;
class FLinkedFilterViewModel;
struct FCurveEditorWidgetOwnerArgs;

/**
 * Instantiates the UI displaying Sequencer's FCurveEditor.
 * 
 * Manages the UI:
 * - binds UI commands
 * - updates the details customization for the key settings in the toolbar
 * - interacts with the tab manager
 */
class FCurveEditorWidgetOwner
{
public:
	
	explicit FCurveEditorWidgetOwner(const FCurveEditorWidgetOwnerArgs& InArgs);
	~FCurveEditorWidgetOwner();
	
	/** Returns whether the curve editor is open */
	bool IsCurveEditorOpen() const;
	
	/** Opens the curve editor */
	void OpenCurveEditor();
	/** Closes the curve editor */
	void CloseCurveEditor();
	
	/** Requests that the UI syncs its selection to that of Sequencer. */
	void SyncSelection() const;
	
	/**
	 * @return The currently active tree view widget.
	 * TODO UE-363527: Maybe this can be factored out. It breaks encapsulation. For that, FCurveEditorExtension::GetCurveEditorTreeView must be deprecated.
	 */
	TSharedPtr<SCurveEditorTree> GetTreeView() const;
	
private:
	
	/** Sequencer instance that owns us. */
	const TWeakPtr<FSequencer> WeakSequencer;
	/** The curve editor being displayed. */
	const TSharedRef<FCurveEditor> CurveEditor;
	
	/** Widgets this presenter manages. */
	const FCurveEditorWidgets Widgets;

	/** Tracks whether this curve editor instance has been shown at least once.
	 * Used to seed the curve editor with the current view range from Sequencer. */
	bool bHasOpenedCurveEditor = false;

	USequencerSettings* GetSequencerSettings() const;
	TSharedPtr<FTabManager> GetTabManager() const;

	void BindCommands();
	void HandleQuickTreeSearch();
	void HandleToggleShowGotoBox();
	
	/** Register an instanced custom property type layout to handle converting FFrameNumber from Tick Resolution to Display Rate. */
	void InitFrameNumberPropertyLayout(const TSharedRef<FSequencer>& InSequencer);
	/** Updates the frame number layout */
	void OnFilterClassChanged();
};
} // namespace UE::Sequencer
