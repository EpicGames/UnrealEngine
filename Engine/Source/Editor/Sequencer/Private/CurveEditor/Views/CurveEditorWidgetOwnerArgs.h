// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

class FUICommandList;
class ITimeSliderController;
class FCurveEditor;
class FSequencer;

namespace UE::Sequencer
{
class FCurveModelSyncer;
class FLinkedFilterViewModel;

/** Init args for FCurveEditorWidgetOwner */
struct FCurveEditorWidgetOwnerArgs
{
	TSharedRef<FSequencer> Sequencer; 
	
	TSharedRef<FCurveEditor> CurveEditor;
	TSharedRef<ITimeSliderController> TimeSliderController;
	
	TSharedRef<FLinkedFilterViewModel> FilteringViewModel;
	TSharedRef<FUICommandList> CommandList;
	
	FCurveModelSyncer& CurveModelSyncer;

	FCurveEditorWidgetOwnerArgs(
		const TSharedRef<FSequencer>& Sequencer, 
		const TSharedRef<FCurveEditor>& CurveEditor, const TSharedRef<ITimeSliderController>& TimeSliderController, 
		const TSharedRef<FLinkedFilterViewModel>& FilteringViewModel, const TSharedRef<FUICommandList>& CommandList,
		FCurveModelSyncer& CurveModelSyncer UE_LIFETIMEBOUND
		)
		: Sequencer(Sequencer)
		, CurveEditor(CurveEditor)
		, TimeSliderController(TimeSliderController)
		, FilteringViewModel(FilteringViewModel)
		, CommandList(CommandList)
		, CurveModelSyncer(CurveModelSyncer)
	{}
};
} // namespace UE::Sequencer
