// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class SCurveEditorPanel;
class SSequencerCurveEditor;

namespace UE::Sequencer
{
class SCurveTreeContent;
class SOldCurveTreeContent;

/** @see FSequencerCurveEditorPresenter */
struct FCurveEditorWidgets
{
	/** The curve editor widget containing the curve editor panel */
	TSharedRef<SSequencerCurveEditor> RootWidget;
	/** The curve editor panel. This is created and updated even if it is not currently visible. */
	TSharedRef<SCurveEditorPanel> CurveEditorPanel;
	
	/** The tree content for curve editor. It displays linked filtering UI. */
	TSharedRef<SCurveTreeContent> TreeContent;

	explicit FCurveEditorWidgets(
		const TSharedRef<SSequencerCurveEditor>& InRootWidget, 
		const TSharedRef<SCurveEditorPanel>& InCurveEditorPanel,
		const TSharedRef<SCurveTreeContent>& InTreeContent
		)
		: RootWidget(InRootWidget)
		, CurveEditorPanel(InCurveEditorPanel)
		, TreeContent(InTreeContent)
	{}
};
} // namespace UE::Sequencer
