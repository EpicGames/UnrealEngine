// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionSyncUtils.h"

#include "CurveEditor.h"
#include "CurveEditor/Integration/Selection/CurveModelSyncer.h"
#include "MVVM/Extensions/ICurveEditorTreeItemExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModelPtr.h"
#include "Misc/ScopeExit.h"
#include "Templates/SharedPointer.h"
#include "Tree/SCurveEditorTree.h"

namespace UE::Sequencer
{
namespace SyncDetail
{
static void GetAllChildrenRecursively(
	const TSharedRef<SCurveEditorTree>& CurveEditorTreeView, const FCurveEditorTreeItemID Item, int32& LastAddedItem,
	TMap<const FCurveEditorTreeItemID, int32>& ItemToSortOrder
	)
{
	ItemToSortOrder.Add(Item, LastAddedItem++);

	TArray<FCurveEditorTreeItemID> Children;

	CurveEditorTreeView->GetTreeItemChildren(Item, Children);
	// Add them to the output array
	for (const FCurveEditorTreeItemID ChildItemID : Children)
	{
		GetAllChildrenRecursively(CurveEditorTreeView, ChildItemID, LastAddedItem, ItemToSortOrder);
	}
}
} // namespace SyncDetail

void ApplySequencerToCurveEditorSelection(
	const TSharedPtr<FSequencerSelection>& InSelection, 
	const TSharedRef<FCurveEditor>& InCurveEditorModel, 
	const TSharedRef<SCurveEditorTree>& InCurveEditorTreeView
	)
{
	//we now sync to the first non-channel in the tree hierarchy, this is needed for hosted channels with control rigs
	//we need to sort the hierarchy each time since the curve id's being used previously were not consistent in order of viewing/creation, persistent now for undo/redo
	InCurveEditorModel->SuspendBroadcast();
	ON_SCOPE_EXIT{ InCurveEditorModel->ResumeBroadcast(); };
	InCurveEditorTreeView->ClearSelection();

	//map of tree items id's to their actually sort order, need to use this since tree items ids are not in order anymore
	TMap<const FCurveEditorTreeItemID, int32>  ItemToSortOrder;
	int32 LastAddedItem = 0;
	TArray<FCurveEditorTreeItemID> Children;
	TArrayView<const FCurveEditorTreeItemID> RootItems = InCurveEditorTreeView->GetRootItems();
	
	for (const FCurveEditorTreeItemID RootItemID : RootItems)
	{
		SyncDetail::GetAllChildrenRecursively(InCurveEditorTreeView, RootItemID, LastAddedItem, ItemToSortOrder);
	}

	auto GetSortOrder = [&ItemToSortOrder](FCurveEditorTreeItemID InTreeItemID) -> int32
	{
		if (const int32* SortOrder = ItemToSortOrder.Find(InTreeItemID))
		{
			return *SortOrder;
		}
		return INDEX_NONE;
	};

	FCurveEditorTreeItemID FirstCurveEditorTreeItemID;
	int32 FirstSortOrder = INDEX_NONE;
	bool bSelectedIsChannel = true;

	for (TViewModelPtr<IOutlinerExtension> SelectedItem : InSelection->Outliner)
	{
		if (TViewModelPtr<ICurveEditorTreeItemExtension> CurveEditorItem = SelectedItem.ImplicitCast())
		{
			FCurveEditorTreeItemID CurveEditorTreeItem = CurveEditorItem->GetCurveEditorItemID();
			if (CurveEditorTreeItem != FCurveEditorTreeItemID::Invalid())
			{
				if (!InCurveEditorTreeView->IsItemSelected(CurveEditorTreeItem))
				{
					InCurveEditorTreeView->SetItemSelection(CurveEditorTreeItem, true);
					bool bIsChannel = SelectedItem.AsModel()->IsA<FChannelModel>() || SelectedItem.AsModel()->IsA<FChannelGroupOutlinerModel>()
					|| SelectedItem.AsModel()->IsA<FChannelGroupModel>();
					if (!FirstCurveEditorTreeItemID.IsValid())
					{
						FirstCurveEditorTreeItemID = CurveEditorTreeItem;
						FirstSortOrder = GetSortOrder(CurveEditorTreeItem);
						bSelectedIsChannel = bIsChannel;
					}
					else
					{
						int32 SortOrder = GetSortOrder(CurveEditorTreeItem);
						if (bIsChannel == false)
						{
							if (bSelectedIsChannel == true)
							{
								FirstCurveEditorTreeItemID = CurveEditorTreeItem;
								FirstSortOrder = SortOrder;
								bSelectedIsChannel = false;
							}
							else if (SortOrder < FirstSortOrder)
							{
								FirstCurveEditorTreeItemID = CurveEditorTreeItem;
								FirstSortOrder = SortOrder;
								bSelectedIsChannel = false;
							}
						}
						else if (bSelectedIsChannel == true) //its a channel only sort if selected is a non channel
						{
							if (SortOrder < FirstSortOrder)
							{
								FirstCurveEditorTreeItemID = CurveEditorTreeItem;
								FirstSortOrder = SortOrder;
							}
						}
					}
				}
			}
		}
	}
	
	if (FirstCurveEditorTreeItemID.IsValid())
	{
		InCurveEditorTreeView->RequestScrollIntoView(FirstCurveEditorTreeItemID);
	}
}

namespace SyncDetail
{
/** Expands all parents such that InItem is visible. Does not expand InItem itself. */
static void SetParentsExpanded(const FCurveEditorTree& InTree, const FCurveEditorTreeItemID& InItem, SCurveEditorTree& InTreeView, bool bInExpanded)
{
	const auto GetParent = [&InTree](const FCurveEditorTreeItem* Item){ return InTree.FindItem(Item->GetParentID()); };
	
	const FCurveEditorTreeItem* TreeItem = InTree.FindItem(InItem);
	const FCurveEditorTreeItem* StartItem = TreeItem ? GetParent(TreeItem) : nullptr;
	if (!StartItem)
	{
		return;
	}
	
	for (const FCurveEditorTreeItem* Current = StartItem; Current; Current = GetParent(Current))
	{
		InTreeView.SetItemExpansion(Current->GetID(), bInExpanded);
	}
}
}

void ExpandSelectedCurveEditorItems(FCurveEditorTree& InTree, SCurveEditorTree& InTreeView)
{
	for (const TPair<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& Pair : InTree.GetSelection())
	{
		if (Pair.Value == ECurveEditorTreeSelectionState::Explicit)
		{
			SyncDetail::SetParentsExpanded(InTree, Pair.Key, InTreeView, true);
		}
	}
}

namespace SyncDetail
{
static void SetParentsExpanded(const TViewModelPtr<IOutlinerExtension>& InNode, const bool bInExpanded)
{
	for (const TViewModelPtr<IOutlinerExtension>& ParentNode : InNode.AsModel()->GetAncestorsOfType<IOutlinerExtension>())
	{
		if (ParentNode->IsExpanded() != bInExpanded)
		{
			ParentNode->SetExpansion(bInExpanded);
		}
	}
}
}

void ApplyCurveEditorToSequencer(FSequencerSelection& InSelection, const FCurveEditorTree& InTree, const FCurveModelSyncer& InCurveModelSyncer)
{
	FOutlinerSelection NewOutlinerSelection;
	const TMap<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& CurveSelection = InTree.GetSelection();
	for (const TPair<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& Pair : CurveSelection)
	{
		if (Pair.Value != ECurveEditorTreeSelectionState::Explicit)
		{
			continue;
		}
		
		const TSharedPtr<FViewModel> ViewModel = InCurveModelSyncer.FindViewModelById(Pair.Key).Pin();
		const TViewModelPtr<IOutlinerExtension> OutlinerNode = ViewModel ? CastViewModel<IOutlinerExtension>(ViewModel.ToSharedRef()) : nullptr;
		if (OutlinerNode)
		{
			NewOutlinerSelection.Select(OutlinerNode);
		}
	}
	
	InSelection.Outliner.ReplaceWith(MoveTemp(NewOutlinerSelection));
}

void ExpandSelectedSequencerItems(const FSequencerSelection& InSelection)
{
	for (const TWeakViewModelPtr<IOutlinerExtension>& WeakOutlinerNode : InSelection.Outliner.GetSelected())
	{
		if (const TViewModelPtr<IOutlinerExtension> OutlinerNode = WeakOutlinerNode.Pin())
		{
			SyncDetail::SetParentsExpanded(OutlinerNode, true);
		}
	}
}
} // namespace UE::Sequencer
