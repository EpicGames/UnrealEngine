// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class FCurveEditorTree;
class FCurveEditor;
class SCurveEditorTree;

namespace UE::Sequencer
{
class FCurveModelSyncer;
class FSequencerSelection;

/** Applies selection in Sequencer to Curve Editor. */
void ApplySequencerToCurveEditorSelection(
	const TSharedPtr<FSequencerSelection>& InSelection, 
	const TSharedRef<FCurveEditor>& InCurveEditorModel, 
	const TSharedRef<SCurveEditorTree>& InCurveEditorTreeView
	);

/** Expands the tree hierarchy to show all items selected in Curve Editor. */
void ExpandSelectedCurveEditorItems(FCurveEditorTree& InTree, SCurveEditorTree& InTreeView);

/** Applies selection in Curve Editor to Sequencer.*/
void ApplyCurveEditorToSequencer(FSequencerSelection& InSelection, const FCurveEditorTree& InTree, const FCurveModelSyncer& InCurveModelSyncer);

/** Expands the tree hierarchy to show all items selected in Sequencer. */
void ExpandSelectedSequencerItems(const FSequencerSelection& InSelection);
} // namespace UE::Sequencer
