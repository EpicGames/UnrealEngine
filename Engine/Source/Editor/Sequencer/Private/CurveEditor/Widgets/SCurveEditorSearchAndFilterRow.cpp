// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCurveEditorSearchAndFilterRow.h"

#include "Filters/Menus/SequencerTrackFilterMenu.h"
#include "Filters/Menus/SequencerViewOptionsMenu.h"
#include "Filters/SequencerFilterBar.h"
#include "Filters/Utils/HideIsolateViewModelUtils.h"
#include "Filters/Utils/LinkedFilteringViewUtils.h"
#include "Filters/ViewModels/HideIsolateShowViewModel.h"
#include "Filters/Linking/Mode/ILinkedFilterViewModel.h"
#include "Filters/Linking/Mode/LinkedFilterViewModel.h"
#include "Filters/Widgets/Linking/SFilterLinkStateSwitcher.h"
#include "Filters/Widgets/Linking/SLinkedFilteringToggleButton.h"
#include "Filters/Widgets/SSearchAndFilterWidget.h"
#include "Sequencer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Sequencer/SSequencerViewOptionsButton.h"

namespace UE::Sequencer
{
namespace CurveEditorRowDetail
{
using FOutlinerExtensionsAttr = TAttribute<TSet<TWeakViewModelPtr<IOutlinerExtension>>>; 

static FOnGetContent MakeDelegate_AddFilterMenuContent(const TSharedRef<FSequencerFilterBar>& InFilterBar)
{
	return FOnGetContent::CreateLambda([Menu = MakeShared<FSequencerTrackFilterMenu>(), WeakFilterBar = InFilterBar.ToWeakPtr()]
	{
		const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
		return ensure(FilterBar) ? Menu->CreateMenu(FilterBar.ToSharedRef()) : SNullWidget::NullWidget;
	});
}
}

void SCurveEditorSearchAndFilterRow::Construct(const FArguments& InArgs)
{
	WeakSequencer = InArgs._Sequencer;
	FilterModel = InArgs._FilterViewModel;
	check(InArgs._CommandList);
	
	ChildSlot
	[
		SNew(SHorizontalBox)
		
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		.Padding(2.f, 0.f, 0.f, 0.f)
		[
			MakeFilterContent(InArgs)
		]
		
		// View Options Combo Button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.f, 0.f, 0.f, 0.f)
		[
			SNew(SSequencerViewOptionsButton)
			.OnGetMenuContent_Lambda([this, ViewOptions = MakeShared<FSequencerViewOptionsMenu>(), CommandList = InArgs._CommandList]
			{
				const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
				return Sequencer 
					? ViewOptions->CreateMenuForCurveEditor(WeakSequencer, CommandList.ToSharedRef()) 
					: SNullWidget::NullWidget;
			})
		]
	];
}

TSharedPtr<SSequencerSearchBox> SCurveEditorSearchAndFilterRow::GetActiveSearchBox() const
{
	static_assert(static_cast<int32>(ELinkedFilterMode::Count) == 2, "Update this switch");
	switch (FilterModel->GetFilterMode())
	{
	case ELinkedFilterMode::Linked: return LinkedFilterWidgets->GetSearchBox();
	case ELinkedFilterMode::Instanced: return InstancedFilterWidgets->GetSearchBox();
	default: checkNoEntry(); return nullptr;
	}
}

TSharedRef<SWidget> SCurveEditorSearchAndFilterRow::MakeFilterContent(const FArguments& InArgs)
{
	const auto MakeWidget = [this, &InArgs](const TSharedRef<FSequencerFilterBar>& InFilterBar)
	{
		const TSharedRef<FSequencer> SequencerPin = WeakSequencer.Pin().ToSharedRef();
		const TSharedRef<FHideIsolateShowViewModel> IsolateViewModel = MakeShared<FHideIsolateShowViewModel>(
			InFilterBar, InFilterBar->GetHideIsolateFilter(),
			InArgs._SelectedItems, InArgs._AllItems
			);
		IsolateViewModel->OnRequestScrollToView().Add(InArgs._ScrollItemIntoView);
			
		const TSharedPtr<ISequencerTrackFilters> Filter = StaticCastSharedPtr<ISequencerTrackFilters>(InFilterBar.ToSharedPtr());
		return SNew(SSearchAndFilterWidget, InFilterBar, IsolateViewModel)
			.OnSearchTextChanged(this, &SCurveEditorSearchAndFilterRow::OnSearchTextChanged, Filter)
			.OnMakeAddFilterMenuContent(CurveEditorRowDetail::MakeDelegate_AddFilterMenuContent(InFilterBar))
			.BeforeSearchBox()
			[
				SNew(SLinkedFilteringToggleButton, InArgs._FilterViewModel.ToSharedRef())
			];
	};
	
	const TSharedPtr<FLinkedFilterViewModel> LinkedFilterViewModel = InArgs._FilterViewModel;
	LinkedFilterWidgets = MakeWidget(LinkedFilterViewModel->GetLinkedFilterBarImpl());
	InstancedFilterWidgets = MakeWidget(LinkedFilterViewModel->GetInstancedFilterBarImpl());
	return SNew(SFilterLinkStateSwitcher)
		.LinkedFilterViewModel(InArgs._FilterViewModel)
		.LinkedContent() 
		[
			LinkedFilterWidgets.ToSharedRef()
		]
		.InstancedContent() 
		[
			InstancedFilterWidgets.ToSharedRef()
		];
}

void SCurveEditorSearchAndFilterRow::OnSearchTextChanged(const FText& InNewText, TSharedPtr<ISequencerTrackFilters> InFilter)
{
	InFilter->SetTextFilterString(InNewText.ToString());
}
} // namespace UE::Sequencer
