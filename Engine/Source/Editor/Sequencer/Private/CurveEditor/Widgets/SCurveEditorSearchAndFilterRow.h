// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Extensions/IOutlinerExtension.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;
class SSequencerSearchBox;
class ISequencerTrackFilters;
class FSequencer;

namespace UE::Sequencer
{
class FLinkedFilterViewModel;
class ILinkedFilterViewModel;
class SSearchAndFilterWidget;

DECLARE_DELEGATE_OneParam(FScrollIntoView, const TWeakViewModelPtr<IOutlinerExtension>&);

/** The row of widgets displayed above the Curve Editor tree view. */
class SCurveEditorSearchAndFilterRow : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SCurveEditorSearchAndFilterRow) {}
		/** Required. Used to generate the subwidgets */
		SLATE_ARGUMENT(TWeakPtr<FSequencer>, Sequencer)
		/** Required. Determines the filtering widgets displayed. */
		SLATE_ARGUMENT(TSharedPtr<FLinkedFilterViewModel>, FilterViewModel)
		/** Required. The command list that should be used, e.g. by the view options context menu. */
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)

		/** Gets all items displayed in the tree view. */
		SLATE_ATTRIBUTE(TSet<TWeakViewModelPtr<IOutlinerExtension>>, AllItems)
		/** Gets the items selected in the tree view. */
		SLATE_ATTRIBUTE(TSet<TWeakViewModelPtr<IOutlinerExtension>>, SelectedItems)
		/** Requests that the given item is scrolled into view. */
		SLATE_EVENT(FScrollIntoView, ScrollItemIntoView)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);
	
	/** @return The search box that is currently being displayed. */
	TSharedPtr<SSequencerSearchBox> GetActiveSearchBox() const;

private:
	
	/** Used to update Sequencer's filters. */
	TWeakPtr<FSequencer> WeakSequencer;
	
	/** Filter model for the Curve Editor. */
	TSharedPtr<ILinkedFilterViewModel> FilterModel;
	
	/** Content displayed when ELinkedFilterMode::Linked */
	TSharedPtr<SSearchAndFilterWidget> LinkedFilterWidgets;
	/** Content displayed when ELinkedFilterMode::Instanced */
	TSharedPtr<SSearchAndFilterWidget> InstancedFilterWidgets;

	/** @return The filtering widget. */
	TSharedRef<SWidget> MakeFilterContent(const FArguments& InArgs);
	
	void OnSearchTextChanged(const FText& InNewText, TSharedPtr<ISequencerTrackFilters> InFilter);
};
} // namespace UE::Sequencer


