// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCurveTreeContent.h"

#include "Components/VerticalBox.h"
#include "CurveEditor.h"
#include "CurveEditor/Integration/Selection/CurveModelSyncer.h"
#include "CurveEditor/Views/SelectionSyncUtils.h"
#include "CurveEditor/Widgets/SCurveEditorSearchAndFilterRow.h"
#include "Filters/Widgets/SSequencerSearchBox.h"
#include "Misc/TransportControlsHelper.h"
#include "SPopoutTabInlineContent.h"
#include "STemporarilyFocusedSpinBox.h"
#include "Sequencer.h"
#include "Styling/AppStyle.h"
#include "Tree/SCurveEditorTree.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBorder.h"

namespace UE::Sequencer
{
void SCurveTreeContent::Construct(const FArguments& InArgs)
{
	const bool bHasCurveModelSyncer = InArgs._CurveModelSyncer.IsBound() || InArgs._CurveModelSyncer.IsSet();
	check(InArgs._Sequencer && InArgs._CurveEditor && InArgs._FilterViewModel && bHasCurveModelSyncer);
	
	WeakSequencer = InArgs._Sequencer;
	CurveEditor = InArgs._CurveEditor;
	CurveModelSyncerAttr = InArgs._CurveModelSyncer;
	
	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			MakeFilterArea(InArgs)
		]

		+SVerticalBox::Slot()
		[
			MakeTreeContent(InArgs)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			MakeFooter(InArgs)
		]
	];
}

void SCurveTreeContent::FocusSearchBox()
{
	FSlateApplication::Get().SetKeyboardFocus(FilterRow->GetActiveSearchBox(), EFocusCause::SetDirectly);
}

void SCurveTreeContent::FocusPlayTimeDisplay()
{
	PlayTimeDisplay->Setup(); 
	FSlateApplication::Get().SetKeyboardFocus(PlayTimeDisplay, EFocusCause::SetDirectly);
}

void SCurveTreeContent::SyncSelection() const
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}
	
	ApplySequencerToCurveEditorSelection(
		Sequencer->GetViewModel()->GetSelection(), CurveEditor.ToSharedRef(), CurveEditorTreeView.ToSharedRef()
		);
	
	if (Sequencer->GetSequencerSettings()->GetAutoExpandCurveEditorOnSelection())
	{
		ExpandSelectedCurveEditorItems(*CurveEditor->GetTree(), *CurveEditorTreeView.ToSharedRef());
	}
}

TSharedRef<SWidget> SCurveTreeContent::MakeFilterArea(const FArguments& InArgs)
{
	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(TEXT("ToolPanel.GroupBorder")))
		.Padding(FAppStyle::GetMargin(TEXT("Sequencer.FilterAreaMargin")))
		.Clipping(EWidgetClipping::ClipToBounds)
		[
			SNew(SVerticalBox)
		
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				// The scroll box adds a shadow to the right when the toolbar is narrow and hiding buttons.
				SNew(SScrollBox)
				.Orientation(Orient_Horizontal)
				.ScrollBarAlwaysVisible(false)
				.ScrollBarVisibility(EVisibility::Collapsed)
				
				+ SScrollBox::Slot()
				.HAlign(HAlign_Fill)
				.FillContentSize(1.f)
				[
					SAssignNew(FilterRow, SCurveEditorSearchAndFilterRow)
					.Sequencer(InArgs._Sequencer)
					.FilterViewModel(InArgs._FilterViewModel)
					.CommandList(InArgs._CommandList)
					.ScrollItemIntoView(this, &SCurveTreeContent::ScrollIntoView)
					.AllItems(this, &SCurveTreeContent::GetAllItems)
					.SelectedItems(this, &SCurveTreeContent::GetSelectedItems)
				]
			]
			
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				InArgs._FilterPills.Widget
			]
		];
}

TSharedRef<SWidget> SCurveTreeContent::MakeTreeContent(const FArguments& InArgs)
{
	const TSharedRef<SCurveEditorTree> NewCurveEditorTree = SNew(SCurveEditorTree, InArgs._CurveEditor);
	CurveEditorTreeView = NewCurveEditorTree;
	return SNew(SScrollBorder, NewCurveEditorTree)
		[
			NewCurveEditorTree
		];
}

TSharedRef<SWidget> SCurveTreeContent::MakeFooter(const FArguments& InArgs)
{
	const TSharedRef<FSequencer> Sequencer = InArgs._Sequencer.ToSharedRef();
	
	PlayTimeDisplay = Sequencer->MakePlayTimeDisplay();
	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Clipping(EWidgetClipping::ClipToBounds)
		.Visibility(this, &SCurveTreeContent::GetTransportControlsVisibility)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			[
				MakePopoutTransportControls(Sequencer, InArgs._TabManager)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.ButtonStyle(FAppStyle::Get(), "NoBorder")
				.ContentPadding(FMargin(1, 0))
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Right)
					.Padding(FMargin(3.f, 0.f, 0.f, 0.f))
					[
						SNew(SBorder)
						.BorderImage(nullptr)
						[
							PlayTimeDisplay.ToSharedRef()
						]
					]
				]
			]
		];
}

void SCurveTreeContent::ScrollIntoView(const TWeakViewModelPtr<IOutlinerExtension>& WeakViewModel)
{
	const FCurveModelSyncer* CurveModelSyncer = CurveModelSyncerAttr.Get();
	if (!CurveModelSyncer)
	{
		return;
	}
	
	const FCurveEditorTreeItemID Id = CurveModelSyncer->FindIdByViewModel(WeakViewModel);
	if (Id.IsValid())
	{
		CurveEditorTreeView->RequestScrollIntoView(Id);
	}
}

TSet<TWeakViewModelPtr<IOutlinerExtension>> SCurveTreeContent::GetAllItems() const
{
	const FCurveModelSyncer* CurveModelSyncer = CurveModelSyncerAttr.Get();
	if (!CurveModelSyncer)
	{
		return {};
	}
	
	TSet<TWeakViewModelPtr<IOutlinerExtension>> AllItems;
	for (const TPair<FCurveEditorTreeItemID, FCurveEditorTreeItem>& Item : CurveEditor->GetTree()->GetAllItems())
	{
		const TSharedPtr<FViewModel> ViewModel = CurveModelSyncer->FindViewModelById(Item.Key).Pin();
		const TViewModelPtr<IOutlinerExtension> OutlinerNode = ViewModel ? CastViewModel<IOutlinerExtension>(ViewModel.ToSharedRef()) : nullptr;
		if (OutlinerNode)
		{
			AllItems.Add(OutlinerNode);
		}
	}
	return AllItems;
}

TSet<TWeakViewModelPtr<IOutlinerExtension>> SCurveTreeContent::GetSelectedItems() const
{
	const FCurveModelSyncer* CurveModelSyncer = CurveModelSyncerAttr.Get();
	if (!CurveModelSyncer)
	{
		return {};
	}
	
	TSet<TWeakViewModelPtr<IOutlinerExtension>> Selected;
	for (const FCurveEditorTreeItemID& Item : CurveEditorTreeView->GetSelectedItems())
	{
		const TSharedPtr<FViewModel> ViewModel = CurveModelSyncer->FindViewModelById(Item).Pin();
		const TViewModelPtr<IOutlinerExtension> OutlinerNode = ViewModel ? CastViewModel<IOutlinerExtension>(ViewModel.ToSharedRef()) : nullptr;
		if (OutlinerNode)
		{
			Selected.Add(OutlinerNode);
		}
	}
	return Selected;
}

EVisibility SCurveTreeContent::GetTransportControlsVisibility() const
{
	if (const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin())
	{
		const bool bShowTransportControls = GetDefault<UCurveEditorSettings>()->GetShowTransportControls();
		return IsPopoutTransportControlsVisible(Sequencer.ToSharedRef(), bShowTransportControls)
			? EVisibility::Visible : EVisibility::Collapsed;
	}
	return EVisibility::Visible;
}

} // namespace UE::Sequencer
