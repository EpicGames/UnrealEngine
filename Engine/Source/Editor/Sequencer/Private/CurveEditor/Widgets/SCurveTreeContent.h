// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Extensions/IOutlinerExtension.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FCurveEditor;
class FSequencer;
class FTabManager;
class FUICommandList;
class SCurveEditorTree;
template<typename T> struct STemporarilyFocusedSpinBox;

namespace UE::Sequencer
{
class FCurveModelSyncer;
class FLinkedFilterViewModel;
class SCurveEditorSearchAndFilterRow;

/** Displays Curve Editor in Sequencer with linked filtering. */
class SCurveTreeContent : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SCurveTreeContent) {}
		/** Required. The Sequencer instance that owns this content */
		SLATE_ARGUMENT(TSharedPtr<FSequencer>, Sequencer)
		/** Required. The curve editor instance that is being displayed */
		SLATE_ARGUMENT(TSharedPtr<FCurveEditor>, CurveEditor)
		/** Required. Determines the filtering widgets displayed. */
		SLATE_ARGUMENT(TSharedPtr<FLinkedFilterViewModel>, FilterViewModel)
		/** Required. The command list that should be used, e.g. by the view options context menu. */
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
		
		/** Explicit tab manager to use, otherwise uses the global tab manager */
		SLATE_ARGUMENT(TSharedPtr<FTabManager>, TabManager)

		/** Used to translate between IOutlinerExtension and FCurveEditorTreeItemID. */
		SLATE_ATTRIBUTE(const FCurveModelSyncer*, CurveModelSyncer)

		/** Shows the filter pills. */
		SLATE_NAMED_SLOT(FArguments, FilterPills)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);
	
	/** Sets keyboard focus to the search box. */
	void FocusSearchBox();
	/** Sets keyboard focus to the play time display. */
	void FocusPlayTimeDisplay();
	
	/** Syncs the tree view's selection to that of Sequencer. */
	void SyncSelection() const;
	
	const TSharedPtr<SCurveEditorTree>& GetCurveEditorTreeView() const { return CurveEditorTreeView; }
	
private:
	
	/** The owning Sequencer instance that the Curve Editor is being displayed for. */
	TWeakPtr<FSequencer> WeakSequencer;
	/** The curve editor being displayed. */
	TSharedPtr<FCurveEditor> CurveEditor;
	
	/** Used to translate between IOutlinerExtension and FCurveEditorTreeItemID. */
	TAttribute<const FCurveModelSyncer*> CurveModelSyncerAttr;
	
	/** The filter row content displayed above the tree. */
	TSharedPtr<SCurveEditorSearchAndFilterRow> FilterRow;
	/** Curve editor tree widget */
	TSharedPtr<SCurveEditorTree> CurveEditorTreeView;
	/** The current playback time display. */
	TSharedPtr<STemporarilyFocusedSpinBox<double>> PlayTimeDisplay;

	TSharedRef<SWidget> MakeFilterArea(const FArguments& InArgs);
	TSharedRef<SWidget> MakeTreeContent(const FArguments& InArgs);
	TSharedRef<SWidget> MakeFooter(const FArguments& InArgs);
	
	/** Scrolls the given item into view. */
	void ScrollIntoView(const TWeakViewModelPtr<IOutlinerExtension>& WeakViewModel);
	/** @return All items visible in the tree view. */
	TSet<TWeakViewModelPtr<IOutlinerExtension>> GetAllItems() const;
	/** @return All items selected in the tree view. */
	TSet<TWeakViewModelPtr<IOutlinerExtension>> GetSelectedItems() const;

	EVisibility GetTransportControlsVisibility() const;
};
} // namespace UE::Sequencer
