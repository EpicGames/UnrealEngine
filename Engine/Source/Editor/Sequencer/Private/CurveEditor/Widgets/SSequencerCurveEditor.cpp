// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerCurveEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "SCurveEditorPanel.h"
#include "Sequencer.h"
#include "Misc/ResizeParamUtils.h"

#define LOCTEXT_NAMESPACE "SSequencerCurveEditor"

void SSequencerCurveEditor::Construct(const FArguments& InArgs, TSharedRef<SCurveEditorPanel> InEditorPanel, TSharedPtr<FSequencer> InSequencer)
{
	WeakSequencer = InSequencer;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			MakeToolbar(InEditorPanel)
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			InEditorPanel
		]
	];
}

TSharedRef<SWidget> SSequencerCurveEditor::MakeToolbar(TSharedRef<SCurveEditorPanel> InEditorPanel)
{
	FSlimHorizontalToolBarBuilder ToolBarBuilder(InEditorPanel->GetCommands(), FMultiBoxCustomization::None, InEditorPanel->GetToolbarExtender(), true);
	ToolBarBuilder.BeginSection("Asset");

	{
		TAttribute<FSlateIcon> SaveIcon;
		SaveIcon.Bind(TAttribute<FSlateIcon>::FGetter::CreateLambda([&] {

			TArray<UMovieScene*> MovieScenesToSave;
			MovieSceneHelpers::GetDescendantMovieScenes(WeakSequencer.Pin()->GetRootMovieSceneSequence(), MovieScenesToSave);
			for (UMovieScene* MovieSceneToSave : MovieScenesToSave)
			{
				UPackage* MovieScenePackageToSave = MovieSceneToSave->GetOuter()->GetOutermost();
				if (MovieScenePackageToSave->IsDirty())
				{
					return FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SaveChanged");
				}
			}

			return FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save");
			}));

		if (WeakSequencer.Pin()->GetHostCapabilities().bSupportsSaveMovieSceneAsset)
		{
			ToolBarBuilder.AddToolBarButton(
				FUIAction(FExecuteAction::CreateLambda([&] { WeakSequencer.Pin()->SaveCurrentMovieScene(); })),
				NAME_None,
				LOCTEXT("SaveDirtyPackages", "Save"),
				LOCTEXT("SaveDirtyPackagesTooltip", "Saves the current sequence and any subsequences"),
				SaveIcon,
				EUserInterfaceActionType::Button, NAME_None, {}, {}, UE::CurveEditor::MakeResizeParams(TEXT("Sequencer.Save"))
			);
		}		
	}
	
	ToolBarBuilder.EndSection();
	// We just use all of the extenders as our toolbar, we don't have a need to create a separate toolbar.
	return ToolBarBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
