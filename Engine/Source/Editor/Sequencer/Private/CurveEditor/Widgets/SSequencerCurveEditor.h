// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FSequencer;
class SCurveEditorPanel;

class SSequencerCurveEditor : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SSequencerCurveEditor)
	{}
	SLATE_END_ARGS()
public:
	void Construct(const FArguments& InArgs, TSharedRef<SCurveEditorPanel> InEditorPanel, TSharedPtr<FSequencer> InSequencer);

	TSharedRef<SWidget> MakeToolbar(TSharedRef<SCurveEditorPanel> InEditorPanel);

private:
	TWeakPtr<FSequencer> WeakSequencer;
};

