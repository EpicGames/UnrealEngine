// Copyright Epic Games, Inc. All Rights Reserved.


#include "SequencerCurveEditorTimeSliderController.h"

#include "CurveEditor.h"
#include "Sequencer.h"
#include "SequencerSettings.h"

FSequencerCurveEditorTimeSliderController::FSequencerCurveEditorTimeSliderController(const FTimeSliderArgs& InArgs, TWeakPtr<FSequencer> InWeakSequencer, TSharedRef<FCurveEditor> InCurveEditor)
	: FSequencerTimeSliderController(InArgs, InWeakSequencer)
{
	WeakSequencer = InWeakSequencer;
	WeakCurveEditor = InCurveEditor;
}


void FSequencerCurveEditorTimeSliderController::ClampViewRange(double& NewRangeMin, double& NewRangeMax)
{
	// Since the CurveEditor uses a different view range (potentially) we have to be careful about which one we clamp.
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const bool bLinkedTimeRange = Sequencer->GetSequencerSettings()->GetLinkCurveEditorTimeRange();
	if (bLinkedTimeRange)
	{
		return FSequencerTimeSliderController::ClampViewRange(NewRangeMin, NewRangeMax);
	}
	else
	{
		TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
		if (CurveEditor.IsValid())
		{
			double InputMin, InputMax;
			CurveEditor->GetBounds().GetInputBounds(InputMin, InputMax);

			bool bNeedsClampSet = false;
			double NewClampRangeMin = InputMin;
			if (NewRangeMin < InputMin)
			{
				NewClampRangeMin = NewRangeMin;
				bNeedsClampSet = true;
			}

			double NewClampRangeMax = InputMax;
			if (NewRangeMax > InputMax)
			{
				NewClampRangeMax = NewRangeMax;
				bNeedsClampSet = true;
			}

			if (bNeedsClampSet)
			{
				CurveEditor->GetBounds().SetInputBounds(NewClampRangeMin, NewClampRangeMax);
			}

		}
	}
}

void FSequencerCurveEditorTimeSliderController::SetViewRange(double NewRangeMin, double NewRangeMax, EViewRangeInterpolation Interpolation)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const bool bLinkedTimeRange = Sequencer->GetSequencerSettings()->GetLinkCurveEditorTimeRange();
	if (bLinkedTimeRange)
	{
		return FSequencerTimeSliderController::SetViewRange(NewRangeMin, NewRangeMax, Interpolation);
	}
	else
	{
		// Clamp to a minimum size to avoid zero-sized or negative visible ranges
		double MinVisibleTimeRange = FFrameNumber(1) / GetTickResolution();
		TRange<double> ExistingViewRange = GetViewRange();

		if (NewRangeMax == ExistingViewRange.GetUpperBoundValue())
		{
			if (NewRangeMin > NewRangeMax - MinVisibleTimeRange)
			{
				NewRangeMin = NewRangeMax - MinVisibleTimeRange;
			}
		}
		else if (NewRangeMax < NewRangeMin + MinVisibleTimeRange)
		{
			NewRangeMax = NewRangeMin + MinVisibleTimeRange;
		}

		TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
		if (CurveEditor.IsValid())
		{
			CurveEditor->GetBounds().SetInputBounds(NewRangeMin, NewRangeMax);
		}
	}
}

FAnimatedRange FSequencerCurveEditorTimeSliderController::GetViewRange() const
{ 
	// If they've linked the Sequencer timerange we can return the internal controller's view range, otherwise we return the bounds (which internally does the same check)
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return FAnimatedRange();
	}
	const bool bLinkedTimeRange = Sequencer->GetSequencerSettings()->GetLinkCurveEditorTimeRange();
	if (bLinkedTimeRange)
	{
		return FSequencerTimeSliderController::GetViewRange();
	}
	else
	{
		TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin();
		if (CurveEditor.IsValid())
		{
			double InputMin, InputMax;
			CurveEditor->GetBounds().GetInputBounds(InputMin, InputMax);

			return FAnimatedRange(InputMin, InputMax);
		}
	}

	return FAnimatedRange();
}
