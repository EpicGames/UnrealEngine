// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SequencerTimeSliderController.h"

class ISequencer;
class FSequencer;

class FSequencerCurveEditorTimeSliderController : public FSequencerTimeSliderController
{
public:

	FSequencerCurveEditorTimeSliderController(const FTimeSliderArgs& InArgs, TWeakPtr<FSequencer> InWeakSequencer, TSharedRef<FCurveEditor> InCurveEditor);

	virtual void ClampViewRange(double& NewRangeMin, double& NewRangeMax) override;
	virtual void SetViewRange(double NewRangeMin, double NewRangeMax, EViewRangeInterpolation Interpolation) override;
	virtual FAnimatedRange GetViewRange() const override;

private:
	TWeakPtr<ISequencer> WeakSequencer;
	TWeakPtr<FCurveEditor> WeakCurveEditor;
};

