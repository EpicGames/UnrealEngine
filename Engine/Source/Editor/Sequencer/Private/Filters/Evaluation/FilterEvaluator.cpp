// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilterEvaluator.h"

#include "Filters/SequencerFilterBar.h"

namespace UE::Sequencer
{
bool FFilterEvaluator::UpdateFiltersOnTrackValueChanged()
{
	if (bHasRequestedFilterUpdate)
	{
		return false;
	}
	
	bool bAnyFilterNeedsRerun = false;
	for (FFilterBarData& FilterBarData : RegisteredFilterBars)
	{
		const bool bNeedsRerun = FilterBarData.FilterBar->ShouldUpdateOnTrackValueChanged();
		FilterBarData.bHasRequestedRefilter |= bNeedsRerun;
		bAnyFilterNeedsRerun |= bNeedsRerun;
	}
	
	if (bAnyFilterNeedsRerun)
	{
		constexpr bool bForce = true;
		const bool bFiltersUpdated = UpdateFilters(bForce);

		// If the filter list was modified, set bHasRequestedFilterUpdate to suppress excessive re-filters between tree update
		bHasRequestedFilterUpdate = bFiltersUpdated;
		return bFiltersUpdated;
	}

	return false;
}

bool FFilterEvaluator::UpdateFilters(bool bForce)
{
	if (!bForce && !bHasRequestedFilterUpdate)
	{
		return false;
	}
	
	TRACE_CPUPROFILER_EVENT_SCOPE(FFilterEvaluator::UpdateFilters);

	bool bHasFilterDataChanged = false; 
	for (FFilterBarData& FilterBarData : RegisteredFilterBars)
	{
		const TSharedRef<FSequencerFilterBar>& FilterBar = FilterBarData.FilterBar;
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(FilterBar->GetIdentifier());
		
		const FSequencerFilterData PreviousFilterData = FilterBar->GetFilterData();
		const FSequencerFilterData& FilterData = FilterBar->FilterNodes();
		const bool bHasFilteredDataChanged = FilterData != PreviousFilterData;
		bHasFilterDataChanged |= bHasFilteredDataChanged;
		FilterBarData.bHasRequestedRefilter = false;
		
		OnFilterBarFilteredDelegate.Broadcast(FilterBar);
	}
	bHasRequestedFilterUpdate = false;

	return bHasFilterDataChanged;
}

void FFilterEvaluator::RequestFilterUpdate(const TSharedRef<FSequencerFilterBar>& InFilterBar)
{
	const int32 Index = IndexOf(InFilterBar);
	if (RegisteredFilterBars.IsValidIndex(Index))
	{
		bHasRequestedFilterUpdate = true;
		RegisteredFilterBars[Index].bHasRequestedRefilter = true;
	}
}

bool FFilterEvaluator::NeedsFilterUpdate() const
{
	return bHasRequestedFilterUpdate;
}

void FFilterEvaluator::IncrementFilterBarUsage(const TSharedRef<FSequencerFilterBar>& InFilterBar)
{
	const int32 Index = IndexOf(InFilterBar);
	if (RegisteredFilterBars.IsValidIndex(Index))
	{
		++RegisteredFilterBars[Index].UsageCount;
	}
	else
	{
		FSimpleDelegate HandleRequestUpdate = FSimpleDelegate::CreateRaw(this, &FFilterEvaluator::OnFilterRequestedRefresh, InFilterBar.ToWeakPtr());
		RegisteredFilterBars.Emplace(InFilterBar, MoveTemp(HandleRequestUpdate));
	}
	
	// We run the filter just in case it's not been evaluated yet. In the future, we could add a dirty flag or serial number to the filter bar.
	bHasRequestedFilterUpdate = true;
}

void FFilterEvaluator::DecrementFilterBarUsage(const TSharedRef<FSequencerFilterBar>& InFilterBar)
{
	const int32 Index = IndexOf(InFilterBar);
	if (!ensure(RegisteredFilterBars.IsValidIndex(Index)))
	{
		return;
	}
	
	FFilterBarData& Data = RegisteredFilterBars[Index];
	check(Data.UsageCount > 0);
	
	if (Data.UsageCount == 1)
	{
		RegisteredFilterBars.RemoveAtSwap(Index);
	}
	else
	{
		--Data.UsageCount;
	}
}

int32 FFilterEvaluator::IndexOf(const TSharedRef<FSequencerFilterBar>& InFilterBar) const
{
	return RegisteredFilterBars.IndexOfByPredicate([&InFilterBar](const FFilterBarData& Data)
	{
		return InFilterBar == Data.FilterBar;
	});
}

void FFilterEvaluator::OnFilterRequestedRefresh(TWeakPtr<FSequencerFilterBar> InWeakFilterBar)
{
	if (const TSharedPtr<FSequencerFilterBar> FilterBar = InWeakFilterBar.Pin(); ensure(FilterBar))
	{
		RequestFilterUpdate(FilterBar.ToSharedRef());
	}
}
} // namespace UE::Sequencer
