// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Filters/SequencerFilterBar.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FSequencerFilterBar;

namespace UE::Sequencer
{
/**
 * Knows of all filter bar instances that need to be run in Sequencer.
 * This is relevant for multiple UI endpoints using the linked filtering system.
 * 
 * Each endpoint can register a filter bar they want evaluated, and request explicit refreshes.
 * The evaluator will be refreshed next tick by Sequencer.
 * @see IncrementFilterBarUsage, DecrementFilterBarUsage
 * 
 * If your filter UI becomes hidden (e.g. tab in background), as optimization, it would be valid to call DecrementFilterBarUsage and follow it up 
 * again with a IncrementFilterBarUsage once the UI becomes active again.
 */
class FFilterEvaluator : public FNoncopyable
{
public:
	
	/** 
	 * Evaluates all filters bars that need updating when a track value changes. 
	 * @return Whether the new list of filtered nodes is different from the previous list
	 */
	bool UpdateFiltersOnTrackValueChanged();
	
	/**
	 * Evaluates all filter bars. 
	 * @return Whether the new list of filtered nodes is different from the previous list
	 */
	bool UpdateFilters(bool bForce = false);
	
	/** Schedules an update of all filters on the given filter bar. The filter bar must be registered. */
	void RequestFilterUpdate(const TSharedRef<FSequencerFilterBar>& InFilterBar);
	/** @return Whether there is a filter update scheduled */
	bool NeedsFilterUpdate() const;

	/** 
	 * Tracks a filter bar that some UI endpoints is displaying filter results for.
	 * 
	 * If this is already tracked, increments the usage counter by 1.
	 * If previously untracked, this creates a subscription ISequencerFilterBar::OnRequestUpdate.
	 * 
	 * @note You should prefer FScopedFilterBarUsage for auto deregistration
	 * @see ScopedFilterBarUsage.
	 * 
	 * Each call must be matched with DecrementFilterBarUsage.
	 * @see DecrementFilterBarUsage
	 */
	void IncrementFilterBarUsage(const TSharedRef<FSequencerFilterBar>& InFilterBar);
	/** Decrements the usage counter by 1. If it reaches 0, the filter bar becomes untracked for filter evaluation. */
	void DecrementFilterBarUsage(const TSharedRef<FSequencerFilterBar>& InFilterBar);
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnFilterBarFiltered, const TSharedRef<FSequencerFilterBar>&)
	/** @return Delegate invoked when a filter bar is refiltered. */
	FOnFilterBarFiltered& OnFilterBarFiltered() { return OnFilterBarFilteredDelegate; }
	
private:
	
	/** Whether any of the filters has a refilter requested. */
	bool bHasRequestedFilterUpdate = false;
	
	struct FFilterBarData : FNoncopyable // Avoid accidental copying so we don't unsubscribe by accident.
	{
		/** The filter bar this data corresponds to. */
		const TSharedRef<FSequencerFilterBar> FilterBar;
		/** Handle to ISequencerFilterBar::OnRequestUpdate. */
		const FDelegateHandle OnRequestUpdateHandle;
		
		/** How many systems have registered to be using this filter bar. Once this reaches 0, the entry is removed. */
		int32 UsageCount = 1;
		/** Whether this filter bar has been marked to be refiltered. */
		bool bHasRequestedRefilter = false;

		explicit FFilterBarData(const TSharedRef<FSequencerFilterBar>& InFilterBar, FSimpleDelegate HandleFilterRequestRefresh)
			: FilterBar(InFilterBar)
			, OnRequestUpdateHandle(FilterBar->OnRequestUpdate().Add(MoveTemp(HandleFilterRequestRefresh)))
		{}
		
		~FFilterBarData()
		{
			FilterBar->OnRequestUpdate().Remove(OnRequestUpdateHandle);
		}
	};
	/** Filters that should be considered for evaluation. */
	TArray<FFilterBarData> RegisteredFilterBars;
	
	/** Invoked when a filter bar is refiltered. */
	FOnFilterBarFiltered OnFilterBarFilteredDelegate;
	
	/** @return Index to RegisteredFilterBars entry that contains InFilterBar. */
	int32 IndexOf(const TSharedRef<FSequencerFilterBar>& InFilterBar) const;
	
	/** Handles ISequencerTrackFilters::OnRequestRefresh by calling RequestFilterUpdate. */
	void OnFilterRequestedRefresh(TWeakPtr<FSequencerFilterBar> InWeakFilterBar);
};
} // namespace UE::Sequencer
