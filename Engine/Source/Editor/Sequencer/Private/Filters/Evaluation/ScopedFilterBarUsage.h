// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FilterEvaluator.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FSequencerFilterBar;

namespace UE::Sequencer
{
class FFilterEvaluator;

/**
 * Registers FSequencerFilterBar with FFilterEvaluator for the lifetime of this scope object.
 * Handles calling FFilterEvaluator::IncrementFilterBarUsage and FFilterEvaluator::DecrementFilterBarUsage.
 */
class FScopedFilterBarUsage : public FNoncopyable
{
public:
	
	explicit FScopedFilterBarUsage(const TSharedRef<FSequencerFilterBar> InFilterBar, const TSharedRef<FFilterEvaluator>& InEvaluator)
		: FilterBar(InFilterBar)
		, Evaluator(InEvaluator)
	{
		Evaluator->IncrementFilterBarUsage(InFilterBar);
	}
	
	FScopedFilterBarUsage(FScopedFilterBarUsage&& InOther)
		: FilterBar(MoveTemp(InOther.FilterBar))
		, Evaluator(MoveTemp(InOther.Evaluator))
	{}

	FScopedFilterBarUsage& operator=(FScopedFilterBarUsage&& Other)
	{
		if (this != &Other)
		{
			CleanUp();
		}
		
		FilterBar = MoveTemp(Other.FilterBar);
		Evaluator = MoveTemp(Other.Evaluator);
		return *this;
	}

	~FScopedFilterBarUsage()
	{
		CleanUp();
	}
	
private:
	
	/** The filter bar that is being used. */
	TSharedPtr<FSequencerFilterBar> FilterBar;
	
	/** The filter evaluator to register and unregister FilterBar with. */
	TSharedPtr<FFilterEvaluator> Evaluator;
	
	/** Calls DecrementFilterBarUsage if this instance state has not been moved. */
	void CleanUp() const
	{
		if (Evaluator && FilterBar)
		{
			Evaluator->DecrementFilterBarUsage(FilterBar.ToSharedRef());
		}
	}
};
} // namespace UE::Sequencer
