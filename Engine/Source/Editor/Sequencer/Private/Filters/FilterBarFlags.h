// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

namespace UE::Sequencer
{
/** Flags that affect filtering */
enum class EFilterFlags : uint8
{
	None = 0,
	
	/** If set, then USequencerSettings::bIncludePinnedInFilter is allowed to effect. If the setting is disabled, pinned items skip filters. */
	PinnedItemsCanSkipFiltering = 1 << 0
};
ENUM_CLASS_FLAGS(EFilterFlags);
} // namespace UE::Sequencer
