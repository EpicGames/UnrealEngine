// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "FilterBarFlags.h"
#include "HAL/Platform.h"
#include "Misc/NotNull.h"
#include "UObject/NameTypes.h"

class FSequencer;

namespace UE::Sequencer
{
/** Init args for FSequencerFilterBar. */
struct FFilterBarInitArgs
{
	/** The Sequencer this filter bar is interacting with */
	TNotNull<FSequencer*> Sequencer;
	
	/**
	 * Used for per filter bar instance config.
	 * This is relevant for the settings to use for reading and saving config.
	 * @see FSequencerFilterBarConfig, USequencerSettings::FindTrackFilterBar, FSequencerFilterBar::GetIdentifier.
	 */
	FString SubIdentifier;
	
	/** Affects how filtering is performed. */
	EFilterFlags Flags = EFilterFlags::None;

	/**
	 * Optional. ID used to look up the filter area config (FSequencerFilterAreaConfig).
	 * When set, allows the filter bar to bind the ToggleFilterBarVisibility command.
	 * @see USequencerSettings::FindOrAddTrackFilterArea
	 */
	FName FilterAreaConfigId = NAME_None;

	explicit FFilterBarInitArgs(FSequencer& InSequencer UE_LIFETIMEBOUND, const FString& InSubIdentifier, EFilterFlags InFlags = EFilterFlags::None)
		: Sequencer(&InSequencer)
		, SubIdentifier(InSubIdentifier)
		, Flags(InFlags)
	{}
};
} // namespace UE::Sequencer
