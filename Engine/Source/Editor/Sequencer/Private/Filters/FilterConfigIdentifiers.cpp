// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilterConfigIdentifiers.h"

namespace UE::Sequencer::ConfigIds
{
const TCHAR* LinkedConfigSubId = TEXT(""); // TODO UE-362151: For now empty so we load using the old config entry, see below.

const TCHAR* Sequencer_InstancedConfigSubId = TEXT("Outliner");
const TCHAR* CurveEditor_InstancedConfigId = TEXT("CurveEditor");

const TCHAR* FilterArea_Outliner = TEXT("Outliner");
const TCHAR* FilterArea_CurveEditor = TEXT("CurveEditor");

} // namespace UE::Sequencer