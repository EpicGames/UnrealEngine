// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

/** 
 * Built-in IDs for sub-config identifiers in FSequencerFilterBar
 * They affect the result of FSequencerFilterBar::GetIdentifier, which is important for the identifier in the filter state config.
 */
namespace UE::Sequencer::ConfigIds
{
/** 
 * Config identifier for the linked filter bar in Sequencer. 
 * @see FSequencerFilterBar::GetIdentifier.
 */
extern const TCHAR* LinkedConfigSubId; // TODO UE-362151: For now empty so we load using the old config entry, see below.

/**
 * Config identifier for the instanced filter bar in Sequencer's outliner. 
 * @see FSequencerFilterBar::GetIdentifier.
 */
extern const TCHAR* Sequencer_InstancedConfigSubId; 
/** 
 * Config identifier for the instanced filter bar in Sequencer's Curve Editor. 
 * @see FSequencerFilterBar::GetIdentifier.
 */
extern const TCHAR* CurveEditor_InstancedConfigId;

/** 
 * ID to use for the Outliner filter area.
 * @see USequencerSettings::FindOrAddTrackFilterArea 
 */
extern const TCHAR* FilterArea_Outliner;
/** 
 * ID to use for the Curve Editor filter area.
 * @see USequencerSettings::FindOrAddTrackFilterArea 
 */
extern const TCHAR* FilterArea_CurveEditor;
} // namespace UE::Sequencer
