// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/SequencerTrackFilterBase.h"
#include "Filters/SequencerFilterBar.h"
#include "ISequencer.h"
#include "MovieSceneSequence.h"
#include "MVVM/ViewModelPtr.h"

using namespace UE::Sequencer;

FSequencerTrackFilter::FSequencerTrackFilter(ISequencerTrackFilters& InOutFilterInterface, TSharedPtr<FFilterCategory>&& InCategory)
	: FSequencerFilterBase<FSequencerTrackFilterType>(InOutFilterInterface, MoveTemp(InCategory))
{
}

bool FSequencerTrackFilter::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return false;
}

ISequencerTrackFilters& FSequencerTrackFilter::GetFilterInterface() const
{
	return static_cast<ISequencerTrackFilters&>(FilterInterface);
}

FFilterResult FSequencerTrackFilter::Evaluate(FSequencerTrackFilterType InItem) const
{
	// This API was added in 5.8. We allow filters to only implement PassesFilter.
	const bool bPassesFilter = PassesFilter(InItem);
	return FFilterResult(bPassesFilter ? EItemFilterState::Include : EItemFilterState::Exclude);
}

UMovieSceneSequence* FSequencerTrackFilter::GetFocusedMovieSceneSequence() const
{
	return GetSequencer().GetFocusedMovieSceneSequence();
}

UMovieScene* FSequencerTrackFilter::GetFocusedGetMovieScene() const
{
	const UMovieSceneSequence* const FocusedMovieSceneSequence = GetFocusedMovieSceneSequence();
	return FocusedMovieSceneSequence ? FocusedMovieSceneSequence->GetMovieScene() : nullptr;
}
