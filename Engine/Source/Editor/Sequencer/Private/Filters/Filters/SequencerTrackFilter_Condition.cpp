// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/SequencerTrackFilter_Condition.h"
#include "Filters/SequencerFilterBar.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "Sequencer.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTrackFilter_Condition"

FSequencerTrackFilter_Condition::FSequencerTrackFilter_Condition(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter(InFilterInterface, MoveTemp(InCategory))
{
}

bool FSequencerTrackFilter_Condition::ShouldUpdateOnTrackValueChanged() const
{
	return true;
}

FText FSequencerTrackFilter_Condition::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_ConditionToolTip", "Show only tracks and sections with conditions");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_Condition::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_Condition;
}

FText FSequencerTrackFilter_Condition::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_Condition", "Condition");
}

FSlateIcon FSequencerTrackFilter_Condition::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.Indicator.Condition"));
}

FString FSequencerTrackFilter_Condition::GetName() const
{
	return StaticName();
}

bool FSequencerTrackFilter_Condition::PassesFilter(FSequencerTrackFilterType InItem) const
{
	for (const TViewModelPtr<IConditionableExtension>& Conditionable : InItem->GetDescendantsOfType<IConditionableExtension>(true))
	{
		if (Conditionable->GetConditionState() != EConditionableConditionState::None)
		{
			return true;
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
