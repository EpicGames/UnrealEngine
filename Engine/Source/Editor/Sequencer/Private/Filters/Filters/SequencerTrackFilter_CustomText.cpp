// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTrackFilter_CustomText.h"
#include "Filters/SequencerFilterBar.h"

using namespace UE::Sequencer;

FSequencerTrackFilter_CustomText::FSequencerTrackFilter_CustomText(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTrackFilter_Text(InFilterInterface)
{
}

void FSequencerTrackFilter_CustomText::SetFromCustomTextFilterData(const FCustomTextFilterData& InFilterData)
{
	Color = InFilterData.FilterColor;
	DisplayName = InFilterData.FilterLabel;
	SetRawFilterText(InFilterData.FilterString);
}

FCustomTextFilterData FSequencerTrackFilter_CustomText::CreateCustomTextFilterData() const
{
	FCustomTextFilterData CustomTextFilterData;
	CustomTextFilterData.FilterColor = Color;
	CustomTextFilterData.FilterLabel = DisplayName;
	CustomTextFilterData.FilterString = GetRawFilterText();

	return CustomTextFilterData;
}

TSharedPtr<FFilterBase<FSequencerTrackFilterType>> FSequencerTrackFilter_CustomText::GetFilter()
{
	return AsShared();
}

bool FSequencerTrackFilter_CustomText::IsCustomTextFilter() const
{
	return true;
}

bool FSequencerTrackFilter_CustomText::ShouldUpdateOnTrackValueChanged() const
{
	return true;
}

FText FSequencerTrackFilter_CustomText::GetDefaultToolTipText() const
{
	return GetDisplayName(); 
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_CustomText::GetToggleCommand() const
{
	return nullptr;
}

FText FSequencerTrackFilter_CustomText::GetDisplayName() const
{
	return DisplayName;
}
FText FSequencerTrackFilter_CustomText::GetToolTipText() const
{
	return GetRawFilterText();
}

FLinearColor FSequencerTrackFilter_CustomText::GetColor() const
{
	return Color;
}

FString FSequencerTrackFilter_CustomText::GetName() const
{
	return FCustomTextFilter<FSequencerTrackFilterType>::GetFilterTypeName().ToString();
}
