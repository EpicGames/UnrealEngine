// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/CustomTextFilters.h"
#include "SequencerTrackFilter_Text.h"

class FSequencerTrackFilter_CustomText
	: public ICustomTextFilter<FSequencerTrackFilterType>
	, public FSequencerTrackFilter_Text
{
public:
	FSequencerTrackFilter_CustomText(ISequencerTrackFilters& InFilterInterface);

	//~ Begin ICustomTextFilter
	virtual void SetFromCustomTextFilterData(const FCustomTextFilterData& InFilterData) override;
	virtual FCustomTextFilterData CreateCustomTextFilterData() const override;
	virtual TSharedPtr<FFilterBase<FSequencerTrackFilterType>> GetFilter() override;
	//~ End ICustomTextFilter

	//~ Begin FSequencerTrackFilter
	virtual bool IsCustomTextFilter() const override;
	virtual bool ShouldUpdateOnTrackValueChanged() const override;
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	//~ End FSequencerTrackFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTipText() const override;
	virtual FLinearColor GetColor() const override;
	//~ End FFilterBase

	//~ Begin FFilterBase
	virtual FString GetName() const override;
	//~ End FFilterBase

protected:
	FText DisplayName = FText::GetEmpty();

	FLinearColor Color = FLinearColor::White;
};
