// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/SequencerTrackFilter_Group.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "MovieScene.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "Sequencer.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTrackFilter_Group"

FSequencerTrackFilter_Group::FSequencerTrackFilter_Group(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter(InFilterInterface, MoveTemp(InCategory))
{
}

FSequencerTrackFilter_Group::~FSequencerTrackFilter_Group()
{
	if (UMovieScene* const MovieScene = MovieSceneWeak.Get())
	{
		MovieScene->GetNodeGroups().OnNodeGroupCollectionChanged().RemoveAll(this);
		MovieSceneWeak.Reset();
	}
}

FText FSequencerTrackFilter_Group::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_GroupToolTip", "Show only Group tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_Group::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_Groups;
}

void FSequencerTrackFilter_Group::OnPreFilter()
{
	if (const UMovieSceneSequence* const FocusedMovieSceneSequence = GetSequencer().GetFocusedMovieSceneSequence())
	{
		UpdateMovieScene(FocusedMovieSceneSequence->GetMovieScene());
	}
}

FText FSequencerTrackFilter_Group::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_Group", "Groups");
}

FSlateIcon FSequencerTrackFilter_Group::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.GroupActors"));
}

FString FSequencerTrackFilter_Group::GetName() const
{
	return StaticName();
}

bool FSequencerTrackFilter_Group::PassesFilter(FSequencerTrackFilterType InItem) const
{
	const UMovieSceneSequence* const FocusedSequence = GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return true;
	}

	UMovieScene* const FocusedMovieScene = FocusedSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return true;
	}

	UMovieSceneNodeGroupCollection& NodeGroups = FocusedMovieScene->GetNodeGroups();
	if (!NodeGroups.HasAnyActiveFilter())
	{
		return true;
	}

	bool bPassed = true;

	ForEachMovieSceneNodeGroup(FocusedMovieScene, InItem,
		[&bPassed](const TViewModelPtr<IOutlinerExtension>& InParent, UMovieSceneNodeGroup* const InNodeGroup)
		{
			const FString GroupPathName = IOutlinerExtension::GetPathName(InParent.AsModel());
			if (InNodeGroup->GetEnableFilter() && !InNodeGroup->ContainsNode(GroupPathName))
			{
				bPassed = false;
				return false;
			}
			return true;
		});

	return bPassed;
}

bool FSequencerTrackFilter_Group::HasActiveGroupFilter() const
{
	return MovieSceneWeak.IsValid() ? MovieSceneWeak->GetNodeGroups().HasAnyActiveFilter() : false;
}

void FSequencerTrackFilter_Group::UpdateMovieScene(UMovieScene* const InMovieScene)
{
	UMovieScene* const OldMovieScene = MovieSceneWeak.Get();

	if (!OldMovieScene || OldMovieScene != InMovieScene)
	{
		if (OldMovieScene)
		{
			OldMovieScene->GetNodeGroups().OnNodeGroupCollectionChanged().RemoveAll(this);
		}

		MovieSceneWeak.Reset();

		if (InMovieScene)
		{
			MovieSceneWeak = InMovieScene;
			InMovieScene->GetNodeGroups().OnNodeGroupCollectionChanged().AddRaw(this, &FSequencerTrackFilter_Group::HandleGroupsChanged);
		}

		HandleGroupsChanged();
	}
}

void FSequencerTrackFilter_Group::HandleGroupsChanged()
{
	if (!MovieSceneWeak.IsValid())
	{
		BroadcastChangedEvent();
		return;
	}

	const UMovieSceneNodeGroupCollection& NodeGroups = MovieSceneWeak->GetNodeGroups();
	if (!NodeGroups.HasAnyActiveFilter())
	{
		BroadcastChangedEvent();
		return;
	}

	BroadcastChangedEvent();
}

void FSequencerTrackFilter_Group::ForEachMovieSceneNodeGroup(UMovieScene* const InMovieScene
	, FSequencerTrackFilterType InItem
	, const TFunctionRef<bool(const TViewModelPtr<IOutlinerExtension>& InParent, UMovieSceneNodeGroup*)>& InFunction)
{
	if (!InMovieScene)
	{
		return;
	}

	const TViewModelPtr<IOutlinerExtension> Parent = InItem.AsModel()->FindAncestorOfType<IOutlinerExtension>(true);
	if (!Parent.IsValid())
	{
		return;
	}

	UMovieSceneNodeGroupCollection& NodeGroups = InMovieScene->GetNodeGroups();

	for (UMovieSceneNodeGroup* const NodeGroup : NodeGroups)
	{
		if (!InFunction(Parent, NodeGroup))
		{
			return;
		}
	}
}

#undef LOCTEXT_NAMESPACE
