// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/SequencerTrackFilter_HideIsolate.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "MovieSceneSection.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/ISectionOwnerExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Extensions/ITrackRowExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTrackFilter_HideIsolate"

namespace UE::Sequencer::HideIsolateShow
{

/**
 * Returns the nearest stable outliner ancestor for a node.
 *
 * The returned node is used as the persistent root for hide/isolate storage.
 * We intentionally stop at object binding, track row, or track because those
 * identities are stable across outliner rebuilds, while children beneath them
 * may be rebuilt or reordered.
 *
 * More specific nodes such as category or channel entries are preserved through
 * the relative suffix stored by BuildPersistentFilterPath().
 *
 * If no stable ancestor is found, the original node is returned so we preserve
 * as much specificity as possible rather than collapsing to an unrelated parent.
 */
static TViewModelPtr<IOutlinerExtension> GetStableRootNode(const TViewModelPtr<IOutlinerExtension>& InNode)
{
	if (!InNode.IsValid())
	{
		return nullptr;
	}

	for (FViewModelPtr Node = InNode.AsModel(); Node; Node = Node->GetParent())
	{
		if (Node->IsA<IObjectBindingExtension>()
			|| Node->IsA<ITrackExtension>()
			|| Node->IsA<ITrackRowExtension>())
		{
			return Node.ImplicitCast();
		}
	}

	return InNode;
}

/**
 * Returns a section-specific disambiguator for nodes that live beneath a
 * single-section owner below the stable root.
 * This is used to distinguish repeated channel/category layouts that belong to
 * different sections, such as Anim Mixer section outliners and section-hosted
 * modifier decorations, without changing the general outliner path identity for
 * all Sequencer models.
 */
static FString GetSectionDisambiguator(const TViewModelPtr<IOutlinerExtension>& InNode
	, const TViewModelPtr<IOutlinerExtension>& InStableRootNode)
{
	if (!InNode.IsValid())
	{
		return FString();
	}

	for (FViewModelPtr Node = InNode.AsModel(); Node && Node != InStableRootNode.AsModel(); Node = Node->GetParent())
	{
		TViewModelPtr<ISectionOwnerExtension> SectionOwner = Node.ImplicitCast();
		if (!SectionOwner.IsValid())
		{
			continue;
		}

		const TArray<UMovieSceneSection*> Sections = SectionOwner->GetSections();
		if (Sections.Num() == 1 && Sections[0])
		{
			return Sections[0]->GetPathName();
		}
	}

	return FString();
}

/**
 * Builds the persistent hide/isolate identity for an outliner node.
 *
 * The identity is split into:
 * - StableRootPath: path of the nearest rebuild-stable ancestor
 * - SectionDisambiguator: optional owning section path when needed
 * - RelativeSuffix: exact suffix from that stable root down to the selected node
 *
 * This allows the filter to survive UI rebuilds while still distinguishing nodes such as:
 * - Transform
 * - Location
 * - Location.X
 *
 * If the full path does not share the stable root as a prefix, we fall back to
 * storing the full path in RelativeSuffix so we preserve the node identity
 * instead of collapsing it.
 */
static FSequencerFilterPersistentPath BuildPersistentFilterPath(const TViewModelPtr<IOutlinerExtension>& InNode)
{
	if (!InNode.IsValid())
	{
		return FSequencerFilterPersistentPath();
	}

	const TViewModelPtr<IOutlinerExtension> StableRootNode = GetStableRootNode(InNode);

	const FString FullPath = IOutlinerExtension::GetPathName(InNode);
	const FString StableRootPath = IOutlinerExtension::GetPathName(StableRootNode);
	const FString SectionDisambiguator = GetSectionDisambiguator(InNode, StableRootNode);

	FString RelativeSuffix;
	if (!StableRootPath.IsEmpty() && FullPath.StartsWith(StableRootPath))
	{
		RelativeSuffix = FullPath.Mid(StableRootPath.Len());
	}
	else
	{
		// Fallback if the exact full path does not share the expected prefix.
		// This still preserves the full node identity instead of collapsing it.
		RelativeSuffix = FullPath;
	}

	return FSequencerFilterPersistentPath(StableRootPath, SectionDisambiguator, RelativeSuffix);
}

/**
 * Rebuilds a cache of live outliner nodes from stored persistent paths.
 *
 * For each current outliner node, we recompute the same persistent identity and
 * match it against the stored set. This remaps persisted hide/isolate state
 * back onto the rebuilt UI tree after outliner refreshes or structural changes.
 *
 * Because the persistent identity includes a stable root, an optional section
 * disambiguator, and an exact relative suffix, this preserves channel/category
 * specificity such as Location.X rather than collapsing everything back to the
 * parent track.
 */
static void RebuildTrackCacheFromPaths(const FSequencerTrackFilter_HideIsolate& InFilter
	, const TSet<FSequencerFilterPersistentPath>& InStoredPaths
	, TSet<TWeakViewModelPtr<IOutlinerExtension>>& OutCache)
{
	OutCache.Empty();

	const TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = InFilter.GetSequencer().GetViewModel();
	if (!SequencerViewModel.IsValid())
	{
		return;
	}

	const FViewModelPtr RootModel = SequencerViewModel->GetRootModel();
	if (!RootModel.IsValid())
	{
		return;
	}

	for (const TViewModelPtr<IOutlinerExtension>& Node : RootModel->GetDescendantsOfType<IOutlinerExtension>())
	{
		const FSequencerFilterPersistentPath PersistentPath = BuildPersistentFilterPath(Node);
		if (InStoredPaths.Contains(PersistentPath))
		{
			OutCache.Add(Node);
		}
	}
}

}

FSequencerTrackFilter_HideIsolate::FSequencerTrackFilter_HideIsolate(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter(InFilterInterface, MoveTemp(InCategory))
{
	IsActiveEvent = FIsActiveEvent::CreateLambda([this]()
		{
			return !HiddenTracks.IsEmpty() || !IsolatedTracks.IsEmpty();
		});
}

void FSequencerTrackFilter_HideIsolate::BindCommands()
{
	const FSequencerTrackFilterCommands& TrackFilterCommands = FSequencerTrackFilterCommands::Get();
	ISequencerTrackFilters& TrackFiltersInterface = GetFilterInterface();
	const TSharedPtr<FUICommandList>& FilterBarBindings = TrackFiltersInterface.GetCommandList();

	FilterBarBindings->MapAction(TrackFilterCommands.HideSelectedTracks,
		FUIAction(
			FExecuteAction::CreateRaw(&TrackFiltersInterface, &ISequencerTrackFilters::HideSelectedTracks),
			FCanExecuteAction::CreateRaw(&TrackFiltersInterface, &ISequencerTrackFilters::HasSelectedTracks)
		));

	FilterBarBindings->MapAction(TrackFilterCommands.IsolateSelectedTracks,
		FUIAction(
			FExecuteAction::CreateRaw(&TrackFiltersInterface, &ISequencerTrackFilters::IsolateSelectedTracks),
			FCanExecuteAction::CreateRaw(&TrackFiltersInterface, &ISequencerTrackFilters::HasSelectedTracks)
		));

	FilterBarBindings->MapAction(TrackFilterCommands.ClearHiddenTracks,
		FUIAction(
			FExecuteAction::CreateSP(this, &FSequencerTrackFilter_HideIsolate::EmptyHiddenTracks, true),
			FCanExecuteAction::CreateSP(this, &FSequencerTrackFilter_HideIsolate::HasHiddenTracks)
		));

	FilterBarBindings->MapAction(TrackFilterCommands.ClearIsolatedTracks,
		FUIAction(
			FExecuteAction::CreateSP(this, &FSequencerTrackFilter_HideIsolate::EmptyIsolatedTracks, true),
			FCanExecuteAction::CreateSP(this, &FSequencerTrackFilter_HideIsolate::HasIsolatedTracks)
		));

	FilterBarBindings->MapAction(TrackFilterCommands.ShowAllTracks,
		FUIAction(
			FExecuteAction::CreateSP(this, &FSequencerTrackFilter_HideIsolate::ShowAllTracks),
			FCanExecuteAction::CreateSP(this, &FSequencerTrackFilter_HideIsolate::HasHiddenOrIsolatedTracks)
		));

	FilterBarBindings->MapAction(TrackFilterCommands.ShowLocationCategoryGroups, FExecuteAction::CreateRaw(&TrackFiltersInterface, &ISequencerTrackFilters::ShowOnlyLocationCategoryGroups));
	FilterBarBindings->MapAction(TrackFilterCommands.ShowRotationCategoryGroups, FExecuteAction::CreateRaw(&TrackFiltersInterface, &ISequencerTrackFilters::ShowOnlyRotationCategoryGroups));
	FilterBarBindings->MapAction(TrackFilterCommands.ShowScaleCategoryGroups, FExecuteAction::CreateRaw(&TrackFiltersInterface, &ISequencerTrackFilters::ShowOnlyScaleCategoryGroups));
}

void FSequencerTrackFilter_HideIsolate::ResetFilter()
{
	HiddenTracks.Empty();
	IsolatedTracks.Empty();

	bHiddenTracksCacheDirty = true;
	bIsolatedTracksCacheDirty = true;

	BroadcastChangedEvent();
}

const TSet<TWeakViewModelPtr<IOutlinerExtension>>& FSequencerTrackFilter_HideIsolate::GetHiddenTracks() const
{
	if (bHiddenTracksCacheDirty)
	{
		RebuildHiddenTracksCache();
	}
	return CachedHiddenTracks;
}

const TSet<TWeakViewModelPtr<IOutlinerExtension>>& FSequencerTrackFilter_HideIsolate::GetIsolatedTracks() const
{
	if (bIsolatedTracksCacheDirty)
	{
		RebuildIsolatedTracksCache();
	}
	return CachedIsolatedTracks;
}

const TSet<FSequencerFilterPersistentPath>& FSequencerTrackFilter_HideIsolate::GetHiddenTrackPaths() const
{
	return HiddenTracks;
}

const TSet<FSequencerFilterPersistentPath>& FSequencerTrackFilter_HideIsolate::GetIsolatedTrackPaths() const
{
	return IsolatedTracks;
}

void FSequencerTrackFilter_HideIsolate::HideTracks(const TSet<TWeakViewModelPtr<IOutlinerExtension>>& InTracks, const bool bInAddToExisting)
{
	if (!bInAddToExisting)
	{
		HiddenTracks.Empty();
	}

	const TSharedPtr<FSequencerSelection>& Selection = GetSequencer().GetViewModel()->GetSelection();
	const FSelectionEventSuppressor SelectionChange = Selection->SuppressEvents();

	for (const TWeakViewModelPtr<IOutlinerExtension>& WeakTrack : InTracks)
	{
		if (const TViewModelPtr<IOutlinerExtension> TrackModel = WeakTrack.Pin())
		{
			if (const FSequencerFilterPersistentPath PersistentPath = HideIsolateShow::BuildPersistentFilterPath(TrackModel))
			{
				HiddenTracks.Add(PersistentPath);
			}

			Selection->Outliner.Deselect(TrackModel);
		}
	}

	bHiddenTracksCacheDirty = true;

	BroadcastChangedEvent();
}

void FSequencerTrackFilter_HideIsolate::UnhideTracks(const TSet<TWeakViewModelPtr<IOutlinerExtension>>& InTracks)
{
	for (const TWeakViewModelPtr<IOutlinerExtension>& TrackModekWeak : InTracks)
	{
		if (const TViewModelPtr<IOutlinerExtension> TrackModel = TrackModekWeak.Pin())
		{
			if (const FSequencerFilterPersistentPath PersistentPath = HideIsolateShow::BuildPersistentFilterPath(TrackModel))
			{
				HiddenTracks.Remove(PersistentPath);
			}
		}
	}

	bHiddenTracksCacheDirty = true;

	BroadcastChangedEvent();
}

void FSequencerTrackFilter_HideIsolate::IsolateTracks(const TSet<TWeakViewModelPtr<IOutlinerExtension>>& InTracks, const bool bInAddToExisting)
{
	if (!bInAddToExisting)
	{
		IsolatedTracks.Empty();
	}

	for (const TWeakViewModelPtr<IOutlinerExtension>& TrackModekWeak : InTracks)
	{
		if (const TViewModelPtr<IOutlinerExtension> TrackModel = TrackModekWeak.Pin())
		{
			if (const FSequencerFilterPersistentPath PersistentPath = HideIsolateShow::BuildPersistentFilterPath(TrackModel))
			{
				IsolatedTracks.Add(PersistentPath);
			}
		}
	}

	bIsolatedTracksCacheDirty = true;

	BroadcastChangedEvent();
}

void FSequencerTrackFilter_HideIsolate::UnisolateTracks(const TSet<TWeakViewModelPtr<IOutlinerExtension>>& InTracks)
{
	for (const TWeakViewModelPtr<IOutlinerExtension>& TrackModekWeak : InTracks)
	{
		if (const TViewModelPtr<IOutlinerExtension> TrackModel = TrackModekWeak.Pin())
		{
			if (const FSequencerFilterPersistentPath PersistentPath = HideIsolateShow::BuildPersistentFilterPath(TrackModel))
			{
				IsolatedTracks.Remove(PersistentPath);
			}
		}
	}

	bIsolatedTracksCacheDirty = true;

	BroadcastChangedEvent();
}

void FSequencerTrackFilter_HideIsolate::IsolateCategoryGroupTracks(const TSet<TWeakViewModelPtr<IOutlinerExtension>>& InTracks
	, const TSet<FName>& InCategoryNames
	, const bool bInAddToExisting)
{
	TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = GetSequencer().GetViewModel();
	if (!SequencerViewModel.IsValid())
	{
		return;
	}

	if (!bInAddToExisting)
	{
		EmptyIsolatedTracks(false);
	}

	TSet<TWeakViewModelPtr<IOutlinerExtension>> TracksToIsolate;
	TSet<TViewModelPtr<IOutlinerExtension>> TracksToExpand;

	auto IsolateChildCategoryGroups = [this, &InCategoryNames, &TracksToIsolate, &TracksToExpand](const TViewModelPtr<IOutlinerExtension>& InTrack)
	{
		const TParentFirstChildIterator<FCategoryGroupModel> ChildTracks = InTrack.AsModel()->GetDescendantsOfType<FCategoryGroupModel>(true);
		for (const TViewModelPtr<FCategoryGroupModel>& ChildCategoryGroup : ChildTracks)
		{
			if (InCategoryNames.Contains(ChildCategoryGroup->GetCategoryName()))
			{
				TracksToIsolate.Add(ChildCategoryGroup);

				const TParentModelIterator<IOutlinerExtension> Ancestors = ChildCategoryGroup->GetAncestorsOfType<IOutlinerExtension>();
				TracksToExpand.Append(Ancestors.ToArray());
				TracksToExpand.Add(ChildCategoryGroup);
			}
		}
	};

	for (const TWeakViewModelPtr<IOutlinerExtension>& WeakTrack : InTracks)
	{
		const TViewModelPtr<IOutlinerExtension> Track = WeakTrack.Pin();
		if (!Track.IsValid())
		{
			continue;
		}

		const TViewModelPtr<FCategoryGroupModel> CategoryGroupModel = Track.AsModel()->FindAncestorOfType<FCategoryGroupModel>(true);
		if (CategoryGroupModel.IsValid())
		{
			const TViewModelPtr<ITrackExtension> ParentTrack = Track.AsModel()->FindAncestorOfType<ITrackExtension>();
			if (ParentTrack.IsValid())
			{
				IsolateChildCategoryGroups(ParentTrack.ImplicitCast());
			}
		}
		else
		{
			IsolateChildCategoryGroups(Track);
		}
	}

	IsolateTracks(TracksToIsolate, true);

	for (const TViewModelPtr<IOutlinerExtension>& Track : TracksToExpand)
	{
		Track->SetExpansion(true);
	}
}

void FSequencerTrackFilter_HideIsolate::ShowAllTracks()
{
	HiddenTracks.Empty();
	IsolatedTracks.Empty();

	bHiddenTracksCacheDirty = true;
	bIsolatedTracksCacheDirty = true;

	BroadcastChangedEvent();
}

bool FSequencerTrackFilter_HideIsolate::HasHiddenTracks() const
{
	return !HiddenTracks.IsEmpty();
}

bool FSequencerTrackFilter_HideIsolate::HasIsolatedTracks() const
{
	return !IsolatedTracks.IsEmpty();
}

bool FSequencerTrackFilter_HideIsolate::HasHiddenOrIsolatedTracks() const
{
	return HasHiddenTracks() || HasIsolatedTracks();
}

bool FSequencerTrackFilter_HideIsolate::IsTrackHidden(const TViewModelPtr<IOutlinerExtension>& InTrack) const
{
	const FSequencerFilterPersistentPath PersistentTrackPath = HideIsolateShow::BuildPersistentFilterPath(InTrack);

	if (HiddenTracks.Contains(PersistentTrackPath))
	{
		return true;
	}

	for (const TViewModelPtr<IOutlinerExtension>& ParentNode : InTrack.AsModel()->GetAncestorsOfType<IOutlinerExtension>())
	{
		const FSequencerFilterPersistentPath ParentNodePath = HideIsolateShow::BuildPersistentFilterPath(ParentNode);
		if (HiddenTracks.Contains(ParentNodePath))
		{
			return true;
		}
	}

	return false;
}

bool FSequencerTrackFilter_HideIsolate::IsTrackIsolated(const TViewModelPtr<IOutlinerExtension>& InTrack) const
{
	const FSequencerFilterPersistentPath PersistentTrackPath = HideIsolateShow::BuildPersistentFilterPath(InTrack);

	if (IsolatedTracks.Contains(PersistentTrackPath))
	{
		return true;
	}

	for (const TViewModelPtr<IOutlinerExtension>& ParentNode : InTrack.AsModel()->GetAncestorsOfType<IOutlinerExtension>())
	{
		const FSequencerFilterPersistentPath ParentNodePath = HideIsolateShow::BuildPersistentFilterPath(ParentNode);
		if (IsolatedTracks.Contains(ParentNodePath))
		{
			return true;
		}
	}

	return false;
}

void FSequencerTrackFilter_HideIsolate::EmptyHiddenTracks(const bool bInBroadcastChange)
{
	HiddenTracks.Empty();
	bHiddenTracksCacheDirty = true;

	if (bInBroadcastChange)
	{
		BroadcastChangedEvent();
	}
}

void FSequencerTrackFilter_HideIsolate::EmptyIsolatedTracks(const bool bInBroadcastChange)
{
	IsolatedTracks.Empty();
	bIsolatedTracksCacheDirty = true;

	if (bInBroadcastChange)
	{
		BroadcastChangedEvent();
	}
}

FText FSequencerTrackFilter_HideIsolate::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_HideIsolate", "Hidden and Isolated");
}

FText FSequencerTrackFilter_HideIsolate::GetToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_HideIsolateToolTip", "Show only Hidden and Isolated tracks");
}

FSlateIcon FSequencerTrackFilter_HideIsolate::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("ExternalImagePicker.BlankImage"));
}

FString FSequencerTrackFilter_HideIsolate::GetName() const
{
	return StaticName();
}

bool FSequencerTrackFilter_HideIsolate::PassesFilter(FSequencerTrackFilterType InItem) const
{
	const TViewModelPtr<IOutlinerExtension> Track = InItem.ImplicitCast();
	if (!Track)
	{
		return false;
	}

	const bool bHasHiddenTracks = !HiddenTracks.IsEmpty();
	const bool bContainsHiddenTrack = IsTrackHidden(Track);

	const bool bHasIsolatedTracks = !IsolatedTracks.IsEmpty();
	const bool bContainsIsolatedTrack = IsTrackIsolated(Track);
	const bool bHasIsolatedDescendant = HasIsolatedDescendant(Track);

	// Can hide isolated tracks, but CANNOT isolate hidden tracks
	if (bHasHiddenTracks && bContainsHiddenTrack)
	{
		return false; // Filter out
	}

	if (!bHasIsolatedTracks)
	{
		return true;
	}

	return bContainsIsolatedTrack || bHasIsolatedDescendant;
}

bool FSequencerTrackFilter_HideIsolate::IsActive() const
{
	return HasHiddenOrIsolatedTracks();
}

bool FSequencerTrackFilter_HideIsolate::HasIsolatedDescendant(const TViewModelPtr<IOutlinerExtension>& InTrack) const
{
	if (!InTrack.IsValid())
	{
		return false;
	}

	const TSet<FSequencerFilterPersistentPath>& IsolatedTrackPaths = GetIsolatedTrackPaths();
	if (IsolatedTrackPaths.IsEmpty())
	{
		return false;
	}

	for (const TViewModelPtr<IOutlinerExtension>& Descendant : InTrack.AsModel()->GetDescendantsOfType<IOutlinerExtension>())
	{
		const FSequencerFilterPersistentPath DescendantPath = HideIsolateShow::BuildPersistentFilterPath(Descendant);
		if (IsolatedTrackPaths.Contains(DescendantPath))
		{
			return true;
		}
	}

	return false;
}

void FSequencerTrackFilter_HideIsolate::RebuildHiddenTracksCache() const
{
	HideIsolateShow::RebuildTrackCacheFromPaths(*this, HiddenTracks, CachedHiddenTracks);
	bHiddenTracksCacheDirty = false;
}

void FSequencerTrackFilter_HideIsolate::RebuildIsolatedTracksCache() const
{
	HideIsolateShow::RebuildTrackCacheFromPaths(*this, IsolatedTracks, CachedIsolatedTracks);
	bIsolatedTracksCacheDirty = false;
}

#undef LOCTEXT_NAMESPACE
