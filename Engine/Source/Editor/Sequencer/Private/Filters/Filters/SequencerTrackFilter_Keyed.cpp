// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/SequencerTrackFilter_Keyed.h"
#include "Filters/SequencerFilterBar.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "Framework/Commands/Commands.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/Extensions/ISectionOwnerExtension.h"
#include "Sequencer.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTrackFilter_Keyed"

FSequencerTrackFilter_Keyed::FSequencerTrackFilter_Keyed(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter(InFilterInterface, MoveTemp(InCategory))
{
}

bool FSequencerTrackFilter_Keyed::ShouldUpdateOnTrackValueChanged() const
{
	return true;
}

FText FSequencerTrackFilter_Keyed::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_KeyedTip", "Show only Keyed tracks"); 
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_Keyed::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_Keyed;
}

bool FSequencerTrackFilter_Keyed::PassesFilter(FSequencerTrackFilterType InItem) const
{
	if (const TViewModelPtr<FCategoryGroupModel> CategoryGroupModel = InItem.ImplicitCast())
	{
		for (const TWeakViewModelPtr<FCategoryModel>& WeakCategory : CategoryGroupModel->GetCategories())
		{
			if (const TViewModelPtr<FCategoryModel> Category = WeakCategory.Pin())
			{
				return Category->IsAnimated();
			}
		}
	}
	else if (const TViewModelPtr<FChannelGroupOutlinerModel> ChannelGroupOutlinerModel = InItem.ImplicitCast())
	{
		return ChannelGroupOutlinerModel->IsAnimated();
	}
	else if (const TViewModelPtr<ISectionOwnerExtension> WeakSectionOwner = InItem.ImplicitCast())
	{
		for (const UMovieSceneSection* const Section : WeakSectionOwner->GetSections())
		{
			for (const FMovieSceneChannelEntry& ChannelEntry : Section->GetChannelProxy().GetAllEntries())
			{
				const TConstArrayView<FMovieSceneChannel*> Channels = ChannelEntry.GetChannels();
				for (int32 ChannelIndex = 0; ChannelIndex < Channels.Num(); ++ChannelIndex)
				{
					const FMovieSceneChannel* const Channel = Channels[ChannelIndex];
					if (Channel && Channel->GetNumKeys() > 0)
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

FText FSequencerTrackFilter_Keyed::GetDisplayName() const
{
	return LOCTEXT("SequenceTrackFilter_Keyed", "Keyed");
}

FSlateIcon FSequencerTrackFilter_Keyed::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.IconKeyUser"));
}

FString FSequencerTrackFilter_Keyed::GetName() const
{
	return StaticName();
}

#undef LOCTEXT_NAMESPACE
