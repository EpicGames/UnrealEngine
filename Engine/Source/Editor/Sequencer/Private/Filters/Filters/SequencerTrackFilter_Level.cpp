// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/SequencerTrackFilter_Level.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "Misc/PackageName.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "Sequencer.h"
#include "UObject/Package.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTrackFilter_Level"

FSequencerTrackFilter_Level::FSequencerTrackFilter_Level(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter(InFilterInterface, MoveTemp(InCategory))
{
}

FSequencerTrackFilter_Level::~FSequencerTrackFilter_Level()
{
	if (CachedWorld.IsValid())
	{
		CachedWorld.Get()->OnLevelsChanged().RemoveAll(this);
		CachedWorld.Reset();
	}
}

FText FSequencerTrackFilter_Level::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_LevelToolTip", "Show only Level tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_Level::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_Levels;
}

void FSequencerTrackFilter_Level::OnPreFilter()
{
	const UObject* const PlaybackContext = GetSequencer().GetPlaybackContext();
	UWorld* const World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
	UpdateWorld(World);
}

FText FSequencerTrackFilter_Level::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_Level", "Levels");
}

FText FSequencerTrackFilter_Level::GetToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_LevelToolTip", "Show only Level tracks");
}

FSlateIcon FSequencerTrackFilter_Level::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("ClassIcon.LevelInstance"));
}

FString FSequencerTrackFilter_Level::GetName() const
{
	return StaticName();
}

bool FSequencerTrackFilter_Level::PassesFilter(FSequencerTrackFilterType InItem) const
{
	if (HiddenLevels.IsEmpty())
	{
		return true;
	}

	const TViewModelPtr<IObjectBindingExtension> BindingExtension = InItem->FindAncestorOfType<IObjectBindingExtension>(true);
	if (!BindingExtension.IsValid())
	{
		return true;
	}

	ISequencer& Sequencer = FilterInterface.GetSequencer();

	for (const TWeakObjectPtr<>& Object : Sequencer.FindObjectsInCurrentSequence(BindingExtension->GetObjectGuid()))
	{
		if (!Object.IsValid())
		{
			continue;
		}

		const UPackage* const Package = Object->GetPackage();
		if (!Package)
		{
			continue;
		}

		// For anything in a level, package should refer to the ULevel that contains it
		const FString LevelName = FPackageName::GetShortName(Package->GetName());

		if (HiddenLevels.Contains(LevelName))
		{
			return false;
		}
	}

	return true;
}

void FSequencerTrackFilter_Level::ResetFilter()
{
	HiddenLevels.Empty();

	BroadcastChangedEvent();
}

const TSet<FString>& FSequencerTrackFilter_Level::GetAllWorldLevels() const
{
	return AllWorldLevels;
}

bool FSequencerTrackFilter_Level::IsActive() const
{
	return HiddenLevels.Num() > 0;
}

bool FSequencerTrackFilter_Level::HasHiddenLevels() const
{
	return !HiddenLevels.IsEmpty();
}

bool FSequencerTrackFilter_Level::HasAllLevelsHidden() const
{
	for (const FString& WorldLevel : AllWorldLevels)
	{
		if (!HiddenLevels.Contains(WorldLevel))
		{
			return false;
		}
	}
	return true;
}

const TSet<FString>& FSequencerTrackFilter_Level::GetHiddenLevels() const
{
	return HiddenLevels;
}

bool FSequencerTrackFilter_Level::IsLevelHidden(const FString& InLevelName) const
{
	return HiddenLevels.Contains(InLevelName);
}

void FSequencerTrackFilter_Level::HideLevel(const FString& InLevelName)
{
	HiddenLevels.Add(InLevelName);

	BroadcastChangedEvent();
}

void FSequencerTrackFilter_Level::UnhideLevel(const FString& InLevelName)
{
	HiddenLevels.Remove(InLevelName);

	BroadcastChangedEvent();
}

void FSequencerTrackFilter_Level::HideAllLevels(const bool bInHide)
{
	if (bInHide)
	{
		HiddenLevels.Append(AllWorldLevels);
	}
	else
	{
		HiddenLevels.Empty();
	}

	BroadcastChangedEvent();
}

bool FSequencerTrackFilter_Level::CanHideAllLevels(const bool bInHide) const
{
	if (bInHide)
	{
		for (const FString& Level : AllWorldLevels)
		{
			if (!HiddenLevels.Contains(Level))
			{
				return true;
			}
		}

		return false;
	}

	return !HiddenLevels.IsEmpty();
}

void FSequencerTrackFilter_Level::UpdateWorld(UWorld* const InWorld)
{
	if (!CachedWorld.IsValid() || CachedWorld.Get() != InWorld)
	{
		if (CachedWorld.IsValid())
		{
			CachedWorld.Get()->OnLevelsChanged().RemoveAll(this);
		}

		CachedWorld.Reset();

		if (InWorld)
		{
			CachedWorld = InWorld;
			CachedWorld.Get()->OnLevelsChanged().AddRaw(this, &FSequencerTrackFilter_Level::HandleLevelsChanged);
		}

		HandleLevelsChanged();
	}
}

void FSequencerTrackFilter_Level::HandleLevelsChanged()
{
	AllWorldLevels.Empty();

	if (!CachedWorld.IsValid())
	{
		ResetFilter();
		return;
	}

	const TArray<ULevel*>& WorldLevels = CachedWorld->GetLevels();
	
	if (WorldLevels.Num() < 2)
	{
		ResetFilter();
		return;
	}

	// Build a list of level names contained in the current world
	TSet<FString> OldWorldLevels = AllWorldLevels;
	for (const ULevel* const Level : WorldLevels)
	{
		if (Level)
		{
			const FString LevelName = FPackageName::GetShortName(Level->GetPackage()->GetName());
			AllWorldLevels.Add(LevelName);
		}
	}

	// Rebuild our list of hidden level names to only include levels which are still in the world
	TSet<FString> OldHiddenLevels = HiddenLevels;
	HiddenLevels.Empty();
	for (const FString& LevelName : OldHiddenLevels)
	{
		if (AllWorldLevels.Contains(LevelName))
		{
			HiddenLevels.Add(LevelName);
		}
	}

	if (OldHiddenLevels.Num() != HiddenLevels.Num())
	{
		BroadcastChangedEvent();
	}
}

#undef LOCTEXT_NAMESPACE
