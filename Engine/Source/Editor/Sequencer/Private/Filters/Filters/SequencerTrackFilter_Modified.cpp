// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/SequencerTrackFilter_Modified.h"
#include "Filters/SequencerFilterBar.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "SequencerTrackFilter_Modified"

FSequencerTrackFilter_Modified::FSequencerTrackFilter_Modified(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter(InFilterInterface, MoveTemp(InCategory))
{
}

bool FSequencerTrackFilter_Modified::ShouldUpdateOnTrackValueChanged() const
{
	return true;
}

FText FSequencerTrackFilter_Modified::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_ModifiedToolTip", "Show only Modified tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_Modified::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_Modified;
}

FText FSequencerTrackFilter_Modified::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_Modified", "Modified");
}

FSlateIcon FSequencerTrackFilter_Modified::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.DirtyBadge"));
}

FString FSequencerTrackFilter_Modified::GetName() const
{
	return StaticName();
}

bool FSequencerTrackFilter_Modified::PassesFilter(FSequencerTrackFilterType InItem) const
{
	FSequencerFilterData& FilterData = GetFilterInterface().GetFilterData();

	const UMovieSceneTrack* const TrackObject = FilterData.ResolveMovieSceneTrackObject(InItem);
	if (!TrackObject)
	{
		return true;
	}

	if (TrackObject->GetPackage()->IsDirty())
	{
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
