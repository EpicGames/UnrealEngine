// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/SequencerTrackFilter_Selected.h"

#include "Filters/SequencerFilterBar.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "Filters/SequencerTrackFilterSelectedExtension.h"
#include "HAL/IConsoleManager.h"
#include "ISequencer.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "Selection.h"
#include "Filters/Utils/HierarchyFilterCache.h"
#include "UObject/UObjectIterator.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTrackFilter_Selected"

namespace UE::Sequencer::SelectionDetail
{
static TArray<TWeakObjectPtr<USequencerTrackFilterSelectedExtension>> GetSelectionExtensions()
{
	TArray<TWeakObjectPtr<USequencerTrackFilterSelectedExtension>> Extensions;
	for (TObjectIterator<USequencerTrackFilterSelectedExtension> ExtensionIt(RF_NoFlags); ExtensionIt; ++ExtensionIt)
	{
		USequencerTrackFilterSelectedExtension* PotentialExtension = *ExtensionIt;
		if (PotentialExtension
			&& PotentialExtension->HasAnyFlags(RF_ClassDefaultObject)
			&& !PotentialExtension->GetClass()->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract))
		{
			Extensions.Emplace(PotentialExtension);
		}
	}
	return Extensions;
}
}

FSequencerTrackFilter_Selected::FSequencerTrackFilter_Selected(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter_HierarchyBased(InFilterInterface, MoveTemp(InCategory))
	, CachedWeakExtensions(SelectionDetail::GetSelectionExtensions())
{}

FSequencerTrackFilter_Selected::~FSequencerTrackFilter_Selected()
{
	UnbindEvents();
}

void FSequencerTrackFilter_Selected::BindSelectionChanged()
{
	if (!OnSelectionChangedHandle.IsValid())
	{
		OnSelectionChangedHandle = USelection::SelectionChangedEvent.AddSP(this, &FSequencerTrackFilter_Selected::OnViewportSelectionChanged);
	}
}

void FSequencerTrackFilter_Selected::UnbindSelectionChanged()
{
	if (OnSelectionChangedHandle.IsValid())
	{
		USelection::SelectionChangedEvent.Remove(OnSelectionChangedHandle);
		OnSelectionChangedHandle.Reset();
	}
}

bool FSequencerTrackFilter_Selected::ShouldUpdateOnTrackValueChanged() const
{
	return true;
}

FText FSequencerTrackFilter_Selected::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_SelectedToolTip", "Show only track selected in the viewport");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_Selected::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_Selected;
}

void FSequencerTrackFilter_Selected::ActiveStateChanged(const bool bInActive)
{
	FSequencerTrackFilter::ActiveStateChanged(bInActive);

	if (bInActive)
	{
		BindEvents();
	}
	else
	{
		UnbindEvents();
	}
}

FText FSequencerTrackFilter_Selected::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_Selected", "Selected");
}

FSlateIcon FSequencerTrackFilter_Selected::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.SelectInViewport"));
}

void FSequencerTrackFilter_Selected::OnPreFilter()
{
	if (!bNeedsPrefilterRebuild)
	{
		return;
	}
	
	TRACE_CPUPROFILER_EVENT_SCOPE(FSequencerTrackFilter_Selected::OnPreFilter);
	bNeedsPrefilterRebuild = false;
	FilterCache.Empty(FilterCache.Num());
	
	const TSharedPtr<FSequencerEditorViewModel> RootViewModel = GetSequencer().GetViewModel();
	const TSharedPtr<FSequencerSelection> SequencerSelection = RootViewModel ? RootViewModel->GetSelection() : nullptr;
	if (SequencerSelection.IsValid())
	{
		IncludeSelection(FilterCache, SequencerSelection->Outliner, GetSequencer(), CachedWeakExtensions);
	}
}

FString FSequencerTrackFilter_Selected::GetName() const
{
	return StaticName();
}

FFilterResult FSequencerTrackFilter_Selected::Evaluate(FSequencerTrackFilterType InItem) const
{
	return EvaluateFilterCache(FilterCache, InItem);
}

void FSequencerTrackFilter_Selected::BindEvents()
{
	BindSelectionChanged();

	const TSharedPtr<FSequencerEditorViewModel> RootViewModel = GetSequencer().GetViewModel();
	WeakViewModel = RootViewModel;
	const TSharedPtr<FSequencerSelection> SequencerSelection = RootViewModel ? RootViewModel->GetSelection() : nullptr;
	if (SequencerSelection)
	{
		SequencerSelection->OnChanged.AddSP(this, &FSequencerTrackFilter_Selected::OnSequencerSelectionChanged);
	}
}

void FSequencerTrackFilter_Selected::UnbindEvents()
{
	UnbindSelectionChanged();

	const TSharedPtr<FSequencerEditorViewModel> RootViewModel = WeakViewModel.Pin();
	WeakViewModel.Reset();
	const TSharedPtr<FSequencerSelection> SequencerSelection = RootViewModel ? RootViewModel->GetSelection() : nullptr;
	if (SequencerSelection)
	{
		SequencerSelection->OnChanged.RemoveAll(this);
	}
}

void FSequencerTrackFilter_Selected::OnSequencerSelectionChanged()
{
	bNeedsPrefilterRebuild = true;
	FilterInterface.RequestFilterUpdate();
}

void FSequencerTrackFilter_Selected::OnViewportSelectionChanged(UObject* const InObject)
{
	bNeedsPrefilterRebuild = true;
	FilterInterface.RequestFilterUpdate();
}

#undef LOCTEXT_NAMESPACE
