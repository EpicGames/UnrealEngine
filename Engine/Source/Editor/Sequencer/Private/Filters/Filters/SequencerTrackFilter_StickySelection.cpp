// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTrackFilter_StickySelection.h"

#include "Sequencer.h"
#include "Filters/SequencerTrackFilterSelectedExtension.h"
#include "Filters/SequencerTrackFilterStickySelectionExtension.h"
#include "Filters/Utils/HierarchyFilterCache.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "UObject/UObjectIterator.h"

namespace UE::Sequencer
{
namespace StickySelection
{
template<typename T>
static TArray<TWeakObjectPtr<T>> GetSelectionExtensions()
{
	TArray<TWeakObjectPtr<T>> Extensions;
	for (TObjectIterator<T> ExtensionIt(RF_NoFlags); ExtensionIt; ++ExtensionIt)
	{
		T* PotentialExtension = *ExtensionIt;
		if (PotentialExtension
			&& PotentialExtension->HasAnyFlags(RF_ClassDefaultObject)
			&& !PotentialExtension->GetClass()->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract))
		{
			Extensions.Emplace(PotentialExtension);
		}
	}
	return Extensions;
}
}

FSequencerTrackFilter_StickySelection::FSequencerTrackFilter_StickySelection(
	const FSequencer& InSequencer, ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory
	)
	: FSequencerTrackFilter_HierarchyBased(InFilterInterface, MoveTemp(InCategory))
	, Sequencer(InSequencer)
	, WeakStickyExtensions(StickySelection::GetSelectionExtensions<USequencerTrackFilterStickySelectionExtension>())
	, WeakSelectionExtensions(StickySelection::GetSelectionExtensions<USequencerTrackFilterSelectedExtension>())
{
	WeakViewModel = InSequencer.GetViewModel();
	if (const TSharedPtr<FSequencerSelection> Selection = GetSequencerSelection())
	{
		Selection->Outliner.OnChanged.AddRaw(this, &FSequencerTrackFilter_StickySelection::OnSelectionChanged);
	}
}

FSequencerTrackFilter_StickySelection::~FSequencerTrackFilter_StickySelection()
{
	const TSharedPtr<FSequencerEditorViewModel> RootViewModel = WeakViewModel.Pin();
	WeakViewModel.Reset();
	const TSharedPtr<FSequencerSelection> SequencerSelection = RootViewModel ? RootViewModel->GetSelection() : nullptr;
	if (SequencerSelection)
	{
		SequencerSelection->Outliner.OnChanged.RemoveAll(this);
	}
}

FString FSequencerTrackFilter_StickySelection::GetName() const
{
	return StaticName();
}

void FSequencerTrackFilter_StickySelection::OnPreFilter()
{
	const TSharedPtr<FSequencerSelection> Selection = GetSequencerSelection();
	if (!bNeedsPrefilterRebuild || !Selection)
	{
		return;
	}
	bNeedsPrefilterRebuild = false;
	const bool bLastResetCacheOnUpdate = bResetCacheOnUpdate;
	bResetCacheOnUpdate = false;
	
	const FOutlinerSelection& OutlinerSelection = Selection->Outliner;
	const bool bWasCleared = OutlinerSelection.Num() == 0;
	if (bLastResetCacheOnUpdate || bWasCleared)
	{
		FilterCache.Empty(FilterCache.Num());
	}
	
	IncludeSelection(FilterCache, OutlinerSelection, GetSequencer(), WeakSelectionExtensions);
}

FFilterResult FSequencerTrackFilter_StickySelection::Evaluate(FSequencerTrackFilterType InItem) const
{
	return EvaluateFilterCache(FilterCache, InItem, EItemFilterState::DoNotCare);
}

TSharedPtr<FSequencerSelection> FSequencerTrackFilter_StickySelection::GetSequencerSelection() const
{
	const TSharedPtr<FSequencerEditorViewModel> RootViewModel = GetSequencer().GetViewModel();
	const TSharedPtr<FSequencerSelection> SequencerSelection = RootViewModel ? RootViewModel->GetSelection() : nullptr;
	return SequencerSelection;
}

void FSequencerTrackFilter_StickySelection::OnSelectionChanged()
{
	bNeedsPrefilterRebuild = true;
	
	const auto ProcessExtension = [this](const TWeakObjectPtr<USequencerTrackFilterStickySelectionExtension>& Extension)
	{
		return Extension.IsValid() && Extension->IsPerformingSelectionThatShouldClearStickySelection(Sequencer);
	};
	const bool bIsExternalSystemClearingStickySelection = Algo::AnyOf(WeakStickyExtensions, ProcessExtension);
	bResetCacheOnUpdate |= Sequencer.IsUpdatingSequencerSelection() || bIsExternalSystemClearingStickySelection;
}
} // namespace UE::Sequencer
