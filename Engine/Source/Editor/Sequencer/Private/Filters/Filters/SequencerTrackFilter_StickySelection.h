// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/Filters/SequencerTrackFilter_HierarchyBased.h"
#include "Filters/SequencerTrackFilterBase.h"
#include "Filters/SequencerTrackFilterSelectedExtension.h"
#include "Filters/Utils/HierarchyFilterCache.h"
#include "HAL/Platform.h"
#include "MVVM/Selection/SequencerOutlinerSelection.h"

class USequencerTrackFilterStickySelectionExtension;
class FSequencer;

namespace UE::Sequencer
{
class FSequencerSelection;
class FSequencerEditorViewModel;

/**
 * Prevents clicks in the Outliner or Curve Editor from changing which items the "Selected" filter shows.
 * Even though clicking items in the Outliner or Curve Editor changes the selection internally, the items clicked will continue to be filtered in 
 * until the user changes the selection externally, e.g. by clicking in the viewport.
 * This filter is designed to be used in conjunction with the "Selected" filter.
 *
 * The workflow that is being enabled is as follows:
 * 1. User enables the "Selected" filter
 * 2. User shift-clicks chest, body, and pelvis controls of a Control Rig in the viewport (-> external selection)
 * 3. User clicks chest in the Sequencer Outliner or Curve Editor (-> internal selection)
 * Without this filter: "Selected" filter collapses to only chest, since the internal click changed the selection.
 * With this filter: chest, body, and pelvis remain filtered in.
 *
 * @see FSequencer::bUpdatingSequencerSelection for the internal/external selection distinction.
 */
class FSequencerTrackFilter_StickySelection : public FSequencerTrackFilter_HierarchyBased
{
public:
	
	static FString StaticName() { return TEXT("StickySelection"); }

	explicit FSequencerTrackFilter_StickySelection(
		const FSequencer& InSequencer UE_LIFETIMEBOUND, 
		ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr
		);
	virtual ~FSequencerTrackFilter_StickySelection() override;
	
	//~ Begin IFilter
	virtual FString GetName() const override;
	virtual void OnPreFilter() override;
	virtual FFilterResult Evaluate(FSequencerTrackFilterType InItem) const override;
	//~ End IFilter
	
private:

	/** The Sequencer for which this filter was created. */
	const FSequencer& Sequencer;

	/** Cached weak reference to the view model captured at construction. */
	TWeakPtr<FSequencerEditorViewModel> WeakViewModel;

	/** Objects extending this filter. Cached at construction to speed up filtering. */
	const TArray<TWeakObjectPtr<USequencerTrackFilterStickySelectionExtension>> WeakStickyExtensions;
	/** Objects extending the selection filter. Cached at construction to speed up filtering. */
	const TArray<TWeakObjectPtr<USequencerTrackFilterSelectedExtension>> WeakSelectionExtensions;
	
	/** Whether IncludedViewModels needs to be rebuilt. */
	bool bNeedsPrefilterRebuild = true;
	/** Whetehr to reset the cache on update. This will be set, e.g. when the user clicks something in the viewport. */
	bool bResetCacheOnUpdate = false;
	/** Contains the selected items. */
	FHierarchyFilterCache FilterCache;
	
	/** @return Sequencer's active selection. */
	TSharedPtr<FSequencerSelection> GetSequencerSelection() const;
	
	/** Invoked when the Sequencer selection changes. */
	void OnSelectionChanged();
};
} // namespace UE::Sequencer
