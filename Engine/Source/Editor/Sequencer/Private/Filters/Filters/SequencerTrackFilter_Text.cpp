// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTrackFilter_Text.h"
#include "Filters/SequencerFilterBar.h"
#include "Filters/SequencerTextFilterExpressionContext.h"
#include "Filters/SequencerTrackFilterTextExpressionExtension.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "MVVM/ViewModelPtr.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_BindingName.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_BindingType.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_Condition.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_ConditionClass.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_ConditionFunc.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_ConditionPasses.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_CustomBinding.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_Deactivated.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_EmptyBinding.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_Filters.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_Group.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_Keyed.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_Level.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_Locked.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_Modified.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_Muted.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_Name.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_ObjectClass.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_Selected.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_Soloed.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_Tag.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_Time.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_TrackClass.h"
#include "Filters/TextExpressions/SequencerTextFilterExpression_Unbound.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "SequencerTrackFilter_Text"

using namespace UE::Sequencer;

FSequencerTrackFilter_Text::FSequencerTrackFilter_Text(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTrackFilter(InFilterInterface, nullptr)
	, TextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::Complex)
{
	IsActiveEvent = FIsActiveEvent::CreateLambda([this]()
		{
			return !GetRawFilterText().IsEmpty();
		});

	// Ordered by importance and most often used. This will dictate the order of display in the text expressions help dialog.
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_Filters>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_Name>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_TrackClass>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_ObjectClass>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_Condition>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_ConditionClass>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_ConditionFunc>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_ConditionPasses>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_Keyed>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_Selected>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_Unbound>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_Group>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_Level>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_Modified>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_Time>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_Locked>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_Deactivated>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_Muted>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_Soloed>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_Tag>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_BindingName>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_BindingType>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_CustomBinding>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FSequencerTextFilterExpression_EmptyBinding>(InFilterInterface));

	// Add global user-defined text expressions
	for (TObjectIterator<USequencerTrackFilterTextExpressionExtension> ExtensionIt(RF_NoFlags); ExtensionIt; ++ExtensionIt)
	{
		const USequencerTrackFilterTextExpressionExtension* const PotentialExtension = *ExtensionIt;
		if (PotentialExtension
			&& PotentialExtension->HasAnyFlags(RF_ClassDefaultObject)
			&& !PotentialExtension->GetClass()->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract))
		{
			TArray<TSharedRef<FSequencerTextFilterExpressionContext>> ExtendedTextExpressions;
			PotentialExtension->AddTrackFilterTextExpressionExtensions(static_cast<ISequencerTrackFilters&>(FilterInterface), ExtendedTextExpressions);

			TextFilterExpressionContexts.Append(ExtendedTextExpressions);
		}
	}

	VerifyNoDuplicateKeys();
}

FText FSequencerTrackFilter_Text::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_Text", "Text");
}

FText FSequencerTrackFilter_Text::GetToolTipText() const
{
	return LOCTEXT("SequencerTrackListFilter_Text", "Show only assets that match the input text");
}

FString FSequencerTrackFilter_Text::GetName() const
{
	return StaticName();
}

bool FSequencerTrackFilter_Text::PassesFilter(FSequencerTrackFilterType InItem) const
{
	for (const TSharedRef<FSequencerTextFilterExpressionContext>& TextFilterExpressionContext : TextFilterExpressionContexts)
	{
		TextFilterExpressionContext->SetFilterItem(InItem);

		const bool bPassedFilter = TextFilterExpressionEvaluator.TestTextFilter(*TextFilterExpressionContext);

		TextFilterExpressionContext->SetFilterItem(nullptr);

		if (!bPassedFilter)
		{
			return false;
		}
	}

	return true;
}

bool FSequencerTrackFilter_Text::IsActive() const
{
	return !GetRawFilterText().IsEmpty();
}

FText FSequencerTrackFilter_Text::GetRawFilterText() const
{
	return TextFilterExpressionEvaluator.GetFilterText();
}

FText FSequencerTrackFilter_Text::GetFilterErrorText() const
{
	return TextFilterExpressionEvaluator.GetFilterErrorText();
}

void FSequencerTrackFilter_Text::SetRawFilterText(const FText& InFilterText)
{
	if (TextFilterExpressionEvaluator.SetFilterText(InFilterText))
	{
		BroadcastChangedEvent();
	}
}

const FTextFilterExpressionEvaluator& FSequencerTrackFilter_Text::GetTextFilterExpressionEvaluator() const
{
	return TextFilterExpressionEvaluator;
}

const TArray<TSharedRef<FSequencerTextFilterExpressionContext>>& FSequencerTrackFilter_Text::GetTextFilterExpressionContexts() const
{
	return TextFilterExpressionContexts;
}

bool FSequencerTrackFilter_Text::DoesTextFilterStringContainExpressionPair(const ISequencerTextFilterExpressionContext& InExpression) const
{
	const TArray<FExpressionToken>& ExpressionTokens = TextFilterExpressionEvaluator.GetFilterExpressionTokens();
	const int32 ExpressionCount = ExpressionTokens.Num();

	// Need atleast three tokens: key, operator, and value
	if (ExpressionTokens.Num() < 3)
	{
		return false;
	}

	const TSet<FName> Keys = InExpression.GetKeys();
	const ESequencerTextFilterValueType ValueType = InExpression.GetValueType();
	const TArray<FSequencerTextFilterKeyword> ValueKeywords = InExpression.GetValueKeywords();

	for (int32 Index = 0; Index < ExpressionCount - 2; ++Index)
	{
		// Match key
		const FExpressionToken& KeyToken = ExpressionTokens[Index];
		const FString KeyTokenString = KeyToken.Context.GetString();

		if (IsTokenKey(KeyToken, Keys))
		{
			// Match operator
			const int32 OperatorIndex = Index + 1;
			const FExpressionToken& OperatorToken = ExpressionTokens[OperatorIndex];
			const FString OperatorTokenString = OperatorToken.Context.GetString();

			if (IsTokenOperator(OperatorToken, ValueType))
			{
				// Match value
				const int32 ValueIndex = Index + 2;
				const FExpressionToken& ValueToken = ExpressionTokens[ValueIndex];
				const FString ValueTokenString = ValueToken.Context.GetString();

				if (IsTokenValueValid(ValueToken, ValueType))
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool FSequencerTrackFilter_Text::IsTokenKey(const FExpressionToken& InToken, const TSet<FName>& InKeys)
{
	const FString KeyTokenString = InToken.Context.GetString();
	
	for (const FName& Key : InKeys)
	{
		if (KeyTokenString.Equals(Key.ToString(), ESearchCase::IgnoreCase))
		{
			return true;
		}
	}

	return false;
};

bool FSequencerTrackFilter_Text::IsTokenOperator(const FExpressionToken& InToken, const ESequencerTextFilterValueType InValueType)
{
	if (InValueType == ESequencerTextFilterValueType::String)
	{
		return InToken.Node.Cast<TextFilterExpressionParser::FEqual>()
			|| InToken.Node.Cast<TextFilterExpressionParser::FNotEqual>();
	}

	return InToken.Node.Cast<TextFilterExpressionParser::FEqual>()
		|| InToken.Node.Cast<TextFilterExpressionParser::FNotEqual>()
		|| InToken.Node.Cast<TextFilterExpressionParser::FLess>()
		|| InToken.Node.Cast<TextFilterExpressionParser::FLessOrEqual>()
		|| InToken.Node.Cast<TextFilterExpressionParser::FGreater>()
		|| InToken.Node.Cast<TextFilterExpressionParser::FGreaterOrEqual>();
}

bool FSequencerTrackFilter_Text::IsTokenValueValid(const FExpressionToken& InToken, const ESequencerTextFilterValueType InValueType)
{
	if (!InToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
	{
		return false;
	}

	// @TODO: better value checking?

	return true;
}

void FSequencerTrackFilter_Text::VerifyNoDuplicateKeys() const
{
	TSet<FName> SeenKeys;

	for (const TSharedRef<FSequencerTextFilterExpressionContext>& TextExpression : TextFilterExpressionContexts)
	{
		const TSet<FName> TextExpressionKeys = TextExpression->GetKeys();
		for (const FName Key : TextExpressionKeys)
		{
			ensureMsgf(!SeenKeys.Contains(Key), TEXT("Sequencer text expression filter key already exists: %s"), *Key.ToString());
		}

		SeenKeys.Append(TextExpressionKeys);
	}
}

#undef LOCTEXT_NAMESPACE
