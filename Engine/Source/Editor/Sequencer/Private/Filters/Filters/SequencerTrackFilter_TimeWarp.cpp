// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/SequencerTrackFilter_TimeWarp.h"
#include "Filters/SequencerFilterBar.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "MovieSceneSection.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "Sequencer.h"
#include "Tracks/MovieSceneTimeWarpTrack.h"
#include "Variants/MovieSceneTimeWarpVariant.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTrackFilter_TimeWarp"

FSequencerTrackFilter_TimeWarp::FSequencerTrackFilter_TimeWarp(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter(InFilterInterface, MoveTemp(InCategory))
{
}

bool FSequencerTrackFilter_TimeWarp::ShouldUpdateOnTrackValueChanged() const
{
	return true;
}

FText FSequencerTrackFilter_TimeWarp::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_TimeWarpToolTip", "Show only Time Warp tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_TimeWarp::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_TimeWarp;
}

FText FSequencerTrackFilter_TimeWarp::GetDisplayName() const
{
	return LOCTEXT("FSequencerTrackFilter_TimeWarp", "Time Warp");
}

FSlateIcon FSequencerTrackFilter_TimeWarp::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.Tracks.TimeWarp"));
}

FString FSequencerTrackFilter_TimeWarp::GetName() const
{
	return StaticName();
}

bool FSequencerTrackFilter_TimeWarp::PassesFilter(FSequencerTrackFilterType InItem) const
{
	if (const TViewModelPtr<ISectionOwnerExtension> SectionOwner = InItem->FindAncestorOfType<ISectionOwnerExtension>(true))
	{
		for (const TViewModelPtr<FSectionModel>& SectionModel : SectionOwner->GetSectionModels().IterateSubList<FSectionModel>())
		{
			UMovieSceneSection* Section = SectionModel->GetSection();
			if (!Section)
			{
				continue;
			}

			if (FMovieSceneTimeWarpVariant* Variant = Section->GetTimeWarp())
			{
				if (Variant->GetType() == EMovieSceneTimeWarpType::Custom)
				{
					return true;
				}
			}
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
