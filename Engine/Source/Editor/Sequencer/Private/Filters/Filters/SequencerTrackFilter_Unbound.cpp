// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/SequencerTrackFilter_Unbound.h"
#include "Filters/SequencerFilterBar.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "Sequencer.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTrackFilter_Unbound"

FSequencerTrackFilter_Unbound::FSequencerTrackFilter_Unbound(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter(InFilterInterface, MoveTemp(InCategory))
{
}

bool FSequencerTrackFilter_Unbound::ShouldUpdateOnTrackValueChanged() const
{
	return true;
}

FText FSequencerTrackFilter_Unbound::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_UnboundToolTip", "Show only Unbound tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_Unbound::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_Unbound;
}

FText FSequencerTrackFilter_Unbound::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_Unbound", "Unbound");
}

FSlateIcon FSequencerTrackFilter_Unbound::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("BTEditor.Graph.BTNode.Decorator.DoesPathExist.Icon"));
}

FString FSequencerTrackFilter_Unbound::GetName() const
{
	return StaticName();
}

bool FSequencerTrackFilter_Unbound::PassesFilter(FSequencerTrackFilterType InItem) const
{
	bool bPassed = false;

	if (const TViewModelPtr<FObjectBindingModel> ObjectBindingModel = InItem->FindAncestorOfType<FObjectBindingModel>(true))
	{
		if (const TViewModelPtr<FSequenceModel> SequenceModel = InItem->FindAncestorOfType<FSequenceModel>())
		{
			const TArrayView<TWeakObjectPtr<>> BoundObjects = FilterInterface.GetSequencer().FindBoundObjects(ObjectBindingModel->GetObjectGuid(), SequenceModel->GetSequenceID());

			bool bAnyValidBinding = false;

			for (const TWeakObjectPtr<>& WeakObj : BoundObjects)
			{
				if (WeakObj.IsValid())
				{
					bAnyValidBinding = true;
					break;
				}
			}

			if (!bAnyValidBinding)
			{
				bPassed = true;
			}
		}
	}

	return bPassed;
}

#undef LOCTEXT_NAMESPACE
