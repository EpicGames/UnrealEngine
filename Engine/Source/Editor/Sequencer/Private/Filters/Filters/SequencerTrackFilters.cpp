// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/SequencerTrackFilters.h"
#include "Filters/SequencerTrackFilterBase.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "Misc/IFilter.h"
#include "Particles/ParticleSystem.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTrackFilters"

FSequencerTrackFilter_Audio::FSequencerTrackFilter_Audio(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter_ClassType<UMovieSceneAudioTrack>(InFilterInterface, InCategory)
{
}

FText FSequencerTrackFilter_Audio::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_Audio", "Audio");
}

FSlateIcon FSequencerTrackFilter_Audio::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.Tracks.Audio"));
}

FText FSequencerTrackFilter_Audio::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_AudioToolTip", "Show only Audio tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_Audio::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_Audio;
}

bool FSequencerTrackFilter_Audio::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return IsSequenceTrackSupported<UMovieSceneAudioTrack>(InSequence);
}

//////////////////////////////////////////////////////////////////////////
//

FSequencerTrackFilter_Event::FSequencerTrackFilter_Event(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter_ClassType<UMovieSceneEventTrack>(InFilterInterface, InCategory)
{
}

FText FSequencerTrackFilter_Event::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_Event", "Event");
}

FSlateIcon FSequencerTrackFilter_Event::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.Tracks.Event"));
}

FText FSequencerTrackFilter_Event::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_EventToolTip", "Show only Event tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_Event::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_Event;
}

bool FSequencerTrackFilter_Event::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return IsSequenceTrackSupported<UMovieSceneEventTrack>(InSequence);
}

//////////////////////////////////////////////////////////////////////////
//

FSequencerTrackFilter_LevelVisibility::FSequencerTrackFilter_LevelVisibility(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter_ClassType<UMovieSceneLevelVisibilityTrack>(InFilterInterface, InCategory)
{
}

FText FSequencerTrackFilter_LevelVisibility::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_LevelVisibility", "Level Visibility");
}

FSlateIcon FSequencerTrackFilter_LevelVisibility::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.Tracks.LevelVisibility"));
}

FText FSequencerTrackFilter_LevelVisibility::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_LevelVisibilityToolTip", "Show only Level Visibility tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_LevelVisibility::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_LevelVisibility;
}

bool FSequencerTrackFilter_LevelVisibility::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return IsSequenceTrackSupported<UMovieSceneLevelVisibilityTrack>(InSequence);
}

//////////////////////////////////////////////////////////////////////////
//

FSequencerTrackFilter_Particle::FSequencerTrackFilter_Particle(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter_ClassType<UMovieSceneParticleTrack>(InFilterInterface, InCategory)
{
}

FText FSequencerTrackFilter_Particle::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_Particle", "Particle System");
}

FSlateIcon FSequencerTrackFilter_Particle::GetIcon() const
{
	return FSlateIconFinder::FindIconForClass(UParticleSystem::StaticClass());
}

FText FSequencerTrackFilter_Particle::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_ParticleToolTip", "Show only Particle System tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_Particle::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_Particle;
}

bool FSequencerTrackFilter_Particle::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return IsSequenceTrackSupported<UMovieSceneParticleTrack>(InSequence);
}

//////////////////////////////////////////////////////////////////////////
//

FSequencerTrackFilter_CinematicShot::FSequencerTrackFilter_CinematicShot(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter_ClassType<UMovieSceneCinematicShotTrack>(InFilterInterface, InCategory)
{
}

FText FSequencerTrackFilter_CinematicShot::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_CinematicShot", "Shot");
}

FSlateIcon FSequencerTrackFilter_CinematicShot::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.Tracks.CinematicShot"));
}

FText FSequencerTrackFilter_CinematicShot::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_CinematicShotToolTip", "Show only Shot tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_CinematicShot::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_CinematicShot;
}

bool FSequencerTrackFilter_CinematicShot::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return IsSequenceTrackSupported<UMovieSceneCinematicShotTrack>(InSequence);
}

//////////////////////////////////////////////////////////////////////////
//

FSequencerTrackFilter_Subsequence::FSequencerTrackFilter_Subsequence(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter_ClassType<UMovieSceneSubTrack>(InFilterInterface, InCategory)
{
}

bool FSequencerTrackFilter_Subsequence::PassesFilter(FSequencerTrackFilterType InItem) const
{
	FSequencerFilterData& FilterData = GetFilterInterface().GetFilterData();
	const UMovieSceneTrack* const TrackObject = FilterData.ResolveMovieSceneTrackObject(InItem);
	return TrackObject
		&& TrackObject->IsA(UMovieSceneSubTrack::StaticClass())
		&& !TrackObject->IsA(UMovieSceneCinematicShotTrack::StaticClass());
}

FText FSequencerTrackFilter_Subsequence::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_Subsequence", "Subsequence");
}

FSlateIcon FSequencerTrackFilter_Subsequence::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.Tracks.Sub"));
}

FText FSequencerTrackFilter_Subsequence::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_SubsequenceToolTip", "Show only Subsequence tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_Subsequence::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_Subsequence;
}

bool FSequencerTrackFilter_Subsequence::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return IsSequenceTrackSupported<UMovieSceneSubTrack>(InSequence);
}

//////////////////////////////////////////////////////////////////////////
//

FSequencerTrackFilter_SkeletalMesh::FSequencerTrackFilter_SkeletalMesh(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter_ComponentType<USkeletalMeshComponent>(InFilterInterface, InCategory)
{
}

FText FSequencerTrackFilter_SkeletalMesh::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_SkeletalMesh", "Skeletal Mesh");
}

FSlateIcon FSequencerTrackFilter_SkeletalMesh::GetIcon() const
{
	return FSlateIconFinder::FindIconForClass(USkeletalMeshComponent::StaticClass());
}

FText FSequencerTrackFilter_SkeletalMesh::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_SkeletalMeshToolTip", "Show only Skeletal Mesh tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_SkeletalMesh::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_SkeletalMesh;
}

bool FSequencerTrackFilter_SkeletalMesh::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return IsSequenceTrackSupported<USkeletalMeshComponent>(InSequence);
}

//////////////////////////////////////////////////////////////////////////
//

FSequencerTrackFilter_Camera::FSequencerTrackFilter_Camera(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter_ComponentType<UCameraComponent>(InFilterInterface, InCategory)
{
}

FText FSequencerTrackFilter_Camera::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_Camera", "Camera");
}

FSlateIcon FSequencerTrackFilter_Camera::GetIcon() const
{
	return FSlateIconFinder::FindIconForClass(UCameraComponent::StaticClass());
}

FText FSequencerTrackFilter_Camera::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_CameraToolTip", "Show only Camera tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_Camera::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_Camera;
}

bool FSequencerTrackFilter_Camera::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return IsSequenceTrackSupported<UCameraComponent>(InSequence);
}

//////////////////////////////////////////////////////////////////////////
//

FSequencerTrackFilter_Light::FSequencerTrackFilter_Light(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter_ComponentType<ULightComponentBase>(InFilterInterface, InCategory)
{
}

FText FSequencerTrackFilter_Light::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_Light", "Light");
}

FSlateIcon FSequencerTrackFilter_Light::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("ClassIcon.Light"));
}

FText FSequencerTrackFilter_Light::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_LightToolTip", "Show only Light tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_Light::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_Light;
}

bool FSequencerTrackFilter_Light::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return IsSequenceTrackSupported<UCameraComponent>(InSequence);
}

//////////////////////////////////////////////////////////////////////////
//

FSequencerTrackFilter_CameraCut::FSequencerTrackFilter_CameraCut(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter_ClassType<UMovieSceneCameraCutTrack>(InFilterInterface, InCategory)
{
}

FText FSequencerTrackFilter_CameraCut::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_CameraCut", "Camera Cut");
}

FSlateIcon FSequencerTrackFilter_CameraCut::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.Tracks.CameraCut"));
}

FText FSequencerTrackFilter_CameraCut::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_CameraCutToolTip", "Show only Camera Cut tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_CameraCut::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_CameraCut;
}

bool FSequencerTrackFilter_CameraCut::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return IsSequenceTrackSupported<UMovieSceneCameraCutTrack>(InSequence);
}

//////////////////////////////////////////////////////////////////////////
//

FSequencerTrackFilter_Fade::FSequencerTrackFilter_Fade(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter_ClassType<UMovieSceneFadeTrack>(InFilterInterface, InCategory)
{
}

FText FSequencerTrackFilter_Fade::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_Fade", "Fade");
}

FSlateIcon FSequencerTrackFilter_Fade::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.Tracks.Fade"));
}

FText FSequencerTrackFilter_Fade::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_FadeToolTip", "Show only Fade tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_Fade::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_Fade;
}

bool FSequencerTrackFilter_Fade::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return IsSequenceTrackSupported<UMovieSceneCameraCutTrack>(InSequence);
}

//////////////////////////////////////////////////////////////////////////
//

FSequencerTrackFilter_DataLayer::FSequencerTrackFilter_DataLayer(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter_ClassType<UMovieSceneDataLayerTrack>(InFilterInterface, InCategory)
{
}

FText FSequencerTrackFilter_DataLayer::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_DataLayer", "Data Layer");
}

FSlateIcon FSequencerTrackFilter_DataLayer::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.Tracks.DataLayer"));
}

FText FSequencerTrackFilter_DataLayer::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_DataLayerToolTip", "Show only Data Layer tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_DataLayer::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_DataLayer;
}

bool FSequencerTrackFilter_DataLayer::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return IsSequenceTrackSupported<UMovieSceneDataLayerTrack>(InSequence);
}

//////////////////////////////////////////////////////////////////////////
//

FSequencerTrackFilter_TimeDilation::FSequencerTrackFilter_TimeDilation(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter_ClassType<UMovieSceneSlomoTrack>(InFilterInterface, InCategory)
{
}

FText FSequencerTrackFilter_TimeDilation::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_TimeDilation", "Time Dilation");
}

FSlateIcon FSequencerTrackFilter_TimeDilation::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.Tracks.Slomo"));
}

FText FSequencerTrackFilter_TimeDilation::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_TimeDilationToolTip", "Show only Time Dilation tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_TimeDilation::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_TimeDilation;
}

bool FSequencerTrackFilter_TimeDilation::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return IsSequenceTrackSupported<UMovieSceneDataLayerTrack>(InSequence);
}

//////////////////////////////////////////////////////////////////////////
//

FSequencerTrackFilter_Folder::FSequencerTrackFilter_Folder(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FSequencerTrackFilter_ModelType<UE::Sequencer::FFolderModel>(InFilterInterface, InCategory)
{
}

FText FSequencerTrackFilter_Folder::GetDisplayName() const
{
	return LOCTEXT("SequencerTrackFilter_Folder", "Folder");
}

FSlateIcon FSequencerTrackFilter_Folder::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("ContentBrowser.AssetTreeFolderClosed"));
}

FText FSequencerTrackFilter_Folder::GetDefaultToolTipText() const
{
	return LOCTEXT("SequencerTrackFilter_FolderToolTip", "Show only Folder tracks");
}

TSharedPtr<FUICommandInfo> FSequencerTrackFilter_Folder::GetToggleCommand() const
{
	return FSequencerTrackFilterCommands::Get().ToggleFilter_Folder;
}

bool FSequencerTrackFilter_Folder::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
