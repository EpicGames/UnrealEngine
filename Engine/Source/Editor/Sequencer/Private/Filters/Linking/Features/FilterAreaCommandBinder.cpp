// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilterAreaCommandBinder.h"

#include "Filters/Linking/Mode/LinkedFilterViewModel.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "Filters/Utils/LinkedFilteringViewUtils.h"
#include "Sequencer.h"
#include "SequencerSettings.h"

namespace UE::Sequencer
{
FFilterAreaCommandBinder::FFilterAreaCommandBinder(
	FName InFilterAreaConfigId, TWeakPtr<FSequencer> InWeakSequencer, const TSharedRef<FLinkedFilterViewModel>& InLinkedFilterModel
	)
	: FilterAreaConfigId(InFilterAreaConfigId)
	, WeakSequencer(MoveTemp(InWeakSequencer))
	, LinkedFilterModel(InLinkedFilterModel)
{
	BindCommands();
	LinkedFilterModel->OnFilterModeChanged().AddRaw(this, &FFilterAreaCommandBinder::OnFilterModeChanged);
}

FFilterAreaCommandBinder::~FFilterAreaCommandBinder()
{
	// The command list may be referenced externally.
	UnbindCommands();
	LinkedFilterModel->OnFilterModeChanged().RemoveAll(this);
}

void FFilterAreaCommandBinder::SetFilterBarVisible(bool bIsVisible)
{
	const TSharedPtr<FSequencer> SequencerPin = WeakSequencer.Pin();
	USequencerSettings* Settings = SequencerPin ? SequencerPin->GetSequencerSettings() : nullptr;
	if (Settings)
	{
		constexpr bool bSaveConfig = false; // Saving unnecessary because we'll save right after settings
		Settings->FindOrAddTrackFilterArea(FilterAreaConfigId, bSaveConfig).SetIsFilterBarVisible(bIsVisible);
		Settings->SaveConfig();
	}
}

bool FFilterAreaCommandBinder::IsFilterBarVisible() const
{
	const TSharedPtr<FSequencer> SequencerPin = WeakSequencer.Pin();
	USequencerSettings* Settings = SequencerPin ? SequencerPin->GetSequencerSettings() : nullptr;
	
	constexpr bool bSaveConfig = false; // Saving unnecessary because we're just getting info and using the default if not found.
	return Settings && Settings->FindOrAddTrackFilterArea(FilterAreaConfigId, bSaveConfig).IsFilterBarVisible();
}

void FFilterAreaCommandBinder::SetPreserveFiltersOnUnlink(bool bValue)
{
	const TSharedPtr<FSequencer> SequencerPin = WeakSequencer.Pin();
	USequencerSettings* Settings = SequencerPin ? SequencerPin->GetSequencerSettings() : nullptr;
	if (Settings)
	{
		constexpr bool bSaveConfig = false; // Saving unnecessary because we'll save right after settings
		Settings->FindOrAddTrackFilterArea(FilterAreaConfigId, bSaveConfig).SetPreserveFiltersOnUnlink(bValue);
		Settings->SaveConfig();
	}
}

bool FFilterAreaCommandBinder::GetPreserveFiltersOnUnlink() const
{
	const TSharedPtr<FSequencer> SequencerPin = WeakSequencer.Pin();
	USequencerSettings* Settings = SequencerPin ? SequencerPin->GetSequencerSettings() : nullptr;
	
	constexpr bool bSaveConfig = false; // Saving unnecessary because we're just getting info and using the default if not found.
	return Settings && Settings->FindOrAddTrackFilterArea(FilterAreaConfigId, bSaveConfig).GetPreserveFiltersOnUnlink();
}

void FFilterAreaCommandBinder::BindCommands()
{
	FSequencerTrackFilterCommands& FilterCommands = FSequencerTrackFilterCommands::Get();
	
	RootCommandList->MapAction(
		FSequencerTrackFilterCommands::Get().ToggleLinkedFiltering ,
		FExecuteAction::CreateLambda([WeakViewModel = LinkedFilterModel.ToWeakPtr()] { ToggleLinkedFiltering(WeakViewModel); }),
		FCanExecuteAction::CreateLambda([]{ return true; }),
		FIsActionChecked::CreateLambda([WeakViewModel = LinkedFilterModel.ToWeakPtr()] { return IsLinkedFilteringActionChecked(WeakViewModel); })
		);
	
	RootCommandList->MapAction(
		FilterCommands.ToggleFilterBarVisibility,
		FExecuteAction::CreateRaw(this, &FFilterAreaCommandBinder::ToggleFilterBarVisibility),
		FCanExecuteAction::CreateLambda([]{ return true; }),
		FIsActionChecked::CreateRaw(this, &FFilterAreaCommandBinder::IsFilterBarVisible)
		);
	
	RootCommandList->MapAction(
		FilterCommands.TogglePreserveFiltersOnUnlink,
		FExecuteAction::CreateRaw(this, &FFilterAreaCommandBinder::TogglePreserveFiltersOnUnlink),
		FCanExecuteAction::CreateLambda([]{ return true; }),
		FIsActionChecked::CreateRaw(this, &FFilterAreaCommandBinder::GetPreserveFiltersOnUnlink)
	);
	
	RebindActiveFilterBarCommands();
}

void FFilterAreaCommandBinder::UnbindCommands()
{
	FSequencerTrackFilterCommands& FilterCommands = FSequencerTrackFilterCommands::Get();
	RootCommandList->UnmapAction(FilterCommands.ToggleLinkedFiltering);
	RootCommandList->UnmapAction(FilterCommands.ToggleFilterBarVisibility);
	RootCommandList->UnmapAction(FilterCommands.TogglePreserveFiltersOnUnlink);
}

void FFilterAreaCommandBinder::RebindActiveFilterBarCommands()
{
	ensure(ActiveFilterBarCommandList.GetSharedReferenceCount() <= 1);
	ActiveFilterBarCommandList = MakeShared<FUICommandList>(*LinkedFilterModel->GetActiveFilterBarImpl()->GetCommandList());
	
	// Append will clean up the previous, now-dead reference to ActiveFilterBarCommandList.
	RootCommandList->Append(ActiveFilterBarCommandList.ToSharedRef());
}
} // namespace UE::Sequencer
