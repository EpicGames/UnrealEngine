// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/UICommandList.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FSequencer;

namespace UE::Sequencer
{
class FLinkedFilterViewModel;

/**
 * Manages UI command bindings for a filter area.
 * - Binds persistent commands that apply regardless of linked/unlinked filter mode (e.g. toggling filter bar visibility and preserve-on-unlink).
 * - Dynamically rebinds RootCommandList to the active filter bar instance whenever ELinkedFilterMode changes.
 */
class FFilterAreaCommandBinder : public FNoncopyable
{
public:
	
	explicit FFilterAreaCommandBinder(
		FName InFilterAreaConfigId, TWeakPtr<FSequencer> InWeakSequencer, const TSharedRef<FLinkedFilterViewModel>& InLinkedFilterModel
		);
	~FFilterAreaCommandBinder();
	
	/** Sets whether the filter bar pills should be visible. */
	void SetFilterBarVisible(bool bIsVisible);
	/** @return Whether the filter bar pills should be visible */
	bool IsFilterBarVisible() const;
	/** Inverts the filter bar pill visibility. */
	void ToggleFilterBarVisibility() { SetFilterBarVisible(!IsFilterBarVisible()); }
	
	/** Sets whether the linked filter state is copied into the unlinked filter bar state when switching into unlinked filter state. */
	void SetPreserveFiltersOnUnlink(bool bValue);
	/** @return Whether the linked filter state is copied into the unlinked filter bar state when switching into unlinked filter state. */
	bool GetPreserveFiltersOnUnlink() const;
	/** Inverts preservation of linked filter state setting. */
	void TogglePreserveFiltersOnUnlink() { SetPreserveFiltersOnUnlink(!GetPreserveFiltersOnUnlink()); }
	
	/** Handles all commands related to the filter bar. The system using FFilterAreaManager should hook this command list up. */
	const TSharedRef<FUICommandList>& GetFilterAreaCommandList() const { return RootCommandList; }
	
private:
	
	/** @see USequencerSettings::FindOrAddTrackFilterArea. */
	const FName FilterAreaConfigId;
	/** Used to get the settings, e.g. to save filter bar visbility. */
	const TWeakPtr<FSequencer> WeakSequencer;
	
	/** Used to implement some of the commands. */
	const TSharedRef<FLinkedFilterViewModel> LinkedFilterModel;
	
	/** Handles all commands related to the filter bar. */
	const TSharedRef<FUICommandList> RootCommandList = MakeShared<FUICommandList>();
	/** The active filter bar's commands are bound to this command list. They are rebound every time the linked filtering mode changes. */
	TSharedPtr<FUICommandList> ActiveFilterBarCommandList;

	void BindCommands();
	void UnbindCommands();
	
	/** Responds to linked filtering mode changing */
	void OnFilterModeChanged() { RebindActiveFilterBarCommands(); }
	/** Rebinds the actions mapped in ActiveFilterBarCommandList to the active filter bar. */
	void RebindActiveFilterBarCommands();
};
} // namespace UE::Sequencer
