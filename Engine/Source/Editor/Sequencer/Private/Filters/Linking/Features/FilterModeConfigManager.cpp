// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilterModeConfigManager.h"

#include "Sequencer.h"
#include "Filters/Linking/Mode/ILinkedFilterViewModel.h"

namespace UE::Sequencer
{
namespace ConfigDetail
{
static void ApplyConfig(
	FName InFilterAreaConfigId, const TWeakPtr<FSequencer>& InWeakSequencer, const TSharedRef<ILinkedFilterViewModel>& InLinkedFilterViewModel
	)
{
	const TSharedPtr<FSequencer> Sequencer = InWeakSequencer.Pin();
	USequencerSettings* Settings = Sequencer ? Sequencer->GetSequencerSettings() : nullptr;
	FSequencerFilterAreaConfig* Config = Settings ? Settings->FindTrackFilterArea(InFilterAreaConfigId) : nullptr;
	if (Config)
	{
		InLinkedFilterViewModel->SetFilterMode(Config->IsLinkedFiltering() ? ELinkedFilterMode::Linked : ELinkedFilterMode::Instanced); 
	}
}
}

FFilterModeConfigManager::FFilterModeConfigManager(
	FName InFilterAreaConfigId, TWeakPtr<FSequencer> InWeakSequencer, const TSharedRef<ILinkedFilterViewModel>& InLinkedFilterViewModel
	)
	: FilterAreaConfigId(InFilterAreaConfigId)
	, WeakSequencer(MoveTemp(InWeakSequencer))
	, LinkingModel(InLinkedFilterViewModel)
{
	ConfigDetail::ApplyConfig(FilterAreaConfigId, WeakSequencer, LinkingModel);
	LinkingModel->OnFilterModeChanged().AddRaw(this, &FFilterModeConfigManager::OnFilterModeChanged);
}

FFilterModeConfigManager::~FFilterModeConfigManager()
{
	LinkingModel->OnFilterModeChanged().RemoveAll(this);
}

void FFilterModeConfigManager::OnFilterModeChanged() const
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	USequencerSettings* Settings = Sequencer ? Sequencer->GetSequencerSettings() : nullptr;
	if (!Settings)
	{
		return;
	}
	
	constexpr bool bSaveConfig = false;
	FSequencerFilterAreaConfig& Config = Settings->FindOrAddTrackFilterArea(FilterAreaConfigId, bSaveConfig);
	Config.SetIsLinkedFiltering(LinkingModel->GetFilterMode() == ELinkedFilterMode::Linked);
	Settings->SaveConfig();
}
} // namespace UE::Sequencer
