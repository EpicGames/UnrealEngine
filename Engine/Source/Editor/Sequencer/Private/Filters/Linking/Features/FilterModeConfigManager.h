// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FSequencer;

namespace UE::Sequencer
{
class ILinkedFilterViewModel;

/** Responsible for saving and loading FSequencerFilterAreaConfig::bIsLinkedFiltering. */
class FFilterModeConfigManager : public FNoncopyable
{
public:
	
	explicit FFilterModeConfigManager(
		FName InFilterAreaConfigId, TWeakPtr<FSequencer> InWeakSequencer, const TSharedRef<ILinkedFilterViewModel>& InLinkedFilterViewModel
		);
	~FFilterModeConfigManager();
	
private:
	
	/** @see USequencerSettings::FindOrAddTrackFilterArea. */
	const FName FilterAreaConfigId;
	/** Used to get Sequencer settings. */
	const TWeakPtr<FSequencer> WeakSequencer;
	
	/** Used to listen when the filter mode changes */
	const TSharedRef<ILinkedFilterViewModel> LinkingModel;
	
	/** Saves the filter mode to config. */
	void OnFilterModeChanged() const;
};
} // namespace UE::Sequencer
