// Copyright Epic Games, Inc. All Rights Reserved.

#include "LinkedFilterStatePreserver.h"

#include "Sequencer.h"
#include "Filters/Filters/SequencerTrackFilter_CustomText.h"
#include "Filters/Linking/Mode/LinkedFilterViewModel.h"

namespace UE::Sequencer
{
FLinkedFilterStatePreserver::FLinkedFilterStatePreserver(
	const TSharedRef<FLinkedFilterViewModel>& LinkingModel, 
	FName FilterAreaConfigId,
	const TWeakPtr<FSequencer>& WeakSequencer
	)
	: LinkingModel(LinkingModel)
	, FilterAreaConfigId(FilterAreaConfigId)
	, WeakSequencer(WeakSequencer)
{
	LinkingModel->OnFilterModeChanged().AddRaw(this, &FLinkedFilterStatePreserver::OnFilterModeChanged);
}

FLinkedFilterStatePreserver::~FLinkedFilterStatePreserver()
{
	LinkingModel->OnFilterModeChanged().RemoveAll(this);
}

void FLinkedFilterStatePreserver::OnFilterModeChanged()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	USequencerSettings* Settings = Sequencer ? Sequencer->GetSequencerSettings() : nullptr;
	if (!Settings)
	{
		return;
	}
	
	FSequencerFilterAreaConfig& Config = Settings->FindOrAddTrackFilterArea(FilterAreaConfigId);
	const bool bShouldCopyLinkedToUnlinked = LinkingModel->GetFilterMode() == ELinkedFilterMode::Instanced && Config.GetPreserveFiltersOnUnlink();
	if (bShouldCopyLinkedToUnlinked)
	{
		const TSharedRef<FSequencerFilterBar> Source = LinkingModel->GetLinkedFilterBarImpl();
		const TSharedRef<FSequencerFilterBar> Target = LinkingModel->GetInstancedFilterBarImpl();
		Target->CopyFiltersFrom(*Source);
	}
}
} // namespace UE::Sequencer
