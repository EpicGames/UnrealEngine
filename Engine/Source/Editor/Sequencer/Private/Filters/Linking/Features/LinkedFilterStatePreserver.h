// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FSequencer;

namespace UE::Sequencer
{
class FLinkedFilterViewModel;

/** When the user switched to unlinked mode, this handles copying the filters from the linked state into the unlinked state. */
class FLinkedFilterStatePreserver : public FNoncopyable
{
public:
	
	explicit FLinkedFilterStatePreserver(
		const TSharedRef<FLinkedFilterViewModel>& InLinkingModel, FName InFilterAreaConfigId, const TWeakPtr<FSequencer>& InWeakSequencer
		);
	~FLinkedFilterStatePreserver();

private:
	
	/** Used to listen when the filter mode changes */
	const TSharedRef<FLinkedFilterViewModel> LinkingModel;
	
	/** @see USequencerSettings::FindOrAddTrackFilterArea. */
	const FName FilterAreaConfigId;
	/** Used to get the settings. */
	const TWeakPtr<FSequencer> WeakSequencer;
	
	void OnFilterModeChanged();
};
} // namespace UE::Sequencer
