// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilterAreaManager.h"

#include "Sequencer.h"
#include "Filters/Linking/Mode/LinkedFilterViewModel.h"

namespace UE::Sequencer
{
FFilterAreaManager::FFilterAreaManager(
	FName InFilterAreaConfigId,
	const TWeakPtr<FSequencer>& InWeakSequencer, 
	const TSharedRef<FLinkedFilterViewModel>& InFilterModel
	)
	: FilterModel(InFilterModel)
	, CommandBinder(InFilterAreaConfigId, InWeakSequencer, FilterModel)
	, LinkedToUnlinkedCopier(FilterModel, InFilterAreaConfigId, InWeakSequencer)
	, FilterModeConfigManager(InFilterAreaConfigId, InWeakSequencer, FilterModel)
{
	FilterModel->GetInstancedFilterBar()->RequestFilterUpdate();
}
}
