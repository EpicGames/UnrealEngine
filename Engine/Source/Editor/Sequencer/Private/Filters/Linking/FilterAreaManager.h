// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/FilterAreaCommandBinder.h"
#include "Features/FilterModeConfigManager.h"
#include "Features/LinkedFilterStatePreserver.h"
#include "Framework/Commands/UICommandList.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FSequencer;
class FUICommandList;

namespace UE::Sequencer
{
class FLinkedFilterViewModel;

/** 
 * Manages a UI area that displays filters that can be linked or instanced.
 * It manages visibility of the filter pills and handles the related commands.
 */
class FFilterAreaManager : public FNoncopyable
{
public:
	
	explicit FFilterAreaManager(
		FName InFilterAreaConfigId,
		const TWeakPtr<FSequencer>& InWeakSequencer, 
		const TSharedRef<FLinkedFilterViewModel>& InFilterModel
		);
	
	const TSharedRef<FLinkedFilterViewModel>& GetFilterModel() const { return FilterModel; }
	
	/** 
	 * @return Command list mapping all commands related to the filter bar and their filters. 
	 * The owner of FFilterAreaManager should hook this up to their input handler.
	 */
	const TSharedRef<FUICommandList>& GetFilterAreaCommandList() const { return CommandBinder.GetFilterAreaCommandList(); }
	
private:
	
	/** Manages the linked and unlinked filter state. */
	const TSharedRef<FLinkedFilterViewModel> FilterModel;
	
	/** Handles the commands that operate on the filter area on a whole. */
	const FFilterAreaCommandBinder CommandBinder;
	
	/** Copies the filter state from the linked to the unlinked filter bar on switch. */
	const FLinkedFilterStatePreserver LinkedToUnlinkedCopier;
	
	/** Saves and loads the filter mode. Example: if you had UI in instanced mode, when you restart the editor, it will be in instanced mode again. */
	const FFilterModeConfigManager FilterModeConfigManager;
};
}

