// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

namespace UE::Sequencer
{
class ILinkedFilterViewModel;
struct FMakeLinkedFilterViewModelArgs;

/** 
 * Factory for creating ILinkedFilterViewModel.
 * Use this if you want to create UI that can be linked to the filters in the Sequencer UI.
 */
class ILinkedFilterFactoryViewModel
{
public:
	
	/**
	 * Creates a new view model instance for UI that wants to be able to link to the filter state in Sequencer.
	 * The view model has a reference to the shared linked filter state and its own filter state, which you can switch between.
	 * @return View model that manages linked and instanced filter state for your UI.
	 */
	virtual TSharedRef<ILinkedFilterViewModel> MakeFilteringModel(const FMakeLinkedFilterViewModelArgs& InArgs) = 0;
	
	virtual ~ILinkedFilterFactoryViewModel() = default;
};
} // namespace UE::Sequencer
