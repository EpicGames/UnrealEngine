// Copyright Epic Games, Inc. All Rights Reserved.

#include "ILinkedFilterViewModel.h"

#include "Filters/ISequencerTrackFilters.h"
#include "Filters/SequencerFilterData.h"

namespace UE::Sequencer
{
bool ILinkedFilterViewModel::IsFilteredOut(const TViewModelPtr<IOutlinerExtension>& InItem) const
{
	return GetActiveFilterBar()->GetFilterData().IsFilteredOut(InItem);
}
} // namespace UE::Sequencer
