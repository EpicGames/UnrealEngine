// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "MVVM/ViewModelPtr.h"

class ISequencerTrackFilters;

namespace UE::Sequencer
{
class IOutlinerExtension;

enum class ELinkedFilterMode : uint8
{
	/** Filtering is using a shared filterinstance */
	Linked,
	/** Filtering is using its own filter state. */
	Instanced,
	
	/** Not a real mode. Useful for static asserting. */
	Count
};

/** 
 * View-model for UI to switch between:
 * - shared filter state: this state is shared by multiple UI, allowing the user to sync multiple UI instances (Sequencer, Curve Editor, etc.)
 * - instanced filter state: this state is used only by the UI instance
 */
class ILinkedFilterViewModel
{
public:
	
	/** Sets the current filtering mode. */
	virtual void SetFilterMode(ELinkedFilterMode InNewMode) = 0;
	/** @return The current filtering mode. */
	virtual ELinkedFilterMode GetFilterMode() const = 0;
	
	/** @return Whether InItem is filtered out by the active filter bar. */
	bool IsFilteredOut(const TViewModelPtr<IOutlinerExtension>& InItem) const;
	
	/** @return The currently active filter bar instance according to the linked mode. */
	virtual TSharedRef<ISequencerTrackFilters> GetActiveFilterBar() const = 0; 
	/** @return The filter bar that is shared with all other UI. */
	virtual TSharedRef<ISequencerTrackFilters> GetLinkedFilterBar() const = 0;
	/** @return The filter bar that is only used by this instance. */
	virtual TSharedRef<ISequencerTrackFilters> GetInstancedFilterBar() const = 0;
	
	/** @return Event when the filter mode changes, i.e. the instance returned by GetActiveFilterBar changes. */
	virtual FSimpleMulticastDelegate& OnFilterModeChanged() = 0;
	
	/** @return Event when the items displayed need to be refreshed, e.g. when ELinkedFilterMode changes, or the active filter bar refilters its items. */
	virtual FSimpleMulticastDelegate& OnFilteredItemsChanged() = 0;
	
	virtual ~ILinkedFilterViewModel() = default;
};
} // namespace UE::Sequencer