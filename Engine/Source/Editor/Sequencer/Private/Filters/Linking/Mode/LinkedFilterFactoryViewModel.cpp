// Copyright Epic Games, Inc. All Rights Reserved.

#include "LinkedFilterFactoryViewModel.h"

#include "Algo/Count.h"
#include "Filters/Linking/Mode/MakeLinkedFilterViewModelArgs.h"
#include "LinkedFilterViewModel.h"

namespace UE::Sequencer
{
FLinkedFilterFactoryViewModel::FLinkedFilterFactoryViewModel(
	FSequencer& InSequencer UE_LIFETIMEBOUND,
	const TSharedRef<FSequencerFilterBar>& InSharedFilterBar,
	const TSharedRef<FFilterEvaluator> InFilterEvaluator
	)
	: Sequencer(InSequencer)
	, SharedFilterBar(InSharedFilterBar)
	, FilterEvaluator(InFilterEvaluator)
{}

int32 FLinkedFilterFactoryViewModel::GetFilterFeatureUseCount() const
{
	return Algo::CountIf(FeatureUsers, [](const TWeakPtr<FLinkedFilterViewModel>& ViewModel)
	{
		return ViewModel.IsValid();
	});
}

TSharedRef<FLinkedFilterViewModel> FLinkedFilterFactoryViewModel::MakeFilteringModelImpl(const FMakeLinkedFilterViewModelArgs& InArgs)
{
	FFilterBarInitArgs InstancedFilterBarArgs(Sequencer, InArgs.ConfigSubIdentifier, InArgs.UnlinkedFilterFlags);
	InstancedFilterBarArgs.FilterAreaConfigId = InArgs.FilterAreaConfigId;
	const TSharedRef<FSequencerFilterBar> InstancedFilterBar = FSequencerFilterBar::Make(InstancedFilterBarArgs);
	const TSharedRef<FLinkedFilterViewModel> ViewModel = MakeShared<FLinkedFilterViewModel>(SharedFilterBar, InstancedFilterBar, FilterEvaluator);
	FeatureUsers.Emplace(ViewModel.ToWeakPtr());
	return ViewModel;
}

TSharedRef<ILinkedFilterViewModel> FLinkedFilterFactoryViewModel::MakeFilteringModel(const FMakeLinkedFilterViewModelArgs& InArgs)
{
	return MakeFilteringModelImpl(InArgs);
}
} // namespace UE::Sequencer
