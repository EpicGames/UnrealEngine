// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerFilterBar.h"
#include "Filters/Linking/Mode/ILinkedFilterFactoryViewModel.h"
#include "HAL/Platform.h"

namespace UE::Sequencer
{
class FFilterEvaluator;
class FLinkedFilterViewModel;

/**
 * Knows how to create FLinkedFilterViewModel. 
 * This factory keep a shared FSequencerFilterBar state that is used when a FLinkedFilterViewModel is put into linked mode.
 */
class FLinkedFilterFactoryViewModel : public ILinkedFilterFactoryViewModel
{
public:
	
	explicit FLinkedFilterFactoryViewModel(
		FSequencer& InSequencer UE_LIFETIMEBOUND,
		const TSharedRef<FSequencerFilterBar>& InSharedFilterBar,
		const TSharedRef<FFilterEvaluator> InFilterEvaluator
		);

	/**
	 * @return The number of features using the linked filtering feature.
	 * 
	 * @note This is determined by looking at the number of valid TWeakPtr<FLinkedFilterViewModel>. 
	 * It is assumed that if a pointer is valid, the system that created it is still using it.
	 */
	int32 GetFilterFeatureUseCount() const;
	
	/** @return View model that manages linked and instanced filter state for your UI.  */
	TSharedRef<FLinkedFilterViewModel> MakeFilteringModelImpl(const FMakeLinkedFilterViewModelArgs& InArgs);
	
	//~ Begin ILinkedFilterFactoryViewModel
	virtual TSharedRef<ILinkedFilterViewModel> MakeFilteringModel(const FMakeLinkedFilterViewModelArgs& InArgs) override;
	//~ End ILinkedFilterFactoryViewModel
	
private:
	
	/** Sequencer */
	FSequencer& Sequencer;
	
	/** The filter bar instance that model edit when they are in linked mode. All created models reference this. */
	const TSharedRef<FSequencerFilterBar> SharedFilterBar;
	/** Instance that the active filter bar instances are registered with. This way Sequencer will issue a refilter eventually. */
	const TSharedRef<FFilterEvaluator> FilterEvaluator;
	
	/** Used to determine how many UI systems are using the linked filtering feature. */
	TArray<TWeakPtr<FLinkedFilterViewModel>> FeatureUsers;
};
} // namespace UE::Sequencer
