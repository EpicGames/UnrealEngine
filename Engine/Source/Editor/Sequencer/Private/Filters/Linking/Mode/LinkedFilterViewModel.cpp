// Copyright Epic Games, Inc. All Rights Reserved.

#include "LinkedFilterViewModel.h"

#include "Filters/SequencerFilterBar.h"
#include "Filters/Evaluation/FilterEvaluator.h"

namespace UE::Sequencer
{
FLinkedFilterViewModel::FLinkedFilterViewModel(
	const TSharedRef<FSequencerFilterBar>& InSharedFilterBar,
	const TSharedRef<FSequencerFilterBar>& InInstancedFilterBar,
	const TSharedRef<FFilterEvaluator> InFilterEvaluator
	)
	: LinkedFilterBar(InSharedFilterBar)
	, InstancedFilterBar(InInstancedFilterBar)
	, FilterEvaluator(InFilterEvaluator)
	, ActiveFilterBarUsage(GetActiveFilterBarImpl(), FilterEvaluator)
{
	FilterEvaluator->OnFilterBarFiltered().AddRaw(this, &FLinkedFilterViewModel::OnFilterBarFiltered);
}

FLinkedFilterViewModel::~FLinkedFilterViewModel()
{
	FilterEvaluator->OnFilterBarFiltered().RemoveAll(this);
}

TSharedRef<FSequencerFilterBar> FLinkedFilterViewModel::GetActiveFilterBarImpl() const
{
	static_assert(static_cast<int32>(ELinkedFilterMode::Count) == 2, "Update this switch");
	switch (FilterMode)
	{
	case ELinkedFilterMode::Linked: return LinkedFilterBar;
	case ELinkedFilterMode::Instanced: return InstancedFilterBar;
	default: checkNoEntry(); return LinkedFilterBar;
	}
}

TSharedRef<FSequencerFilterBar> FLinkedFilterViewModel::GetLinkedFilterBarImpl() const
{
	return LinkedFilterBar;
}

TSharedRef<FSequencerFilterBar> FLinkedFilterViewModel::GetInstancedFilterBarImpl() const
{
	return InstancedFilterBar;
}

void FLinkedFilterViewModel::SetFilterMode(ELinkedFilterMode InNewMode)
{
	if (InNewMode != FilterMode)
	{
		FilterMode = InNewMode;
		
		const TSharedRef<FSequencerFilterBar> NewActiveFilterBar = GetActiveFilterBarImpl();
		ActiveFilterBarUsage = FScopedFilterBarUsage(NewActiveFilterBar, FilterEvaluator);
		OnFilterModeChangedDelegate.Broadcast();
		OnFilteredItemsChangedDelegate.Broadcast();
	}
}

TSharedRef<ISequencerTrackFilters> FLinkedFilterViewModel::GetActiveFilterBar() const
{
	return GetActiveFilterBarImpl();
}

TSharedRef<ISequencerTrackFilters> FLinkedFilterViewModel::GetLinkedFilterBar() const
{
	return LinkedFilterBar;
}

TSharedRef<ISequencerTrackFilters> FLinkedFilterViewModel::GetInstancedFilterBar() const
{
	return InstancedFilterBar;
}

void FLinkedFilterViewModel::OnFilterBarFiltered(const TSharedRef<FSequencerFilterBar>& InFilterBar)
{
	if (InFilterBar == GetActiveFilterBarImpl())
	{
		OnFilteredItemsChangedDelegate.Broadcast();
	}
}
} // namespace UE::Sequencer
