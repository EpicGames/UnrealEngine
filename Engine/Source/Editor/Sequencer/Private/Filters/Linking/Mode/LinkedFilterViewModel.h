// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/Evaluation/ScopedFilterBarUsage.h"
#include "Filters/Linking/Mode/ILinkedFilterViewModel.h"
#include "Templates/SharedPointer.h"

class FSequencer;
class FSequencerFilterBar;

namespace UE::Sequencer
{
class FFilterEvaluator;

class FLinkedFilterViewModel : public ILinkedFilterViewModel
{
public:
	
	explicit FLinkedFilterViewModel(
		const TSharedRef<FSequencerFilterBar>& InSharedFilterBar,
		const TSharedRef<FSequencerFilterBar>& InInstancedFilterBar,
		const TSharedRef<FFilterEvaluator> InFilterEvaluator
		);
	~FLinkedFilterViewModel();
	
	TSharedRef<FSequencerFilterBar> GetActiveFilterBarImpl() const;
	TSharedRef<FSequencerFilterBar> GetLinkedFilterBarImpl() const;
	TSharedRef<FSequencerFilterBar> GetInstancedFilterBarImpl() const;
	
	//~ Begin ILinkedFilterViewModel
	virtual void SetFilterMode(ELinkedFilterMode InNewMode) override;
	virtual ELinkedFilterMode GetFilterMode() const override { return FilterMode; }
	virtual TSharedRef<ISequencerTrackFilters> GetActiveFilterBar() const override;
	virtual TSharedRef<ISequencerTrackFilters> GetLinkedFilterBar() const override;
	virtual TSharedRef<ISequencerTrackFilters> GetInstancedFilterBar() const override;
	virtual FSimpleMulticastDelegate& OnFilterModeChanged() override { return OnFilterModeChangedDelegate; }
	virtual FSimpleMulticastDelegate& OnFilteredItemsChanged() override { return OnFilteredItemsChangedDelegate; }
	//~ End ILinkedFilterViewModel
	
private:
	
	/** The filter bar instance that is shared across all UI. Other UI can choose to link to this instance. */
	const TSharedRef<FSequencerFilterBar> LinkedFilterBar;
	/** The filter bar instance that is owned by the particular view that is displaying the filters, e.g. Curve Editor. */
	const TSharedRef<FSequencerFilterBar> InstancedFilterBar;
	
	/** The current filter mode. It determines which filter bar instance is used. */
	ELinkedFilterMode FilterMode = ELinkedFilterMode::Linked;
	
	/** Ensures that the active filter is refiltered when a change is requested on it. The active filter is always registered with it. */
	const TSharedRef<FFilterEvaluator> FilterEvaluator;
	/** When the active filter bar changes, this is leveraged to orchestrate FFilterEvaluator::IncrementFilterBarUsage and DecrementFilterBarUsage. */
	FScopedFilterBarUsage ActiveFilterBarUsage;
	
	/** Event when the filter mode changes, i.e. the instance returned by GetActiveFilterBar changes. */
	FSimpleMulticastDelegate OnFilterModeChangedDelegate;
	/** Event when the items displayed need to be refreshed, e.g. when ELinkedFilterMode changes, or the active filter bar refilters its items. */
	FSimpleMulticastDelegate OnFilteredItemsChangedDelegate;
	
	void OnFilterBarFiltered(const TSharedRef<FSequencerFilterBar>& InFilterBar);
};
} // namespace UE::Sequencer
