// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Filters/FilterBarFlags.h"
#include "UObject/NameTypes.h"

namespace UE::Sequencer
{
/** @see ILinkedFilterFactoryViewModel::MakeFilteringModel */
struct FMakeLinkedFilterViewModelArgs
{
	/**
	 * The sub-identifier to used for saving and loading filter entries.
	 * 
	 * The final identifier is the name of the settings (USequencerSettings::GetName) + "." + ConfigSubIdentifier.
	 * Example: "LevelSequencerEditor.Linked", "LevelSequencerEditor.CurveEditor", etc.
	 * 
	 * @see USequencerSettings::TrackFilterBars.
	 */
	FString ConfigSubIdentifier;
	
	/**
	 * The filter flags to use for the unlinked filter bar.
	 *
	 * By default, unlinked filter state should not be affected by the USequencerSettings::bIncludePinnedInFilter setting, so you likely don't want
	 * to set PinnedItemsCanSkipFiltering.
	 */
	EFilterFlags UnlinkedFilterFlags = EFilterFlags::None;

	/**
	 * Optional. ID used to look up the filter area config (FSequencerFilterAreaConfig).
	 * When set, allows the instanced filter bar to bind the ToggleFilterBarVisibility command.
	 * @see USequencerSettings::FindOrAddTrackFilterArea
	 */
	FName FilterAreaConfigId = NAME_None;

	explicit FMakeLinkedFilterViewModelArgs(FString InConfigSubIdentifier, EFilterFlags InFilterFlags = EFilterFlags::None)
		: ConfigSubIdentifier(MoveTemp(InConfigSubIdentifier))
		, UnlinkedFilterFlags(InFilterFlags)
		{}
};
} // namespace UE::Sequencer
