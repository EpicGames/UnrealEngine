// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "SequencerFilterBarContext.generated.h"

class FSequencerFilterBar;
class UToolMenu;

DECLARE_DELEGATE_OneParam(FOnPopulateFilterBarMenu, UToolMenu*);

UCLASS()
class USequencerFilterBarContext : public UObject
{
	GENERATED_BODY()

public:
	void Init(const TWeakPtr<FSequencerFilterBar>& InFilterBarWeak)
	{
		WeakFilterBar = InFilterBarWeak;
	}

	TSharedPtr<FSequencerFilterBar> GetFilterBar() const
	{
		return WeakFilterBar.IsValid() ? WeakFilterBar.Pin() : nullptr;
	}

	FOnPopulateFilterBarMenu OnPopulateFilterBarMenu;

protected:
	TWeakPtr<FSequencerFilterBar> WeakFilterBar;
};
