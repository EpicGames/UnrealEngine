// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerFilterBarContextMenu.h"
#include "SequencerFilterBarContext.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Filters/SequencerFilterBar.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "Filters/Utils/FilterViewUtils.h"
#include "Filters/Widgets/SSequencerFilterBar.h"

#define LOCTEXT_NAMESPACE "SequencerFilterBarContextMenu"

TSharedRef<SWidget> FSequencerFilterBarContextMenu::CreateMenu(const TSharedRef<FSequencerFilterBar>& InFilterBar)
{
	const FName FilterMenuName = TEXT("Sequencer.FilterBarContextMenu");
	if (!UToolMenus::Get()->IsMenuRegistered(FilterMenuName))
	{
		UToolMenu* const Menu = UToolMenus::Get()->RegisterMenu(FilterMenuName);
		Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([this](UToolMenu* const InMenu)
			{
				if (USequencerFilterBarContext* const Context = InMenu->FindContext<USequencerFilterBarContext>())
				{
					Context->OnPopulateFilterBarMenu.ExecuteIfBound(InMenu);
				}
			}));
	}

	USequencerFilterBarContext* const ContextObject = NewObject<USequencerFilterBarContext>();
	ContextObject->Init(InFilterBar);
	ContextObject->OnPopulateFilterBarMenu = FOnPopulateFilterBarMenu::CreateRaw(this, &FSequencerFilterBarContextMenu::PopulateMenu);

	const FToolMenuContext MenuContext(InFilterBar->GetCommandList(), nullptr, ContextObject);
	return UToolMenus::Get()->GenerateWidget(FilterMenuName, MenuContext);
}

void FSequencerFilterBarContextMenu::PopulateMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	USequencerFilterBarContext* const Context = InMenu->FindContext<USequencerFilterBarContext>();
	if (!Context)
	{
		return;
	}

	WeakFilterBar = Context->GetFilterBar();

	UToolMenu& MenuRef = *InMenu;

	PopulateOptionsSection(MenuRef);
}

void FSequencerFilterBarContextMenu::PopulateOptionsSection(UToolMenu& InMenu)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const FSequencerTrackFilterCommands& TrackFilterCommands = FSequencerTrackFilterCommands::Get();

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("Options")
		, LOCTEXT("OptionsHeading", "Filter Bar Options"));

	Section.AddMenuEntry(TrackFilterCommands.ToggleFilterBarVisibility);

	Section.AddSeparator(NAME_Name);
	Section.AddMenuEntry(
		TEXT("SaveCurrentFilterSetAsCustomTextFilter"),
		LOCTEXT("SaveCurrentFilterSetAsCustomTextFilter", "Save Current as New Filter"),
		LOCTEXT("SaveCurrentFilterSetAsCustomTextFilterTooltip", "Saves the enabled and active set of common filters as a custom text filter"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("LevelEditor.SaveAs")),
		FUIAction(FExecuteAction::CreateLambda([WeakFilterBar = FilterBar.ToWeakPtr()]()
			{
				if (const TSharedPtr<FSequencerFilterBar> FilterBarPin = WeakFilterBar.Pin())
				{
					UE::Sequencer::OpenDialog_SaveCurrentFilterSetAsCustomTextFilter(FilterBarPin.ToSharedRef());
				}
			}))
		);
	Section.AddSeparator(NAME_Name);

	PopulateFilterBulkOptionsSection(InMenu);
}

void FSequencerFilterBarContextMenu::PopulateFilterBulkOptionsSection(UToolMenu& InMenu)
{
	const FSequencerTrackFilterCommands& TrackFilterCommands = FSequencerTrackFilterCommands::Get();

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("FilterBulkOptions")
		, LOCTEXT("BulkOptionsContextHeading", "Filter Bulk Options"));

	Section.AddMenuEntry(TEXT("ActivateAllFilters"),
		LOCTEXT("ActivateAllFilters", "Activate All Filters"),
		LOCTEXT("ActivateAllFiltersTooltip", "Activates all enabled filters."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Plus")),
		FUIAction(FExecuteAction::CreateRaw(this, &FSequencerFilterBarContextMenu::OnActivateAllFilters, true)));

	Section.AddMenuEntry(TEXT("DeactivateAllFilters"),
		LOCTEXT("DeactivateAllFilters", "Deactivate All Filters"),
		LOCTEXT("DeactivateAllFiltersTooltip", "Deactivates all enabled filters."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Minus")),
		FUIAction(FExecuteAction::CreateRaw(this, &FSequencerFilterBarContextMenu::OnActivateAllFilters, false)));

	Section.AddSeparator(NAME_None);

	Section.AddMenuEntry(TrackFilterCommands.ResetFilters
		, TrackFilterCommands.ResetFilters->GetLabel()
		, TrackFilterCommands.ResetFilters->GetDescription()
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault")));

	Section.AddSeparator(NAME_None);

	Section.AddMenuEntry(TrackFilterCommands.ToggleMuteFilters
		, TrackFilterCommands.ToggleMuteFilters->GetLabel()
		, TrackFilterCommands.ToggleMuteFilters->GetDescription()
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Denied")));
}

void FSequencerFilterBarContextMenu::OnActivateAllFilters(const bool bInActivate)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FilterBar->ActivateAllEnabledFilters(bInActivate, {});
}

#undef LOCTEXT_NAMESPACE
