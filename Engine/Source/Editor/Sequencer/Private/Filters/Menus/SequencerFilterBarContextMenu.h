// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FSequencerFilterBar;
class SWidget;
class UToolMenu;

class FSequencerFilterBarContextMenu : public TSharedFromThis<FSequencerFilterBarContextMenu>
{
public:
	TSharedRef<SWidget> CreateMenu(const TSharedRef<FSequencerFilterBar>& InFilterBar);

protected:
	void PopulateMenu(UToolMenu* const InMenu);

	void PopulateOptionsSection(UToolMenu& InMenu);
	void PopulateFilterBulkOptionsSection(UToolMenu& InMenu);

	void OnActivateAllFilters(const bool bInActivate);

	TWeakPtr<FSequencerFilterBar> WeakFilterBar;
};
