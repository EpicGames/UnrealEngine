// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerTrackFilterBase.h"
#include "SequencerFilterBarContext.h"
#include "Templates/SharedPointer.h"
#include "SequencerFilterMenuContext.generated.h"

UCLASS()
class USequencerFilterMenuContext : public UObject
{
	GENERATED_BODY()

public:
	void Init(const TWeakPtr<FSequencerTrackFilter>& InFilter)
	{
		WeakFilter = InFilter;
	}

	TSharedPtr<FSequencerTrackFilter> GetFilter() const
	{
		return WeakFilter.Pin();
	}

	FOnPopulateFilterBarMenu OnPopulateFilterBarMenu;

protected:
	TWeakPtr<FSequencerTrackFilter> WeakFilter;
};
