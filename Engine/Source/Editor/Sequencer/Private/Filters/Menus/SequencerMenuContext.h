// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerFilterBarContext.h"
#include "Templates/SharedPointer.h"
#include "SequencerMenuContext.generated.h"

class FSequencer;

UCLASS()
class USequencerMenuContext : public UObject
{
	GENERATED_BODY()

public:
	void Init(const TWeakPtr<FSequencer>& InWeakSequencer)
	{
		WeakSequencer = InWeakSequencer;
	}

	TSharedPtr<FSequencer> GetSequencer() const
	{
		return WeakSequencer.Pin();
	}

	FOnPopulateFilterBarMenu OnPopulateFilterBarMenu;

protected:
	TWeakPtr<FSequencer> WeakSequencer;
};
