// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTrackFilterContextMenu.h"
#include "Filters/Filters/SequencerTrackFilter_CustomText.h"
#include "Filters/Menus/SequencerFilterMenuContext.h"
#include "Filters/SequencerFilterBar.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "Filters/Widgets/SSequencerCustomTextFilterDialog.h"
#include "Sequencer.h"
#include "SequencerSettings.h"
#include "ToolMenu.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "SequencerTrackFilterContextMenu"

TSharedRef<SWidget> FSequencerTrackFilterContextMenu::CreateMenuWidget(const TSharedRef<FSequencerTrackFilter>& InFilter)
{
	UToolMenus* const ToolMenus = UToolMenus::Get();
	check(ToolMenus);

	const FName FilterMenuName = TEXT("Sequencer.TrackFilterContextMenu");
	if (!ToolMenus->IsMenuRegistered(FilterMenuName))
	{
		UToolMenu* const Menu = ToolMenus->RegisterMenu(FilterMenuName);
		Menu->AddDynamicSection(TEXT("PopulateMenu"), FNewToolMenuDelegate::CreateLambda([this](UToolMenu* const InMenu)
			{
				if (USequencerFilterMenuContext* const Context = InMenu->FindContext<USequencerFilterMenuContext>())
				{
					Context->OnPopulateFilterBarMenu.ExecuteIfBound(InMenu);
				}
			}));
	}

	const TSharedPtr<FUICommandList> CommandList = InFilter->GetFilterInterface().GetCommandList();

	USequencerFilterMenuContext* const ContextObject = NewObject<USequencerFilterMenuContext>();
	ContextObject->Init(InFilter);
	ContextObject->OnPopulateFilterBarMenu = FOnPopulateFilterBarMenu::CreateSP(this, &FSequencerTrackFilterContextMenu::PopulateMenu);

	const FToolMenuContext MenuContext(CommandList, nullptr, ContextObject);
	return UToolMenus::Get()->GenerateWidget(FilterMenuName, MenuContext);
}

void FSequencerTrackFilterContextMenu::PopulateMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	USequencerFilterMenuContext* const Context = InMenu->FindContext<USequencerFilterMenuContext>();
	if (!Context)
	{
		return;
	}

	WeakFilter = Context->GetFilter();

	UToolMenu& MenuRef = *InMenu;

	PopulateFilterOptionsSection(MenuRef);
	PopulateCustomFilterOptionsSection(MenuRef);
	PopulateBulkOptionsSection(MenuRef);
}

void FSequencerTrackFilterContextMenu::PopulateFilterOptionsSection(UToolMenu& InMenu)
{
	const TSharedPtr<FSequencerTrackFilter> Filter = GetFilter();
	if (!Filter.IsValid())
	{
		return;
	}

	const FText FilterName = GetFilterDisplayName();

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("FilterOptions")
		, LOCTEXT("FilterOptionsContextHeading", "Filter Options"));

	Section.AddMenuEntry(TEXT("ActivateOnlyThisFilter"),
		FText::Format(LOCTEXT("ActivateOnlyThisFilter", "Activate Only: {0}"), FilterName),
		LOCTEXT("ActivateOnlyThisFilterTooltip", "Activate only this filter from the list."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Plus")),
		FUIAction(FExecuteAction::CreateSP(this, &FSequencerTrackFilterContextMenu::OnActivateWithFilterException)));

	Section.AddMenuEntry(TEXT("DisableFilter"),
		FText::Format(LOCTEXT("DisableFilter", "Remove: {0}"), FilterName),
		LOCTEXT("DisableFilterTooltip", "Disable this filter and remove it from the list. It can be added again in the filters menu."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Minus")),
		FUIAction(FExecuteAction::CreateSP(this, &FSequencerTrackFilterContextMenu::OnDisableFilter)));
}

void FSequencerTrackFilterContextMenu::PopulateCustomFilterOptionsSection(UToolMenu& InMenu)
{
	const TSharedPtr<FSequencerTrackFilter_CustomText> CustomTextFilter
		= StaticCastSharedPtr<FSequencerTrackFilter_CustomText>(GetFilter());
	if (!CustomTextFilter.IsValid() || !CustomTextFilter->IsCustomTextFilter())
	{
		return;
	}

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("CustomFilterOptions")
		, LOCTEXT("CustomFilterOptionsContextHeading", "Custom Filter Options"));

	Section.AddMenuEntry(TEXT("EditCustomTextFilter"),
		FText::Format(LOCTEXT("EditCustomTextFilter", "Edit: {0}"), CustomTextFilter->GetDisplayName()),
		LOCTEXT("EditCustomTextFilterTooltip", "Edit this custom text filter saved to config."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Edit")),
		FUIAction(FExecuteAction::CreateSP(this, &FSequencerTrackFilterContextMenu::OnEditFilter)));

	Section.AddMenuEntry(TEXT("DeleteCustomTextFilter"),
		FText::Format(LOCTEXT("DeleteCustomTextFilter", "Delete: {0}"), CustomTextFilter->GetDisplayName()),
		LOCTEXT("DeleteCustomTextFilterTooltip", "Delete this custom text filter from config.\n\nCAUTION: This cannot be undone!"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Delete")),
		FUIAction(FExecuteAction::CreateSP(this, &FSequencerTrackFilterContextMenu::OnDeleteFilter)));
}

void FSequencerTrackFilterContextMenu::PopulateBulkOptionsSection(UToolMenu& InMenu)
{
	const FSequencerTrackFilterCommands& TrackFilterCommands = FSequencerTrackFilterCommands::Get();

	const FText FilterName = GetFilterDisplayName();

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("FilterBulkOptions")
		, LOCTEXT("BulkOptionsContextHeading", "Filter Bulk Options"));

	Section.AddMenuEntry(TrackFilterCommands.ActivateAllFilters
		, TrackFilterCommands.ActivateAllFilters->GetLabel()
		, TrackFilterCommands.ActivateAllFilters->GetDescription()
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Plus")));

	Section.AddMenuEntry(TrackFilterCommands.DeactivateAllFilters
		, TrackFilterCommands.DeactivateAllFilters->GetLabel()
		, TrackFilterCommands.DeactivateAllFilters->GetDescription()
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Minus")));

	Section.AddSeparator(NAME_None);

	Section.AddMenuEntry(TrackFilterCommands.ResetFilters
		, TrackFilterCommands.ResetFilters->GetLabel()
		, TrackFilterCommands.ResetFilters->GetDescription()
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault")));

	Section.AddSeparator(NAME_None);

	Section.AddMenuEntry(TrackFilterCommands.ToggleMuteFilters
		, TrackFilterCommands.ToggleMuteFilters->GetLabel()
		, TrackFilterCommands.ToggleMuteFilters->GetDescription()
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Denied")));
}

FText FSequencerTrackFilterContextMenu::GetFilterDisplayName() const
{
	const TSharedPtr<FSequencerTrackFilter> Filter = GetFilter();
	if (!Filter.IsValid())
	{
		return FText::GetEmpty();
	}

	return Filter.IsValid() ? Filter->GetDisplayName() : FText::GetEmpty();
}

void FSequencerTrackFilterContextMenu::OnDisableFilter()
{
	const TSharedPtr<FSequencerTrackFilter> Filter = GetFilter();
	if (!Filter.IsValid())
	{
		return;
	}

	const FString FilterName = Filter->GetDisplayName().ToString();
	Filter->GetFilterInterface().SetFilterEnabledByDisplayName(FilterName, false);
}

void FSequencerTrackFilterContextMenu::OnActivateWithFilterException()
{
	const TSharedPtr<FSequencerTrackFilter> Filter = GetFilter();
	if (!Filter.IsValid())
	{
		return;
	}

	ISequencerTrackFilters& FilterInterface = Filter->GetFilterInterface();
	const FString FilterName = Filter->GetDisplayName().ToString();

	FilterInterface.ActivateAllEnabledFilters(false, { FilterName });
	FilterInterface.SetFilterActiveByDisplayName(FilterName, true);
}

void FSequencerTrackFilterContextMenu::OnActivateAllFilters(const bool bInActivate)
{
	const TSharedPtr<FSequencerTrackFilter> Filter = GetFilter();
	if (!Filter.IsValid())
	{
		return;
	}

	Filter->GetFilterInterface().ActivateAllEnabledFilters(bInActivate, {});
}

void FSequencerTrackFilterContextMenu::OnEditFilter()
{
	const TSharedPtr<FSequencerTrackFilter> Filter = GetFilter();
	if (!Filter.IsValid() || !Filter->IsCustomTextFilter())
	{
		return;
	}

	const TSharedPtr<FSequencerTrackFilter_CustomText> CustomTextFilter = StaticCastSharedPtr<FSequencerTrackFilter_CustomText>(Filter);
	if (!CustomTextFilter.IsValid())
	{
		return;
	}

	ISequencerTrackFilters& FilterInterface = Filter->GetFilterInterface();
	const TSharedRef<ISequencerTrackFilters> FilterBar = StaticCastSharedRef<ISequencerTrackFilters>(FilterInterface.AsShared());

	SSequencerCustomTextFilterDialog::CreateWindow_EditCustomTextFilter(FilterBar, CustomTextFilter);
}

void FSequencerTrackFilterContextMenu::OnDeleteFilter()
{
	const TSharedPtr<FSequencerTrackFilter> Filter = GetFilter();
	if (!Filter.IsValid() || !Filter->IsCustomTextFilter())
	{
		return;
	}

	const TSharedPtr<FSequencerTrackFilter_CustomText> CustomTextFilter = StaticCastSharedPtr<FSequencerTrackFilter_CustomText>(Filter);
	if (!CustomTextFilter.IsValid())
	{
		return;
	}

	ISequencerTrackFilters& FilterInterface = Filter->GetFilterInterface();
	const FString FilterName = Filter->GetDisplayName().ToString();

	FilterInterface.SetFilterActiveByDisplayName(FilterName, false);
	FilterInterface.RemoveCustomTextFilter(CustomTextFilter.ToSharedRef(), false);

	if (USequencerSettings* const SequencerSettings = FilterInterface.GetSequencer().GetSequencerSettings())
	{
		FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(FilterInterface.GetIdentifier(), false);

		if (!Config.RemoveCustomTextFilter(FilterName))
		{
			return;
		}

		SequencerSettings->SaveConfig();
	}
}

const TSharedPtr<FSequencerTrackFilter> FSequencerTrackFilterContextMenu::GetFilter() const
{
	return WeakFilter.IsValid() ? WeakFilter.Pin() : nullptr;
}

#undef LOCTEXT_NAMESPACE
