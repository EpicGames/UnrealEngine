// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FSequencerTrackFilter;
class SWidget;
class UToolMenu;

class FSequencerTrackFilterContextMenu : public TSharedFromThis<FSequencerTrackFilterContextMenu>
{
public:
	TSharedRef<SWidget> CreateMenuWidget(const TSharedRef<FSequencerTrackFilter>& InFilter);

protected:
	void PopulateMenu(UToolMenu* const InMenu);

	void PopulateFilterOptionsSection(UToolMenu& InMenu);
	void PopulateCustomFilterOptionsSection(UToolMenu& InMenu);
	void PopulateBulkOptionsSection(UToolMenu& InMenu);

	FText GetFilterDisplayName() const;

	void OnDisableFilter();

	void OnActivateWithFilterException();

	void OnActivateAllFilters(const bool bInActivate);

	void OnEditFilter();
	void OnDeleteFilter();

	const TSharedPtr<FSequencerTrackFilter> GetFilter() const;

	TWeakPtr<FSequencerTrackFilter> WeakFilter;
};
