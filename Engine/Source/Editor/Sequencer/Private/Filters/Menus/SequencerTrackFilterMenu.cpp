// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTrackFilterMenu.h"
#include "Filters/SequencerFilterBar.h"
#include "Filters/Filters/SequencerTrackFilter_CustomText.h"
#include "Filters/Filters/SequencerTrackFilter_Level.h"
#include "Filters/Filters/SequencerTrackFilter_Group.h"
#include "Filters/Widgets/SSequencerFilterBar.h"
#include "Filters/Widgets/SSequencerCustomTextFilterDialog.h"
#include "ISequencer.h"
#include "LevelSequence.h"
#include "Misc/MessageDialog.h"
#include "MovieScene.h"
#include "SequencerFilterBarContext.h"
#include "SSequencer.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "Filters/Utils/FilterViewUtils.h"

#define LOCTEXT_NAMESPACE "SequencerTrackFilterMenu"

TSharedRef<SWidget> FSequencerTrackFilterMenu::CreateMenu(const TSharedRef<FSequencerFilterBar>& InFilterBar)
{
	const FName FilterMenuName = TEXT("Sequencer.TrackFilterMenu");
	if (!UToolMenus::Get()->IsMenuRegistered(FilterMenuName))
	{
		UToolMenu* const Menu = UToolMenus::Get()->RegisterMenu(FilterMenuName);
		Menu->bShouldCloseWindowAfterMenuSelection = false;
		Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([this](UToolMenu* const InMenu)
			{
				if (USequencerFilterBarContext* const Context = InMenu->FindContext<USequencerFilterBarContext>())
				{
					Context->OnPopulateFilterBarMenu.ExecuteIfBound(InMenu);
				}
			}));
	}

	USequencerFilterBarContext* const ContextObject = NewObject<USequencerFilterBarContext>();
	ContextObject->Init(InFilterBar);
	ContextObject->OnPopulateFilterBarMenu = FOnPopulateFilterBarMenu::CreateSP(this, &FSequencerTrackFilterMenu::PopulateMenu);

	const FToolMenuContext MenuContext(InFilterBar->GetCommandList(), nullptr, ContextObject);
	return UToolMenus::Get()->GenerateWidget(FilterMenuName, MenuContext);
}

void FSequencerTrackFilterMenu::PopulateMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	USequencerFilterBarContext* const Context = InMenu->FindContext<USequencerFilterBarContext>();
	if (!Context)
	{
		return;
	}

	WeakFilterBar = Context->GetFilterBar();

	UToolMenu& Menu = *InMenu;

	PopulateFilterOptionsSection(Menu);
	PopulateCustomsSection(Menu);
	PopulateCommonFilterSections(Menu);
	PopulateOtherFilterSections(Menu);
}

void FSequencerTrackFilterMenu::PopulateCustomsSection(UToolMenu& InMenu)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("UserCreated"));

	Section.AddSubMenu(TEXT("CustomTextFilters"),
		LOCTEXT("CustomTextFilters_SubMenuLabel", "Custom Text Filters"),
		LOCTEXT("CustomTextFilters_SubMenuTooltip", "Custom Text Filters"),
		FNewToolMenuDelegate::CreateSP(this, &FSequencerTrackFilterMenu::FillCustomTextFiltersMenu),
		false,
		FSlateIcon(),
		false);
}

void FSequencerTrackFilterMenu::PopulateFilterOptionsSection(UToolMenu& InMenu)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("FilterOptions"), LOCTEXT("FilterOptionsHeading", "Filters"));

	Section.AddMenuEntry(TEXT("ResetFilters"),
		LOCTEXT("FilterListResetFilters", "Reset Filters"),
		LOCTEXT("FilterListResetToolTip", "Resets current filter selection"),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault")),
		FUIAction(
			FExecuteAction::CreateSP(this, &FSequencerTrackFilterMenu::ResetFilters),
			FCanExecuteAction::CreateSP(this, &FSequencerTrackFilterMenu::CanResetFilters)
		));

	const USequencerSettings* const SequencerSettings = FilterBar->GetSequencer().GetSequencerSettings();
	if (SequencerSettings && SequencerSettings->GetUseFilterSubmenusForCategories())
	{
		Section.AddSeparator(NAME_None);
	}
}

void FSequencerTrackFilterMenu::PopulateCommonFilterSections(UToolMenu& InMenu)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const USequencerSettings* const SequencerSettings = FilterBar->GetSequencer().GetSequencerSettings();
	if (!SequencerSettings)
	{
		return;
	}

	if (SequencerSettings->GetUseFilterSubmenusForCategories())
	{
		FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("OtherFilters"), LOCTEXT("OtherFiltersHeading", "Other Filters"));

		for (const TSharedRef<FFilterCategory>& Category : FilterBar->GetFilterCategories())
		{
			Section.AddSubMenu(*Category->Title.ToString(),
				Category->Title,
				Category->Tooltip,
				FNewToolMenuDelegate::CreateSP(this, &FSequencerTrackFilterMenu::FillFiltersMenuCategory, Category),
				FUIAction(
					FExecuteAction::CreateSP(this, &FSequencerTrackFilterMenu::OnFilterCategoryClicked, Category),
					FCanExecuteAction(),
					FGetActionCheckState::CreateSP(this, &FSequencerTrackFilterMenu::GetFilterCategoryCheckedState, Category)
				),
				EUserInterfaceActionType::ToggleButton,
				false,
				FSlateIcon(),
				false);
		}
	}
	else
	{
		for (const TSharedRef<FFilterCategory>& Category : FilterBar->GetFilterCategories())
		{
			FToolMenuSection& Section = InMenu.FindOrAddSection(*Category->Title.ToString(), Category->Title);
			FillFiltersMenuCategory(Section, Category);
		}
	}
}

void FSequencerTrackFilterMenu::PopulateOtherFilterSections(UToolMenu& InMenu)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("OtherFilters"), LOCTEXT("OtherFiltersHeading", "Other Filters"));

	if (FilterBar->IsFilterSupported(FSequencerTrackFilter_Level::StaticName()))
	{
		Section.AddSubMenu(TEXT("LevelFilters"),
			LOCTEXT("LevelFilters", "Level Filters"),
			LOCTEXT("LevelFiltersToolTip", "Filter tracks by level"),
			FNewToolMenuDelegate::CreateSP(this, &FSequencerTrackFilterMenu::FillLevelFilterMenu),
			FUIAction(
				FExecuteAction::CreateSP(this, &FSequencerTrackFilterMenu::ToggleAllLevelFilters),
				FCanExecuteAction(),
				FGetActionCheckState::CreateSP(this, &FSequencerTrackFilterMenu::GetAllLevelsCheckState)
			),
			EUserInterfaceActionType::ToggleButton,
			false,
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("ClassIcon.LevelInstance")),
			false);
	}

	if (FilterBar->IsFilterSupported(FSequencerTrackFilter_Group::StaticName()))
	{
		Section.AddSubMenu(TEXT("GroupFilters"),
			LOCTEXT("GroupFilters", "Group Filters"),
			LOCTEXT("GroupFiltersToolTip", "Filter tracks by group"),
			FNewToolMenuDelegate::CreateSP(this, &FSequencerTrackFilterMenu::FillGroupFilterMenu),
			FUIAction(
				FExecuteAction::CreateSP(this, &FSequencerTrackFilterMenu::ToggleAllGroupFilters),
				FCanExecuteAction(),
				FGetActionCheckState::CreateSP(this, &FSequencerTrackFilterMenu::GetAllGroupsCheckState)
			),
			EUserInterfaceActionType::ToggleButton,
			false,
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.GroupActors")),
			false);
	}
}

void FSequencerTrackFilterMenu::FillLevelFilterMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FToolMenuSection& LevelsSection = InMenu->FindOrAddSection(TEXT("Levels"), LOCTEXT("LevelFilters", "Level Filters"));

	const UWorld* const World = FilterBar->GetWorld();
	if (!World)
	{
		return;
	}

	for (const ULevel* const Level : World->GetLevels())
	{
		const FString LevelName = FPackageName::GetShortName(Level->GetPackage()->GetName());
		LevelsSection.AddMenuEntry(*LevelName,
			FText::FromString(LevelName),
			FText::FromString(Level->GetPackage()->GetName()),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FSequencerTrackFilterMenu::OnTrackLevelFilterClicked, LevelName),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FSequencerTrackFilterMenu::IsLevelFilterActive, LevelName)
			),
			EUserInterfaceActionType::ToggleButton);
	}
}

void FSequencerTrackFilterMenu::FillGroupFilterMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const ISequencer& Sequencer = FilterBar->GetSequencer();

	FToolMenuSection& GroupFilterOptionsSection = InMenu->FindOrAddSection(TEXT("GroupFilterOptions")
		, FText(), FToolMenuInsert(NAME_None, EToolMenuInsertType::First));

	GroupFilterOptionsSection.AddMenuEntry(TEXT("OpenGroupFilters"),
		LOCTEXT("OpenGroupFilters", "Open Group Filters..."),
		LOCTEXT("OpenGroupFiltersToolTip", "Opens the group filter dialog for managing groups"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.GroupActors")),
		FUIAction(FExecuteAction::CreateSP(this, &FSequencerTrackFilterMenu::OpenNodeGroupsManager)));

	const UMovieSceneSequence* const FocusedMovieSequence = Sequencer.GetRootMovieSceneSequence();
	if (!FocusedMovieSequence)
	{
		return;
	}

	UMovieScene* const FocusedMovieScene = FocusedMovieSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	FToolMenuSection& GroupsSection = InMenu->FindOrAddSection(TEXT("Groups"), LOCTEXT("GroupFilters", "Group Filters"));

	for (UMovieSceneNodeGroup* const NodeGroup : FocusedMovieScene->GetNodeGroups())
	{
		const FName GroupName = NodeGroup->GetName();
		GroupsSection.AddMenuEntry(GroupName,
			FText::FromName(GroupName),
			FText::FromName(GroupName),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FSequencerTrackFilterMenu::OnNodeGroupFilterClicked, NodeGroup),
				FCanExecuteAction::CreateLambda([FocusedMovieScene]() { return !FocusedMovieScene->IsReadOnly(); }),
				FIsActionChecked::CreateLambda([NodeGroup]() { return NodeGroup->GetEnableFilter(); })
			),
			EUserInterfaceActionType::ToggleButton);
	}
}

void FSequencerTrackFilterMenu::FillCustomTextFiltersMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FToolMenuSection& CustomTextOptionsSection = InMenu->FindOrAddSection(TEXT("CustomTextFilterOptions")
		, LOCTEXT("CustomTextFilterOptions", "Custom Text Filter Options")
		, FToolMenuInsert(NAME_None, EToolMenuInsertType::First));

	CustomTextOptionsSection.AddMenuEntry(TEXT("TextExpressionHelp"),
		LOCTEXT("TextExpressionHelp", "Text Expression Help"),
		LOCTEXT("TextExpressionHelpToolTip", "Opens the help dialog for the advanced search syntax text expressions"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Help")),
		FUIAction(FExecuteAction::CreateLambda([WeakFilterBar = FilterBar.ToWeakPtr()]
			{
				if (const TSharedPtr<FSequencerFilterBar> FilterBarPin = WeakFilterBar.Pin())
				{
					UE::Sequencer::OpenDialog_TextExpressionHelp(FilterBarPin.ToSharedRef());
				}
			}))
		);

	CustomTextOptionsSection.AddMenuEntry(TEXT("SaveCurrentAsNewTextFilter"),
		LOCTEXT("SaveCurrentAsNewTextFilter", "Save Current as New Filter"),
		LOCTEXT("SaveCurrentAsNewTextFilterToolTip", "Saves the enabled and active set of common filters as a custom text filter"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("LevelEditor.SaveAs")),
		FUIAction(FExecuteAction::CreateLambda([WeakFilterBar = FilterBar.ToWeakPtr()]
			{
				if (const TSharedPtr<FSequencerFilterBar> FilterBarPin = WeakFilterBar.Pin())
				{
					UE::Sequencer::OpenDialog_SaveCurrentFilterSetAsCustomTextFilter(FilterBarPin.ToSharedRef());
				}
			}))
		);

	CustomTextOptionsSection.AddMenuEntry(TEXT("CreateNewTextFilter"),
		LOCTEXT("CreateNewTextFilter", "Create New Filter"),
		LOCTEXT("CreateNewTextFilterTooltip", "Create a new text filter"),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.PlusCircle")),
		FUIAction(FExecuteAction::CreateSP(this, &FSequencerTrackFilterMenu::CreateNewTextFilter)));

	USequencerSettings* const SequencerSettings = FilterBar->GetSequencer().GetSequencerSettings();
	check(SequencerSettings);

	FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(FilterBar->GetIdentifier(), true);

	const TArray<FCustomTextFilterData>& CustomTextFilterDatas = Config.GetCustomTextFilters();
	if (CustomTextFilterDatas.IsEmpty())
	{
		return;
	}

	FToolMenuSection& FiltersSection = InMenu->AddSection(TEXT("CustomTextFilters"), LOCTEXT("CustomTextFilters", "Custom Text Filters"));

	for (const FCustomTextFilterData& CustomTextFilterData : CustomTextFilterDatas)
	{
		const TSharedPtr<SWidget> MenuItem = ConstructCustomMenuItemWidget(CustomTextFilterData.FilterLabel
			, FSimpleDelegate::CreateSP(this, &FSequencerTrackFilterMenu::OnCustomTextFilterClicked, CustomTextFilterData.FilterLabel)
			, TAttribute<ECheckBoxState>::CreateSP(this, &FSequencerTrackFilterMenu::GetCustomTextFilerCheckState, CustomTextFilterData.FilterLabel)
			, FSimpleDelegate::CreateSP(this, &FSequencerTrackFilterMenu::OnEditCustomTextFilterClicked, CustomTextFilterData.FilterLabel));
		FiltersSection.AddEntry(FToolMenuEntry::InitWidget(*CustomTextFilterData.FilterLabel.ToString(), MenuItem.ToSharedRef(), FText::GetEmpty()));
	}
}

void FSequencerTrackFilterMenu::FillFiltersMenuCategory(FToolMenuSection& InOutSection, const TSharedRef<FFilterCategory> InMenuCategory)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	for (const TSharedRef<FSequencerTrackFilter>& Filter : FilterBar->GetCommonFilters())
	{
		if (Filter->GetCategory() != InMenuCategory)
		{
			continue;
		}

		const FText DisplayName = Filter->GetDisplayName();

		FToolMenuEntry& NewMenuEntry = InOutSection.AddMenuEntry(*DisplayName.ToString(),
			DisplayName,
			Filter->GetToolTipText(),
			Filter->GetIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FSequencerTrackFilterMenu::OnFilterClicked, Filter),
				FCanExecuteAction::CreateLambda([this]()
					{
						return WeakFilterBar.IsValid() ? !WeakFilterBar.Pin()->AreFiltersMuted() : false;
					}),
				FIsActionChecked::CreateLambda([this, Filter]()
					{
						return WeakFilterBar.IsValid() ? WeakFilterBar.Pin()->IsFilterEnabled(Filter) : false;
					})
			),
			EUserInterfaceActionType::ToggleButton);

		if (const TSharedPtr<FUICommandInfo> ToggleCommand = Filter->GetToggleCommand())
		{
			NewMenuEntry.InputBindingLabel = ToggleCommand->GetInputText();
		}
	}
}

void FSequencerTrackFilterMenu::FillFiltersMenuCategory(UToolMenu* const InMenu, const TSharedRef<FFilterCategory> InMenuCategory)
{
	if (InMenu)
	{
		FillFiltersMenuCategory(InMenu->AddSection(TEXT("Section")), InMenuCategory);
	}
}

TSharedRef<SWidget> FSequencerTrackFilterMenu::ConstructCustomMenuItemWidget(const TAttribute<FText>& InItemText
	, const FSimpleDelegate& OnItemClicked
	, const TAttribute<ECheckBoxState>& InIsChecked
	, const FSimpleDelegate OnEditItemClicked)
{
	return SNew(SButton)
		.ContentPadding(0.f)
		.ButtonStyle(FAppStyle::Get(), TEXT("Menu.Button"))
		.ForegroundColor(FSlateColor::UseForeground())
		.ToolTipText(InItemText)
		.OnClicked_Lambda([OnItemClicked]()
			{
				OnItemClicked.ExecuteIfBound();
				return FReply::Handled();
			})
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(12.f, 0.f, 12.f, 0.f)
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), TEXT("Menu.CheckBox"))
				.OnCheckStateChanged_Lambda([OnItemClicked](const ECheckBoxState InNewState)
					{
						OnItemClicked.ExecuteIfBound();
					})
				.IsChecked(InIsChecked)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FStyleColors::White)
				.Text(InItemText)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(12.f, 0.f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
				.ContentPadding(0.f)
				.OnClicked_Lambda([OnEditItemClicked]()
					{
						OnEditItemClicked.ExecuteIfBound();
						return FReply::Handled();
					})
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(16.f, 12.f))
					.Image(FAppStyle::GetBrush(TEXT("Icons.Edit")))
				]
			]
		];
}

void FSequencerTrackFilterMenu::OnFilterCategoryClicked(const TSharedRef<FFilterCategory> InMenuCategory)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const ECheckBoxState CategoryCheckState = GetFilterCategoryCheckedState(InMenuCategory);
	const bool bIsCategoryEnabled = (CategoryCheckState == ECheckBoxState::Checked);
	FilterBar->ActivateCommonFilters(bIsCategoryEnabled, { InMenuCategory }, {});
}

ECheckBoxState FSequencerTrackFilterMenu::GetFilterCategoryCheckedState(const TSharedRef<FFilterCategory> InMenuCategory) const
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return ECheckBoxState::Unchecked;
	}

	for (const TSharedRef<FSequencerTrackFilter>& Filter : FilterBar->GetCommonFilters())
	{
		if (Filter->GetCategory() == InMenuCategory && !FilterBar->IsFilterEnabled(Filter))
		{
			return ECheckBoxState::Unchecked;
		}
	}

	return ECheckBoxState::Checked;
}

void FSequencerTrackFilterMenu::OnFilterClicked(const TSharedRef<FSequencerTrackFilter> InFilter)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	if (FilterBar->IsFilterEnabled(InFilter))
	{
		FilterBar->SetFilterEnabled(InFilter, false, true);
	}
	else
	{
		FilterBar->SetFilterActive(InFilter, true, true);
	}
}

void FSequencerTrackFilterMenu::OnCustomTextFilterClicked(const FText InFilterLabel)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencerTrackFilter_CustomText> CustomTextFilter = FilterBar->FindCustomTextFilterByDisplayName(InFilterLabel.ToString());
	if (!CustomTextFilter.IsValid())
	{
		return;
	}

	const TSharedRef<FSequencerTrackFilter_CustomText> CustomTextFilterRef = CustomTextFilter.ToSharedRef();

	if (FilterBar->IsFilterEnabled(CustomTextFilterRef))
	{
		FilterBar->SetFilterEnabled(CustomTextFilterRef, false, true);
	}
	else
	{
		FilterBar->SetFilterActive(CustomTextFilterRef, true, true);
	}
}

ECheckBoxState FSequencerTrackFilterMenu::GetCustomTextFilerCheckState(const FText InFilterLabel) const
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return ECheckBoxState::Unchecked;
	}

	if (const TSharedPtr<FSequencerTrackFilter_CustomText> CustomTextFilter = FilterBar->FindCustomTextFilterByDisplayName(InFilterLabel.ToString()))
	{
		return FilterBar->IsFilterEnabled(CustomTextFilter.ToSharedRef()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Unchecked;
}

void FSequencerTrackFilterMenu::OnEditCustomTextFilterClicked(const FText InFilterLabel)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	if (const TSharedPtr<FSequencerTrackFilter_CustomText> CustomTextFilter = FilterBar->FindCustomTextFilterByDisplayName(InFilterLabel.ToString()))
	{
		SSequencerCustomTextFilterDialog::CreateWindow_EditCustomTextFilter(FilterBar.ToSharedRef(), CustomTextFilter);
	}
}

void FSequencerTrackFilterMenu::OnTrackLevelFilterClicked(const FString InLevelName)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const bool bNewActiveState = !FilterBar->IsLevelFilterActive(InLevelName);
	FilterBar->ActivateLevelFilter(InLevelName, bNewActiveState);
}

void FSequencerTrackFilterMenu::ToggleAllLevelFilters()
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const ECheckBoxState AllState = GetAllLevelsCheckState();
	switch (AllState)
	{
	case ECheckBoxState::Checked:
		FilterBar->EnableAllLevelFilters(false);
		break;
	case ECheckBoxState::Undetermined:
	case ECheckBoxState::Unchecked:
		FilterBar->EnableAllLevelFilters(true);
		break;
	}
}

ECheckBoxState FSequencerTrackFilterMenu::GetAllLevelsCheckState() const
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return ECheckBoxState::Unchecked;
	}

	if (FilterBar->HasAllLevelFiltersActive())
	{
		return ECheckBoxState::Unchecked;
	}

	return FilterBar->HasActiveLevelFilter() ? ECheckBoxState::Undetermined : ECheckBoxState::Checked;
}

void FSequencerTrackFilterMenu::OnNodeGroupFilterClicked(UMovieSceneNodeGroup* const InNodeGroup)
{
	if (InNodeGroup)
	{
		InNodeGroup->SetEnableFilter(!InNodeGroup->GetEnableFilter());
	}
}

void FSequencerTrackFilterMenu::ToggleAllGroupFilters()
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const ECheckBoxState AllState = GetAllGroupsCheckState();
	switch (AllState)
	{
	case ECheckBoxState::Checked:
		FilterBar->EnableAllGroupFilters(false);
		break;
	case ECheckBoxState::Undetermined:
	case ECheckBoxState::Unchecked:
		FilterBar->EnableAllGroupFilters(true);
		break;
	}
}

ECheckBoxState FSequencerTrackFilterMenu::GetAllGroupsCheckState() const
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return ECheckBoxState::Unchecked;
	}

	const UMovieSceneSequence* const RootSequence = FilterBar->GetSequencer().GetRootMovieSceneSequence();
	if (!RootSequence)
	{
		return ECheckBoxState::Unchecked;
	}

	UMovieScene* const FocusedMovieScene = RootSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return ECheckBoxState::Unchecked;
	}

	int32 ActiveFilterCount = 0;

	const UMovieSceneNodeGroupCollection& Groups = FocusedMovieScene->GetNodeGroups();
	for (const UMovieSceneNodeGroup* const NodeGroup : Groups)
	{
		if (NodeGroup->GetEnableFilter())
		{
			++ActiveFilterCount;
		}
	}

	if (ActiveFilterCount > 0)
	{
		if (ActiveFilterCount == Groups.Num())
		{
			return ECheckBoxState::Checked;
		}

		return ECheckBoxState::Undetermined;
	}

	return ECheckBoxState::Unchecked;
}

bool FSequencerTrackFilterMenu::CanResetFilters() const
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return false;
	}

	if (FilterBar->AreFiltersMuted())
	{
		return true;
	}

	return FilterBar->CanResetFilters();
}

void FSequencerTrackFilterMenu::ResetFilters()
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FilterBar->ResetFilters();
}

bool FSequencerTrackFilterMenu::IsLevelFilterActive(const FString InLevelName) const
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return false;
	}

	return FilterBar->IsLevelFilterActive(InLevelName);
}

void FSequencerTrackFilterMenu::OpenNodeGroupsManager()
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const FSequencer* Sequencer = static_cast<FSequencer*>(&FilterBar->GetSequencer());
	const TSharedPtr<SSequencer> SequencerWidget = Sequencer->GetUnderlyingSequencerWidget();
	if (!SequencerWidget.IsValid())
	{
		return;
	}

	SequencerWidget->OpenNodeGroupsManager();
}

void FSequencerTrackFilterMenu::CreateNewTextFilter()
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	SSequencerCustomTextFilterDialog::CreateWindow_AddCustomTextFilter(FilterBar.ToSharedRef(), FCustomTextFilterData(), nullptr);
}

#undef LOCTEXT_NAMESPACE
