// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"
#include "Styling/SlateTypes.h"

class FFilterCategory;
class FSequencerFilterBar;
class FSequencerTrackFilter;
class UMovieSceneNodeGroup;
class UToolMenu;
struct FToolMenuSection;

class FSequencerTrackFilterMenu : public TSharedFromThis<FSequencerTrackFilterMenu>
{
public:
	TSharedRef<SWidget> CreateMenu(const TSharedRef<FSequencerFilterBar>& InFilterBar);

protected:
	void PopulateMenu(UToolMenu* const InMenu);

	void PopulateCustomsSection(UToolMenu& InMenu);
	void PopulateFilterOptionsSection(UToolMenu& InMenu);
	void PopulateCommonFilterSections(UToolMenu& InMenu);
	void PopulateOtherFilterSections(UToolMenu& InMenu);

	void FillLevelFilterMenu(UToolMenu* const InMenu);
	void FillGroupFilterMenu(UToolMenu* const InMenu);

	void FillCustomTextFiltersMenu(UToolMenu* const InMenu);

	void FillFiltersMenuCategory(FToolMenuSection& InOutSection, const TSharedRef<FFilterCategory> InMenuCategory);
	void FillFiltersMenuCategory(UToolMenu* const InMenu, const TSharedRef<FFilterCategory> InMenuCategory);

	static TSharedRef<SWidget> ConstructCustomMenuItemWidget(const TAttribute<FText>& InItemText
		, const FSimpleDelegate& OnItemClicked
		, const TAttribute<ECheckBoxState>& InIsChecked
		, const FSimpleDelegate OnEditItemClicked);

 	void OnFilterCategoryClicked(const TSharedRef<FFilterCategory> InMenuCategory);
 	ECheckBoxState GetFilterCategoryCheckedState(const TSharedRef<FFilterCategory> InMenuCategory) const;

 	void OnFilterClicked(const TSharedRef<FSequencerTrackFilter> InFilter);

	void OnCustomTextFilterClicked(const FText InFilterLabel);
	ECheckBoxState GetCustomTextFilerCheckState(const FText InFilterLabel) const;
	void OnEditCustomTextFilterClicked(const FText InFilterLabel);

	void OnTrackLevelFilterClicked(const FString InLevelName);

	void OnNodeGroupFilterClicked(UMovieSceneNodeGroup* const InNodeGroup);

	void ToggleAllLevelFilters();
	ECheckBoxState GetAllLevelsCheckState() const;

	void ToggleAllGroupFilters();
	ECheckBoxState GetAllGroupsCheckState() const;

	bool CanResetFilters() const;
	void ResetFilters();

	bool IsLevelFilterActive(const FString InLevelName) const;

	void OpenNodeGroupsManager();

	void CreateNewTextFilter();

private:
	TWeakPtr<FSequencerFilterBar> WeakFilterBar;
};
