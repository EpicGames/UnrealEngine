// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerViewOptionsMenu.h"

#include "CurveEditor.h"
#include "CurveEditorCommands.h"
#include "CurveEditor/SequencerCurveEditorApp.h"
#include "CurveEditor/Views/CurveEditorWidgetOwner.h"
#include "Filters/Menus/SequencerMenuContext.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "SSequencer.h"
#include "Sequencer.h"
#include "SequencerCommands.h"
#include "SequencerFilterBarContext.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Filters/Utils/LinkedFilteringViewUtils.h"

#define LOCTEXT_NAMESPACE "SequencerViewOptionsMenu"

namespace UE::Sequencer
{
namespace ViewOptionsDetail
{
static void AddFilterBarSection(UToolMenu& InMenu)
{
	const FSequencerTrackFilterCommands& TrackFilterCommands = FSequencerTrackFilterCommands::Get();
	FToolMenuSection& FilterBarSection = InMenu.FindOrAddSection(
		TEXT("FilterBarVisibility") , LOCTEXT("FilterBarVisibilityHeading", "Filter Bar")
		);
	
	FilterBarSection.AddMenuEntry(TrackFilterCommands.ToggleFilterBarVisibility);
	FilterBarSection.AddMenuEntry(TrackFilterCommands.TogglePreserveFiltersOnUnlink);
}

static TSharedRef<SWidget> CreateMenu(FName InMenuName, FOnPopulateFilterBarMenu InPopulateMenu, const TSharedRef<FUICommandList>& InCommandList)
{
	if (!UToolMenus::Get()->IsMenuRegistered(InMenuName))
	{
		UToolMenu* const Menu = UToolMenus::Get()->RegisterMenu(InMenuName);
		Menu->bShouldCloseWindowAfterMenuSelection = false;
		Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
		   if (USequencerMenuContext* const Context = InMenu->FindContext<USequencerMenuContext>())
		   {
			  Context->OnPopulateFilterBarMenu.ExecuteIfBound(InMenu);
		   }
		}));
	}
 
	USequencerMenuContext* ContextObject = NewObject<USequencerMenuContext>();
	ContextObject->OnPopulateFilterBarMenu = InPopulateMenu;
 
	const FToolMenuContext MenuContext(InCommandList, nullptr, ContextObject);
	return UToolMenus::Get()->GenerateWidget(InMenuName, MenuContext);
}
}

TSharedRef<SWidget> FSequencerViewOptionsMenu::CreateMenuForSequencer(
	TWeakPtr<FSequencer> InWeakSequencer, const TSharedRef<FUICommandList>& InCommandList
	)
{
	return ViewOptionsDetail::CreateMenu(
		TEXT("Sequencer.ViewOptionsMenu"),
		FOnPopulateFilterBarMenu::CreateSP(this, &FSequencerViewOptionsMenu::PopulateForSequencer, MoveTemp(InWeakSequencer)),
		InCommandList
		);
}

TSharedRef<SWidget> FSequencerViewOptionsMenu::CreateMenuForCurveEditor(
	TWeakPtr<FSequencer> InWeakSequencer, const TSharedRef<FUICommandList>& InCommandList
	)
{
	return ViewOptionsDetail::CreateMenu(
		TEXT("Sequencer.ViewOptionsMenu.CurveEditor"),
		FOnPopulateFilterBarMenu::CreateSP(this, &FSequencerViewOptionsMenu::PopulateForCurveEditor, MoveTemp(InWeakSequencer)),
		InCommandList
		);
}

void FSequencerViewOptionsMenu::PopulateForSequencer(UToolMenu* const InMenu, TWeakPtr<FSequencer> InWeakSequencer)
{
	const TSharedPtr<FSequencer> Sequencer = InWeakSequencer.Pin();
	if (ensure(Sequencer))
	{
		WeakSequencer = Sequencer;
    	
		PopulateFiltersSection(*InMenu);
		PopulateSortAndOrganizeSection(*InMenu);
		PopulateFilterOptionsSection(*InMenu);
	}
}

void FSequencerViewOptionsMenu::PopulateForCurveEditor(UToolMenu* const InMenu, TWeakPtr<FSequencer> InWeakSequencer)
{
	const TSharedPtr<FSequencer> Sequencer = InWeakSequencer.Pin();
	if (ensure(Sequencer))
	{
		WeakSequencer = Sequencer;
    	
		PopulateFiltersSection(*InMenu);
		PopulateCurveEditorOrganizeSection(*InMenu);
		ViewOptionsDetail::AddFilterBarSection(*InMenu);
	}
}

void FSequencerViewOptionsMenu::PopulateMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	USequencerMenuContext* const Context = InMenu->FindContext<USequencerMenuContext>();
	if (!Context)
	{
		return;
	}

	WeakSequencer = Context->GetSequencer();

	UToolMenu& Menu = *InMenu;

	PopulateFiltersSection(Menu);
	PopulateSortAndOrganizeSection(Menu);
	PopulateFilterOptionsSection(Menu);
}

void FSequencerViewOptionsMenu::PopulateFiltersSection(UToolMenu& InMenu)
{
	const FSequencerTrackFilterCommands& TrackFilterCommands = FSequencerTrackFilterCommands::Get();

	FToolMenuSection& HiddenTracksSection = InMenu.FindOrAddSection(TEXT("HiddenTracks"), LOCTEXT("HiddenTracksHeading", "Hidden Tracks"));

	HiddenTracksSection.AddMenuEntry(TrackFilterCommands.HideSelectedTracks);
	HiddenTracksSection.AddMenuEntry(TrackFilterCommands.ClearHiddenTracks);

	FToolMenuSection& IsolateTracksSection = InMenu.FindOrAddSection(TEXT("IsolatedTracks"), LOCTEXT("IsolatedTracksHeading", "Isolated Tracks"));

	IsolateTracksSection.AddMenuEntry(TrackFilterCommands.IsolateSelectedTracks);
	IsolateTracksSection.AddMenuEntry(TrackFilterCommands.ClearIsolatedTracks);

	FToolMenuSection& ShowTracksSection = InMenu.FindOrAddSection(TEXT("ShowTracks"), LOCTEXT("ShowTracksHeading", "Show Tracks"));

	ShowTracksSection.AddMenuEntry(TrackFilterCommands.ShowAllTracks);
	ShowTracksSection.AddSeparator(NAME_None);
	ShowTracksSection.AddMenuEntry(TrackFilterCommands.ShowLocationCategoryGroups);
	ShowTracksSection.AddMenuEntry(TrackFilterCommands.ShowRotationCategoryGroups);
	ShowTracksSection.AddMenuEntry(TrackFilterCommands.ShowScaleCategoryGroups);
}

void FSequencerViewOptionsMenu::PopulateSortAndOrganizeSection(UToolMenu& InMenu)
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<FUICommandList> SequencerBindings = Sequencer->GetCommandBindings();
	const FSequencerCommands& SequencerCommands = FSequencerCommands::Get();

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("OrganizeAndSort"), LOCTEXT("OrganizeAndSortHeader", "Organize and Sort"));

	Section.AddMenuEntryWithCommandList(SequencerCommands.ToggleAutoExpandNodesOnSelection, SequencerBindings);
	Section.AddMenuEntryWithCommandList(SequencerCommands.ToggleExpandCollapseNodes, SequencerBindings);
	Section.AddMenuEntryWithCommandList(SequencerCommands.ToggleExpandCollapseNodesAndDescendants, SequencerBindings);
	Section.AddMenuEntryWithCommandList(SequencerCommands.ExpandAllNodes, SequencerBindings);
	Section.AddMenuEntryWithCommandList(SequencerCommands.CollapseAllNodes, SequencerBindings);
	Section.AddMenuEntryWithCommandList(SequencerCommands.SortAllNodesAndDescendants, SequencerBindings);
}

void FSequencerViewOptionsMenu::PopulateFilterOptionsSection(UToolMenu& InMenu)
{
	FToolMenuSection& OptionsSection = InMenu.FindOrAddSection(TEXT("FilterOptions"), LOCTEXT("FilterOptionsHeading", "Filter Options"));

	OptionsSection.AddMenuEntry(TEXT("FilterPinned"),
		LOCTEXT("FilterPinned", "Filter Pinned"),
		LOCTEXT("FilterPinnedToolTip", "Toggle inclusion of pinned items when filtering"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FSequencerViewOptionsMenu::ToggleIncludePinnedInFilter),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw(this, &FSequencerViewOptionsMenu::IsIncludePinnedInFilter)
		),
		EUserInterfaceActionType::ToggleButton);

	OptionsSection.AddMenuEntry(TEXT("AutoExpandPassedFilterNodes"),
		LOCTEXT("AutoExpandPassedFilterNodes", "Auto Expand Filtered Items"),
		LOCTEXT("AutoExpandPassedFilterNodesToolTip", "Toggle expansion of items when a filter is passed"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FSequencerViewOptionsMenu::ToggleAutoExpandPassedFilterNodes),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw(this, &FSequencerViewOptionsMenu::IsAutoExpandPassedFilterNodes)
		),
		EUserInterfaceActionType::ToggleButton);

	ViewOptionsDetail::AddFilterBarSection(InMenu);
}

void FSequencerViewOptionsMenu::PopulateCurveEditorOrganizeSection(UToolMenu& InMenu)
{
	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("CurveEditor.OrganizeAndSort"), LOCTEXT("CurveEditor.OrganizeAndSortHeader", "Organize and Sort"));

	Section.AddMenuEntry(FSequencerCommands::Get().ToggleAutoExpandCurveEditorOnSelection);

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const FSequencerCurveEditorApp* const CurveEditorApp = FSequencerCurveEditorApp::Get(*Sequencer);
	if (!CurveEditorApp)
	{
		return;
	}

	const TSharedPtr<FCurveEditor> CurveEditor = CurveEditorApp->GetCurveEditor();
	if (!CurveEditor.IsValid())
	{
		return;
	}

	const TSharedPtr<FUICommandList> CurveEditorCommandList = CurveEditor->GetCommands();
	const FCurveEditorCommands& CurveEditorCommands = FCurveEditorCommands::Get();

	Section.AddMenuEntryWithCommandList(CurveEditorCommands.ToggleExpandCollapseNodes, CurveEditorCommandList);
	Section.AddMenuEntryWithCommandList(CurveEditorCommands.ToggleExpandCollapseNodesAndDescendants, CurveEditorCommandList);
	Section.AddMenuEntryWithCommandList(CurveEditorCommands.ExpandAllNodes, CurveEditorCommandList);
	Section.AddMenuEntryWithCommandList(CurveEditorCommands.CollapseAllNodes, CurveEditorCommandList);
	Section.AddMenuEntryWithCommandList(CurveEditorCommands.SortAllNodes, CurveEditorCommandList);
}

bool FSequencerViewOptionsMenu::IsIncludePinnedInFilter() const
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	const USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!SequencerSettings)
	{
		return false;
	}

	return SequencerSettings->GetIncludePinnedInFilter();
}

void FSequencerViewOptionsMenu::ToggleIncludePinnedInFilter()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!SequencerSettings)
	{
		return;
	}

	SequencerSettings->SetIncludePinnedInFilter(!SequencerSettings->GetIncludePinnedInFilter());

	Sequencer->GetFilterInterface()->RequestFilterUpdate();
}

bool FSequencerViewOptionsMenu::IsAutoExpandPassedFilterNodes() const
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	const USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!SequencerSettings)
	{
		return false;
	}

	return SequencerSettings->GetAutoExpandNodesOnFilterPass();
}

void FSequencerViewOptionsMenu::ToggleAutoExpandPassedFilterNodes()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!SequencerSettings)
	{
		return;
	}

	SequencerSettings->SetAutoExpandNodesOnFilterPass(!SequencerSettings->GetAutoExpandNodesOnFilterPass());

	Sequencer->GetFilterInterface()->RequestFilterUpdate();
}

TSharedPtr<SSequencer> FSequencerViewOptionsMenu::GetSequencerWidget() const
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	return Sequencer ? Sequencer->GetUnderlyingSequencerWidget().ToSharedPtr() : nullptr;
}
} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
