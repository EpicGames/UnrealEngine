// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FUICommandList;
class FSequencer;
class SSequencer;
class SWidget;
class UToolMenu;
enum class EFilterBarLayout : uint8;

namespace UE::Sequencer
{
class FSequencerViewOptionsMenu : public TSharedFromThis<FSequencerViewOptionsMenu>
{
public:
	
	/** Creates the view options menu for Sequencer's Outliner. */
	TSharedRef<SWidget> CreateMenuForSequencer(TWeakPtr<FSequencer> InWeakSequencer, const TSharedRef<FUICommandList>& InCommandList);
	
	/** Creates the view options menu for Sequencer's Curve Editor. */
	TSharedRef<SWidget> CreateMenuForCurveEditor(TWeakPtr<FSequencer> InWeakSequencer, const TSharedRef<FUICommandList>& InCommandList);

protected:
	
	void PopulateForSequencer(UToolMenu* const InMenu, TWeakPtr<FSequencer> InWeakSequencer);
	void PopulateForCurveEditor(UToolMenu* const InMenu, TWeakPtr<FSequencer> InWeakSequencer);
	
	void PopulateMenu(UToolMenu* const InMenu);
	void PopulateFiltersSection(UToolMenu& InMenu);
	void PopulateSortAndOrganizeSection(UToolMenu& InMenu);
	void PopulateFilterOptionsSection(UToolMenu& InMenu);
	
	void PopulateCurveEditorOrganizeSection(UToolMenu& InMenu);

	bool IsIncludePinnedInFilter() const;
	void ToggleIncludePinnedInFilter();

	bool IsAutoExpandPassedFilterNodes() const;
	void ToggleAutoExpandPassedFilterNodes();

	TSharedPtr<SSequencer> GetSequencerWidget() const;

	TWeakPtr<FSequencer> WeakSequencer;
};
} // namespace UE::Sequencer
