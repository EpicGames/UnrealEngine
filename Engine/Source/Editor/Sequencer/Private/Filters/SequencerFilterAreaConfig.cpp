// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/SequencerFilterAreaConfig.h"

void FSequencerFilterAreaConfig::SetIsFilterBarVisible(bool bNewIsVisible)
{
	bIsFilterBarVisible = bNewIsVisible;
}

bool FSequencerFilterAreaConfig::IsFilterBarVisible() const
{
	return bIsFilterBarVisible;
}

void FSequencerFilterAreaConfig::SetPreserveFiltersOnUnlink(bool bValue)
{
	bPreserveFiltersOnUnlink = bValue;
}

bool FSequencerFilterAreaConfig::GetPreserveFiltersOnUnlink() const
{
	return bPreserveFiltersOnUnlink;
}

void FSequencerFilterAreaConfig::SetIsLinkedFiltering(bool bValue)
{
	bIsLinkedFiltering = bValue;
}

bool FSequencerFilterAreaConfig::IsLinkedFiltering() const
{
	return bIsLinkedFiltering;
}
