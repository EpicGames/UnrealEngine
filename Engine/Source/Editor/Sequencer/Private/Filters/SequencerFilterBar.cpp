// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/SequencerFilterBar.h"

#include "FilterBarInitArgs.h"
#include "Filters/Filters/SequencerTrackFilter_Keyed.h"
#include "Filters/Filters/SequencerTrackFilter_Condition.h"
#include "Filters/Filters/SequencerTrackFilter_CustomText.h"
#include "Filters/Filters/SequencerTrackFilter_Group.h"
#include "Filters/Filters/SequencerTrackFilter_HideIsolate.h"
#include "Filters/Filters/SequencerTrackFilter_Level.h"
#include "Filters/Filters/SequencerTrackFilter_Modified.h"
#include "Filters/Filters/SequencerTrackFilter_Selected.h"
#include "Filters/Filters/SequencerTrackFilter_Text.h"
#include "Filters/Filters/SequencerTrackFilter_TimeWarp.h"
#include "Filters/Filters/SequencerTrackFilter_Unbound.h"
#include "Filters/Filters/SequencerTrackFilters.h"
#include "Filters/SequencerFilterBarConfig.h"
#include "Filters/SequencerTextFilterExpressionContext.h"
#include "Filters/SequencerTrackFilterCollection.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "Filters/SequencerTrackFilterExtension.h"
#include "Filters/Utils/FilterViewUtils.h"
#include "Filters/Widgets/SSequencerSearchBox.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"
#include "MovieScene.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/Views/SOutlinerView.h"
#include "Sequencer.h"
#include "SSequencer.h"
#include "Algo/ForEach.h"
#include "Filters/SequencerFilterAreaConfig.h"
#include "Filters/SequencerTrackFilter_StickySelection.h"
#include "Stats/StatsMisc.h"
#include "UObject/UObjectIterator.h"
#include "Utils/HideIsolateViewModelUtils.h"
#include "Utils/Evaluation/FilterBarEvaluation.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerFilterBar"

int32 FSequencerFilterBar::InstanceCount = 0;

TSharedRef<FSequencerFilterBar> FSequencerFilterBar::Make(FFilterBarInitArgs InArgs)
{
	TSharedRef<FSequencerFilterBar> FilterBar = MakeShared<FSequencerFilterBar>(MoveTemp(InArgs), FPrivateToken());
	
	// Requires TSharedFromThis to be initialized.
	FilterBar->BindCommands(); 
	
	return FilterBar;
}

FSequencerFilterBar::FSequencerFilterBar(FFilterBarInitArgs InArgs, FPrivateToken)
	: Sequencer(*InArgs.Sequencer)
	, FilterData(FString())
	, CommandList(MakeShared<FUICommandList>())
	, ClassTypeCategory(MakeShared<FFilterCategory>(LOCTEXT("ActorTypeFilterCategory", "Actor Type Filters"), FText()))
	, ComponentTypeCategory(MakeShared<FFilterCategory>(LOCTEXT("ObjectTypeFilterCategory", "Object Type Filters"), FText()))
	, MiscCategory(MakeShared<FFilterCategory>(LOCTEXT("MiscFilterCategory", "Misc Filters"), FText()))
	, TransientCategory(MakeShared<FFilterCategory>(LOCTEXT("TransientFilterCategory", "Transient Filters"), FText()))
	, CommonFilters(MakeShared<FSequencerTrackFilterCollection>(*this))
	, InternalFilters(MakeShared<FSequencerTrackFilterCollection>(*this))
	, TextFilter(MakeShared<FSequencerTrackFilter_CustomText>(*this))
	, HideIsolateFilter(MakeShared<FSequencerTrackFilter_HideIsolate>(*this))
	, LevelFilter(MakeShared<FSequencerTrackFilter_Level>(*this, TransientCategory))
	, GroupFilter(MakeShared<FSequencerTrackFilter_Group>(*this, TransientCategory))
	, SelectedFilter(MakeShared<FSequencerTrackFilter_Selected>(*this, MiscCategory))
	, MultiSelectFilter(MakeShared<FSequencerTrackFilter_StickySelection>(*InArgs.Sequencer, *this, TransientCategory))
	, ModifiedFilter(MakeShared<FSequencerTrackFilter_Modified>(*this, MiscCategory))
	, SubIdentifier(MoveTemp(InArgs.SubIdentifier))
	, Flags(InArgs.Flags)
	, FilterAreaConfigId(InArgs.FilterAreaConfigId)
{
	InstanceCount++;

	FSequencerTrackFilterCommands::Register();

	CommonFilters->OnChanged().AddRaw(this, &FSequencerFilterBar::RequestFilterUpdate);
	InternalFilters->OnChanged().AddRaw(this, &FSequencerFilterBar::RequestFilterUpdate);
	TextFilter->OnChanged().AddRaw(this, &FSequencerFilterBar::HandleTextFilterChanged);
	LevelFilter->OnChanged().AddRaw(this, &FSequencerFilterBar::RequestFilterUpdate);
	HideIsolateFilter->OnChanged().AddRaw(this, &FSequencerFilterBar::RequestFilterUpdate);
	SelectedFilter->OnChanged().AddRaw(this, &FSequencerFilterBar::RequestFilterUpdate);

	CreateDefaultFilters();

	CreateCustomTextFiltersFromConfig();
	
	FilterData.OnChangeFilterState().AddLambda([this](const TViewModelPtr<IOutlinerExtension>& InItem, bool bIsFilteredOut)
	{
		ChangeOutlinerExtensionFilterStateEvent.Broadcast(InItem, bIsFilteredOut);
	});
}

FSequencerFilterBar::~FSequencerFilterBar()
{
	InstanceCount--;

	if (InstanceCount == 0)
	{
		FSequencerTrackFilterCommands::Unregister();
	}

	CommonFilters->OnChanged().RemoveAll(this);
    InternalFilters->OnChanged().RemoveAll(this);
    TextFilter->OnChanged().RemoveAll(this);
    LevelFilter->OnChanged().RemoveAll(this);
    HideIsolateFilter->OnChanged().RemoveAll(this);
    SelectedFilter->OnChanged().RemoveAll(this);
}

TSharedPtr<ICustomTextFilter<FSequencerTrackFilterType>> FSequencerFilterBar::CreateTextFilter()
{
	return MakeShared<FSequencerTrackFilter_CustomText>(*this);
}

void FSequencerFilterBar::CopyFiltersFrom(const FSequencerFilterBar& InSource)
{
	USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings();
	if (!SequencerSettings)
	{
		return;
	}
	
	// Needed to notify UI to update.
	const auto BroadcastDeactivate = [this](const TSharedRef<FSequencerTrackFilter>& InFilter)
	{
		FiltersChangedEvent.Broadcast(ESequencerFilterChange::Disable, InFilter);
		return true;
	};
	CommonFilters->ForEachFilter(BroadcastDeactivate);
	InternalFilters->ForEachFilter(BroadcastDeactivate);
	Algo::ForEach(CustomTextFilters, BroadcastDeactivate);
	
	// We'll be lazy and simply override the config...
	FSequencerFilterBarConfig& SourceSettings = SequencerSettings->FindOrAddTrackFilterBar(InSource.GetIdentifier(), false);
	FSequencerFilterBarConfig& TargetSettings = SequencerSettings->FindOrAddTrackFilterBar(GetIdentifier(), false);
	TargetSettings = SourceSettings;
	SequencerSettings->SaveConfig();
	
	// ... and reuse the initialization mechanisms to recreate filters from config.
	CreateDefaultFilters();
	CreateCustomTextFiltersFromConfig();
	
	// Needed to notify UI to update.
	const auto BroadcastFilter = [this](const TSharedRef<FSequencerTrackFilter>& InFilter)
	{
		if (IsFilterActive(InFilter))
		{
			FiltersChangedEvent.Broadcast(ESequencerFilterChange::Activate, InFilter);
		}
		if (IsFilterEnabled(InFilter))
		{
			FiltersChangedEvent.Broadcast(ESequencerFilterChange::Enable, InFilter);
		}
		return true;
	};
	CommonFilters->ForEachFilter(BroadcastFilter);
	InternalFilters->ForEachFilter(BroadcastFilter);
	Algo::ForEach(CustomTextFilters, BroadcastFilter);
	
	// The filters are now dirty
	RequestFilterUpdate();
}

void FSequencerFilterBar::CreateDefaultFilters()
{
	auto AddFilterIfSupported = [this]
		(const TSharedPtr<FSequencerTrackFilterCollection>& InFilterCollection, const TSharedRef<FSequencerTrackFilter>& InFilter)
	{
		if (IsFilterSupported(InFilter))
		{
			InFilterCollection->Add(InFilter);
		}
	};

	// Add internal filters that won't be saved to config
	InternalFilters->RemoveAll();

	AddFilterIfSupported(InternalFilters, LevelFilter);
	AddFilterIfSupported(InternalFilters, GroupFilter);

	// Add class type category filters
	CommonFilters->RemoveAll();

	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_Audio>(*this, ClassTypeCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_CameraCut>(*this, ClassTypeCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_DataLayer>(*this, ClassTypeCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_Event>(*this, ClassTypeCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_Fade>(*this, ClassTypeCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_Folder>(*this, ClassTypeCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_LevelVisibility>(*this, ClassTypeCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_Particle>(*this, ClassTypeCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_CinematicShot>(*this, ClassTypeCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_Subsequence>(*this, ClassTypeCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_TimeDilation>(*this, ClassTypeCategory));

	// Add component type category filters
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_Camera>(*this, ComponentTypeCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_Light>(*this, ComponentTypeCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_SkeletalMesh>(*this, ComponentTypeCategory));

	// Add misc category filters
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_Keyed>(*this, MiscCategory));
	//AddFilterIfSupported(CommonFilters, ModifiedFilter); // Disabling until clear direction on what this should do
	AddFilterIfSupported(CommonFilters, SelectedFilter);
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_Unbound>(*this, MiscCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_Condition>(*this, MiscCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FSequencerTrackFilter_TimeWarp>(*this, MiscCategory));

	// Add global user-defined filters
	for (TObjectIterator<USequencerTrackFilterExtension> ExtensionIt(RF_NoFlags); ExtensionIt; ++ExtensionIt)
	{
		const USequencerTrackFilterExtension* const PotentialExtension = *ExtensionIt;
		if (PotentialExtension
			&& PotentialExtension->HasAnyFlags(RF_ClassDefaultObject)
			&& !PotentialExtension->GetClass()->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract))
		{
			TArray<TSharedRef<FSequencerTrackFilter>> ExtendedTrackFilters;
			PotentialExtension->AddTrackFilterExtensions(*this, ClassTypeCategory, ExtendedTrackFilters);

			for (const TSharedRef<FSequencerTrackFilter>& ExtendedTrackFilter : ExtendedTrackFilters)
			{
				AddFilterIfSupported(CommonFilters, ExtendedTrackFilter);
			}
		}
	}

	CommonFilters->Sort();

	// Activate filters internally based on saved config
	if (USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings())
	{
		const FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(GetIdentifier(), false);

		auto ActivateFilterIfConfigActive = [this, &Config](const TSharedRef<FSequencerTrackFilter>& InFilter)
			{
				const FString FilterName = InFilter->GetDisplayName().ToString();
				if (Config.IsFilterActive(FilterName))
				{
					InFilter->SetActive(true);
				}
			};

		CommonFilters->ForEachFilter([ActivateFilterIfConfigActive](const TSharedRef<FSequencerTrackFilter>& InFilter)
			{
				ActivateFilterIfConfigActive(InFilter);
				return true;
			});
		InternalFilters->ForEachFilter([ActivateFilterIfConfigActive](const TSharedRef<FSequencerTrackFilter>& InFilter)
			{
				ActivateFilterIfConfigActive(InFilter);
				return true;
			});
	}
	
	// Always active
	SetFilterActive(MultiSelectFilter, true);

	// Broadcast changes
	CommonFilters->OnChanged().Broadcast();
	InternalFilters->OnChanged().Broadcast();
	TextFilter->OnChanged().Broadcast();
	LevelFilter->OnChanged().Broadcast();
	HideIsolateFilter->OnChanged().Broadcast();
	SelectedFilter->OnChanged().Broadcast();
}

bool FSequencerFilterBar::IsFilterBarVisible() const
{
	USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings();
	if (!SequencerSettings)
	{
		return true;
	}
	const FSequencerFilterAreaConfig* const Config = SequencerSettings->FindTrackFilterArea(FilterAreaConfigId);
	return !Config || Config->IsFilterBarVisible();
}

void FSequencerFilterBar::ToggleFilterBarVisibilityCommand()
{
	USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings();
	if (!SequencerSettings)
	{
		return;
	}
	constexpr bool bSaveConfig = false;
	FSequencerFilterAreaConfig& Config = SequencerSettings->FindOrAddTrackFilterArea(FilterAreaConfigId, bSaveConfig);
	Config.SetIsFilterBarVisible(!Config.IsFilterBarVisible());
	SequencerSettings->SaveConfig();
}

void FSequencerFilterBar::BindCommands()
{
	const FSequencerTrackFilterCommands& TrackFilterCommands = FSequencerTrackFilterCommands::Get();

	if (!FilterAreaConfigId.IsNone())
	{
		CommandList->MapAction(TrackFilterCommands.ToggleFilterBarVisibility,
			FExecuteAction::CreateSP(this, &FSequencerFilterBar::ToggleFilterBarVisibilityCommand),
			FCanExecuteAction::CreateLambda([]{ return true; }),
			FIsActionChecked::CreateSP(this, &FSequencerFilterBar::IsFilterBarVisible));
	}

	CommandList->MapAction(TrackFilterCommands.ResetFilters,
		FExecuteAction::CreateSP(this, &FSequencerFilterBar::ResetFilters),
		FCanExecuteAction::CreateSP(this, &FSequencerFilterBar::CanResetFilters));

	CommandList->MapAction(TrackFilterCommands.ToggleMuteFilters,
		FExecuteAction::CreateSP(this, &FSequencerFilterBar::ToggleMuteFilters),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FSequencerFilterBar::AreFiltersMuted));

	CommandList->MapAction(TrackFilterCommands.DisableAllFilters,
		FExecuteAction::CreateLambda([this]()
			{
				EnableAllFilters(false, {});
			}),
		FCanExecuteAction::CreateSP(this, &FSequencerFilterBar::HasAnyFilterEnabled));

	CommandList->MapAction(TrackFilterCommands.ToggleActivateEnabledFilters,
		FExecuteAction::CreateSP(this, &FSequencerFilterBar::ToggleActivateAllEnabledFilters),
		FCanExecuteAction::CreateSP(this, &FSequencerFilterBar::HasAnyFilterEnabled));

	CommandList->MapAction(TrackFilterCommands.ActivateAllFilters,
		FExecuteAction::CreateSP(this, &FSequencerFilterBar::ActivateAllEnabledFilters, true, TArray<FString>()));

	CommandList->MapAction(TrackFilterCommands.DeactivateAllFilters,
		FExecuteAction::CreateSP(this, &FSequencerFilterBar::ActivateAllEnabledFilters, false, TArray<FString>()));

	// Bind all filter actions
	UMovieSceneSequence* const FocusedSequence = Sequencer.GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return;
	}

	const TArray<TSharedRef<FSequencerTrackFilter>> AllFilters = GetFilterList(true);
	for (const TSharedRef<FSequencerTrackFilter>& Filter : AllFilters)
	{
		Filter->BindCommands();
	}
}

void FSequencerFilterBar::CreateCustomTextFiltersFromConfig()
{
	USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings();
	if (!SequencerSettings)
	{
		return;
	}

	CustomTextFilters.Empty();

	FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(GetIdentifier(), false);

	for (const FCustomTextFilterData& CustomTextFilterData : Config.GetCustomTextFilters())
	{
		const TSharedRef<FSequencerTrackFilter_CustomText> NewCustomTextFilter = MakeShared<FSequencerTrackFilter_CustomText>(*this);
		NewCustomTextFilter->SetFromCustomTextFilterData(CustomTextFilterData);
		CustomTextFilters.Add(NewCustomTextFilter);
	}
}

ISequencer& FSequencerFilterBar::GetSequencer() const
{
	return Sequencer;
}

TSharedPtr<FUICommandList> FSequencerFilterBar::GetCommandList() const
{
	return CommandList;
}

FName FSequencerFilterBar::GetIdentifier() const
{
	if (const USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings())
	{
		return SubIdentifier.IsEmpty() 
			? *SequencerSettings->GetName()
			// E.g. "LevelSequencerEditor.Linked"
			: *FString::Printf(TEXT("%s.%s"), *SequencerSettings->GetName(), *SubIdentifier);
	}
	return TEXT("SequencerMain");
}

bool FSequencerFilterBar::AreFiltersMuted() const
{
	return bFiltersMuted;
}

void FSequencerFilterBar::MuteFilters(const bool bInMute)
{
	if (bFiltersMuted == bInMute)
	{
		return;
	}
	
	bFiltersMuted = bInMute;
	
	OnMuteFiltersChangedEvent.Broadcast(bFiltersMuted);
	RequestFilterUpdate();
}

void FSequencerFilterBar::ToggleMuteFilters()
{
	MuteFilters(!AreFiltersMuted());
}

bool FSequencerFilterBar::CanResetFilters() const
{
	return HasAnyFiltersEnabled();
}

void FSequencerFilterBar::ResetFilters()
{
	EnableAllFilters(false, {});
	EnableCustomTextFilters(false);
	EnableAllGroupFilters(false);
	LevelFilter->ResetFilter();
}

FString FSequencerFilterBar::GetTextFilterString() const
{
	return TextFilter->GetRawFilterText().ToString();
}

void FSequencerFilterBar::SetTextFilterString(const FString& InText)
{
	TextFilter->SetRawFilterText(FText::FromString(InText));
	RequestFilterUpdate();
}

bool FSequencerFilterBar::DoesTextFilterStringContainExpressionPair(const ISequencerTextFilterExpressionContext& InExpression) const
{
	return TextFilter->DoesTextFilterStringContainExpressionPair(InExpression);
}

TSharedRef<FSequencerTrackFilter_Text> FSequencerFilterBar::GetTextFilter() const
{
	return TextFilter;
}

TSharedRef<FSequencerTrackFilter_HideIsolate> FSequencerFilterBar::GetHideIsolateFilter() const
{
	return HideIsolateFilter;
}

FText FSequencerFilterBar::GetFilterErrorText() const
{
	return TextFilter->GetFilterErrorText();
}

void FSequencerFilterBar::HideTracks(const TSet<TWeakViewModelPtr<IOutlinerExtension>>& InTracks, const bool bInAddToExisting)
{
	HideIsolateFilter->HideTracks(InTracks, bInAddToExisting);
}

void FSequencerFilterBar::UnhideTracks(const TSet<TWeakViewModelPtr<IOutlinerExtension>>& InTracks)
{
	HideIsolateFilter->UnhideTracks(InTracks);
}

void FSequencerFilterBar::IsolateTracks(const TSet<TWeakViewModelPtr<IOutlinerExtension>>& InTracks, const bool bInAddToExisting)
{
	HideIsolateFilter->IsolateTracks(InTracks, bInAddToExisting);
}

void FSequencerFilterBar::UnisolateTracks(const TSet<TWeakViewModelPtr<IOutlinerExtension>>& InTracks)
{
	HideIsolateFilter->UnisolateTracks(InTracks);
}

bool FSequencerFilterBar::HasHiddenTracks() const
{
	return HideIsolateFilter->HasHiddenTracks();
}

bool FSequencerFilterBar::HasIsolatedTracks() const
{
	return HideIsolateFilter->HasIsolatedTracks();
}

void FSequencerFilterBar::RequestFilterUpdate()
{
	RequestUpdateEvent.Broadcast();
}

TSharedPtr<FSequencerTrackFilter> FSequencerFilterBar::FindFilterByDisplayName(const FString& InFilterName) const
{
	TSharedPtr<FSequencerTrackFilter> OutFilter;

	CommonFilters->ForEachFilter([&InFilterName, &OutFilter]
		(const TSharedRef<FSequencerTrackFilter>& InFilter)
		{
			const FString FilterName = InFilter->GetDisplayName().ToString();
			if (FilterName.Equals(InFilterName, ESearchCase::IgnoreCase))
			{
				OutFilter = InFilter;
				return false;
			}
			return true;
		});

	return OutFilter;
}

TSharedPtr<FSequencerTrackFilter_CustomText> FSequencerFilterBar::FindCustomTextFilterByDisplayName(const FString& InFilterName) const
{
	TSharedPtr<FSequencerTrackFilter_CustomText> OutFilter;

	for (const TSharedRef<FSequencerTrackFilter_CustomText>& CustomTextFilter : CustomTextFilters)
	{
		const FString FilterName = CustomTextFilter->GetDisplayName().ToString();
		if (FilterName.Equals(InFilterName, ESearchCase::IgnoreCase))
		{
			OutFilter = CustomTextFilter;
			break;
		}
	}

	return OutFilter;
}

bool FSequencerFilterBar::HasAnyFiltersEnabled() const
{
	return HasEnabledCommonFilters() || HasEnabledCustomTextFilters() || AnyInternalFilterActive();
}

bool FSequencerFilterBar::IsFilterActiveByDisplayName(const FString& InFilterName) const
{
	if (const TSharedPtr<FSequencerTrackFilter> Filter = FindFilterByDisplayName(InFilterName))
	{
		return IsFilterActive(Filter.ToSharedRef());
	}
	return false;
}

bool FSequencerFilterBar::IsFilterEnabledByDisplayName(const FString& InFilterName) const
{
	if (const TSharedPtr<FSequencerTrackFilter> Filter = FindFilterByDisplayName(InFilterName))
	{
		return IsFilterEnabled(Filter.ToSharedRef());
	}
	return false;
}

bool FSequencerFilterBar::SetFilterActiveByDisplayName(const FString& InFilterName, const bool bInActive, const bool bInRequestFilterUpdate)
{
	if (const TSharedPtr<FSequencerTrackFilter> Filter = FindFilterByDisplayName(InFilterName))
	{
		return SetFilterActive(Filter.ToSharedRef(), bInActive, bInRequestFilterUpdate);
	}

	if (const TSharedPtr<FSequencerTrackFilter> Filter = FindCustomTextFilterByDisplayName(InFilterName))
	{
		return SetFilterActive(Filter.ToSharedRef(), bInActive, bInRequestFilterUpdate);
	}

	return false;
}

bool FSequencerFilterBar::SetFilterEnabledByDisplayName(const FString& InFilterName, const bool bInEnabled, const bool bInRequestFilterUpdate)
{
	if (const TSharedPtr<FSequencerTrackFilter> Filter = FindFilterByDisplayName(InFilterName))
	{
		return SetFilterEnabled(Filter.ToSharedRef(), bInEnabled, bInRequestFilterUpdate);
	}

	if (const TSharedPtr<FSequencerTrackFilter> Filter = FindCustomTextFilterByDisplayName(InFilterName))
	{
		return SetFilterEnabled(Filter.ToSharedRef(), bInEnabled, bInRequestFilterUpdate);
	}

	return false;
}

bool FSequencerFilterBar::AnyCommonFilterActive() const
{
	bool bOutActiveFilter = false;

	CommonFilters->ForEachFilter([this, &bOutActiveFilter]
		(const TSharedRef<FSequencerTrackFilter>& InFilter)
		{
			if (IsFilterActive(InFilter))
			{
				bOutActiveFilter = true;
				return false;
			}
			return true;
		});

	return bOutActiveFilter;
}

bool FSequencerFilterBar::AnyInternalFilterActive() const
{
	const bool bLevelFilterActive = LevelFilter->HasHiddenLevels();
	const bool bGroupFilterActive = GroupFilter->HasActiveGroupFilter();
	return bLevelFilterActive || bGroupFilterActive;
}

bool FSequencerFilterBar::HasAnyFilterActive(const bool bCheckTextFilter
	, const bool bInCheckHideIsolateFilter
	, const bool bInCheckCommonFilters
	, const bool bInCheckInternalFilters
	, const bool bInCheckCustomTextFilters) const
{
	if (bFiltersMuted)
	{
		return false;
	}

	const bool bTextFilterActive = bCheckTextFilter && TextFilter->IsActive();
	const bool bHideIsolateFilterActive = bInCheckHideIsolateFilter && HideIsolateFilter->IsActive();
	const bool bCommonFilterActive = bInCheckCommonFilters && AnyCommonFilterActive();
	const bool bInternalFilterActive = bInCheckInternalFilters && AnyInternalFilterActive();
	const bool bCustomTextFilterActive = bInCheckCustomTextFilters && AnyCustomTextFilterActive();

	return bTextFilterActive
		|| bHideIsolateFilterActive
		|| bCommonFilterActive
		|| bInternalFilterActive
		|| bCustomTextFilterActive;
}

bool FSequencerFilterBar::IsFilterActive(const TSharedRef<FSequencerTrackFilter> InFilter) const
{
	USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings();
	if (!SequencerSettings)
	{
		return false;
	}

	const FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(GetIdentifier(), false);

	const FString FilterName = InFilter->GetDisplayName().ToString();
	return Config.IsFilterActive(FilterName);
}

bool FSequencerFilterBar::SetFilterActive(const TSharedRef<FSequencerTrackFilter>& InFilter, const bool bInActive, const bool bInRequestFilterUpdate)
{
	USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings();
	if (!SequencerSettings)
	{
		return false;
	}

	const bool bNewActive = InFilter->IsInverseFilter() ? !bInActive : bInActive;

	FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(GetIdentifier(), true);

	const FString FilterName = InFilter->GetDisplayName().ToString();
	const bool bSuccess = Config.SetFilterActive(FilterName, bNewActive);

	if (bSuccess)
	{
		SequencerSettings->SaveConfig();

		InFilter->SetActive(bNewActive);

		const ESequencerFilterChange FilterChangeType = bNewActive
			? ESequencerFilterChange::Activate : ESequencerFilterChange::Deactivate;
		FiltersChangedEvent.Broadcast(FilterChangeType, InFilter);

		if (bInRequestFilterUpdate)
		{
			RequestFilterUpdate();
		}
	}

	return bSuccess;
}

void FSequencerFilterBar::EnableAllFilters(const bool bInEnable, const TArray<FString>& InExceptionFilterNames)
{
	TArray<TSharedRef<FSequencerTrackFilter>> ExceptionFilters;
	TArray<TSharedRef<FSequencerTrackFilter_CustomText>> ExceptionCustomTextFilters;

	for (const FString& FilterName : InExceptionFilterNames)
	{
		if (const TSharedPtr<FSequencerTrackFilter> Filter = FindFilterByDisplayName(FilterName))
		{
			ExceptionFilters.Add(Filter.ToSharedRef());
		}
		else if (const TSharedPtr<FSequencerTrackFilter_CustomText> CustomTextFilter = FindCustomTextFilterByDisplayName(FilterName))
		{
			ExceptionCustomTextFilters.Add(CustomTextFilter.ToSharedRef());
		}
	}

	EnableFilters(bInEnable, {}, ExceptionFilters);
	EnableCustomTextFilters(bInEnable, ExceptionCustomTextFilters);
}

void FSequencerFilterBar::GetCommonFiltersImpl(TArray<TSharedPtr<void>>& OutTypeErasedFilters) const
{
	for (const TSharedRef<FSequencerTrackFilter>& Filter : CommonFilters->GetAllFilters())
	{
		OutTypeErasedFilters.Add(Filter);
	}
}

void FSequencerFilterBar::ActivateCommonFilters(const bool bInActivate, const TArray<FString>& InExceptionFilterNames)
{
	TArray<TSharedRef<FSequencerTrackFilter>> ExceptionFilters;

	for (const FString& FilterName : InExceptionFilterNames)
	{
		if (const TSharedPtr<FSequencerTrackFilter> Filter = FindFilterByDisplayName(FilterName))
		{
			ExceptionFilters.Add(Filter.ToSharedRef());
		}
	}

	return ActivateCommonFilters(bInActivate, {}, ExceptionFilters);
}

void FSequencerFilterBar::ActivateCommonFilters(const bool bInActivate
    , const TArray<TSharedRef<FFilterCategory>> InMatchCategories
    , const TArray<TSharedRef<FSequencerTrackFilter>>& InExceptions)
{
	USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings();
	if (!SequencerSettings)
	{
		return;
	}

	FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(GetIdentifier(), false);

	bool bNeedsSave = false;

	CommonFilters->ForEachFilter([this, bInActivate, &InExceptions, &Config, &bNeedsSave]
		(const TSharedRef<FSequencerTrackFilter>& InFilter)
		{
			if (InExceptions.Contains(InFilter))
			{
				return true;
			}

			const FString FilterName = InFilter->GetDisplayName().ToString();
			if (Config.SetFilterActive(FilterName, bInActivate))
			{
				const ESequencerFilterChange FilterChangeType = bInActivate
					? ESequencerFilterChange::Activate
					: ESequencerFilterChange::Deactivate;
				FiltersChangedEvent.Broadcast(FilterChangeType, InFilter);

				InFilter->SetActive(bInActivate);

				bNeedsSave = true;
			}

			return true;
		}
		, InMatchCategories);

	if (bNeedsSave)
	{
		SequencerSettings->SaveConfig();
	}

	RequestFilterUpdate();
}

bool FSequencerFilterBar::AreAllEnabledFiltersActive(const bool bInActive, const TArray<FString> InExceptionFilterNames) const
{
	const TArray<TSharedRef<FSequencerTrackFilter>> EnabledFilters = GetEnabledFilters();
	for (const TSharedRef<FSequencerTrackFilter>& Filter : EnabledFilters)
	{
		const FString FilterName = Filter->GetDisplayName().ToString();
		if (InExceptionFilterNames.Contains(FilterName))
		{
			continue;
		}

		if (IsFilterActive(Filter) != bInActive)
		{
			return false;
		}
	}

	const TArray<TSharedRef<FSequencerTrackFilter_CustomText>> EnabledCustomTextFilters = GetEnabledCustomTextFilters();
	for (const TSharedRef<FSequencerTrackFilter_CustomText>& CustomTextFilter : EnabledCustomTextFilters)
	{
		const FString FilterName = CustomTextFilter->GetDisplayName().ToString();
		if (InExceptionFilterNames.Contains(FilterName))
		{
			continue;
		}

		if (IsFilterActive(CustomTextFilter) != bInActive)
		{
			return false;
		}
	}

	return true;
}

void FSequencerFilterBar::ActivateAllEnabledFilters(const bool bInActivate, const TArray<FString> InExceptionFilterNames)
{
	const TArray<TSharedRef<FSequencerTrackFilter>> EnabledFilters = GetEnabledFilters();
	for (const TSharedRef<FSequencerTrackFilter>& Filter : EnabledFilters)
	{
		const FString FilterName = Filter->GetDisplayName().ToString();
		if (InExceptionFilterNames.Contains(FilterName))
        {
        	continue;
        }

		if (IsFilterActive(Filter) != bInActivate)
		{
			SetFilterActive(Filter, bInActivate);
		}
	}

	const TArray<TSharedRef<FSequencerTrackFilter_CustomText>> EnabledCustomTextFilters = GetEnabledCustomTextFilters();
	for (const TSharedRef<FSequencerTrackFilter_CustomText>& CustomTextFilter : EnabledCustomTextFilters)
	{
		const FString FilterName = CustomTextFilter->GetDisplayName().ToString();
		if (InExceptionFilterNames.Contains(FilterName))
		{
			continue;
		}

		if (IsFilterActive(CustomTextFilter) != bInActivate)
		{
			SetFilterActive(CustomTextFilter, bInActivate);
		}
	}
}

void FSequencerFilterBar::ToggleActivateAllEnabledFilters()
{
	const bool bNewActive = !AreAllEnabledFiltersActive(true, {});
	ActivateAllEnabledFilters(bNewActive, {});
}

TArray<TSharedRef<FSequencerTrackFilter>> FSequencerFilterBar::GetActiveFilters() const
{
	TArray<TSharedRef<FSequencerTrackFilter>> OutFilters;

	CommonFilters->ForEachFilter([this, &OutFilters]
		(const TSharedRef<FSequencerTrackFilter>& InFilter)
		{
			if (IsFilterActive(InFilter))
			{
				OutFilters.Add(InFilter);
			}
			return true;
		});

	return OutFilters;
}

bool FSequencerFilterBar::HasEnabledCommonFilters() const
{
	bool bOutReturn = false;

	CommonFilters->ForEachFilter([this, &bOutReturn]
		(const TSharedRef<FSequencerTrackFilter>& InFilter)
		{
			if (IsFilterEnabled(InFilter))
			{
				bOutReturn = true;
				return false;
			}
			return true;
		});

	if (bOutReturn)
	{
		return true;
	}

	InternalFilters->ForEachFilter([this, &bOutReturn]
		(const TSharedRef<FSequencerTrackFilter>& InFilter)
		{
			if (IsFilterEnabled(InFilter))
			{
				bOutReturn = true;
				return false;
			}
			return true;
		});

	return bOutReturn;
}

bool FSequencerFilterBar::HasEnabledFilter(const TArray<TSharedRef<FSequencerTrackFilter>>& InFilters) const
{
	const TArray<TSharedRef<FSequencerTrackFilter>>& Filters = InFilters.IsEmpty() ? GetCommonFilters() : InFilters;

	for (const TSharedRef<FSequencerTrackFilter>& Filter : Filters)
	{
		if (IsFilterEnabled(Filter))
		{
			return true;
		}
	}

	return false;
}

bool FSequencerFilterBar::HasAnyFilterEnabled() const
{
	const bool bCommonFilterEnabled = HasEnabledCommonFilters();
	const bool bCustomTextFilterEnabled = HasEnabledCustomTextFilters();

	return bCommonFilterEnabled
		|| bCustomTextFilterEnabled;
}

bool FSequencerFilterBar::IsFilterEnabled(TSharedRef<FSequencerTrackFilter> InFilter) const
{
	USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings();
	if (!SequencerSettings)
	{
		return false;
	}

	FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(GetIdentifier(), false);

	const FString FilterName = InFilter->GetDisplayName().ToString();
	return Config.IsFilterEnabled(FilterName);
}

bool FSequencerFilterBar::SetFilterEnabled(const TSharedRef<FSequencerTrackFilter> InFilter, const bool bInEnabled, const bool bInRequestFilterUpdate)
{
	USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings();
	if (!SequencerSettings)
	{
		return false;
	}

	FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(GetIdentifier(), true);

	const FString FilterName = InFilter->GetDisplayName().ToString();
	const bool bSuccess = Config.SetFilterEnabled(FilterName, bInEnabled);

	if (bSuccess)
	{
		SequencerSettings->SaveConfig();

		const ESequencerFilterChange FilterChangeType = bInEnabled
			? ESequencerFilterChange::Enable : ESequencerFilterChange::Disable;
		FiltersChangedEvent.Broadcast(FilterChangeType, InFilter);

		if (!bInEnabled && IsFilterActive(InFilter))
		{
			InFilter->SetActive(false);
		}

		if (bInRequestFilterUpdate)
		{
			RequestFilterUpdate();
		}
	}

	return bSuccess;
}

void FSequencerFilterBar::EnableFilters(const bool bInEnable
	, const TArray<TSharedRef<FFilterCategory>> InMatchCategories
	, const TArray<TSharedRef<FSequencerTrackFilter>> InExceptions)
{
	USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings();
	if (!SequencerSettings)
	{
		return;
	}

	FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(GetIdentifier(), true);

	CommonFilters->ForEachFilter([this, bInEnable, &InExceptions, &Config]
		(const TSharedRef<FSequencerTrackFilter>& InFilter)
		{
			if (InExceptions.IsEmpty() || !InExceptions.Contains(InFilter))
			{
				const FString FilterName = InFilter->GetDisplayName().ToString();
				if (Config.SetFilterEnabled(FilterName, bInEnable))
				{
					const ESequencerFilterChange FilterChangeType = bInEnable
						? ESequencerFilterChange::Enable : ESequencerFilterChange::Disable;
					FiltersChangedEvent.Broadcast(FilterChangeType, InFilter);

					if (!bInEnable && IsFilterActive(InFilter))
					{
						InFilter->SetActive(false);
					}
				}
			}
			return true;
		}
		, InMatchCategories);

	SequencerSettings->SaveConfig();

	RequestFilterUpdate();
}

void FSequencerFilterBar::ToggleFilterEnabled(const TSharedRef<FSequencerTrackFilter> InFilter)
{
	SetFilterEnabled(InFilter, !IsFilterEnabled(InFilter), true);
}

TArray<TSharedRef<FSequencerTrackFilter>> FSequencerFilterBar::GetEnabledFilters() const
{
	TArray<TSharedRef<FSequencerTrackFilter>> OutFilters;

	CommonFilters->ForEachFilter([this, &OutFilters]
		(const TSharedRef<FSequencerTrackFilter>& InFilter)
		{
			if (IsFilterEnabled(InFilter))
			{
				OutFilters.Add(InFilter);
			}
			return true;
		});

	return OutFilters;
}

bool FSequencerFilterBar::HasAnyCommonFilters() const
{
	return !CommonFilters->IsEmpty();
}

bool FSequencerFilterBar::AddFilter(const TSharedRef<FSequencerTrackFilter>& InFilter)
{
	const bool bSuccess = CommonFilters->Add(InFilter) == 1;

	return bSuccess;
}

bool FSequencerFilterBar::RemoveFilter(const TSharedRef<FSequencerTrackFilter>& InFilter)
{
	const bool bSuccess = CommonFilters->Remove(InFilter) == 1;

	if (bSuccess)
	{
		FiltersChangedEvent.Broadcast(ESequencerFilterChange::Disable, InFilter);
	}

	return bSuccess;
}

TArray<FText> FSequencerFilterBar::GetFilterDisplayNames() const
{
	return CommonFilters->GetFilterDisplayNames();
}

TArray<FText> FSequencerFilterBar::GetCustomTextFilterNames() const
{
	TArray<FText> OutLabels;

	for (const TSharedRef<FSequencerTrackFilter_CustomText>& CustomTextFilter : CustomTextFilters)
	{
		const FCustomTextFilterData TextFilterData = CustomTextFilter->CreateCustomTextFilterData();
		OutLabels.Add(TextFilterData.FilterLabel);
	}

	return OutLabels;
}

int32 FSequencerFilterBar::GetTotalDisplayNodeCount() const
{
	return FilterData.GetTotalNodeCount();
}

int32 FSequencerFilterBar::GetFilteredDisplayNodeCount() const
{
	return FilterData.GetDisplayNodeCount();
}

TArray<TSharedRef<FSequencerTrackFilter>> FSequencerFilterBar::GetCommonFilters(const TArray<TSharedRef<FFilterCategory>>& InCategories) const
{
	return CommonFilters->GetAllFilters(InCategories);
}

bool FSequencerFilterBar::AnyCustomTextFilterActive() const
{
	for (const TSharedRef<FSequencerTrackFilter_CustomText>& Filter : CustomTextFilters)
	{
		if (IsFilterActive(Filter))
		{
			return true;
		}
	}

	return false;
}

bool FSequencerFilterBar::HasEnabledCustomTextFilters() const
{
	for (const TSharedRef<FSequencerTrackFilter_CustomText>& Filter : CustomTextFilters)
	{
		if (IsFilterEnabled(Filter))
		{
			return true;
		}
	}
	return false;
}

TArray<TSharedRef<FSequencerTrackFilter_CustomText>> FSequencerFilterBar::GetAllCustomTextFilters() const
{
	return CustomTextFilters;
}

bool FSequencerFilterBar::AddCustomTextFilter(const TSharedRef<FSequencerTrackFilter_CustomText>& InFilter, const bool bInAddToConfig)
{
	if (CustomTextFilters.Add(InFilter) != 1)
	{
		return false;
	}

	if (bInAddToConfig)
	{
		if (USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings())
		{
			FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(GetIdentifier(), false);

			if (Config.AddCustomTextFilter(InFilter->CreateCustomTextFilterData()))
			{
				SequencerSettings->SaveConfig();
			}
		}
	}

	FiltersChangedEvent.Broadcast(ESequencerFilterChange::Activate, InFilter);

	return true;
}

bool FSequencerFilterBar::RemoveCustomTextFilter(const TSharedRef<FSequencerTrackFilter_CustomText>& InFilter, const bool bInAddToConfig)
{
	if (CustomTextFilters.Remove(InFilter) != 1)
	{
		return false;
	}

	if (bInAddToConfig)
	{
		if (USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings())
		{
			FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(GetIdentifier(), false);

			const FString FilterName = InFilter->GetDisplayName().ToString();
			if (Config.RemoveCustomTextFilter(FilterName))
			{
				SequencerSettings->SaveConfig();
			}
		}
	}

	FiltersChangedEvent.Broadcast(ESequencerFilterChange::Disable, InFilter);

	return true;
}

void FSequencerFilterBar::ActivateCustomTextFilters(const bool bInActivate, const TArray<TSharedRef<FSequencerTrackFilter_CustomText>> InExceptions)
{
	USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings();
	if (!SequencerSettings)
	{
		return;
	}

	FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(GetIdentifier(), false);

	for (const TSharedRef<FSequencerTrackFilter_CustomText>& CustomTextFilter : CustomTextFilters)
	{
		if (InExceptions.IsEmpty() || !InExceptions.Contains(CustomTextFilter))
		{
			const FString FilterName = CustomTextFilter->GetDisplayName().ToString();
			if (Config.SetFilterActive(FilterName, bInActivate))
			{
				if (!bInActivate && IsFilterActive(CustomTextFilter))
				{
					CustomTextFilter->SetActive(false);
				}

				const ESequencerFilterChange FilterChangeType = bInActivate
					? ESequencerFilterChange::Activate : ESequencerFilterChange::Deactivate;
				FiltersChangedEvent.Broadcast(FilterChangeType, CustomTextFilter);
			}
		}
	}

	SequencerSettings->SaveConfig();

	RequestFilterUpdate();
}

void FSequencerFilterBar::EnableCustomTextFilters(const bool bInEnable, const TArray<TSharedRef<FSequencerTrackFilter_CustomText>> InExceptions)
{
	USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings();
	if (!SequencerSettings)
	{
		return;
	}

	FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(GetIdentifier(), false);

	for (const TSharedRef<FSequencerTrackFilter_CustomText>& CustomTextFilter : CustomTextFilters)
	{
		if (InExceptions.IsEmpty() || !InExceptions.Contains(CustomTextFilter))
		{
			const FString FilterName = CustomTextFilter->GetDisplayName().ToString();
			if (Config.SetFilterEnabled(FilterName, bInEnable))
			{
				if (!bInEnable && IsFilterActive(CustomTextFilter))
				{
					CustomTextFilter->SetActive(false);
				}

				const ESequencerFilterChange FilterChangeType = bInEnable
					? ESequencerFilterChange::Enable : ESequencerFilterChange::Disable;
				FiltersChangedEvent.Broadcast(FilterChangeType, CustomTextFilter);
			}
		}
	}

	SequencerSettings->SaveConfig();

	RequestFilterUpdate();
}

TArray<TSharedRef<FSequencerTrackFilter_CustomText>> FSequencerFilterBar::GetEnabledCustomTextFilters() const
{
	TArray<TSharedRef<FSequencerTrackFilter_CustomText>> OutFilters;

	for (const TSharedRef<FSequencerTrackFilter_CustomText>& CustomTextFilter : CustomTextFilters)
	{
		if (IsFilterEnabled(CustomTextFilter))
		{
			OutFilters.Add(CustomTextFilter);
		}
	}

	return OutFilters;
}

TSet<TSharedRef<FFilterCategory>> FSequencerFilterBar::GetFilterCategories(const TSet<TSharedRef<FSequencerTrackFilter>>* InFilters) const
{
	return CommonFilters->GetCategories(InFilters);
}

TSet<TSharedRef<FFilterCategory>> FSequencerFilterBar::GetConfigCategories() const
{
	return { ClassTypeCategory, ComponentTypeCategory, MiscCategory };
}

TSharedRef<FFilterCategory> FSequencerFilterBar::GetClassTypeCategory() const
{
	return ClassTypeCategory;
}

TSharedRef<FFilterCategory> FSequencerFilterBar::GetComponentTypeCategory() const
{
	return ComponentTypeCategory;
}

TSharedRef<FFilterCategory> FSequencerFilterBar::GetMiscCategory() const
{
	return MiscCategory;
}

bool FSequencerFilterBar::HasActiveLevelFilter() const
{
	return LevelFilter->HasHiddenLevels();
}

bool FSequencerFilterBar::HasAllLevelFiltersActive() const
{
	return LevelFilter->HasAllLevelsHidden();
}

const TSet<FString>& FSequencerFilterBar::GetActiveLevelFilters() const
{
	return LevelFilter->GetHiddenLevels();
}

void FSequencerFilterBar::ActivateLevelFilter(const FString& InLevelName, const bool bInActivate)
{
	if (bInActivate)
	{
		LevelFilter->UnhideLevel(InLevelName);
	}
	else
	{
		LevelFilter->HideLevel(InLevelName);
	}
}

bool FSequencerFilterBar::IsLevelFilterActive(const FString InLevelName) const
{
	return !LevelFilter->IsLevelHidden(InLevelName);
}

void FSequencerFilterBar::EnableAllLevelFilters(const bool bInEnable)
{
	LevelFilter->HideAllLevels(!bInEnable);
}

bool FSequencerFilterBar::CanEnableAllLevelFilters(const bool bInEnable)
{
	return LevelFilter->CanHideAllLevels(!bInEnable);
}

void FSequencerFilterBar::EnableAllGroupFilters(const bool bInEnable)
{
	UMovieSceneSequence* const FocusedMovieSequence = Sequencer.GetFocusedMovieSceneSequence();
	if (!FocusedMovieSequence)
	{
		return;
	}

	UMovieScene* const FocusedMovieScene = FocusedMovieSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	for (UMovieSceneNodeGroup* const NodeGroup : FocusedMovieScene->GetNodeGroups())
	{
		NodeGroup->SetEnableFilter(bInEnable);
	}
}

bool FSequencerFilterBar::IsGroupFilterActive() const
{
	return GroupFilter->HasActiveGroupFilter();
}

bool FSequencerFilterBar::PassesAnyCommonFilter(const TViewModelPtr<IOutlinerExtension>& InNode)
{
	UMovieSceneSequence* const FocusedSequence = Sequencer.GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return true;
	}

	bool bPassedAnyFilters = false;
	bool bAnyFilterActive = false;

	// Only one common filter needs to pass for this node to be included in the filtered set
	CommonFilters->ForEachFilter([this, &InNode, &bPassedAnyFilters, &bAnyFilterActive, FocusedSequence]
		(const TSharedRef<FSequencerTrackFilter>& InFilter)
		{
		   if (IsFilterActive(InFilter))
		   {
			  bAnyFilterActive = true;
       
			  if (InFilter->PassesFilter(InNode))
			  {
				 bPassedAnyFilters = true;
				 return false; // Stop processing filters
			  }
		   }

		   return true;
		});

	if (!bAnyFilterActive)
	{
		return true;
	}

	return bPassedAnyFilters;
}

bool FSequencerFilterBar::PassesAllInternalFilters(const TViewModelPtr<IOutlinerExtension>& InNode)
{
	UMovieSceneSequence* const FocusedSequence = Sequencer.GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return true;
	}

	bool bPassedAllFilters = true;

	InternalFilters->ForEachFilter([this, &InNode, &bPassedAllFilters, FocusedSequence]
		(const TSharedRef<FSequencerTrackFilter>& InFilter)
		{
			if (!InFilter->PassesFilter(InNode))
			{
				bPassedAllFilters = false;
				return false; // Stop processing filters
			}
			return true;
		});

	return bPassedAllFilters;
}

bool FSequencerFilterBar::PassesAllCustomTextFilters(const TViewModelPtr<IOutlinerExtension>& InNode)
{
	for (const TSharedRef<FSequencerTrackFilter_CustomText>& Filter : CustomTextFilters)
	{
		if (IsFilterActive(Filter))
		{
			if (!Filter->PassesFilter(InNode))
			{
				return false;
			}
		}
	}

	return true;
}

UWorld* FSequencerFilterBar::GetWorld() const
{
	if (UObject* const PlaybackContext = Sequencer.GetPlaybackContext())
	{
		return PlaybackContext->GetWorld();
	}
	return nullptr;
}

void FSequencerFilterBar::PreFilter() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSequencerFilterBar::PreFilter);
	
	const auto PreFilter = [this](const TSharedRef<FSequencerTrackFilter>& InFilter)
		{
			if (IsFilterActive(InFilter))
			{
				InFilter->OnPreFilter();
			}
		};

	const auto PreFilterCollection = [PreFilter](const TSharedRef<FSequencerTrackFilterCollection>& InCollection)
		{
			InCollection->ForEachFilter([PreFilter]
				(const TSharedRef<FSequencerTrackFilter>& InFilter)
				{
					PreFilter(InFilter);
					return true;
				});
		};

	PreFilterCollection(CommonFilters);
	PreFilterCollection(InternalFilters);

	PreFilter(TextFilter);
	PreFilter(HideIsolateFilter);
	PreFilter(LevelFilter);
	PreFilter(GroupFilter);
	PreFilter(SelectedFilter);
	PreFilter(ModifiedFilter);
}

const FSequencerFilterData& FSequencerFilterBar::FilterNodes()
{
	++FilterSerial;

	EvaluateFilters();

	// Always filter in spacer node
	if (const FSequenceModel* const SequenceModel = Sequencer.GetNodeTree()->GetRootNode()->CastThis<FSequenceModel>())
	{
		if (const TViewModelPtr<IOutlinerExtension> SpacerNode = CastViewModelChecked<IOutlinerExtension>(SequenceModel->GetBottomSpacer()))
		{
			ChangeOutlinerExtensionFilterStateEvent.Broadcast(SpacerNode, false);
		}
	}

	return FilterData;
}

FSequencerFilterData& FSequencerFilterBar::GetFilterData()
{
	return FilterData;
}

const FTextFilterExpressionEvaluator& FSequencerFilterBar::GetTextFilterExpressionEvaluator() const
{
	return GetTextFilter()->GetTextFilterExpressionEvaluator();
}

TArray<TSharedRef<ISequencerTextFilterExpressionContext>> FSequencerFilterBar::GetTextFilterExpressionContexts() const
{
	TArray<TSharedRef<ISequencerTextFilterExpressionContext>> TextExpressions;

	Algo::Transform(TextFilter->GetTextFilterExpressionContexts(), TextExpressions
		, [](TSharedRef<FSequencerTextFilterExpressionContext> InExpressionContext)
		{
			return InExpressionContext;
		});

	CommonFilters->ForEachFilter([]
		(const TSharedRef<FSequencerTrackFilter>& InFilter)
		{

			return true;
		});

	return TextExpressions;
}

FString FSequencerFilterBar::GenerateTextFilterStringFromEnabledFilters() const
{
	TArray<TSharedRef<FSequencerTrackFilter>> FiltersToSave;

	FiltersToSave.Append(GetCommonFilters());

	for (const TSharedRef<FSequencerTrackFilter_CustomText>& Filter : CustomTextFilters)
	{
		FiltersToSave.Add(Filter);
	}

	FString GeneratedFilterString = TextFilter->GetRawFilterText().ToString();

	for (const TSharedRef<FSequencerTrackFilter>& Filter : FiltersToSave)
	{
		if (IsFilterActive(Filter) && IsFilterEnabled(Filter))
		{
			const FString AndAddString = GeneratedFilterString.IsEmpty() ? TEXT("") : TEXT(" AND ");
			const FString ThisFilterGeneratedString = FString::Format(TEXT("{0}{1}==TRUE"), { AndAddString, *Filter->GetName() });
			GeneratedFilterString.Append(ThisFilterGeneratedString);
		}
	}

	return GeneratedFilterString;
}

void FSequencerFilterBar::EvaluateFilters()
{
	// Reset all filter data
	FilterData.Reset();
	
	if (HasAnyFilterActive())
	{
		const FSequencerFilterContext Filters = BuildActiveFilters();
		InvokePreFilter(Filters);
		
		// Loop through all nodes and filter recursively
		for (const TViewModelPtr<IOutlinerExtension>& RootNode : Sequencer.GetNodeTree()->GetRootNodes())
		{
			FilterNodesRecursive(RootNode, Filters, {});
		}
	}
	else
	{
		for (const TViewModelPtr<IOutlinerExtension>& RootNode : Sequencer.GetNodeTree()->GetRootNodes())
		{
			FilterData.FilterInParentChildNodes(RootNode, ENodeVisitFlags::IncludeSelf | ENodeVisitFlags::IncludeChildren);
		}
	}
}

bool FSequencerFilterBar::FilterNodesRecursive(
	const TViewModelPtr<IOutlinerExtension>& InStartNode,
	const FSequencerFilterContext& InFilters,
	const FRecursiveFilterOverrides& InOverrides
	)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSequencerFilterBar::FilterNodesRecursive);
	
	/**
	 * Main Filtering Logic
	 *
	 * - Pinning overrides all other filters
	 * - Hidden/Isolated tracks will take precedence over common filters
	 * - Can hide sub tracks of isolated tracks
	 */

	const USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings();

	// Pinning overrides all other filters
	if (EnumHasAnyFlags(Flags, EFilterFlags::PinnedItemsCanSkipFiltering) && SequencerSettings && !SequencerSettings->GetIncludePinnedInFilter())
	{
		const TSharedPtr<IPinnableExtension> Pinnable = InStartNode.AsModel()->FindAncestorOfType<IPinnableExtension>(true);
		if (Pinnable.IsValid() && Pinnable->IsPinned())
		{
			const ENodeVisitFlags VisitFlags = ENodeVisitFlags::IncludeSelf | ENodeVisitFlags::IncludeChildren;
			FilterData.FilterInParentChildNodes(InStartNode, VisitFlags);
			return true;
		}
	}
	
	const FSequencerFilterEvaluation EvaluationResult = EvaluateSequencerFilterBar(InStartNode, InFilters, InOverrides);
	const bool bAllFiltersPassed = PassesFilterState(EvaluationResult.ItemResult);
	bool bAnyChildPassed = false;
	
	// Some filters, such as text filters, need to be run on all children. If every filter has overrides, we can optimize by trimming the subtree.
	const bool bHasOverrideForAllFilters = InFilters.NumFilters() == EvaluationResult.NewOverrides.Overrides.Num();
	if (bHasOverrideForAllFilters)
	{
		bAnyChildPassed = bAllFiltersPassed;
		const ENodeVisitFlags VisitFlags = ENodeVisitFlags::IncludeSelf | ENodeVisitFlags::IncludeChildren;
		const int32 Num = bAllFiltersPassed 
			? FilterData.FilterInParentChildNodes(InStartNode, VisitFlags) 
			: FilterData.FilterOutParentChildNodes(InStartNode, VisitFlags);
		FilterData.AddTotalNodeCount(Num);
	}
	else
	{
		FilterData.IncrementTotalNodeCount();
		
		// Child nodes should always be processed, as they may force their parents to pass
		for (const TViewModelPtr<IOutlinerExtension>& Node : InStartNode.AsModel()->GetChildrenOfType<IOutlinerExtension>())
		{
			if (FilterNodesRecursive(Node, InFilters, EvaluationResult.NewOverrides))
			{
				bAnyChildPassed = true;
			}
		}
	}
	
	if (bAllFiltersPassed || bAnyChildPassed)
	{
		// Don't expand if a mouse capture is effecting TrackValueChanged events to avoid keys
		// being hidden and expanding tracks while dragging
		if (SequencerSettings && SequencerSettings->GetAutoExpandNodesOnFilterPass()
			&& !FSlateApplication::Get().HasAnyMouseCaptor())
		{
			SetTrackParentsExpanded(InStartNode.ImplicitCast(), true);
		}

		FilterData.FilterInNodeWithAncestors(InStartNode);
		return true;
	}

	// After child nodes are processed, fail anything that didn't pass
	FilterData.FilterOutNode(InStartNode);
	return false;
}

FSequencerFilterContext FSequencerFilterBar::BuildActiveFilters()
{
	FSequencerFilterContext Filters;
	
	if (TextFilter->IsActive())
	{
		Filters.TextFilter = &TextFilter.Get();
	}
	
	if (HideIsolateFilter->IsActive())
	{
		Filters.HideIsolate = &HideIsolateFilter.Get();
	}
	
	CommonFilters->ForEachFilter([this, &Filters](const TSharedRef<FSequencerTrackFilter>& InFilter)
	{
		if (IsFilterActive(InFilter))
		{
			Filters.CommonFilters.Add(&InFilter.Get());
		}
		return true;
	});
	// When filtering treat MultiSelectFilter like any other common filter: any common filter needs to pass to filter in a node.
	// But: it is not supposed to get a filter bar widget when filtering.
	if (IsFilterActive(SelectedFilter) && IsFilterActive(MultiSelectFilter))
	{
		Filters.CommonFilters.Add(&MultiSelectFilter.Get());
	}
	
	InternalFilters->ForEachFilter([this, &Filters](const TSharedRef<FSequencerTrackFilter>& InFilter)
	{
		if (IsFilterActive(InFilter))
		{
			Filters.InternalFilters.Add(&InFilter.Get());
		}
		return true;
	});
	
	for (const TSharedRef<FSequencerTrackFilter_CustomText>& Filter : CustomTextFilters)
	{
		if (IsFilterActive(Filter))
		{
			Filters.CustomTextFilters.Add(&Filter.Get());
		}
	}
	
	return Filters;
}

void FSequencerFilterBar::HandleTextFilterChanged()
{
	OnTextFilterTextChangedEvent.Broadcast();
	RequestFilterUpdate();
}

TSet<TWeakViewModelPtr<IOutlinerExtension>> FSequencerFilterBar::GetSelectedTracksOrAll() const
{
	TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = GetSequencer().GetViewModel();
	return SequencerViewModel ? ::GetSelectedTracksOrAll(*SequencerViewModel) : TSet<TWeakViewModelPtr<IOutlinerExtension>>();
}

TSet<TWeakViewModelPtr<IOutlinerExtension>> FSequencerFilterBar::GetSelectedTracks() const
{
	TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = GetSequencer().GetViewModel();
	return SequencerViewModel ? ::GetSelectedTracks(*SequencerViewModel) : TSet<TWeakViewModelPtr<IOutlinerExtension>>();
}

bool FSequencerFilterBar::HasSelectedTracks() const
{
	return !GetSelectedTracks().IsEmpty();
}

void FSequencerFilterBar::HideSelectedTracks()
{
	const bool bAddToExisting = !FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Shift);
	const TSet<TWeakViewModelPtr<IOutlinerExtension>> TracksToHide = GetSelectedTracks();
	HideIsolateFilter->HideTracks(TracksToHide, bAddToExisting);
}

void FSequencerFilterBar::IsolateSelectedTracks()
{
	const bool bAddToExisting = FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Shift);
	const TSet<TWeakViewModelPtr<IOutlinerExtension>> TracksToIsolate = GetSelectedTracks();
	HideIsolateFilter->IsolateTracks(TracksToIsolate, bAddToExisting);
}

void FSequencerFilterBar::ShowOnlyLocationCategoryGroups()
{
	HideIsolateFilter->IsolateCategoryGroupTracks(GetSelectedTracksOrAll(), { TEXT("Location") }, false);
}

void FSequencerFilterBar::ShowOnlyRotationCategoryGroups()
{
	HideIsolateFilter->IsolateCategoryGroupTracks(GetSelectedTracksOrAll(), { TEXT("Rotation") }, false);
}

void FSequencerFilterBar::ShowOnlyScaleCategoryGroups()
{
	HideIsolateFilter->IsolateCategoryGroupTracks(GetSelectedTracksOrAll(), { TEXT("Scale") }, false);
}

void FSequencerFilterBar::SetTrackParentsExpanded(const TViewModelPtr<IOutlinerExtension>& InNode, const bool bInExpanded)
{
	for (const TViewModelPtr<IOutlinerExtension>& ParentNode : InNode.AsModel()->GetAncestorsOfType<IOutlinerExtension>())
	{
		if (ParentNode->IsExpanded() != bInExpanded)
		{
			ParentNode->SetExpansion(bInExpanded);
		}
	}
}

TArray<TSharedRef<FSequencerTrackFilter>> FSequencerFilterBar::GetFilterList(const bool bInIncludeCustomTextFilters) const
{
	TArray<TSharedRef<FSequencerTrackFilter>> AllFilters;

	AllFilters.Append(CommonFilters->GetAllFilters());
	AllFilters.Append(InternalFilters->GetAllFilters());

	AllFilters.Add(TextFilter);
	AllFilters.Add(HideIsolateFilter);

	if (bInIncludeCustomTextFilters)
	{
		for (const TSharedRef<FSequencerTrackFilter_CustomText>& Filter : CustomTextFilters)
		{
			AllFilters.Add(Filter);
		}
	}

	return AllFilters;
}

bool FSequencerFilterBar::ShouldUpdateOnTrackValueChanged() const
{
	if (bFiltersMuted)
	{
		return false;
	}

	const TArray<TSharedRef<FSequencerTrackFilter>> AllFilters = GetFilterList();

	for (const TSharedRef<FSequencerTrackFilter>& Filter : AllFilters)
	{
		if (Filter->ShouldUpdateOnTrackValueChanged() && IsFilterActive(Filter))
		{
			return true;
		}
	}

	return false;
}

bool FSequencerFilterBar::IsFilterSupported(const TSharedRef<FSequencerTrackFilter>& InFilter) const
{
	UMovieSceneSequence* const MovieSceneSequence = Sequencer.GetFocusedMovieSceneSequence();
	if (!MovieSceneSequence)
	{
		return false;
	}

	const FString FilterName = InFilter->GetName();
	const bool bFilterSupportsSequence = InFilter->SupportsSequence(MovieSceneSequence);
	const bool bSequenceSupportsFilter = MovieSceneSequence->IsFilterSupported(FilterName);
	return bFilterSupportsSequence || bSequenceSupportsFilter;
}

bool FSequencerFilterBar::IsFilterSupported(const FString& InFilterName) const
{
	const TArray<TSharedRef<FSequencerTrackFilter>> FilterList = GetFilterList();
	const TSharedRef<FSequencerTrackFilter>* const FoundFilter = FilterList.FindByPredicate(
		[InFilterName](const TSharedRef<FSequencerTrackFilter>& InFilter)
		{
			return InFilter->GetName().Equals(InFilterName, ESearchCase::IgnoreCase);
		});
	return FoundFilter ? IsFilterSupported(*FoundFilter) : false;
}

void FSequencerFilterBar::OpenTextExpressionHelp()
{
	OpenDialog_TextExpressionHelp(SharedThis(this));
}

void FSequencerFilterBar::SaveCurrentFilterSetAsCustomTextFilter()
{
	const FText NewFilterString = FText::FromString(GenerateTextFilterStringFromEnabledFilters());
	// @TODO: update to the new way of handling this (similar to Sequence Navigator)
	//CreateWindow_AddCustomTextFilter(DefaultNewCustomTextFilterData(NewFilterString));
}

void FSequencerFilterBar::CreateNewTextFilter()
{
	const FText NewFilterString = FText::FromString(GetTextFilterString());
	// @TODO: update to the new way of handling this (similar to Sequence Navigator)
	//CreateWindow_AddCustomTextFilter(DefaultNewCustomTextFilterData(NewFilterString));
}

bool FSequencerFilterBar::DoesCommonFilterPass(const TViewModelPtr<FViewModel>& InViewModel, const FString& InFilterName) const
{
	bool bAnyPassed = false;

	CommonFilters->ForEachFilter([this, &bAnyPassed, &InViewModel, &InFilterName]
		(const TSharedRef<FSequencerTrackFilter>& InFilter)
		{
			if (InFilter->GetName().Equals(InFilterName, ESearchCase::IgnoreCase)
				&& InFilter->PassesFilter(InViewModel.ImplicitCast()))
			{
				bAnyPassed = true;
				return false; // Stop looping
			}
			return true;
		});

	return bAnyPassed;
}

void FSequencerFilterBar::GetCommonFilterKeySuggestions(TArray<FSequencerFilterSuggestion>& OutSuggestions) const
{
	for (const TSharedRef<FSequencerTrackFilter>& CommonFilter : CommonFilters->GetAllFilters())
	{
		FSequencerFilterSuggestion NewSuggestion;
		NewSuggestion.Suggestion = CommonFilter->GetName();
		NewSuggestion.DisplayName = CommonFilter->GetDisplayName();
		NewSuggestion.Description = CommonFilter->GetDefaultToolTipText();

		OutSuggestions.AddUnique(NewSuggestion);
	}
}

void FSequencerFilterBar::GetCommonFilterValueSuggestions(const FString& InKey, TArray<FSequencerFilterSuggestion>& OutSuggestions) const
{
	for (const TSharedRef<FSequencerTrackFilter>& CommonFilter : CommonFilters->GetAllFilters())
	{
		if (InKey.Equals(CommonFilter->GetName(), ESearchCase::IgnoreCase))
		{
			FSequencerFilterSuggestion::MakeBooleanSuggestions(OutSuggestions);
			break;
		}
	}
}

bool FSequencerFilterBar::IsCommonFilterName(const FString& InFilterName) const
{
	bool bFoundFilter = false;

	CommonFilters->ForEachFilter([&InFilterName, &bFoundFilter](const TSharedRef<FSequencerTrackFilter>& InFilter)
		{
			bFoundFilter = InFilter->GetName().Equals(InFilterName, ESearchCase::IgnoreCase);
			const bool bContinue = !bFoundFilter;
			return bContinue;
		});

	return bFoundFilter;
}

#undef LOCTEXT_NAMESPACE
