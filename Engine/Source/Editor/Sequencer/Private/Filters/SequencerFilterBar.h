// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FilterBarInitArgs.h"
#include "Filters/CustomTextFilters.h"
#include "Filters/FilterBarDelegates.h"
#include "Filters/ISequencerTrackFilters.h"
#include "Filters/SequencerFilterData.h"
#include "Filters/SequencerTrackFilterBase.h"

class FSequencer;
class FSequencerTrackFilter;
class FSequencerTrackFilterCollection;
class FSequencerTrackFilterMenu;
class FSequencerTrackFilter_CustomText;
class FSequencerTrackFilter_Group;
class FSequencerTrackFilter_HideIsolate;
class FSequencerTrackFilter_Level;
class FSequencerTrackFilter_Modified;
class FSequencerTrackFilter_Selected;
class FSequencerTrackFilter_Text;
class FUICommandList;
class SComboButton;
class SFilterBarIsolateHideShow;
class SSequencerFilterBar;
class SSequencerSearchBox;
enum class EFilterBarLayout : uint8;
namespace UE::Sequencer {
struct FSequencerFilterOverrides;
struct FSequencerFilterContext;
class FSequencerTrackFilter_StickySelection;
struct FRecursiveFilterOverrides;
class IOutlinerExtension; }
namespace UE::Sequencer { struct FFilterBarInitArgs; }

enum class ESequencerFilterChange : uint8
{
	/** The filter shows up in the filter bar. In the combo menu, it is checked. */
	Enable,
	/** The filter does not show up in the filter bar. In the combo menu, it is unchecked. */
	Disable,
	/** The filter shows up in the filter bar and is currently filtering. In the combo menu, it is checked. */
	Activate,
	/** The filter shows up in the filter bar but is currently not filtering. In the combo menu, it is checked. */
	Deactivate
};

/** Holds the Sequencer track filter collection, the current text filter, and hidden/isolated lists. */
class FSequencerFilterBar : public ISequencerTrackFilters
{
	struct FPrivateToken { explicit FPrivateToken() = default; }; // Force construction through FSequencerFilterBar::Make.
public:
	
	DECLARE_EVENT_TwoParams(FSequencerFilterBar, FSequencerFiltersChanged, const ESequencerFilterChange /*InChangeType*/, const TSharedRef<FSequencerTrackFilter>& /*InFilter*/);
	DECLARE_EVENT_TwoParams(FSequencerFilterBar, FSequencerCustomTextFiltersChanged, const ESequencerFilterChange /*InChangeType*/, const TSharedRef<FSequencerTrackFilter_CustomText>& /*InFilter*/);
	
	/** Creates a filter bar instance. */
	static TSharedRef<FSequencerFilterBar> Make(UE::Sequencer::FFilterBarInitArgs InArgs);
	
	explicit FSequencerFilterBar(UE::Sequencer::FFilterBarInitArgs InArgs, FPrivateToken);
	virtual ~FSequencerFilterBar() override;

	TSharedPtr<ICustomTextFilter<FSequencerTrackFilterType>> CreateTextFilter();
	
	/**
	 * Copies the activation and enabled state of all filters from InSource
	 * to this filter bar. After the call, both instances contain identical
	 * filter states.
	 */
	void CopyFiltersFrom(const FSequencerFilterBar& InSource);

	virtual bool AreFiltersMuted() const override;
	virtual void MuteFilters(const bool bInMute) override;
	void ToggleMuteFilters();

	virtual bool CanResetFilters() const override;
	virtual void ResetFilters() override;

	FSequencerFiltersChanged& OnFiltersChanged() { return FiltersChangedEvent; }

	TSharedRef<FSequencerTrackFilter_Text> GetTextFilter() const;
	virtual TSharedRef<FSequencerTrackFilter_HideIsolate> GetHideIsolateFilter() const override;
	FText GetFilterErrorText() const;

	/** Hide/Isolate/Show Filter Functions */

	void HideTracks(const TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>>& InTracks, const bool bInAddToExisting);
	void UnhideTracks(const TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>>& InTracks);

	void IsolateTracks(const TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>>& InTracks, const bool bInAddToExisting);
	void UnisolateTracks(const TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>>& InTracks);
	
	bool HasHiddenTracks() const;
	bool HasIsolatedTracks() const;

	TSharedPtr<FSequencerTrackFilter> FindFilterByDisplayName(const FString& InFilterName) const;
	TSharedPtr<FSequencerTrackFilter_CustomText> FindCustomTextFilterByDisplayName(const FString& InFilterName) const;

	bool HasAnyFiltersEnabled() const;

	//~ Begin ISequencerFilterBar

	virtual FName GetIdentifier() const override;

	virtual ISequencer& GetSequencer() const override;

	virtual TSharedPtr<FUICommandList> GetCommandList() const override;

	virtual FString GetTextFilterString() const override;
    virtual void SetTextFilterString(const FString& InText) override;

	virtual bool DoesTextFilterStringContainExpressionPair(const ISequencerTextFilterExpressionContext& InExpression) const override;

	virtual void RequestFilterUpdate() override;

	virtual void EnableAllFilters(const bool bInEnable, const TArray<FString>& InExceptionFilterNames) override;

	virtual void ActivateCommonFilters(const bool bInActivate, const TArray<FString>& InExceptionFilterNames) override;

	virtual bool AreAllEnabledFiltersActive(const bool bInActive, const TArray<FString> InExceptionFilterNames) const override;
	virtual void ActivateAllEnabledFilters(const bool bInActivate, const TArray<FString> InExceptionFilterNames) override;
	virtual void ToggleActivateAllEnabledFilters();

	virtual bool IsFilterActiveByDisplayName(const FString& InFilterName) const override;
	virtual bool IsFilterEnabledByDisplayName(const FString& InFilterName) const override;
	virtual bool SetFilterActiveByDisplayName(const FString& InFilterName, const bool bInActive, const bool bInRequestFilterUpdate = true) override;
	virtual bool SetFilterEnabledByDisplayName(const FString& InFilterName, const bool bInEnabled, const bool bInRequestFilterUpdate = true) override;

	virtual TArray<FText> GetFilterDisplayNames() const override;
	virtual TArray<FText> GetCustomTextFilterNames() const override;

	virtual int32 GetTotalDisplayNodeCount() const override;
	virtual int32 GetFilteredDisplayNodeCount() const override;

	//~ End ISequencerFilterBar

	//~ End ISequencerTrackFilters

	virtual void HideSelectedTracks() override;
	virtual void IsolateSelectedTracks() override;

	virtual void ShowOnlyLocationCategoryGroups() override;
	virtual void ShowOnlyRotationCategoryGroups() override;
	virtual void ShowOnlyScaleCategoryGroups() override;

	virtual bool HasSelectedTracks() const override;

	virtual FSequencerFilterData& GetFilterData() override;

	virtual const FTextFilterExpressionEvaluator& GetTextFilterExpressionEvaluator() const override;
	virtual TArray<TSharedRef<ISequencerTextFilterExpressionContext>> GetTextFilterExpressionContexts() const override;
	virtual FString GenerateTextFilterStringFromEnabledFilters() const override;

	//~ End ISequencerTrackFilters

	/** Active Filter Functions */

	bool AnyCommonFilterActive() const;
	bool AnyInternalFilterActive() const;
	virtual bool HasAnyFilterActive(const bool bCheckTextFilter = true
		, const bool bInCheckHideIsolateFilter = true
		, const bool bInCheckCommonFilters = true
		, const bool bInCheckInternalFilters = true
		, const bool bInCheckCustomTextFilters = true) const override;
	virtual bool IsFilterActive(const TSharedRef<FSequencerTrackFilter> InFilter) const override;
	virtual bool SetFilterActive(const TSharedRef<FSequencerTrackFilter>& InFilter, const bool bInActive, const bool bInRequestFilterUpdate = true) override;
	void ActivateCommonFilters(const bool bInActivate
		, const TArray<TSharedRef<FFilterCategory>> InMatchCategories
		, const TArray<TSharedRef<FSequencerTrackFilter>>& InExceptions);
	TArray<TSharedRef<FSequencerTrackFilter>> GetActiveFilters() const;

	/** Enabled Filter Functions */

	bool HasEnabledCommonFilters() const;
	bool HasEnabledFilter(const TArray<TSharedRef<FSequencerTrackFilter>>& InFilters = {}) const;
	virtual bool HasAnyFilterEnabled() const override;
	virtual bool IsFilterEnabled(TSharedRef<FSequencerTrackFilter> InFilter) const override;
	virtual bool SetFilterEnabled(const TSharedRef<FSequencerTrackFilter> InFilter, const bool bInEnabled, const bool bInRequestFilterUpdate = true) override;
	void EnableFilters(const bool bInEnable
		, const TArray<TSharedRef<FFilterCategory>> InMatchCategories = {}
		, const TArray<TSharedRef<FSequencerTrackFilter>> InExceptions = {});
	void ToggleFilterEnabled(const TSharedRef<FSequencerTrackFilter> InFilter);
	TArray<TSharedRef<FSequencerTrackFilter>> GetEnabledFilters() const;

	/** Filter Functions */

	bool HasAnyCommonFilters() const;
	bool AddFilter(const TSharedRef<FSequencerTrackFilter>& InFilter);
	bool RemoveFilter(const TSharedRef<FSequencerTrackFilter>& InFilter);

	TArray<TSharedRef<FSequencerTrackFilter>> GetCommonFilters(const TArray<TSharedRef<FFilterCategory>>& InCategories = {}) const;

	/** Custom Text Filter Functions */

	bool AnyCustomTextFilterActive() const;
	bool HasEnabledCustomTextFilters() const;
	TArray<TSharedRef<FSequencerTrackFilter_CustomText>> GetAllCustomTextFilters() const;
	virtual bool AddCustomTextFilter(const TSharedRef<FSequencerTrackFilter_CustomText>& InFilter, const bool bInAddToConfig) override;
	virtual bool RemoveCustomTextFilter(const TSharedRef<FSequencerTrackFilter_CustomText>& InFilter, const bool bInAddToConfig) override;
	void ActivateCustomTextFilters(const bool bInActivate, const TArray<TSharedRef<FSequencerTrackFilter_CustomText>> InExceptions = {});
	void EnableCustomTextFilters(const bool bInEnable, const TArray<TSharedRef<FSequencerTrackFilter_CustomText>> InExceptions = {});
	TArray<TSharedRef<FSequencerTrackFilter_CustomText>> GetEnabledCustomTextFilters() const;

	/** Filter Category Functions */

	TSet<TSharedRef<FFilterCategory>> GetFilterCategories(const TSet<TSharedRef<FSequencerTrackFilter>>* InFilters = nullptr) const;
	TSet<TSharedRef<FFilterCategory>> GetConfigCategories() const;
	TSharedRef<FFilterCategory> GetClassTypeCategory() const;
	TSharedRef<FFilterCategory> GetComponentTypeCategory() const;
	TSharedRef<FFilterCategory> GetMiscCategory() const;

	/** Level Filter Functions */

	bool HasActiveLevelFilter() const;
	bool HasAllLevelFiltersActive() const;
	const TSet<FString>& GetActiveLevelFilters() const;
	void ActivateLevelFilter(const FString& InLevelName, const bool bInActivate);
	bool IsLevelFilterActive(const FString InLevelName) const;
	void EnableAllLevelFilters(const bool bInEnable);
	bool CanEnableAllLevelFilters(const bool bInEnable);

	/** Group Filter Functions */

	void EnableAllGroupFilters(const bool bInEnable);
	bool IsGroupFilterActive() const;

	/** Misc Functions */

	void SetTrackParentsExpanded(const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InNode, const bool bInExpanded);

	UWorld* GetWorld() const;

	const FSequencerFilterData& FilterNodes();

	/**
	 * Monotonically incrementing serial that bumps once per FilterNodes() invocation. Lets cached
	 * views (e.g. SChannelView's key renderer) detect that filter state has been re-evaluated and
	 * the set of filtered-in descendants may have changed, without subscribing to per-item events.
	 */
	uint32 GetFilterSerial() const { return FilterSerial; }

	bool ShouldUpdateOnTrackValueChanged() const;

	bool IsFilterSupported(const TSharedRef<FSequencerTrackFilter>& InFilter) const;
	bool IsFilterSupported(const FString& InFilterName) const;

	virtual FSimpleMulticastDelegate& OnRequestUpdate() override { return RequestUpdateEvent; }
	virtual FSimpleMulticastDelegate& OnTextFilterTextChanged() override { return OnTextFilterTextChangedEvent; }
	virtual FMuteStateChangedDelegate& OnMuteFiltersChanged() override { return OnMuteFiltersChangedEvent; }
	UE::Sequencer::FChangeOutlinerExtensionFilterState& OnChangeOutlinerExtensionFilterState() { return ChangeOutlinerExtensionFilterStateEvent; }
	
	virtual void OpenTextExpressionHelp() override;
	virtual void SaveCurrentFilterSetAsCustomTextFilter() override;
	virtual void CreateNewTextFilter() override;

	virtual bool DoesCommonFilterPass(const UE::Sequencer::TViewModelPtr<UE::Sequencer::FViewModel>& InViewModel, const FString& InFilterName) const override;

	virtual void GetCommonFilterKeySuggestions(TArray<FSequencerFilterSuggestion>& OutSuggestions) const override;
	virtual void GetCommonFilterValueSuggestions(const FString& InKey, TArray<FSequencerFilterSuggestion>& OutSuggestions) const override;

	virtual bool IsCommonFilterName(const FString& InFilterName) const override;
	
	TArray<TSharedRef<FSequencerTrackFilter>> GetFilterList(const bool bInIncludeCustomTextFilters = false) const;

protected:
	virtual void GetCommonFiltersImpl(TArray<TSharedPtr<void>>& OutTypeErasedFilters) const override;

	void CreateDefaultFilters();
	void CreateCustomTextFiltersFromConfig();

	bool PassesAnyCommonFilter(const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InNode);
	bool PassesAllInternalFilters(const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InNode);
	bool PassesAllCustomTextFilters(const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InNode);

	/** Calls the event before nodes are filtered to give filters a chance to pre-process */
	void PreFilter() const;

	/** The Sequencer this filter bar is interacting with */
	FSequencer& Sequencer;
	
	FSequencerFiltersChanged FiltersChangedEvent;
	FSimpleMulticastDelegate RequestUpdateEvent;
	FSimpleMulticastDelegate OnTextFilterTextChangedEvent;
	FMuteStateChangedDelegate OnMuteFiltersChangedEvent;
	/** Broadcasts when the filter state of IOutlinerExtension is changed. */
	UE::Sequencer::FChangeOutlinerExtensionFilterState ChangeOutlinerExtensionFilterStateEvent;
	
	/** Caches the applied filter state. */
	FSequencerFilterData FilterData;

	/** Bumped once per FilterNodes() call. See GetFilterSerial(). */
	uint32 FilterSerial = 0;

	/** Binds the commands that the filters handle. */
	const TSharedRef<FUICommandList> CommandList;

	/** Global override to enable/disable all filters */
	bool bFiltersMuted = false;

	// As C++ destroys in reverse declaration order, put filters last. This will cause them to be destroyed before the above members. 
	// The filters' destructors cause the above members to be accessed, e.g. RequestUpdateEvent.
	const TSharedRef<FFilterCategory> ClassTypeCategory;
	const TSharedRef<FFilterCategory> ComponentTypeCategory;
	const TSharedRef<FFilterCategory> MiscCategory;
	const TSharedRef<FFilterCategory> TransientCategory;

	const TSharedRef<FSequencerTrackFilterCollection> CommonFilters;
	const TSharedRef<FSequencerTrackFilterCollection> InternalFilters;

	const TSharedRef<FSequencerTrackFilter_Text> TextFilter;
	const TSharedRef<FSequencerTrackFilter_HideIsolate> HideIsolateFilter;
	const TSharedRef<FSequencerTrackFilter_Level> LevelFilter;
	const TSharedRef<FSequencerTrackFilter_Group> GroupFilter;
	const TSharedRef<FSequencerTrackFilter_Selected> SelectedFilter;
	const TSharedRef<UE::Sequencer::FSequencerTrackFilter_StickySelection> MultiSelectFilter;
	const TSharedRef<FSequencerTrackFilter_Modified> ModifiedFilter;
	
	TArray<TSharedRef<FSequencerTrackFilter_CustomText>> CustomTextFilters;

private:
	
	static int32 InstanceCount;
	
	/**
	 * Used for per-filter bar instance config.
	 * This is relevant for the settings to use for reading and saving config.
	 * @see FSequencerFilterBarConfig, USequencerSettings::FindTrackFilterBar, FSequencerFilterBar::GetIdentifier.
	 */
	const FString SubIdentifier;

	/** Affects how filtering is performed. */
	const UE::Sequencer::EFilterFlags Flags;

	/**
	 * ID used to look up the filter area config (FSequencerFilterAreaConfig).
	 * When set, enables the ToggleFilterBarVisibility command binding.
	 * @see USequencerSettings::FindOrAddTrackFilterArea
	 */
	FName FilterAreaConfigId;

	bool IsFilterBarVisible() const;
	void ToggleFilterBarVisibilityCommand();
	
	/** @return Gets the selected IOutlinerExtensions, or all of them if nothing is selected. */
	TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>> GetSelectedTracksOrAll() const;

	/** @return Gets the selected IOutlinerExtensions. */
	TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>> GetSelectedTracks() const;

	/** Binds all commands that the filter bar or its filters handle. */
	void BindCommands();
	
	/** New algorithm for filtering nodes. */
	void EvaluateFilters();
	/** @return Filter that are active and should be evaluated. */
	UE::Sequencer::FSequencerFilterContext BuildActiveFilters();
	/** Do not call directly! Should only be called by FilterNodes(). */
	bool FilterNodesRecursive(
		const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InStartNode,
		const UE::Sequencer::FSequencerFilterContext& InFilters,
		const UE::Sequencer::FRecursiveFilterOverrides& InOverrides
		);
	
	/** Invoked when the text filter changes. */
	void HandleTextFilterChanged();
};
