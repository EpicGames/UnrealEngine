// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/SequencerFilterBarConfig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequencerFilterBarConfig)

bool FSequencerFilterBarConfig::IsFilterEnabled(const FString& InFilterName) const
{
	return ActiveFilters.EnabledStates.Contains(InFilterName);
}

bool FSequencerFilterBarConfig::SetFilterEnabled(const FString& InFilterName, const bool bInActive)
{
	if (bInActive)
	{
		ActiveFilters.EnabledStates.FindOrAdd(InFilterName);
		return true;
	}
	else
	{
		return ActiveFilters.EnabledStates.Remove(InFilterName) == 1;
	}
}

bool FSequencerFilterBarConfig::IsFilterActive(const FString& InFilterName) const
{
	if (ActiveFilters.EnabledStates.Contains(InFilterName))
	{
		return ActiveFilters.EnabledStates[InFilterName];
	}
	return false;
}

bool FSequencerFilterBarConfig::SetFilterActive(const FString& InFilterName, const bool bInActive)
{
	if (bool* EnabledStatePtr = ActiveFilters.EnabledStates.Find(InFilterName))
	{
		if (*EnabledStatePtr == bInActive)
		{
			return false;
		}

		*EnabledStatePtr = bInActive;

		return true;
	}

	ActiveFilters.EnabledStates.Add(InFilterName) = bInActive;

	return true;
}

const FSequencerFilterSet& FSequencerFilterBarConfig::GetCommonActiveSet() const
{
	return ActiveFilters;
}

TArray<FCustomTextFilterData>& FSequencerFilterBarConfig::GetCustomTextFilters()
{
	return CustomTextFilters;
}

bool FSequencerFilterBarConfig::HasCustomTextFilter(const FString& InFilterName) const
{
	const FCustomTextFilterData* const FoundFilterData = CustomTextFilters.FindByPredicate([&InFilterName](const FCustomTextFilterData& InFilterData)
	{
		return InFilterData.FilterLabel.ToString().Equals(InFilterName, ESearchCase::IgnoreCase);
	});
	return FoundFilterData != nullptr;
}

FCustomTextFilterData* FSequencerFilterBarConfig::FindCustomTextFilter(const FString& InFilterName)
{
	return CustomTextFilters.FindByPredicate([&InFilterName](const FCustomTextFilterData& InFilterData)
		{
			return InFilterData.FilterLabel.ToString().Equals(InFilterName, ESearchCase::IgnoreCase);
		});
}

bool FSequencerFilterBarConfig::AddCustomTextFilter(FCustomTextFilterData InFilterData)
{
	if (!HasCustomTextFilter(InFilterData.FilterLabel.ToString()))
	{
		CustomTextFilters.Add(MoveTemp(InFilterData));
		return true;
	}
	return false;
}

bool FSequencerFilterBarConfig::RemoveCustomTextFilter(const FString& InFilterName)
{
	if (HasCustomTextFilter(InFilterName))
	{
		const int32 IndexToRemove = CustomTextFilters.IndexOfByPredicate([&InFilterName](const FCustomTextFilterData& InFilterData)
			{
				return InFilterData.FilterLabel.ToString().Equals(InFilterName, ESearchCase::IgnoreCase);
			});
		if (IndexToRemove != INDEX_NONE)
		{
			CustomTextFilters.RemoveAt(IndexToRemove);
			return true;
		}
	}
	return false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
EFilterBarLayout FSequencerFilterBarConfig::GetFilterBarLayout() const
{
	return FilterBarLayout_DEPRECATED;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FSequencerFilterBarConfig::SetFilterBarLayout(const EFilterBarLayout InLayout)
{
	FilterBarLayout_DEPRECATED = InLayout;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
