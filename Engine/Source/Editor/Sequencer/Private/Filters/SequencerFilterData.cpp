// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/SequencerFilterData.h"
#include "ISequencer.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

using namespace UE::Sequencer;

FSequencerFilterData::FSequencerFilterData(const FString& InRawFilterText)
	: RawFilterText(InRawFilterText)
{
}

bool FSequencerFilterData::operator==(const FSequencerFilterData& InRhs) const
{
	return GetTotalNodeCount() == InRhs.GetTotalNodeCount()
		&& GetDisplayNodeCount() == InRhs.GetDisplayNodeCount()
		&& ContainsFilterInNodes(InRhs);
}

bool FSequencerFilterData::operator!=(const FSequencerFilterData& InRhs) const
{
	return !(*this == InRhs);
}

void FSequencerFilterData::Reset()
{
	ProcessedNodes.Reset();
	FilterInNodes.Reset();
	
	TotalNodeCount = 0;
}

FString FSequencerFilterData::GetRawFilterText() const
{
	return RawFilterText;
}

uint32 FSequencerFilterData::GetDisplayNodeCount() const
{
	return FilterInNodes.Num();
}

uint32 FSequencerFilterData::GetTotalNodeCount() const
{
	return TotalNodeCount;
}

uint32 FSequencerFilterData::GetFilterInCount() const
{
	return FilterInNodes.Num();
}

uint32 FSequencerFilterData::GetFilterOutCount() const
{
	return GetTotalNodeCount() - GetFilterInCount();
}

void FSequencerFilterData::IncrementTotalNodeCount()
{
	++TotalNodeCount;
}

void FSequencerFilterData::AddTotalNodeCount(int32 InNodeCount)
{
	TotalNodeCount += FMath::Max(0, InNodeCount);
}

void FSequencerFilterData::FilterInNode(TWeakViewModelPtr<IOutlinerExtension> InNodeWeak)
{
	ProcessedNodes.Add(InNodeWeak);
	
	FilterInNodes.Add(InNodeWeak);

	if (const TViewModelPtr<IOutlinerExtension> Node = InNodeWeak.Pin())
	{
		ChangeFilterStateEvent.Broadcast(Node, false);
	}
}

void FSequencerFilterData::FilterOutNode(TWeakViewModelPtr<IOutlinerExtension> InNodeWeak)
{
	ProcessedNodes.Add(InNodeWeak);
	
	const FSetElementId ElementId = FilterInNodes.FindId(InNodeWeak);
	if (ElementId.IsValidId())
	{
		FilterInNodes.Remove(ElementId);
	}

	if (const TViewModelPtr<IOutlinerExtension> Node = InNodeWeak.Pin())
	{
		ChangeFilterStateEvent.Broadcast(Node, true);
	}
}

void FSequencerFilterData::FilterInParentChildNodes(
	const TViewModelPtr<IOutlinerExtension>& InNode, const bool bInIncludeSelf, const bool bInIncludeParents, const bool bInIncludeChildren
	)
{
	const ENodeVisitFlags Flags = 
		(bInIncludeSelf ? ENodeVisitFlags::IncludeSelf : ENodeVisitFlags::None)
		| (bInIncludeParents ? ENodeVisitFlags::IncludeParents : ENodeVisitFlags::None)
		| (bInIncludeChildren ? ENodeVisitFlags::IncludeChildren : ENodeVisitFlags::None);
	FilterInParentChildNodes(InNode, Flags);
}

int32 FSequencerFilterData::FilterInParentChildNodes(const TViewModelPtr<IOutlinerExtension>& InNode, ENodeVisitFlags InFlags)
{
	const auto FilterInLambda = [this](const TViewModelPtr<IOutlinerExtension>& Node){ FilterInNode(Node); };
	return ApplyToNodes(InNode, FilterInLambda, InFlags);
}

int32 FSequencerFilterData::FilterOutParentChildNodes(const TViewModelPtr<IOutlinerExtension>& InNode, ENodeVisitFlags InFlags)
{
	const auto FilterOutLambda = [this](const TViewModelPtr<IOutlinerExtension>& Node){ FilterOutNode(Node); };
	return ApplyToNodes(InNode, FilterOutLambda, InFlags);
}

void FSequencerFilterData::FilterInNodeWithAncestors(const TViewModelPtr<IOutlinerExtension>& InNode)
{
	FilterInParentChildNodes(InNode, ENodeVisitFlags::IncludeSelf | ENodeVisitFlags::IncludeParents);
}

bool FSequencerFilterData::ContainsFilterInNodes(const FSequencerFilterData& InOtherData) const
{
	return FilterInNodes.Includes(InOtherData.FilterInNodes);
}

bool FSequencerFilterData::IsFilteredOut(const TViewModelPtr<IOutlinerExtension>& InNode) const
{
	// If an item has been added since the last filtered run, then we'll not consider it filtered out.
	return ProcessedNodes.Contains(InNode) && !FilterInNodes.Contains(InNode);
}

TWeakViewModelPtr<ITrackExtension> FSequencerFilterData::ResolveTrack(FSequencerTrackFilterType InNode)
{
	if (!InNode.IsValid())
	{
		return nullptr;
	}

	const TWeakViewModelPtr<IOutlinerExtension> WeakOutlinerNode = InNode.ImplicitCast();

	// Use cache version if it exists, otherwise resolve below
	if (ResolvedTracks.Contains(WeakOutlinerNode))
	{
		if (const TViewModelPtr<ITrackExtension> ResolvedTrack = ResolvedTracks[WeakOutlinerNode].Pin())
		{
			return ResolvedTrack;
		}

		ResolvedTracks.Remove(WeakOutlinerNode);
	}

	// Resolve and cache
	if (const TViewModelPtr<ITrackExtension> AncestorTrack = InNode->FindAncestorOfType<ITrackExtension>(true))
	{
		ResolvedTracks.Add(WeakOutlinerNode, AncestorTrack);
		return AncestorTrack;
	}

	return nullptr;
}

UMovieSceneTrack* FSequencerFilterData::ResolveMovieSceneTrackObject(FSequencerTrackFilterType InNode)
{
	if (!InNode.IsValid())
	{
		return nullptr;
	}

	const TWeakViewModelPtr<IOutlinerExtension> WeakOutlinerNode = InNode.ImplicitCast();

	// Use cache version if it exists, otherwise resolve below
	if (ResolvedTrackObjects.Contains(WeakOutlinerNode))
	{
		if (ResolvedTrackObjects[WeakOutlinerNode].IsValid())
		{
			return ResolvedTrackObjects[WeakOutlinerNode].Get();
		}

		ResolvedTrackObjects.Remove(WeakOutlinerNode);
	}

	if (const TViewModelPtr<ITrackExtension> AncestorTrackModel = InNode->FindAncestorOfType<ITrackExtension>(true))
	{
		if (UMovieSceneTrack* const TrackObject = AncestorTrackModel->GetTrack())
		{
			ResolvedTrackObjects.Add(WeakOutlinerNode, TrackObject);
			return TrackObject;
		}
	}

	return nullptr;
}

UObject* FSequencerFilterData::ResolveTrackBoundObject(ISequencer& InSequencer, FSequencerTrackFilterType InNode)
{
	if (!InNode.IsValid())
	{
		return nullptr;
	}

	const TWeakViewModelPtr<IOutlinerExtension> WeakOutlinerNode = InNode.ImplicitCast();

	// Use cache version if it exists, otherwise resolve below
	if (ResolvedObjects.Contains(WeakOutlinerNode))
	{
		if (ResolvedObjects[WeakOutlinerNode].IsValid())
		{
			return ResolvedObjects[WeakOutlinerNode].Get();
		}

		ResolvedObjects.Remove(WeakOutlinerNode);
	}

	if (const TViewModelPtr<IObjectBindingExtension> ObjectBindingModel = InNode->FindAncestorOfType<IObjectBindingExtension>(true))
	{
		if (UObject* const BoundObject = InSequencer.FindSpawnedObjectOrTemplate(ObjectBindingModel->GetObjectGuid()))
		{
			ResolvedObjects.Add(WeakOutlinerNode, BoundObject);
			return BoundObject;
		}
	}

	return nullptr;
}

int32 FSequencerFilterData::ApplyToNodes(
	const TViewModelPtr<IOutlinerExtension>& InNode, 
	TFunctionRef<void(const TViewModelPtr<IOutlinerExtension>&)>&& InCallback,
	ENodeVisitFlags InFlags
	)
{
	int32 NodeCount = 0;
	if (!InNode.IsValid())
	{
		return NodeCount;
	}
	
	if (EnumHasAnyFlags(InFlags, ENodeVisitFlags::IncludeParents))
	{
		for (TViewModelPtr<IOutlinerExtension> ParentNode : InNode.AsModel()->GetAncestorsOfType<IOutlinerExtension>())
		{
			++NodeCount;
			InCallback(ParentNode);
		}
	}

	if (EnumHasAnyFlags(InFlags, ENodeVisitFlags::IncludeSelf))
	{
		++NodeCount;
		InCallback(InNode);
	}

	if (EnumHasAnyFlags(InFlags, ENodeVisitFlags::IncludeChildren))
	{
		for (TViewModelPtr<IOutlinerExtension> ChildNode : InNode.AsModel()->GetDescendantsOfType<IOutlinerExtension>())
		{
			++NodeCount;
			InCallback(ChildNode);
		}
	}
	
	return NodeCount;
}