// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/SequencerTrackFilterCollection.h"
#include "Sequencer.h"

using namespace UE::Sequencer;

FSequencerTrackFilterCollection::FSequencerTrackFilterCollection(ISequencerTrackFilters& InFilterInterface)
	: FilterInterface(InFilterInterface)
{
}

bool FSequencerTrackFilterCollection::ContainsFilter(const TSharedRef<FSequencerTrackFilter>& InItem) const
{
	bool bOutContains = false;

	ForEachFilter([&InItem, &bOutContains]
		(const TSharedRef<FSequencerTrackFilter>& InTrackFilter) -> bool
		{
			if (InItem == InTrackFilter)
			{
				bOutContains = true;
				return false;
			}
			return true;
		});

	return bOutContains;
}

void FSequencerTrackFilterCollection::RemoveAll()
{
	using namespace UE::Sequencer;

	for (const TSharedPtr<IFilter<TViewModelPtr<FViewModel>>>& Filter : ChildFilters)
	{
		Filter->OnChanged().RemoveAll(this);
	}

	ChildFilters.Empty();

	ChangedEvent.Broadcast();
}

int32 FSequencerTrackFilterCollection::Add(const TSharedRef<FSequencerTrackFilter>& InFilter)
{
	int32 ExistingIdx = INDEX_NONE;
	if (ChildFilters.Find(InFilter, ExistingIdx))
	{
		// The filter already exists, don't add a new one but return the index where it was found.
		return ExistingIdx;
	}

	InFilter->OnChanged().AddSP(this, &FSequencerTrackFilterCollection::OnChildFilterChanged);

	int32 Result = ChildFilters.Add(InFilter);

	ChangedEvent.Broadcast();

	return Result;
}

int32 FSequencerTrackFilterCollection::Remove(const TSharedRef<FSequencerTrackFilter>& InFilter)
{
	InFilter->OnChanged().RemoveAll(this);

	int32 Result = ChildFilters.Remove(InFilter);

	// Don't broadcast if the collection didn't change
	if (Result > 0)
	{
		ChangedEvent.Broadcast();
	}

	return Result;
}

TSharedRef<FSequencerTrackFilter> FSequencerTrackFilterCollection::GetFilterAtIndex(const int32 InIndex)
{
	check(ChildFilters.IsValidIndex(InIndex));
	return StaticCastSharedPtr<FSequencerTrackFilter>(ChildFilters[InIndex]).ToSharedRef();
}

int32 FSequencerTrackFilterCollection::Num() const
{
	return ChildFilters.Num();
}

bool FSequencerTrackFilterCollection::IsEmpty() const
{
	return ChildFilters.IsEmpty();
}

void FSequencerTrackFilterCollection::Sort()
{
	ChildFilters.Sort([](const TSharedPtr<IFilter<FSequencerTrackFilterType>>& LHS, const TSharedPtr<IFilter<FSequencerTrackFilterType>>& RHS)
		{
			const TSharedPtr<FSequencerTrackFilter> CastedLHS = StaticCastSharedPtr<FSequencerTrackFilter>(LHS);
			const TSharedPtr<FSequencerTrackFilter> CastedRHS = StaticCastSharedPtr<FSequencerTrackFilter>(RHS);
			if (CastedLHS.IsValid() && CastedRHS.IsValid())
			{
				return CastedLHS->GetDisplayName().ToString() < CastedRHS->GetDisplayName().ToString();
			}
			return true;
		});
}

void FSequencerTrackFilterCollection::OnChildFilterChanged()
{
	ChangedEvent.Broadcast();
}

TArray<FText> FSequencerTrackFilterCollection::GetFilterDisplayNames() const
{
	TArray<FText> OutDisplayNames;

	ForEachFilter([&OutDisplayNames]
		(const TSharedRef<FSequencerTrackFilter>& InTrackFilter) -> bool
		{
			OutDisplayNames.Add(InTrackFilter->GetDisplayName());
			return true;
		});

	return OutDisplayNames;
}

TArray<TSharedRef<FSequencerTrackFilter>> FSequencerTrackFilterCollection::GetAllFilters(const TArray<TSharedRef<FFilterCategory>>& InCategories) const
{
	TArray<TSharedRef<FSequencerTrackFilter>> OutFilters;

	ForEachFilter([&OutFilters]
		(const TSharedRef<FSequencerTrackFilter>& InTrackFilter) -> bool
		{
			OutFilters.Add(InTrackFilter);
			return true;
		}
		, InCategories);

	return OutFilters;
}

TSet<TSharedRef<FFilterCategory>> FSequencerTrackFilterCollection::GetCategories(const TSet<TSharedRef<FSequencerTrackFilter>>* InFilters) const
{
	TSet<TSharedRef<FFilterCategory>> OutCategories;

	ForEachFilter([InFilters, &OutCategories]
		(const TSharedRef<FSequencerTrackFilter>& InTrackFilter) -> bool
		{
			if (!InFilters || InFilters->Contains(InTrackFilter))
			{
				if (const TSharedPtr<FFilterCategory> Category = InTrackFilter->GetCategory())
				{
					if (const TSharedPtr<FFilterCategory> CategoryCasted = StaticCastSharedPtr<FFilterCategory>(Category))
					{
						OutCategories.Add(CategoryCasted.ToSharedRef());
					}
				}
			}
			return true;
		});

	return OutCategories;
}

TArray<TSharedRef<FSequencerTrackFilter>> FSequencerTrackFilterCollection::GetCategoryFilters(const TSharedRef<FFilterCategory>& InCategory) const
{
	TArray<TSharedRef<FSequencerTrackFilter>> OutFilters;

	ForEachFilter([&InCategory, &OutFilters]
		(const TSharedRef<FSequencerTrackFilter>& InTrackFilter) -> bool
		{
			if (InTrackFilter->GetCategory() == InCategory)
			{
				OutFilters.Add(InTrackFilter);
			}
			return true;
		});

	return OutFilters;
}

void FSequencerTrackFilterCollection::ForEachFilter(const TFunctionRef<bool(const TSharedRef<FSequencerTrackFilter>&)>& InFunction
	, const TArray<TSharedRef<FFilterCategory>>& InCategories) const
{
	using namespace UE::Sequencer;

	UMovieSceneSequence* const RootSequence = FilterInterface.GetSequencer().GetRootMovieSceneSequence();
	if (!RootSequence)
	{
		return;
	}

	for (const TSharedPtr<IFilter<TViewModelPtr<FViewModel>>>& Filter : ChildFilters)
	{
		const TSharedPtr<FSequencerTrackFilter> FilterCasted = StaticCastSharedPtr<FSequencerTrackFilter>(Filter);
		if (!FilterCasted.IsValid())
		{
			continue;
		}

		if (!InCategories.IsEmpty() && InCategories.Contains(FilterCasted->GetCategory()))
		{
			continue;
		}

		if (!InFunction(FilterCasted.ToSharedRef()))
		{
			return;
		}
	}
}
