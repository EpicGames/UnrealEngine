// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/SequencerTrackFilterCommands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SequencerTrackFilterCommands"

FSequencerTrackFilterCommands::FSequencerTrackFilterCommands()
	: TCommands<FSequencerTrackFilterCommands>("SequencerTrackFilters"
		, LOCTEXT("SequencerTrackFilters", "Sequencer Track Filters")
		, TEXT("Sequencer")
		, FAppStyle::GetAppStyleSetName())
{
}

void FSequencerTrackFilterCommands::RegisterCommands()
{
	UI_COMMAND(ToggleFilterBarVisibility, "Filter Bar Visibility", "Toggle the visibility of the filter bar. The filter bar will only be displayed when there are enabled filters", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::F));

	UI_COMMAND(ResetFilters, "Reset Filters", "Reset all enabled filters", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ToggleMuteFilters, "Mute Filters", "Mute or Unmute all active filters", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(DisableAllFilters, "Remove All Filters", "Disables all filters and removes them from the filter bar", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ToggleActivateEnabledFilters, "Toggle Active State", "Activates or deactivates all enabled filters", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ActivateAllFilters, "Activate All Filters", "Activates all enabled filters", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeactivateAllFilters, "Deactivate All Filters", "Deactivates all enabled filters", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(ToggleLinkedFiltering, "Toggle Linked Filtering", "Toggles whether filtering should use the filter state shared with other UI or its own instance.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(TogglePreserveFiltersOnUnlink, "Preserve Filters on Unlink", "Toggles whether when you unlink your filters, you want the filters in your linked filter bar to be copied into the unlinked filter bar.", EUserInterfaceActionType::ToggleButton, FInputChord())
	
	UI_COMMAND(HideSelectedTracks, "Hide Selected Tracks", "Hide all selected tracks", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::W));
	UI_COMMAND(IsolateSelectedTracks, "Isolate Selected Tracks", "Isolate all selected tracks", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::Q));

	UI_COMMAND(ClearHiddenTracks, "Show Hidden Tracks", "Show all hidden tracks", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Alt, EKeys::W));
	UI_COMMAND(ClearIsolatedTracks, "Unisolate Isolated Tracks", "Unisolate all isolated tracks", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Alt, EKeys::Q));

	UI_COMMAND(ShowAllTracks, "Show All Tracks", "Show all tracks", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::E));

	UI_COMMAND(ShowLocationCategoryGroups, "Show Location Groups", "Show all location category groups and hide everything else", EUserInterfaceActionType::Button, FInputChord(EKeys::P));
	UI_COMMAND(ShowRotationCategoryGroups, "Show Rotation Groups", "Show all rotation category groups and hide everything else", EUserInterfaceActionType::Button, FInputChord(EKeys::R));
	UI_COMMAND(ShowScaleCategoryGroups, "Show Scale Groups", "Show all scale category groups and hide everything else", EUserInterfaceActionType::Button, FInputChord(EKeys::C));

	UI_COMMAND(ToggleFilter_Audio, "Toggle Audio Filter", "Toggle the filter for Audio tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFilter_DataLayer, "Toggle Data Layer Filter", "Toggle the filter for Data Layer tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFilter_Event, "Toggle Event Filter", "Toggle the filter for Event tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFilter_Fade, "Toggle Fade Filter", "Toggle the filter for Fade tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFilter_Folder, "Toggle Folder Filter", "Toggle the filter for Folder tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFilter_LevelVisibility, "Toggle Level Visibility Filter", "Toggle the filter for Level Visibility tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFilter_Particle, "Toggle Particle Filter", "Toggle the filter for Particle tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFilter_CinematicShot, "Toggle Cinematic Shot Filter", "Toggle the filter for Cinematic Shot tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFilter_Subsequence, "Toggle Sub Tracks Filter", "Toggle the filter for Sub tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFilter_TimeDilation, "Toggle Time Dilation Filter", "Toggle the filter for Time Dilation tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFilter_TimeWarp, "Toggle Time Warp Filter", "Toggle the filter for Time Warp tracks", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(ToggleFilter_Camera, "Toggle Camera Filter", "Toggle the filter for Camera tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFilter_CameraCut, "Toggle Camera Cut Filter", "Toggle the filter for Camera Cut tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFilter_Light, "Toggle Light Filter", "Toggle the filter for Light tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFilter_SkeletalMesh, "Toggle Skeletal Mesh Filter", "Toggle the filter for Skeletal Mesh tracks", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ToggleFilter_Condition, "Toggle Condition Filter", "Toggle the filter for tracks with conditions", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFilter_Keyed, "Toggle Keyed Filter", "Toggle the filter for Keyed tracks", EUserInterfaceActionType::Button, FInputChord(EKeys::U));
	UI_COMMAND(ToggleFilter_Modified, "Toggle Modified Filter", "Toggle the filter for Modified tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFilter_Selected, "Toggle Selected In Viewport Filter", "Toggle the filter for Selected In Viewport tracks", EUserInterfaceActionType::Button, FInputChord(EKeys::F10));
	UI_COMMAND(ToggleFilter_Unbound, "Toggle Unbound Filter", "Toggle the filter for Unbound tracks", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ToggleFilter_Groups, "Toggle Groups Filter", "Toggle the filter for Group tracks", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleFilter_Levels, "Toggle Levels Filter", "Toggle the filter for Level tracks", EUserInterfaceActionType::Button, FInputChord());
}

TArray<TSharedPtr<FUICommandInfo>> FSequencerTrackFilterCommands::GetAllCommands() const
{
	TArray<TSharedPtr<FUICommandInfo>> OutCommands;

	OutCommands.Add(ToggleFilterBarVisibility);

	OutCommands.Add(ResetFilters);

	OutCommands.Add(ToggleMuteFilters);

	OutCommands.Add(DisableAllFilters);

	OutCommands.Add(ToggleActivateEnabledFilters);
	
	OutCommands.Add(HideSelectedTracks);
	OutCommands.Add(IsolateSelectedTracks);

	OutCommands.Add(ClearHiddenTracks);
	OutCommands.Add(ClearIsolatedTracks);

	OutCommands.Add(ShowAllTracks);

	OutCommands.Add(ShowLocationCategoryGroups);
	OutCommands.Add(ShowRotationCategoryGroups);
	OutCommands.Add(ShowScaleCategoryGroups);

	OutCommands.Add(ToggleFilter_Audio);
	OutCommands.Add(ToggleFilter_DataLayer);
	OutCommands.Add(ToggleFilter_Event);
	OutCommands.Add(ToggleFilter_Fade);
	OutCommands.Add(ToggleFilter_Folder);
	OutCommands.Add(ToggleFilter_LevelVisibility);
	OutCommands.Add(ToggleFilter_Particle);
	OutCommands.Add(ToggleFilter_CinematicShot);
	OutCommands.Add(ToggleFilter_Subsequence);
	OutCommands.Add(ToggleFilter_TimeDilation);
	OutCommands.Add(ToggleFilter_TimeWarp);
	
	OutCommands.Add(ToggleFilter_Camera);
	OutCommands.Add(ToggleFilter_CameraCut);
	OutCommands.Add(ToggleFilter_Light);
	OutCommands.Add(ToggleFilter_SkeletalMesh);

	OutCommands.Add(ToggleFilter_Condition);
	OutCommands.Add(ToggleFilter_Keyed);
	OutCommands.Add(ToggleFilter_Modified);
	OutCommands.Add(ToggleFilter_Selected);
	OutCommands.Add(ToggleFilter_Unbound);

	OutCommands.Add(ToggleFilter_Groups);
	OutCommands.Add(ToggleFilter_Levels);

	return OutCommands;
}

#undef LOCTEXT_NAMESPACE
