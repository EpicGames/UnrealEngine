// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FUICommandInfo;

class FSequencerTrackFilterCommands : public TCommands<FSequencerTrackFilterCommands>
{
public:
	FSequencerTrackFilterCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	TArray<TSharedPtr<FUICommandInfo>> GetAllCommands() const;

	/** FilterBar Commands */

	TSharedPtr<FUICommandInfo> ToggleFilterBarVisibility;

	TSharedPtr<FUICommandInfo> ResetFilters;

	TSharedPtr<FUICommandInfo> ToggleMuteFilters;

	TSharedPtr<FUICommandInfo> DisableAllFilters;

	TSharedPtr<FUICommandInfo> ToggleActivateEnabledFilters;

	TSharedPtr<FUICommandInfo> ActivateAllFilters;
	TSharedPtr<FUICommandInfo> DeactivateAllFilters;
	
	TSharedPtr<FUICommandInfo> ToggleLinkedFiltering;
	TSharedPtr<FUICommandInfo> TogglePreserveFiltersOnUnlink;

	/** Hide/Isolate Filter Commands */

	TSharedPtr<FUICommandInfo> HideSelectedTracks;
	TSharedPtr<FUICommandInfo> IsolateSelectedTracks;

	TSharedPtr<FUICommandInfo> ClearHiddenTracks;
	TSharedPtr<FUICommandInfo> ClearIsolatedTracks;

	TSharedPtr<FUICommandInfo> ShowAllTracks;

	TSharedPtr<FUICommandInfo> ShowLocationCategoryGroups;
	TSharedPtr<FUICommandInfo> ShowRotationCategoryGroups;
	TSharedPtr<FUICommandInfo> ShowScaleCategoryGroups;

	/** Filter toggle commands */

	TSharedPtr<FUICommandInfo> ToggleFilter_Audio;
	TSharedPtr<FUICommandInfo> ToggleFilter_DataLayer;
	TSharedPtr<FUICommandInfo> ToggleFilter_Event;
	TSharedPtr<FUICommandInfo> ToggleFilter_Fade;
	TSharedPtr<FUICommandInfo> ToggleFilter_Folder;
	TSharedPtr<FUICommandInfo> ToggleFilter_LevelVisibility;
	TSharedPtr<FUICommandInfo> ToggleFilter_Particle;
	TSharedPtr<FUICommandInfo> ToggleFilter_CinematicShot;
	TSharedPtr<FUICommandInfo> ToggleFilter_Subsequence;
	TSharedPtr<FUICommandInfo> ToggleFilter_TimeDilation;
	TSharedPtr<FUICommandInfo> ToggleFilter_TimeWarp;

	TSharedPtr<FUICommandInfo> ToggleFilter_Camera;
	TSharedPtr<FUICommandInfo> ToggleFilter_CameraCut;
	TSharedPtr<FUICommandInfo> ToggleFilter_Light;
	TSharedPtr<FUICommandInfo> ToggleFilter_SkeletalMesh;

	TSharedPtr<FUICommandInfo> ToggleFilter_Condition;
	TSharedPtr<FUICommandInfo> ToggleFilter_Keyed;
	TSharedPtr<FUICommandInfo> ToggleFilter_Modified;
	TSharedPtr<FUICommandInfo> ToggleFilter_Selected;
	TSharedPtr<FUICommandInfo> ToggleFilter_Unbound;

	TSharedPtr<FUICommandInfo> ToggleFilter_Groups;
	TSharedPtr<FUICommandInfo> ToggleFilter_Levels;
};
