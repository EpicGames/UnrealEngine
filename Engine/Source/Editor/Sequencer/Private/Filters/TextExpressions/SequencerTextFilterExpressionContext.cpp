// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/SequencerTextFilterExpressionContext.h"
#include "Filters/SequencerTrackFilterBase.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModelPtr.h"
#include "Sequencer.h"
#include "MVVM/ViewModels/CategoryModel.h"

using namespace UE::Sequencer;

FSequencerTextFilterExpressionContext::FSequencerTextFilterExpressionContext(ISequencerTrackFilters& InFilterInterface)
	: FilterInterface(InFilterInterface)
{
}

void FSequencerTextFilterExpressionContext::SetFilterItem(FSequencerTrackFilterType InFilterItem)
{
	FilterItem = InFilterItem;
}

bool FSequencerTextFilterExpressionContext::TestBasicStringExpression(const FTextFilterString& InValue
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	constexpr ETextFilterTextComparisonMode TextComparisonMode = ETextFilterTextComparisonMode::Partial;

	for (const FViewModelPtr& OutlinerItem : FilterItem->GetAncestors(/*bIncludeThis=*/true))
	{
		if (const TViewModelPtr<FChannelGroupOutlinerModel> ChannelGroupOutlinerModel = OutlinerItem.ImplicitCast())
		{
			const FTextFilterString TrackLabel = ChannelGroupOutlinerModel->GetLabel().ToString();
			if (TextFilterUtils::TestBasicStringExpression(TrackLabel, InValue, TextComparisonMode))
			{
				return true;
			}

			const FTextFilterString ChannelName = ChannelGroupOutlinerModel->GetChannelName();
			if (TextFilterUtils::TestBasicStringExpression(ChannelName, InValue, TextComparisonMode))
			{
				return true;
			}
		}
		else if (const TViewModelPtr<FCategoryGroupModel> CategoryGroupModel = OutlinerItem.ImplicitCast())
		{
			const FTextFilterString CategoryName = CategoryGroupModel->GetCategoryName();
			if (TextFilterUtils::TestBasicStringExpression(CategoryName, InValue, TextComparisonMode))
			{
				return true;
			}
		}
		else if (const TViewModelPtr<FChannelGroupModel> ChannelGroupModel = OutlinerItem.ImplicitCast())
		{
			const FTextFilterString CategoryName = ChannelGroupModel->GetChannelName();
			if (TextFilterUtils::TestBasicStringExpression(CategoryName, InValue, TextComparisonMode))
			{
				return true;
			}
		}
		else if (const TViewModelPtr<IOutlinerExtension> OutlinerExtension = OutlinerItem.ImplicitCast())
		{
			const FTextFilterString TrackLabel = OutlinerExtension->GetLabel().ToString();
			if (TextFilterUtils::TestBasicStringExpression(TrackLabel, InValue, TextComparisonMode))
			{
				return true;
			}
		}
	}

	return false;
}

bool FSequencerTextFilterExpressionContext::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FilterItem.IsValid())
	{
		return false;
	}

	const TSet<FName> Keys = GetKeys();
	if (!Keys.IsEmpty() && !Keys.Contains(InKey))
	{
		return false;
	}

	return !InValue.IsEmpty();
}

UMovieSceneSequence* FSequencerTextFilterExpressionContext::GetFocusedMovieSceneSequence() const
{
	return FilterInterface.GetSequencer().GetFocusedMovieSceneSequence();
}

UMovieScene* FSequencerTextFilterExpressionContext::GetFocusedGetMovieScene() const
{
	const UMovieSceneSequence* const FocusedMovieSceneSequence = GetFocusedMovieSceneSequence();
	return FocusedMovieSceneSequence ? FocusedMovieSceneSequence->GetMovieScene() : nullptr;
}

UMovieSceneTrack* FSequencerTextFilterExpressionContext::GetMovieSceneTrack() const
{
	return FilterInterface.GetFilterData().ResolveMovieSceneTrackObject(FilterItem);
}

UObject* FSequencerTextFilterExpressionContext::GetBoundObject() const
{
	return FilterInterface.GetFilterData().ResolveTrackBoundObject(FilterInterface.GetSequencer(), FilterItem);
}
