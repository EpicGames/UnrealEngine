// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_BindingName.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_BindingName"

FSequencerTextFilterExpression_BindingName::FSequencerTextFilterExpression_BindingName(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_BindingName::GetKeys() const
{
	return { TEXT("BindingName") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_BindingName::GetValueType() const
{
	return ESequencerTextFilterValueType::String;
}

FText FSequencerTextFilterExpression_BindingName::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_BindingName", "Filter by track object binding name");
}

bool FSequencerTextFilterExpression_BindingName::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	for (const TViewModelPtr<FObjectBindingModel>& ObjectBindingModel : FilterItem->GetAncestorsOfType<FObjectBindingModel>())
	{
		const FString Label = ObjectBindingModel->GetLabel().ToString();
		if (TextFilterUtils::TestComplexExpression(Label, InValue, InComparisonOperation, InTextComparisonMode))
		{
			return true;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
