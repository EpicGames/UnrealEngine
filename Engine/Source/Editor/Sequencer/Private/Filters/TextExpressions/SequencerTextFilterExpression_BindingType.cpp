// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_BindingType.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/SharedViewModelData.h"
#include "MovieSceneBindingReferences.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "Bindings/MovieSceneReplaceableBinding.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_BindingType"

FSequencerTextFilterExpression_BindingType::FSequencerTextFilterExpression_BindingType(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_BindingType::GetKeys() const
{
	return { TEXT("BindingClass"), TEXT("BindingType") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_BindingType::GetValueType() const
{
	return ESequencerTextFilterValueType::String;
}

FText FSequencerTextFilterExpression_BindingType::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_BindingType", "Filter by presence of a binding with the given type");
}

bool FSequencerTextFilterExpression_BindingType::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const TViewModelPtr<IObjectBindingExtension> ObjectBindingExtension = FilterItem->FindAncestorOfType<IObjectBindingExtension>(true);
	if (!ObjectBindingExtension.IsValid())
	{
		return false;
	}

	FGuid ObjectBindingID = ObjectBindingExtension->GetObjectGuid();

	const UMovieSceneSequence* FocusedSequence = GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return true;
	}

	const FMovieSceneBindingReferences* BindingReferences = FocusedSequence->GetBindingReferences();
	const FMovieSceneBindingReference* BindingReference = BindingReferences ? BindingReferences->GetReference(ObjectBindingID, 0) : nullptr;

	// Special cases- possessable, spawnable, replaceable
	if (TextFilterUtils::TestComplexExpression(TEXT("Possessable"), InValue, InComparisonOperation, InTextComparisonMode))
	{
		if (!BindingReferences)
		{
			return FocusedSequence->GetMovieScene()->FindPossessable(ObjectBindingID) != nullptr;
		}
		else if (!BindingReference)
		{
			return false;
		}
		else 
		{
			return !BindingReference->Locator.IsEmpty() && BindingReference->CustomBinding == nullptr;
		}
	}
	else if (TextFilterUtils::TestComplexExpression(TEXT("Spawnable"), InValue, InComparisonOperation, InTextComparisonMode))
	{
		if (!BindingReferences)
		{
			return FocusedSequence->GetMovieScene()->FindSpawnable(ObjectBindingID) != nullptr;
		}
		else if (!BindingReference)
		{
			return false;
		}
		else
		{
			return Cast<UMovieSceneSpawnableBindingBase>(BindingReference->CustomBinding) != nullptr;
		}
	}
	else if (TextFilterUtils::TestComplexExpression(TEXT("Replaceable"), InValue, InComparisonOperation, InTextComparisonMode))
	{
		if (!BindingReferences)
		{
			return FocusedSequence->GetMovieScene()->FindSpawnable(ObjectBindingID) != nullptr;
		}
		else if (!BindingReference)
		{
			return false;
		}
		else
		{
			return Cast<UMovieSceneReplaceableBindingBase>(BindingReference->CustomBinding) != nullptr;
		}
	}
	else if (!BindingReference || !BindingReference->CustomBinding)
	{
		return false;
	}
	else
	{
		return TextFilterUtils::TestComplexExpression(BindingReference->CustomBinding->GetClass()->GetName(), InValue, InComparisonOperation, InTextComparisonMode);
	}
}

#undef LOCTEXT_NAMESPACE
