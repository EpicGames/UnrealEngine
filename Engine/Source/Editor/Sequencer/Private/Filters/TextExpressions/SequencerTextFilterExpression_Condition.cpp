// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_Condition.h"
#include "MVVM/Extensions/IConditionableExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/SharedViewModelData.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_Condition"

FSequencerTextFilterExpression_Condition::FSequencerTextFilterExpression_Condition(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_Condition::GetKeys() const
{
	return { TEXT("Condition"), TEXT("HasCondition") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_Condition::GetValueType() const
{
	return ESequencerTextFilterValueType::Boolean;
}

FText FSequencerTextFilterExpression_Condition::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_Condition", "Filter by presence of a condition");
}

bool FSequencerTextFilterExpression_Condition::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const TViewModelPtr<IConditionableExtension> ConditionableExtension = FilterItem->FindAncestorOfType<IConditionableExtension>(true);
	if (!ConditionableExtension.IsValid())
	{
		return false;
	}

	bool bPassed = ConditionableExtension->GetConditionState() != EConditionableConditionState::None;

	if (TViewModelPtr<FConditionStateCacheExtension> StateCache = CastViewModel<FConditionStateCacheExtension>(FilterItem.AsModel()->GetSharedData()))
	{
		bPassed |= EnumHasAnyFlags(StateCache->GetCachedFlags(FilterItem->GetModelID()), ECachedConditionState::ParentHasCondition);
	}

	return CompareFStringForExactBool(InValue, InComparisonOperation, bPassed);
}

#undef LOCTEXT_NAMESPACE
