// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_ConditionClass.h"
#include "MVVM/Extensions/IConditionableExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/SharedViewModelData.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_ConditionClass"

FSequencerTextFilterExpression_ConditionClass::FSequencerTextFilterExpression_ConditionClass(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_ConditionClass::GetKeys() const
{
	return { TEXT("ConditionClass"), TEXT("ConditionType") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_ConditionClass::GetValueType() const
{
	return ESequencerTextFilterValueType::String;
}

FText FSequencerTextFilterExpression_ConditionClass::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_ConditionClass", "Filter by presence of a condition with the given type");
}

bool FSequencerTextFilterExpression_ConditionClass::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const TViewModelPtr<IConditionableExtension> ConditionableExtension = FilterItem->FindAncestorOfType<IConditionableExtension>(true);
	if (!ConditionableExtension.IsValid())
	{
		return false;
	}

	if (ConditionableExtension->GetConditionState() == EConditionableConditionState::None)
	{
		return false;
	}

	if (const UMovieSceneCondition* Condition = ConditionableExtension->GetCondition())
	{
		return TextFilterUtils::TestComplexExpression(Condition->GetClass()->GetName(), InValue, InComparisonOperation, InTextComparisonMode);
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
