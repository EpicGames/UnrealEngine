// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_ConditionFunc.h"
#include "MVVM/Extensions/IConditionableExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/SharedViewModelData.h"
#include "Conditions/MovieSceneDirectorBlueprintCondition.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_ConditionFunc"

FSequencerTextFilterExpression_ConditionFunc::FSequencerTextFilterExpression_ConditionFunc(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_ConditionFunc::GetKeys() const
{
	return { TEXT("ConditionFunc"), TEXT("ConditionEndpoint") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_ConditionFunc::GetValueType() const
{
	return ESequencerTextFilterValueType::String;
}

FText FSequencerTextFilterExpression_ConditionFunc::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_ConditionFunc", "Filter by presence of director blueprint condition with the given function/endpoint name");
}

bool FSequencerTextFilterExpression_ConditionFunc::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const TViewModelPtr<IConditionableExtension> ConditionableExtension = FilterItem->FindAncestorOfType<IConditionableExtension>(true);
	if (!ConditionableExtension.IsValid())
	{
		return false;
	}

	if (const UMovieSceneDirectorBlueprintCondition* Condition = Cast<UMovieSceneDirectorBlueprintCondition>(ConditionableExtension->GetCondition()))
	{
		if (Condition->DirectorBlueprintConditionData.Function && Condition->DirectorBlueprintConditionData.Function->Next)
		{
			return TextFilterUtils::TestComplexExpression(Condition->DirectorBlueprintConditionData.Function->Next->GetName(), InValue, InComparisonOperation, InTextComparisonMode);
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
