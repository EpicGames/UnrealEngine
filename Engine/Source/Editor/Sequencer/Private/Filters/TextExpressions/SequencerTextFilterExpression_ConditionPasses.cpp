// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_ConditionPasses.h"
#include "MVVM/Extensions/IConditionableExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/SharedViewModelData.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_Condition"

FSequencerTextFilterExpression_ConditionPasses::FSequencerTextFilterExpression_ConditionPasses(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_ConditionPasses::GetKeys() const
{
	return { TEXT("ConditionPasses"), TEXT("ConditionEvaluates") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_ConditionPasses::GetValueType() const
{
	return ESequencerTextFilterValueType::Boolean;
}

FText FSequencerTextFilterExpression_ConditionPasses::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_ConditionPasses", "Filters based on the presence of a condition that is passing/failing");
}

bool FSequencerTextFilterExpression_ConditionPasses::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const TViewModelPtr<IConditionableExtension> ConditionableExtension = FilterItem->FindAncestorOfType<IConditionableExtension>(true);
	if (!ConditionableExtension.IsValid())
	{
		return false;
	}

	if (ConditionableExtension->GetConditionState() == EConditionableConditionState::None)
	{
		return false;
	}

	if (TViewModelPtr<FConditionStateCacheExtension> StateCache = CastViewModel<FConditionStateCacheExtension>(FilterItem.AsModel()->GetSharedData()))
	{
		bool bPassed = EnumHasAnyFlags(StateCache->GetCachedFlags(FilterItem->GetModelID()), ECachedConditionState::ConditionEvaluatingTrue)
						|| EnumHasAnyFlags(StateCache->GetCachedFlags(FilterItem->GetModelID()), ECachedConditionState::ParentHasConditionEvaluatingTrue);
		
		return CompareFStringForExactBool(InValue, InComparisonOperation, bPassed);
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
