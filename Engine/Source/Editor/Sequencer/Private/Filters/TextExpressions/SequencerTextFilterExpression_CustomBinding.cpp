// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_CustomBinding.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/SharedViewModelData.h"
#include "MovieSceneBindingReferences.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_CustomBinding"

FSequencerTextFilterExpression_CustomBinding::FSequencerTextFilterExpression_CustomBinding(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_CustomBinding::GetKeys() const
{
	return { TEXT("Custom"), TEXT("CustomBinding"), TEXT("IsCustom"), TEXT("IsCustomBinding") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_CustomBinding::GetValueType() const
{
	return ESequencerTextFilterValueType::Boolean;
}

FText FSequencerTextFilterExpression_CustomBinding::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_CustomBinding", "Filter by presence of a custom binding");
}

bool FSequencerTextFilterExpression_CustomBinding::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const TViewModelPtr<IObjectBindingExtension> ObjectBindingExtension = FilterItem->FindAncestorOfType<IObjectBindingExtension>(true);
	if (!ObjectBindingExtension.IsValid())
	{
		return false;
	}

	FGuid ObjectBindingID = ObjectBindingExtension->GetObjectGuid();

	const UMovieSceneSequence* FocusedSequence = GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return true;
	}

	const FMovieSceneBindingReferences* BindingReferences = FocusedSequence->GetBindingReferences();
	if (!BindingReferences)
	{
		return false;
	}

	bool bHasCustomBinding = BindingReferences->GetCustomBinding(ObjectBindingID, 0) != nullptr;

	return CompareFStringForExactBool(InValue, InComparisonOperation, bHasCustomBinding);
}

#undef LOCTEXT_NAMESPACE
