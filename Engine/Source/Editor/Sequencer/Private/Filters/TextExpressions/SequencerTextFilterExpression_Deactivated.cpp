// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_Deactivated.h"

#include "MVVM/Extensions/IDeactivatableExtension.h"
#include "MVVM/Extensions/IMutableExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_Deactivated"

FSequencerTextFilterExpression_Deactivated::FSequencerTextFilterExpression_Deactivated(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_Deactivated::GetKeys() const
{
	return { TEXT("Deactive"), TEXT("Deactived"), TEXT("Deactivate"), TEXT("Deactivated") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_Deactivated::GetValueType() const
{
	return ESequencerTextFilterValueType::Boolean;
}

FText FSequencerTextFilterExpression_Deactivated::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_Deactivated", "Filter by track deactivated state");
}

bool FSequencerTextFilterExpression_Deactivated::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const TViewModelPtr<IDeactivatableExtension> DeactivatableExtension
		= FilterItem->FindAncestorOfType<IDeactivatableExtension>(/*bIncludeThis=*/true);
	if (!DeactivatableExtension.IsValid())
	{
		return false;
	}

	const bool bPassed = DeactivatableExtension->IsDeactivated();
	return CompareFStringForExactBool(InValue, InComparisonOperation, bPassed);
}

#undef LOCTEXT_NAMESPACE
