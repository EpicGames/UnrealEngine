// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_EmptyBinding.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/SharedViewModelData.h"
#include "MovieSceneBindingReferences.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_EmptyBinding"

FSequencerTextFilterExpression_EmptyBinding::FSequencerTextFilterExpression_EmptyBinding(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_EmptyBinding::GetKeys() const
{
	return { TEXT("Empty"), TEXT("EmptyBinding"), TEXT("IsEmpty"), TEXT("IsEmptyBinding") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_EmptyBinding::GetValueType() const
{
	return ESequencerTextFilterValueType::Boolean;
}

FText FSequencerTextFilterExpression_EmptyBinding::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_EmptyBinding", "Filter by presence of an empty binding");
}

bool FSequencerTextFilterExpression_EmptyBinding::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const TViewModelPtr<IObjectBindingExtension> ObjectBindingExtension = FilterItem->FindAncestorOfType<IObjectBindingExtension>(true);
	if (!ObjectBindingExtension.IsValid())
	{
		return false;
	}

	FGuid ObjectBindingID = ObjectBindingExtension->GetObjectGuid();

	const UMovieSceneSequence* FocusedSequence = GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return true;
	}

	const FMovieSceneBindingReferences* BindingReferences = FocusedSequence->GetBindingReferences();
	if (!BindingReferences)
	{
		return false;
	}

	const FMovieSceneBindingReference* Reference = BindingReferences->GetReference(ObjectBindingID, 0);

	if (!Reference)
	{
		return false;
	}

	bool bIsEmptyBinding = Reference->Locator.IsEmpty() && Reference->CustomBinding == nullptr;

	return CompareFStringForExactBool(InValue, InComparisonOperation, bIsEmptyBinding);
}

#undef LOCTEXT_NAMESPACE
