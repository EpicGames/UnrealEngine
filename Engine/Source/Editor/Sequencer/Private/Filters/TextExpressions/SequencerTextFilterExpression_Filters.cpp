// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_Filters.h"
#include "Filters/ISequencerFilterBar.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "Sequencer.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_Filters"

FSequencerTextFilterExpression_Filters::FSequencerTextFilterExpression_Filters(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_Filters::GetKeys() const
{
	// No keys means this will not act like a traditional text expression, but as a general text expression
	// that works for all common filters.
	// This also means this will not show up in the text expression help dialog.
	return {};
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_Filters::GetValueType() const
{
	return ESequencerTextFilterValueType::Boolean;
}

FText FSequencerTextFilterExpression_Filters::GetDescription() const
{
	return FText();
}

bool FSequencerTextFilterExpression_Filters::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	// This expression tries to expose the names of common filters as expression.
	// If the expression is an actual expression, like "Lock", "Locked", etc., then ignore it. We'll only handle actual common filter names.
	const FString FilterName = InKey.ToString();
	const bool bIsCommonFilter = FilterInterface.IsCommonFilterName(FilterName);
	if (!bIsCommonFilter)
	{
		return true;
	}

	const bool bPassesFilter = FilterInterface.DoesCommonFilterPass(FilterItem.ImplicitCast(), InKey.ToString());
	return CompareFStringForExactBool(InValue, InComparisonOperation, bPassesFilter);
}

#undef LOCTEXT_NAMESPACE
