// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_Group.h"
#include "Filters/Filters/SequencerTrackFilter_Group.h"
#include "ISequencer.h"
#include "MovieScene.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_Group"

FSequencerTextFilterExpression_Group::FSequencerTextFilterExpression_Group(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_Group::GetKeys() const
{
	return { TEXT("Group") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_Group::GetValueType() const
{
	return ESequencerTextFilterValueType::String;
}

FText FSequencerTextFilterExpression_Group::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_Group", "Filter by group name");
}

bool FSequencerTextFilterExpression_Group::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const ISequencer& Sequencer = FilterInterface.GetSequencer();

	const UMovieSceneSequence* const FocusedSequence = Sequencer.GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return true;
	}

	UMovieScene* const FocusedMovieScene = FocusedSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return true;
	}

	bool bPassed = false;

	FSequencerTrackFilter_Group::ForEachMovieSceneNodeGroup(FocusedMovieScene, FilterItem,
		[&InValue, InComparisonOperation, InTextComparisonMode, &bPassed](const TViewModelPtr<IOutlinerExtension>& InParent, UMovieSceneNodeGroup* const InNodeGroup)
		{
			const FString GroupPathName = IOutlinerExtension::GetPathName(InParent.AsModel());
			if (InNodeGroup->ContainsNode(GroupPathName)
				&& TextFilterUtils::TestComplexExpression(InNodeGroup->GetName(), InValue, InComparisonOperation, InTextComparisonMode))
			{
				bPassed = true;
				return false;
			}
			return true;
		});

	return bPassed;
}

#undef LOCTEXT_NAMESPACE
