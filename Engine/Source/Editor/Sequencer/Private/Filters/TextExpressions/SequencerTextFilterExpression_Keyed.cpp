// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_Keyed.h"
#include "Filters/Filters/SequencerTrackFilter_Keyed.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_Keyed"

FSequencerTextFilterExpression_Keyed::FSequencerTextFilterExpression_Keyed(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_Keyed::GetKeys() const
{
	return { TEXT("Keyed"), TEXT("Keys"), TEXT("Animated") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_Keyed::GetValueType() const
{
	return ESequencerTextFilterValueType::Boolean;
}

FText FSequencerTextFilterExpression_Keyed::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_Keys", "Filter by presence of keys");
}

bool FSequencerTextFilterExpression_Keyed::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const TSharedPtr<FSequencerTrackFilter_Keyed> Filter = MakeShared<FSequencerTrackFilter_Keyed>(FilterInterface);
	const bool bPassesFilter = Filter->PassesFilter(FilterItem);
	return CompareFStringForExactBool(InValue, InComparisonOperation, bPassesFilter);
}

#undef LOCTEXT_NAMESPACE
