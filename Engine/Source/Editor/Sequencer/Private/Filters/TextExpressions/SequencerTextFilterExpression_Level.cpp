// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_Level.h"
#include "Engine/Level.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "Sequencer.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_Level"

FSequencerTextFilterExpression_Level::FSequencerTextFilterExpression_Level(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_Level::GetKeys() const
{
	return { TEXT("Level"), TEXT("Map") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_Level::GetValueType() const
{
	return ESequencerTextFilterValueType::String;
}

FText FSequencerTextFilterExpression_Level::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_Level", "Filter by level name");
}

bool FSequencerTextFilterExpression_Level::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	// Track nodes do not belong to a level, but might be a child of an objectbinding node that does
	constexpr bool bIncludeThis = true;
	const TViewModelPtr<IObjectBindingExtension> BindingExtension = FilterItem->FindAncestorOfType<IObjectBindingExtension>(bIncludeThis);
	if (!BindingExtension.IsValid())
	{
		return false;
	}

	ISequencer& Sequencer = FilterInterface.GetSequencer();

	for (const TWeakObjectPtr<>& Object : Sequencer.FindObjectsInCurrentSequence(BindingExtension->GetObjectGuid()))
	{
		if (!Object.IsValid())
		{
			continue;
		}

		const UPackage* const Package = Object->GetPackage();
		if (!Package)
		{
			continue;
		}

		// For anything in a level, package should refer to the ULevel that contains it
		const FString LevelName = FPackageName::GetShortName(Package->GetName());
		
		if (!TextFilterUtils::TestComplexExpression(LevelName, InValue, InComparisonOperation, InTextComparisonMode))
		{
			return false;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
