// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_Locked.h"
#include "MVVM/Extensions/ILockableExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_Locked"

FSequencerTextFilterExpression_Locked::FSequencerTextFilterExpression_Locked(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_Locked::GetKeys() const
{
	return { TEXT("Lock"), TEXT("Locked") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_Locked::GetValueType() const
{
	return ESequencerTextFilterValueType::Boolean;
}

FText FSequencerTextFilterExpression_Locked::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_Locked", "Filter by track locked state");
}

bool FSequencerTextFilterExpression_Locked::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const TViewModelPtr<ILockableExtension> LockableExtension = FilterItem->FindAncestorOfType<ILockableExtension>(true);
	if (!LockableExtension.IsValid())
	{
		return false;
	}

	const bool bPassed = LockableExtension->GetLockState() == ELockableLockState::Locked;
	return CompareFStringForExactBool(InValue, InComparisonOperation, bPassed);
}

#undef LOCTEXT_NAMESPACE
