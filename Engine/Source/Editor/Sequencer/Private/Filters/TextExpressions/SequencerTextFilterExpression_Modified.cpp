// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_Modified.h"
#include "Filters/Filters/SequencerTrackFilter_Modified.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_Modified"

FSequencerTextFilterExpression_Modified::FSequencerTextFilterExpression_Modified(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_Modified::GetKeys() const
{
	return { TEXT("Modified"), TEXT("Changed"), TEXT("Dirty") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_Modified::GetValueType() const
{
	return ESequencerTextFilterValueType::Boolean;
}

FText FSequencerTextFilterExpression_Modified::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_Modified", "Filter by modified state");
}

bool FSequencerTextFilterExpression_Modified::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const TSharedPtr<FSequencerTrackFilter_Modified> Filter = MakeShared<FSequencerTrackFilter_Modified>(FilterInterface);
	const bool bFilterPassed = Filter->PassesFilter(FilterItem);
	return CompareFStringForExactBool(InValue, InComparisonOperation, bFilterPassed);
}

#undef LOCTEXT_NAMESPACE
