// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_Muted.h"
#include "MVVM/Extensions/IMutableExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_Muted"

FSequencerTextFilterExpression_Muted::FSequencerTextFilterExpression_Muted(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_Muted::GetKeys() const
{
	return { TEXT("Mute"), TEXT("Muted") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_Muted::GetValueType() const
{
	return ESequencerTextFilterValueType::Boolean;
}

FText FSequencerTextFilterExpression_Muted::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_Muted", "Filter by track muted state");
}

bool FSequencerTextFilterExpression_Muted::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const TViewModelPtr<IMutableExtension> MutableExtension = FilterItem->FindAncestorOfType<IMutableExtension>(true);
	if (!MutableExtension.IsValid())
	{
		return false;
	}

	const bool bPassed = MutableExtension->IsMuted();
	return CompareFStringForExactBool(InValue, InComparisonOperation, bPassed);
}

#undef LOCTEXT_NAMESPACE
