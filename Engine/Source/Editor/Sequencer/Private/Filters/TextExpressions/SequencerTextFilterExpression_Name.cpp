// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_Name.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_Name"

FSequencerTextFilterExpression_Name::FSequencerTextFilterExpression_Name(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_Name::GetKeys() const
{
	return { TEXT("Name") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_Name::GetValueType() const
{
	return ESequencerTextFilterValueType::String;
}

FText FSequencerTextFilterExpression_Name::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_Name", "Filter by track name");
}

bool FSequencerTextFilterExpression_Name::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	/**
	 * Evaluate a keyed text-filter expression against the "Name" of an item, with outliner/subtree semantics.
	 *
	 * Key points:
	 * - This filter is evaluated against the labels of the current row AND its ancestors (bIncludeThis=true).
	 *   That means a match on a parent label causes the child row to be considered a match as well.
	 *
	 * - Most comparison operators (==, contains, startswith, etc.) use "any-match" aggregation:
	 *     Pass if ANY ancestor/self label matches the operator against InValue.
	 *
	 * - "!=" is intentionally special-cased to support subtree exclusion:
	 *     The desired behavior is: "exclude this row and its subtree if ANY ancestor/self label equals the value".
	 *     Said differently: for "name != X" we want to keep rows only when NO ancestor/self label equals X.
	 *
	 * Implemented as:
	 *   1) For "!=" we run per-label comparisons using "==" (Equal) to detect "any equals".
	 *   2) After the scan, we invert the aggregate result for "!=".
	 *
	 * Notes on the early return:
	 * - The base context's TestComplexExpression(...) determines whether this expression applies to this filter key.
	 *   If it does NOT apply, we return true so that unrelated keys don't accidentally filter out items.
	 */

	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const bool bIsNotEqual = (InComparisonOperation == ETextFilterComparisonOperation::NotEqual);

	// Subtree exclusion for "!=" is based on "any ancestor/self equals".
	// For all other operators, apply the operator normally.
	const ETextFilterComparisonOperation ActualOperation = bIsNotEqual
		? ETextFilterComparisonOperation::Equal : InComparisonOperation;

	bool bAnyMatched = false;

	for (const TViewModelPtr<IOutlinerExtension>& OutlinerExtension
		: FilterItem->GetAncestorsOfType<IOutlinerExtension>(/*bIncludeThis=*/true))
	{
		const FString Label = OutlinerExtension->GetLabel().ToString();
		if (TextFilterUtils::TestComplexExpression(Label, InValue, ActualOperation, InTextComparisonMode))
		{
			bAnyMatched = true;
			break;
		}
	}

	return bIsNotEqual ? !bAnyMatched : bAnyMatched;
}

#undef LOCTEXT_NAMESPACE
