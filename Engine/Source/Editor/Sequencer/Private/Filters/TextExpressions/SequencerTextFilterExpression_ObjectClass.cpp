// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_ObjectClass.h"
#include "Sequencer.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_ObjectClass"

FSequencerTextFilterExpression_ObjectClass::FSequencerTextFilterExpression_ObjectClass(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_ObjectClass::GetKeys() const
{
	return { TEXT("ObjectClass"), TEXT("ObjectType") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_ObjectClass::GetValueType() const
{
	return ESequencerTextFilterValueType::String;
}

FText FSequencerTextFilterExpression_ObjectClass::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_ObjectClass", "Filter by bound object class name");
}

bool FSequencerTextFilterExpression_ObjectClass::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	ISequencer& Sequencer = FilterInterface.GetSequencer();

	if (UObject* const BoundObject = FilterInterface.GetFilterData().ResolveTrackBoundObject(Sequencer, FilterItem))
	{
		const FString BoundObjectClassName = BoundObject->GetClass()->GetName();
		if (TextFilterUtils::TestComplexExpression(BoundObjectClassName, InValue, InComparisonOperation, InTextComparisonMode))
		{
			return true;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
