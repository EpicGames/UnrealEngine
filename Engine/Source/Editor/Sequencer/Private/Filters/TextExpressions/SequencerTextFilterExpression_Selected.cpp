// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/TextExpressions/SequencerTextFilterExpression_Selected.h"
#include "Filters/Filters/SequencerTrackFilter_Selected.h"
#include "Selection.h"
#include "Sequencer.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_Selected"

FSequencerTextFilterExpression_Selected::FSequencerTextFilterExpression_Selected(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
	BindSelectionChanged();
}

FSequencerTextFilterExpression_Selected::~FSequencerTextFilterExpression_Selected()
{
	UnbindSelectionChanged();
}

void FSequencerTextFilterExpression_Selected::BindSelectionChanged()
{
	if (!OnSelectionChangedHandle.IsValid())
	{
		OnSelectionChangedHandle = USelection::SelectionChangedEvent.AddRaw(this, &FSequencerTextFilterExpression_Selected::OnSelectionChanged);
	}
}

void FSequencerTextFilterExpression_Selected::UnbindSelectionChanged()
{
	if (OnSelectionChangedHandle.IsValid())
	{
		USelection::SelectionChangedEvent.Remove(OnSelectionChangedHandle);
		OnSelectionChangedHandle.Reset();
	}
}

TSet<FName> FSequencerTextFilterExpression_Selected::GetKeys() const
{
	return { TEXT("Selected"), TEXT("Viewport") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_Selected::GetValueType() const
{
	return ESequencerTextFilterValueType::Boolean;
}

FText FSequencerTextFilterExpression_Selected::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_Selected", "Filter by viewport selection state");
}

bool FSequencerTextFilterExpression_Selected::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const TSharedPtr<FSequencerTrackFilter_Selected> Filter = MakeShared<FSequencerTrackFilter_Selected>(FilterInterface);
	const bool bFilterPassed = Filter->PassesFilter(FilterItem);
	return CompareFStringForExactBool(InValue, InComparisonOperation, bFilterPassed);
}

void FSequencerTextFilterExpression_Selected::OnSelectionChanged(UObject* const InObject)
{
	if (FilterInterface.DoesTextFilterStringContainExpressionPair(*this))
	{
		FilterInterface.RequestFilterUpdate();
	}
}

#undef LOCTEXT_NAMESPACE
