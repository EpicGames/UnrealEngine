// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_Soloed.h"
#include "MVVM/Extensions/ISoloableExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_Soloed"

FSequencerTextFilterExpression_Soloed::FSequencerTextFilterExpression_Soloed(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_Soloed::GetKeys() const
{
	return { TEXT("Solo"), TEXT("Soloed") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_Soloed::GetValueType() const
{
	return ESequencerTextFilterValueType::Boolean;
}

FText FSequencerTextFilterExpression_Soloed::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_Soloed", "Filter by track soloed state");
}

bool FSequencerTextFilterExpression_Soloed::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const TViewModelPtr<ISoloableExtension> SoloableExtension = FilterItem->FindAncestorOfType<ISoloableExtension>(true);
	if (!SoloableExtension.IsValid())
	{
		return false;
	}

	const bool bPassed = SoloableExtension->IsSolo();
	return CompareFStringForExactBool(InValue, InComparisonOperation, bPassed);
}

#undef LOCTEXT_NAMESPACE
