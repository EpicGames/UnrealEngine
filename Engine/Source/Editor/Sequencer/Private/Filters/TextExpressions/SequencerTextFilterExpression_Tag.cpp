// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_Tag.h"
#include "ISequencer.h"
#include "MovieScene.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_Tag"

FSequencerTextFilterExpression_Tag::FSequencerTextFilterExpression_Tag(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_Tag::GetKeys() const
{
	return { TEXT("Tag") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_Tag::GetValueType() const
{
	return ESequencerTextFilterValueType::String;
}


FText FSequencerTextFilterExpression_Tag::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_Tag", "Filter by track Sequencer tag");
}

bool FSequencerTextFilterExpression_Tag::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const UMovieScene* const MovieScene = GetFocusedGetMovieScene();
	if (!MovieScene)
	{
		return true;
	}

	if (const TViewModelPtr<FObjectBindingModel> BindingModel = FilterItem->FindAncestorOfType<FObjectBindingModel>())
	{
		const ISequencer& Sequencer = FilterInterface.GetSequencer();

		const FMovieSceneObjectBindingID ObjectBindingID(UE::MovieScene::FFixedObjectBindingID(BindingModel->GetObjectGuid(), Sequencer.GetFocusedTemplateID()));
		const TMap<FName, FMovieSceneObjectBindingIDs>& AllTaggedBindings = MovieScene->AllTaggedBindings();

		for (const TTuple<FName, FMovieSceneObjectBindingIDs>& TagBindingPair : AllTaggedBindings)
		{
			if (TagBindingPair.Value.IDs.Contains(ObjectBindingID))
			{
				const FString TagName = TagBindingPair.Key.ToString().ToUpper();
				const FTextFilterString TagTextFilterString = TagName;
				if (TagTextFilterString.CompareText(InValue, ETextFilterTextComparisonMode::Partial))
				{
					return true;
				}
			}
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
