// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerTextFilterExpressionContext.h"

class FSequencerTextFilterExpression_Tag : public FSequencerTextFilterExpressionContext
{
public:
	FSequencerTextFilterExpression_Tag(ISequencerTrackFilters& InFilterInterface);

	//~ Begin FSequencerTextFilterExpressionContext
	virtual TSet<FName> GetKeys() const override;
	virtual ESequencerTextFilterValueType GetValueType() const override;
	virtual FText GetDescription() const override;
	//~ End FSequencerTextFilterExpressionContext

	//~ Begin ITextFilterExpressionContext
	virtual bool TestComplexExpression(const FName& InKey
		, const FTextFilterString& InValue
		, const ETextFilterComparisonOperation InComparisonOperation
		, const ETextFilterTextComparisonMode InTextComparisonMode) const override;
	//~ End ITextFilterExpressionContext
};
