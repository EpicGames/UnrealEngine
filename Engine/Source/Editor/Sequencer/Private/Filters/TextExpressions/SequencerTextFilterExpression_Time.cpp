// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_Time.h"
#include "Filters/SequencerFilterBar.h"
#include "IKeyArea.h"
#include "ISequencer.h"
#include "MovieScene.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/ViewModels/ChannelModel.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_Time"

FSequencerTextFilterExpression_Time::FSequencerTextFilterExpression_Time(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_Time::GetKeys() const
{
	return { TEXT("Time") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_Time::GetValueType() const
{
	return ESequencerTextFilterValueType::Integer;
}

FText FSequencerTextFilterExpression_Time::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_Time", "Filter by time range (frame number)");
}

TArray<FSequencerTextFilterKeyword> FSequencerTextFilterExpression_Time::GetValueKeywords() const
{
	return {
		{ TEXT("Now"), LOCTEXT("NowKeywordDescription", "Use the current playhead time value") },
		{ TEXT("Start"), LOCTEXT("StartKeywordDescription", "Use the start time of the sequence") },
		{ TEXT("End"), LOCTEXT("EndKeywordDescription", "Use the end time of the sequence") }
	};
}

bool FSequencerTextFilterExpression_Time::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const UMovieScene* const FocusedMovieScene = GetFocusedGetMovieScene();
	if (!FocusedMovieScene)
	{
		return true;
	}

	const TViewModelPtr<FChannelGroupOutlinerModel> ChannelGroupOutlinerModel = FilterItem.ImplicitCast();
	if (!ChannelGroupOutlinerModel.IsValid())
	{
		return false;
	}

	ISequencer& Sequencer = FilterInterface.GetSequencer();

	// Assume the value is a specified as a numeric frame number
	FTextFilterString ValueToCheck = InValue;

	// Check for time entered instead of frame number
	if (ValueToCheck.AsString().Contains(":"))
	{
		FTimespan Timespan;
		FTimespan::Parse(InValue.AsString(), Timespan);

		const double TotalSeconds = Timespan.GetTotalSeconds();
		const FQualifiedFrameTime FrameTime(FFrameTime::FromDecimal(TotalSeconds), FocusedMovieScene->GetDisplayRate());

		ValueToCheck = FString::FromInt(FrameTime.Time.FloorToFrame().Value);
	}

	const TArray<TSharedRef<IKeyArea>> CachedKeyAreas = ChannelGroupOutlinerModel->GetAllKeyAreas();

	if (ValueToCheck.CompareFString(TEXT("Now"), ETextFilterTextComparisonMode::Exact))
	{
		const FTextFilterString NowTimeValue = FString::FromInt(Sequencer.GetGlobalTime().Time.FrameNumber.Value);
		return CompareTime(Sequencer, NowTimeValue, CachedKeyAreas, InComparisonOperation);
	}
	if (ValueToCheck.CompareFString(TEXT("Start"), ETextFilterTextComparisonMode::Exact))
	{
		const FTextFilterString NowTimeValue = FString::FromInt(0);
		return CompareTime(Sequencer, NowTimeValue, CachedKeyAreas, InComparisonOperation);
	}
	if (ValueToCheck.CompareFString(TEXT("End"), ETextFilterTextComparisonMode::Exact))
	{
		const FFrameNumber PlaybackRange = FocusedMovieScene->GetPlaybackRange().Size<FFrameNumber>();
		const FTextFilterString NowTimeValue = FString::FromInt(PlaybackRange.Value);
		return CompareTime(Sequencer, NowTimeValue, CachedKeyAreas, InComparisonOperation);
	}

	return CompareTime(Sequencer, ValueToCheck, CachedKeyAreas, InComparisonOperation);
}

bool FSequencerTextFilterExpression_Time::CompareTime(ISequencer& InSequencer
	, const FTextFilterString& InValue
	, const TArray<TSharedRef<IKeyArea>>& InKeyAreas
	, const ETextFilterComparisonOperation InComparisonOperation)
{
	if (!InValue.AsString().IsNumeric() || InKeyAreas.IsEmpty())
	{
		return false;
	}

	const UMovieSceneSequence* const MovieSceneSequence = InSequencer.GetFocusedMovieSceneSequence();
	if (!MovieSceneSequence)
	{
		return false;
	}

	const UMovieScene* const FocusedMovieScene = MovieSceneSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return false;
	}

	const FFrameRate TickResolution = FocusedMovieScene->GetTickResolution();
	const FFrameRate DisplayRate = FocusedMovieScene->GetDisplayRate();

	for (const TSharedRef<IKeyArea>& KeyArea : InKeyAreas)
	{
		TArray<FKeyHandle> OutKeyHandles;
		KeyArea->GetKeyHandles(OutKeyHandles);
		if (OutKeyHandles.IsEmpty())
		{
			continue;
		}

		TArray<FFrameNumber> OutKeyTimes;
		OutKeyTimes.AddUninitialized(OutKeyHandles.Num());
		KeyArea->GetKeyTimes(OutKeyHandles, OutKeyTimes);

		for (const FFrameNumber& FrameTime : OutKeyTimes)
		{
			const FFrameNumber ConvertedFrameNumber = ConvertFrameTime(FrameTime, TickResolution, DisplayRate).RoundToFrame();
			const FTextFilterString KeyFrameString = FString::FromInt(ConvertedFrameNumber.Value);
			if (!InValue.CompareNumeric(KeyFrameString, InComparisonOperation))
			{
				return true;
			}
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
