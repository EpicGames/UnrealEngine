// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerTextFilterExpressionContext.h"

class FSequencerTextFilterExpression_Time : public FSequencerTextFilterExpressionContext
{
public:
	static bool CompareTime(ISequencer& InSequencer
		, const FTextFilterString& InValue
		, const TArray<TSharedRef<IKeyArea>>& InKeyAreas
		, const ETextFilterComparisonOperation InComparisonOperation);

	FSequencerTextFilterExpression_Time(ISequencerTrackFilters& InFilterInterface);

	//~ Begin FSequencerTextFilterExpressionContext
	virtual TSet<FName> GetKeys() const override;
	virtual ESequencerTextFilterValueType GetValueType() const override;
	virtual FText GetDescription() const override;
	virtual TArray<FSequencerTextFilterKeyword> GetValueKeywords() const override;
	//~ End FSequencerTextFilterExpressionContext

	//~ Begin ITextFilterExpressionContext
	virtual bool TestComplexExpression(const FName& InKey
		, const FTextFilterString& InValue
		, const ETextFilterComparisonOperation InComparisonOperation
		, const ETextFilterTextComparisonMode InTextComparisonMode) const override;
	//~ End ITextFilterExpressionContext
};
