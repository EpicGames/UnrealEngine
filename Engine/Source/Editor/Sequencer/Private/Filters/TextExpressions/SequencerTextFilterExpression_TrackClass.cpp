// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_TrackClass.h"
#include "Filters/SequencerFilterData.h"
#include "Sequencer.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_TrackClass"

FSequencerTextFilterExpression_TrackClass::FSequencerTextFilterExpression_TrackClass(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_TrackClass::GetKeys() const
{
	return { TEXT("TrackClass"), TEXT("TrackType") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_TrackClass::GetValueType() const
{
	return ESequencerTextFilterValueType::String;
}

FText FSequencerTextFilterExpression_TrackClass::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_TrackClass", "Filter by track class name");
}

bool FSequencerTextFilterExpression_TrackClass::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	if (UMovieSceneTrack* const TrackObject = GetMovieSceneTrack())
	{
		const FString TrackClassName = TrackObject->GetClass()->GetName();
		if (TextFilterUtils::TestComplexExpression(TrackClassName, InValue, InComparisonOperation, InTextComparisonMode))
		{
			return true;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
