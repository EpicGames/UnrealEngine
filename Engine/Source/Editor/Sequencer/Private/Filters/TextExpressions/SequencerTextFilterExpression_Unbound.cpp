// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTextFilterExpression_Unbound.h"
#include "Filters/Filters/SequencerTrackFilter_Unbound.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "Sequencer.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SequencerTextFilterExpression_Unbound"

FSequencerTextFilterExpression_Unbound::FSequencerTextFilterExpression_Unbound(ISequencerTrackFilters& InFilterInterface)
	: FSequencerTextFilterExpressionContext(InFilterInterface)
{
}

TSet<FName> FSequencerTextFilterExpression_Unbound::GetKeys() const
{
	return { TEXT("Unbound"), TEXT("Missing") };
}

ESequencerTextFilterValueType FSequencerTextFilterExpression_Unbound::GetValueType() const
{
	return ESequencerTextFilterValueType::Boolean;
}

FText FSequencerTextFilterExpression_Unbound::GetDescription() const
{
	return LOCTEXT("ExpressionDescription_Unbound", "Filter by track unbound or missing objects");
}

bool FSequencerTextFilterExpression_Unbound::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FSequencerTextFilterExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	const TSharedPtr<FSequencerTrackFilter_Unbound> Filter = MakeShared<FSequencerTrackFilter_Unbound>(FilterInterface);
	const bool bPassesFilter = Filter->PassesFilter(FilterItem);
	return CompareFStringForExactBool(InValue, InComparisonOperation, bPassesFilter);
}

#undef LOCTEXT_NAMESPACE
