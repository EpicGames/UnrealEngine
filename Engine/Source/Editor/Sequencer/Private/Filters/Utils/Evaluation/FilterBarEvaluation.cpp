// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilterBarEvaluation.h"

#include "Filters/SequencerTrackFilterBase.h"
#include "Filters/Filters/SequencerTrackFilter_HideIsolate.h"
#include "Filters/Filters/SequencerTrackFilter_Text.h"

namespace UE::Sequencer
{
void InvokePreFilter(const FSequencerFilterContext& InFilters)
{
	if (InFilters.TextFilter)
	{
		InFilters.TextFilter.GetValue()->OnPreFilter();
	}
	
	if (InFilters.HideIsolate)
	{
		InFilters.HideIsolate.GetValue()->OnPreFilter();
	}
	
	for (const TNotNull<FSequencerTrackFilter*> CustomTextFilter : InFilters.CustomTextFilters)
	{
		CustomTextFilter->OnPreFilter();
	}
	
	for (const TNotNull<FSequencerTrackFilter*> CommonFilter : InFilters.CommonFilters)
	{
		CommonFilter->OnPreFilter();
	}
	
	for (const TNotNull<FSequencerTrackFilter*> InternalFilter : InFilters.InternalFilters)
	{
		InternalFilter->OnPreFilter();
	}
}

namespace FilterEval
{
template<typename TFilter> requires std::is_base_of_v<FSequencerTrackFilter, TFilter>
static EItemFilterState EvaluateFilter(
	const TViewModelPtr<IOutlinerExtension>& InItem, 
	const TOptional<TNotNull<TFilter*>> InFilter,
	FRecursiveFilterOverrides& InOverrides
	)
{
	return InFilter ? EvaluateSingle(InItem, *InFilter.GetValue(), InOverrides) : EItemFilterState::DoNotCare;
}
}

FSequencerFilterEvaluation EvaluateSequencerFilterBar(
	const TViewModelPtr<IOutlinerExtension>& InItem, 
	const FSequencerFilterContext& InFilters,
	const FRecursiveFilterOverrides& InOverrides
	)
{
	FRecursiveFilterOverrides NewOverrides = InOverrides;
	
	// The below is sorted in the order we expect to evaluate the fastest.
	
	const EItemFilterState HideIsolateFilterResult = FilterEval::EvaluateFilter(InItem, InFilters.HideIsolate, NewOverrides);
	if (!PassesFilterState(HideIsolateFilterResult))
	{
		return FSequencerFilterEvaluation(HideIsolateFilterResult, NewOverrides);
	}
	
	const EItemFilterState TextFilterResult = FilterEval::EvaluateFilter(InItem, InFilters.TextFilter, NewOverrides);
	if (!PassesFilterState(TextFilterResult))
	{
		return FSequencerFilterEvaluation(TextFilterResult, NewOverrides);
	}
	
	const EItemFilterState CustomTextResult = EvaluateConjunctive(InItem, InFilters.CustomTextFilters, NewOverrides);
	if (!PassesFilterState(CustomTextResult))
	{
		return FSequencerFilterEvaluation(CustomTextResult, NewOverrides);
	}
	
	// The internal filters act as a pre-filter... before user filters are applied.
	const EItemFilterState InternalEval = EvaluateConjunctive(InItem, InFilters.InternalFilters, NewOverrides);
	if (!PassesFilterState(InternalEval))
	{
		return FSequencerFilterEvaluation(InternalEval, NewOverrides);
	}
	
	// The filter pills that the user manually checks and unchecks add or remove items... which encodes an OR condition
	const EItemFilterState CommonEval = EvaluateDisjunctive(InItem, InFilters.CommonFilters, NewOverrides);
	if (!PassesFilterState(CommonEval))
	{
		return FSequencerFilterEvaluation(CommonEval, NewOverrides);
	}
	
	// All filter results must pass. Effectively, this will combine to Include or DoNotCare.
	const EItemFilterState CombinedResult = CombineConjunctive(
		TextFilterResult, HideIsolateFilterResult, CommonEval, InternalEval, CustomTextResult
		);
	return FSequencerFilterEvaluation(CombinedResult, NewOverrides);
}
} // namespace UE::Sequencer
