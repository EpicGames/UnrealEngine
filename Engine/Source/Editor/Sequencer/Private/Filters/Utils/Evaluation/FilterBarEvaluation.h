// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FilterGroupEvaluation.h"
#include "Containers/Array.h"
#include "Misc/NotNull.h"

class FSequencerTrackFilter_HideIsolate;
class FSequencerTrackFilter_CustomText;
class FSequencerTrackFilter;
class FSequencerTrackFilter_Text;

namespace UE::Sequencer
{
constexpr SIZE_T NumInlineFilters = 8;
template<typename T>
using TFilterArray = TArray<T, TInlineAllocator<NumInlineFilters>>;

/**
 * The filters run by FSequencerFilterBar. The members are specific to the needs of FSequencerFilterBar. 
 * An item passes if the conjunction of filters passes, i.e. all filters pass on the item.
 */
struct FSequencerFilterContext
{
	/** The search box filter, if active. */
	TOptional<TNotNull<FSequencerTrackFilter_Text*>> TextFilter;
	
	/** The isolate / hide / show filter, if active. */
	TOptional<TNotNull<FSequencerTrackFilter_HideIsolate*>> HideIsolate;
	
	TFilterArray<TNotNull<FSequencerTrackFilter*>> CommonFilters;
	
	TFilterArray<TNotNull<FSequencerTrackFilter*>> InternalFilters;
	
	TFilterArray<TNotNull<FSequencerTrackFilter*>> CustomTextFilters;
	
	/** @return The number of active filters. */
	int32 NumFilters() const
	{
		return static_cast<int32>(TextFilter.IsSet()) 
			+ static_cast<int32>(HideIsolate.IsSet()) 
			+ CommonFilters.Num()
			+ InternalFilters.Num()
			+ CustomTextFilters.Num();
	}
};

/** Result of calling EvaluateSequencerFilterBar. */
struct FSequencerFilterEvaluation
{
	/** Indicates what to do with the item. */
	EItemFilterState ItemResult;
	
	/** The new overrides to apply to children of the item. */
	FRecursiveFilterOverrides NewOverrides;
};

/** Calls FSequencerFilterBase::OnPreFilter on all filters that require it. */
void InvokePreFilter(const FSequencerFilterContext& InFilters);

/** Evaluates the filters if FSequencerFilterBar. */
FSequencerFilterEvaluation EvaluateSequencerFilterBar(
	const TViewModelPtr<IOutlinerExtension>& InItem, 
	const FSequencerFilterContext& InFilters,
	const FRecursiveFilterOverrides& InOverrides
	);
}
