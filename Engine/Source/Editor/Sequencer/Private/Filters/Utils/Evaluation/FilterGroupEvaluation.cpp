// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilterGroupEvaluation.h"

#include "Filters/FilterResult.h"
#include "Filters/SequencerTrackFilterBase.h"
#include "Filters/SequencerTrackFilterCollection.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/Extensions/IOutlinerExtension.h"

namespace UE::Sequencer
{
TOptional<EItemFilterState> FRecursiveFilterOverrides::FindOverride(const FSequencerTrackFilter& InFilter) const
{
	for (const FFilterOverride& Override : Overrides)
	{
		if (&Override.Filter == &InFilter)
		{
			return Override.ResultToApply;
		}
	}
	
	return {};
}

void FRecursiveFilterOverrides::AddOverride(const FSequencerTrackFilter& InFilter, EItemFilterState InOverride)
{
	for (FFilterOverride& Override : Overrides)
	{
		if (&Override.Filter == &InFilter)
		{
			Override.ResultToApply = InOverride;
			return;
		}
	}
	
	Overrides.Emplace(InFilter, InOverride);
}

namespace FilterGroup
{
template<EItemFilterState TStopCondition, typename TCombineFunc> requires std::is_invocable_r_v<EItemFilterState, TCombineFunc, EItemFilterState, EItemFilterState>
static EItemFilterState Evaluate(
	const TViewModelPtr<IOutlinerExtension>& InItem, 
	TConstArrayView<TNotNull<FSequencerTrackFilter*>> InFilters,
	FRecursiveFilterOverrides& InOutOverrides,
	TCombineFunc&& InCombineFunc
	)
{
	EItemFilterState CombinedResult = EItemFilterState::DoNotCare;
	
	for (const FSequencerTrackFilter* Filter : InFilters)
	{
		const EItemFilterState FilterResult = EvaluateSingle(InItem, *Filter, InOutOverrides);
		CombinedResult = InCombineFunc(CombinedResult, FilterResult);
		if (CombinedResult == TStopCondition) // Short-circuit evaluation of the other filters, to skip unneeded evaluation.
		{
			return CombinedResult;
		}
	}
	
	return CombinedResult;
}
}

EItemFilterState EvaluateDisjunctive(
	const TViewModelPtr<IOutlinerExtension>& InItem, 
	TConstArrayView<TNotNull<FSequencerTrackFilter*>> InFilters,
	FRecursiveFilterOverrides& InOutOverrides
	)
{
	return FilterGroup::Evaluate<EItemFilterState::Include>(InItem, InFilters, InOutOverrides, [](EItemFilterState Left, EItemFilterState Right)
	{
		return CombineDisjunctive(Left, Right);
	});
}

EItemFilterState EvaluateConjunctive(
	const TViewModelPtr<IOutlinerExtension>& InItem,
	TConstArrayView<TNotNull<FSequencerTrackFilter*>> InFilters, 
	FRecursiveFilterOverrides& InOutOverrides
	)
{
	return FilterGroup::Evaluate<EItemFilterState::Exclude>(InItem, InFilters, InOutOverrides, [](EItemFilterState Left, EItemFilterState Right)
	{
		return CombineConjunctive(Left, Right);
	});
}

EItemFilterState EvaluateSingle(
	const TViewModelPtr<IOutlinerExtension>& InItem,
	const FSequencerTrackFilter& InFilter,
	FRecursiveFilterOverrides& InOutOverrides
	)
{
	const TOptional<EItemFilterState> Override = InOutOverrides.FindOverride(InFilter);
	const FFilterResult FilterResult = Override 
		? FFilterResult(*Override, EFilterResultFlags::ApplyResultToChildren) 
		: InFilter.Evaluate(InItem);
		
	if (EnumHasAnyFlags(FilterResult.Flags, EFilterResultFlags::ApplyResultToChildren))
	{
		InOutOverrides.AddOverride(InFilter, FilterResult.ItemState);
	}
	
	return FilterResult.ItemState;
}
}
