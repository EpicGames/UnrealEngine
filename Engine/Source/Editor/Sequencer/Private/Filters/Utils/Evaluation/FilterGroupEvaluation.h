// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Filters/FilterResult.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModelPtr.h"

class FSequencerTrackFilter;
class FSequencerTrackFilterCollection;

namespace UE::Sequencer
{
enum class EItemFilterState : uint8;
struct FFilterResult;

/** Override of the result of a single filter */
struct FFilterOverride
{
	/** The filter this override applies to */
	const FSequencerTrackFilter& Filter;
	/** The state to assume for any items passed to this filter */
	EItemFilterState ResultToApply;

	explicit FFilterOverride(const FSequencerTrackFilter& Filter UE_LIFETIMEBOUND, EItemFilterState ResultToApply)
		: Filter(Filter)
		, ResultToApply(ResultToApply)
	{}
};

/** Holds overrides for various filters. */
struct FRecursiveFilterOverrides
{
	/** Filters in this array should not be evaluated. Instead, just assume FFilterOverride::ResultToApply as result. */
	TArray<FFilterOverride, TInlineAllocator<8>> Overrides;
	
	TOptional<EItemFilterState> FindOverride(const FSequencerTrackFilter& InFilter) const;
	void AddOverride(const FSequencerTrackFilter& InFilter, EItemFilterState InOverride);
};

/** Runs all filters and returns Include if any filter passes. */
EItemFilterState EvaluateDisjunctive(
	const TViewModelPtr<IOutlinerExtension>& InItem, 
	TConstArrayView<TNotNull<FSequencerTrackFilter*>> InFilters,
	FRecursiveFilterOverrides& InOutOverrides
	);

/** Runs all filters and returns Include if all filters pass. */
EItemFilterState EvaluateConjunctive(
	const TViewModelPtr<IOutlinerExtension>& InItem, 
	TConstArrayView<TNotNull<FSequencerTrackFilter*>> InFilters,
	FRecursiveFilterOverrides& InOutOverrides
	);

/** Runs the filter and appends its result to FRecursiveFilterOverrides if the filter reports that the result applies to child items. */
EItemFilterState EvaluateSingle(
	const TViewModelPtr<IOutlinerExtension>& InItem,
	const FSequencerTrackFilter& InFilter,
	FRecursiveFilterOverrides& InOutOverrides
	);
}
