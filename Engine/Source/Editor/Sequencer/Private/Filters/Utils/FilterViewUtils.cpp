// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilterViewUtils.h"

#include "HideIsolateViewModelUtils.h"
#include "Filters/ISequencerTrackFilters.h"
#include "Filters/Menus/SequencerTrackFilterMenu.h"
#include "Filters/Widgets/SFilterExpressionHelpDialog.h"
#include "Filters/Widgets/SSequencerCustomTextFilterDialog.h"
#include "Templates/SharedPointer.h"

#define LOCTEXT_NAMESPACE "FilterViewUtils"

namespace UE::Sequencer
{
void OpenDialog_SaveCurrentFilterSetAsCustomTextFilter(const TSharedRef<ISequencerTrackFilters>& InFilterBar)
{
	FCustomTextFilterData CustomTextFilterData;
	CustomTextFilterData.FilterString = FText::FromString(InFilterBar->GenerateTextFilterStringFromEnabledFilters());
	if (CustomTextFilterData.FilterLabel.IsEmpty())
	{
		CustomTextFilterData.FilterLabel = LOCTEXT("NewFilterName", "New Filter Name");
	}

	SSequencerCustomTextFilterDialog::CreateWindow_AddCustomTextFilter(InFilterBar, MoveTemp(CustomTextFilterData));
}

void OpenDialog_TextExpressionHelp(const TSharedRef<ISequencerTrackFilters>& InFilterBar)
{
	FFilterExpressionHelpDialogConfig Config;
	Config.IdentifierName = TEXT("SequencerCustomTextFilterHelp");
	Config.DialogTitle = LOCTEXT("SequencerCustomTextFilterHelp", "Sequencer Custom Text Filter Help");
	Config.AddCommonFilterHelpEntries<FSequencerTrackFilter>(InFilterBar);

	SFilterExpressionHelpDialog::Open(MoveTemp(Config));
}

TSharedRef<SComboButton> MakeAddFilterButton(
	const TSharedRef<ISequencerTrackFilters>& InTrackFilters,
	FOnGetContent InGetMenuContent
	)
{
	const TSharedPtr<SLayeredImage> FilterImage = SNew(SLayeredImage)
		.Image(FAppStyle::Get().GetBrush(TEXT("Icons.Filter")))
		.ColorAndOpacity_Lambda([WeakFilters = InTrackFilters.ToWeakPtr()]
		{
			const TSharedPtr<ISequencerTrackFilters> Filters = WeakFilters.Pin();
			return Filters && Filters->AreFiltersMuted() ? FLinearColor(1.f, 1.f, 1.f, 0.2f) : FSlateColor::UseForeground();
		});

	// Badge the filter icon if there are filters enabled or active
	FilterImage->AddLayer(
		TAttribute<const FSlateBrush*>::CreateLambda([WeakFilters = InTrackFilters.ToWeakPtr()]() -> const FSlateBrush*
		{
			const TSharedPtr<ISequencerTrackFilters> Filters = WeakFilters.Pin();
			if (!Filters || Filters->AreFiltersMuted() || !Filters->HasAnyFilterEnabled())
			{
				return nullptr;
			}

			if (Filters->HasAnyFilterActive(false, false))
			{
				return FAppStyle::Get().GetBrush(TEXT("Icons.BadgeModified"));
			}

			return FAppStyle::Get().GetBrush(TEXT("Icons.Badge"));
		}));

	const TSharedRef<SComboButton> ComboButton = SNew(SComboButton)
		.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>(TEXT("SimpleComboButtonWithIcon")))
		.ForegroundColor(FSlateColor::UseStyle())
		.ToolTipText_Lambda([WeakFilters = InTrackFilters.ToWeakPtr()]
		{
			const TSharedPtr<ISequencerTrackFilters> Filters = WeakFilters.Pin();
			if (!Filters)
			{
				return FText::GetEmpty();
			}
				
			return FText::Format(
				LOCTEXT("AddFilterToolTip", "Open the Add Filter Menu to add or manage filters\n\n"
				"Shift + Click to temporarily mute all active filters\n\n{0}"), 
				MakeHiddenAndIsolatedCountText_WithTotal(Filters->GetFilterData(), *Filters->GetHideIsolateFilter())
				);
		})
		.OnComboBoxOpened_Lambda([WeakFilters = InTrackFilters.ToWeakPtr()]
		{
			const TSharedPtr<ISequencerTrackFilters> Filters = WeakFilters.Pin();
				
			// Don't allow opening the menu if filters are muted or we are toggling the filter mute state
			if (!Filters || Filters->AreFiltersMuted() || FSlateApplication::Get().GetModifierKeys().IsShiftDown())
			{
				FSlateApplication::Get().DismissAllMenus();
			}
		})
		.OnGetMenuContent_Lambda([WeakFilters = InTrackFilters.ToWeakPtr(), InGetMenuContent = MoveTemp(InGetMenuContent)]() -> TSharedRef<SWidget>
		{
			const TSharedPtr<ISequencerTrackFilters> Filters = WeakFilters.Pin();
			if (!Filters)
			{
				return SNullWidget::NullWidget;
			}
				
			if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
			{
				Filters->MuteFilters(!Filters->AreFiltersMuted());
				FSlateApplication::Get().DismissAllMenus();
				return SNullWidget::NullWidget;
			}
				
			return InGetMenuContent.IsBound() ? InGetMenuContent.Execute() : SNullWidget::NullWidget;
		})
		.ContentPadding(FMargin(1, 0))
		.ButtonContent()
		[
			FilterImage.ToSharedRef()
		];
	ComboButton->AddMetadata(MakeShared<FTagMetaData>(TEXT("SequencerTrackFiltersCombo")));

	return ComboButton;
}
}

#undef LOCTEXT_NAMESPACE