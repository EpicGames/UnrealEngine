// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Templates/SharedPointerFwd.h"

class SComboButton;
class ISequencerTrackFilters;

namespace UE::Sequencer
{
/** Opens a dialog for saving the current filter set as a custom text filter. */
void OpenDialog_SaveCurrentFilterSetAsCustomTextFilter(const TSharedRef<ISequencerTrackFilters>& InFilterBar);

/** Opens a dialog that helps the user for creating custom text expressions. */
void OpenDialog_TextExpressionHelp(const TSharedRef<ISequencerTrackFilters>& InFilterBar);

/** @return Combo button for adding filters to ISequencerTrackFilters. */
TSharedRef<SComboButton> MakeAddFilterButton(
	const TSharedRef<ISequencerTrackFilters>& InTrackFilters,
	FOnGetContent InGetMenuContent
	);
}

