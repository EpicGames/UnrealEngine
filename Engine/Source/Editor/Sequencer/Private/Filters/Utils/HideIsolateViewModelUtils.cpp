// Copyright Epic Games, Inc. All Rights Reserved.

#include "HideIsolateViewModelUtils.h"

#include "Filters/Filters/SequencerTrackFilter_HideIsolate.h"
#include "Filters/SequencerFilterData.h"
#include "Internationalization/Text.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"

#define LOCTEXT_NAMESPACE "FitlerViewModelUtils"

namespace UE::Sequencer
{
TSet<TWeakViewModelPtr<IOutlinerExtension>> GetSelectedTracksOrAll(const FSequencerEditorViewModel& InViewModel)
{
	const TSharedPtr<FSequencerSelection> Selection = InViewModel.GetSelection();
	if (!Selection.IsValid())
	{
		return {};
	}

	const TSet<TWeakViewModelPtr<IOutlinerExtension>> SelectedSet = Selection->Outliner.GetSelected();
	return SelectedSet.IsEmpty() ? GetAllTracks(InViewModel) : SelectedSet;
}

TSet<TWeakViewModelPtr<IOutlinerExtension>> GetSelectedTracks(const FSequencerEditorViewModel& InViewModel)
{
	const TSharedPtr<FSequencerSelection> Selection = InViewModel.GetSelection();
	if (!Selection.IsValid())
	{
		return {};
	}

	const TSet<TWeakViewModelPtr<IOutlinerExtension>> SelectedSet = Selection->Outliner.GetSelected();
	return SelectedSet;
}

TSet<TWeakViewModelPtr<IOutlinerExtension>> GetAllTracks(const FSequencerEditorViewModel& InViewModel)
{
	TSet<TWeakViewModelPtr<IOutlinerExtension>> OutTracks;
	for (const TViewModelPtr<IOutlinerExtension>& TrackModel : InViewModel.GetRootModel()->GetDescendantsOfType<IOutlinerExtension>())
	{
		OutTracks.Add(TrackModel);
	}
	return OutTracks;
}

FText MakeHiddenTracksCountText(const FSequencerTrackFilter_HideIsolate& InFilter)
{
	return FText::Format(
		LOCTEXT("HiddenTracksSummary", "{0} hidden tracks"), 
		InFilter.GetHiddenTrackPaths().Num()
		);
}

FText MakeHiddenTracksCountText_WithTotal(
	const FSequencerFilterData& InFilterData, const FSequencerTrackFilter_HideIsolate& InFilter
	)
{
	return FText::Format(
		LOCTEXT("HiddenTracksSummaryWithTotal", "{0} of {1} total tracks"), 
		MakeHiddenTracksCountText(InFilter),
		InFilterData.GetTotalNodeCount()
		);
}

FText MakeIsolatedTracksCountText(const FSequencerTrackFilter_HideIsolate& InFilter)
{
	return FText::Format(
		LOCTEXT("IsolatedTracksSummary", "{0} isolated tracks"), 
		InFilter.GetIsolatedTrackPaths().Num()
		);
}

FText MakeIsolatedTracksCountText_WithTotal(
	const FSequencerFilterData& InFilterData, const FSequencerTrackFilter_HideIsolate& InFilterBar
	)
{
	return FText::Format(
		LOCTEXT("IsolatedTracksSummaryWithTotal", "{0} of {1} total tracks"), 
		MakeIsolatedTracksCountText(InFilterBar),
		InFilterData.GetTotalNodeCount()
		);
}

FText MakeHiddenAndIsolatedCountText(const FSequencerTrackFilter_HideIsolate& InFilterBar)
{
	const FText HiddenTracksSummary = MakeHiddenTracksCountText(InFilterBar);
	const FText IsolatedTracksSummary = MakeIsolatedTracksCountText(InFilterBar);
	return FText::Format(
		LOCTEXT("HideIsolateSummary", "{0}, {1}"),
		HiddenTracksSummary, 
		IsolatedTracksSummary
		);
}

FText MakeHiddenAndIsolatedCountText_WithTotal(
	const FSequencerFilterData& InFilterData, const FSequencerTrackFilter_HideIsolate& InFilterBar
	)
{
	const int32 FilteredCount = InFilterData.GetDisplayNodeCount();
	const int32 TotalCount = InFilterData.GetTotalNodeCount();
	const FText HideIsolateSummary = MakeHiddenAndIsolatedCountText(InFilterBar);
	return FText::Format(
		LOCTEXT("LongDisplaySummary", "Showing {0} of {1} total tracks\n{2}"),
		FilteredCount, TotalCount, HideIsolateSummary
		);
}
} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE