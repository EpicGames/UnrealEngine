// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "MVVM/ViewModelPtr.h"

class FSequencerTrackFilter_HideIsolate;
class FText;
class ISequencerFilterBar;
struct FSequencerFilterData;

namespace UE::Sequencer
{
class FSequencerEditorViewModel;
class IOutlinerExtension;

/** @return Gets the selected IOutlinerExtensions, or all of them if nothing is selected. */
TSet<TWeakViewModelPtr<IOutlinerExtension>> GetSelectedTracksOrAll(const FSequencerEditorViewModel& InViewModel);
/** @return Gets the selected IOutlinerExtensions. */
TSet<TWeakViewModelPtr<IOutlinerExtension>> GetSelectedTracks(const FSequencerEditorViewModel& InViewModel);
/** @return Gets all tracks */
TSet<TWeakViewModelPtr<IOutlinerExtension>> GetAllTracks(const FSequencerEditorViewModel& InViewModel);

/** @return "{0} hidden tracks" */
FText MakeHiddenTracksCountText(const FSequencerTrackFilter_HideIsolate& InFilter);
/** @return "{0} hidden tracks of {1} total tracks" */
FText MakeHiddenTracksCountText_WithTotal(const FSequencerFilterData& InFilterData, const FSequencerTrackFilter_HideIsolate& InFilter);

/** @return "{0} isolated tracks" */
FText MakeIsolatedTracksCountText(const FSequencerTrackFilter_HideIsolate& InFilter);
/** @return "{0} isolated tracks of {1} total tracks" */
FText MakeIsolatedTracksCountText_WithTotal(const FSequencerFilterData& InFilterData, const FSequencerTrackFilter_HideIsolate& InFilterBar);

/** @return "{0} hidden tracks, {1} isolated tracks" */
FText MakeHiddenAndIsolatedCountText(const FSequencerTrackFilter_HideIsolate& InFilterBar);
/** @return "Showing {0} of {1} total tracks\n{0} hidden tracks, {1} isolated tracks" */
FText MakeHiddenAndIsolatedCountText_WithTotal(const FSequencerFilterData& InFilterData, const FSequencerTrackFilter_HideIsolate& InFilterBar);
} // namespace UE::Sequencer
