// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Utils/HierarchyFilterCache.h"

#include "Filters/SequencerTrackFilterSelectedExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Selection/SequencerOutlinerSelection.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

namespace UE::Sequencer
{
void IncludeTree(FHierarchyFilterCache& InCache, const FViewModelPtr& InItem)
{
	// If it's already included the parents were already implicitly added before so avoid iterating it again...
	if (!InCache.Contains(InItem))
	{
		// If "Foo.A" is selected, make "Foo.A.Child", etc. filtered in...
		InCache.Add(InItem, EFilterResultFlags::ApplyResultToChildren);
		
		for (const FViewModelPtr& Parent : FParentModelIterator(InItem))
		{
			// Parent should be visible but do not tell them to propagate to all their children. 
			// If Foo.A is selected, then Foo should be filtered in, but not Foo.B.
			InCache.FindOrAdd(Parent);
		}
	}
	
	// ... but not all of its children might be added, yet.
	for (const FViewModelPtr& Child : FParentFirstChildIterator(InItem))
	{
		// ... an if the filtered is asked whether "Foo.A.Child" is filtered in, it should also say, yes.
		InCache.Add(Child, EFilterResultFlags::ApplyResultToChildren);
	}
}

FFilterResult EvaluateFilterCache(const FHierarchyFilterCache& InCache, const FViewModelPtr& InItem, EItemFilterState InNotIncludedResult)
{
	const EFilterResultFlags* Flags = InCache.Find(InItem);
	const bool bPassesFilter = Flags != nullptr;
	
	const EItemFilterState ResultState = bPassesFilter ? EItemFilterState::Include : InNotIncludedResult;
	const EFilterResultFlags ResultFlags = bPassesFilter ? *Flags : EFilterResultFlags::ApplyResultToChildren;
	return FFilterResult(ResultState, ResultFlags);
}

void IncludeSelection(
	FHierarchyFilterCache& InCache, const FOutlinerSelection& InOutlinerSelection, 
	const ISequencer& InSequencer, TConstArrayView<TWeakObjectPtr<USequencerTrackFilterSelectedExtension>> InExtensions
	)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IncludeSelection);
	
	const auto ProcessSelected = [&InCache](const FViewModelPtr& ViewModel)
	{
		if (!InCache.Contains(ViewModel))
		{
			IncludeTree(InCache, ViewModel);
		}
	};

	for (const FViewModelPtr& ViewModel : InOutlinerSelection)
	{
		ProcessSelected(ViewModel);
	}
	
	for (const TWeakObjectPtr<USequencerTrackFilterSelectedExtension>& Extension : InExtensions)
	{
		if (!Extension.IsValid())
		{
			continue;
		}
		
		Extension->EnumerateViewModelsConsideredAsSelected(InSequencer, [&ProcessSelected](const FViewModelPtr& ViewModel) 
		{
			ProcessSelected(ViewModel);
			return USequencerTrackFilterSelectedExtension::EBreakBehavior::Continue;
		});
	}
	
	InCache.Shrink();
}
} // UE::Sequencer
