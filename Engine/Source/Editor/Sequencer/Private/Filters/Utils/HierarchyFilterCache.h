// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Filters/FilterResult.h"
#include "MVVM/ViewModelPtr.h"

#define UE_API SEQUENCER_API

class ISequencer;
class USequencerTrackFilterSelectedExtension;

namespace UE::Sequencer
{
class FOutlinerSelection;
using FHierarchyFilterCache = TMap<TWeakViewModelPtr<FViewModel>, EFilterResultFlags>;

/**
 * Causes InItem, its parents, and all of InItem's children to pass the filter. 
 * - Adds all InItem and InItem's children with the ApplyResultToChildren flag. 
 * - Adds InItem's parent without the flag.
 */
void IncludeTree(FHierarchyFilterCache& InCache, const FViewModelPtr& InItem);
	
/** 
 * @param InItem The item to evaluate
 * @param InNotIncludedResult What to return when InItem is not in the hiearchy.
 * @return Whether InItem passes the filter result. 
 */
FFilterResult EvaluateFilterCache(const FHierarchyFilterCache& InCache, const FViewModelPtr& InItem, EItemFilterState InNotIncludedResult = EItemFilterState::Exclude);

/** 
 * Adds the items that are supposed to be considered selected to the cache: 
 * 1. the items explicitly selected, and 
 * 2. the items that USequencerTrackFilterSelectedExtension reports should be considered selected.
 */
void IncludeSelection(
	FHierarchyFilterCache& InCache, const FOutlinerSelection& InOutlinerSelection, 
	const ISequencer& InSequencer, TConstArrayView<TWeakObjectPtr<USequencerTrackFilterSelectedExtension>> InExtensions
	);
} // namespace UE::Sequencer

#undef UE_API
