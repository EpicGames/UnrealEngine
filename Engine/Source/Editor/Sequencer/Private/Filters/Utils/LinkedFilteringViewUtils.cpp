// Copyright Epic Games, Inc. All Rights Reserved.

#include "LinkedFilteringViewUtils.h"

#include "Filters/Linking/Mode/ILinkedFilterViewModel.h"
#include "Misc/ConsoleVariables.h"
#include "Templates/SharedPointer.h"

namespace UE::Sequencer
{
void ToggleLinkedFiltering(TWeakPtr<ILinkedFilterViewModel> InWeakViewModel)
{
	const TSharedPtr<ILinkedFilterViewModel> ViewModelPin = InWeakViewModel.Pin();
	if (!ViewModelPin)
	{
		return;
	}
	
	static_assert(static_cast<int32>(ELinkedFilterMode::Count) == 2, "Update this switch");
	switch (ViewModelPin->GetFilterMode())
	{
	case ELinkedFilterMode::Linked: ViewModelPin->SetFilterMode(ELinkedFilterMode::Instanced); break;
	case ELinkedFilterMode::Instanced: ViewModelPin->SetFilterMode(ELinkedFilterMode::Linked); break;
	default: checkNoEntry(); break;
	}
}

bool IsLinkedFilteringActionChecked(TWeakPtr<ILinkedFilterViewModel> InWeakViewModel)
{
	const TSharedPtr<ILinkedFilterViewModel> ViewModelPin = InWeakViewModel.Pin();
	return ViewModelPin && ViewModelPin->GetFilterMode() == ELinkedFilterMode::Linked;
}
} // namespace UE::Sequencer
