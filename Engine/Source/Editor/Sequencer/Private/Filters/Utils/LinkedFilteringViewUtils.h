// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

namespace UE::Sequencer
{
class ILinkedFilterViewModel;

/** Utility for binding ToggleLinkedFiltering to FExecuteAction. */
void ToggleLinkedFiltering(TWeakPtr<ILinkedFilterViewModel> InWeakViewModel);
/** Utility for binding ToggleLinkedFiltering to FIsActionChecked. */
bool IsLinkedFilteringActionChecked(TWeakPtr<ILinkedFilterViewModel> InWeakViewModel);
}
