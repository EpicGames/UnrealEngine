// Copyright Epic Games, Inc. All Rights Reserved.

#include "HideIsolateShowViewModel.h"

#include "Filters/Filters/SequencerTrackFilter_HideIsolate.h"
#include "Filters/ISequencerFilterBar.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "Filters/Utils/HideIsolateViewModelUtils.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"

#define LOCTEXT_NAMESPACE "FIsolateHideShowViewModel"

namespace UE::Sequencer
{
FHideIsolateShowViewModel::FHideIsolateShowViewModel(
	const TSharedRef<ISequencerTrackFilters>& InFilterBar,
	const TSharedRef<FSequencerTrackFilter_HideIsolate>& InIsolateFilter,
	TAttribute<TSet<TWeakViewModelPtr<IOutlinerExtension>>> InSelectedTracksAttr,
	TAttribute<TSet<TWeakViewModelPtr<IOutlinerExtension>>> InAllTracksAttr
	)
	: FilterBar(InFilterBar)
	, IsolateFilter(InIsolateFilter)
	, SelectedTracksAttr(MoveTemp(InSelectedTracksAttr))
	, AllTracksAttr(MoveTemp(InAllTracksAttr))
{}

void FHideIsolateShowViewModel::EmptyIsolatedTracks()
{
	IsolateFilter->EmptyIsolatedTracks();
	FilterBar->RequestFilterUpdate();
}

void FHideIsolateShowViewModel::IsolateSelectedTracks(bool bAddToExisting)
{
	IsolateFilter->IsolateTracks(GetSelectedTracks(), bAddToExisting);
}

void FHideIsolateShowViewModel::EmptyHiddenTracks()
{
	IsolateFilter->EmptyHiddenTracks();
	FilterBar->RequestFilterUpdate();
}

void FHideIsolateShowViewModel::HideSelectedTracks(const bool bAddToExisting)
{
	IsolateFilter->HideTracks(GetSelectedTracks(), bAddToExisting);
}

void FHideIsolateShowViewModel::ShowAllTracks()
{
	IsolateFilter->ShowAllTracks();
	
	const TSet<TWeakViewModelPtr<IOutlinerExtension>> SelectedTracks = SelectedTracksAttr.Get();
	if (SelectedTracks.Num() > 0)
	{
		const TWeakViewModelPtr<IOutlinerExtension> FirstItem = *SelectedTracks.CreateConstIterator();
		RequestScrollToViewDelegate.Broadcast(FirstItem);
	}
	
	FilterBar->RequestFilterUpdate();
}

bool FHideIsolateShowViewModel::AreFiltersMuted() const
{
	return FilterBar->AreFiltersMuted();
}

bool FHideIsolateShowViewModel::HasIsolatedTracks() const
{
	return IsolateFilter->GetIsolatedTrackPaths().Num() > 0;
}

bool FHideIsolateShowViewModel::HasHiddenTracks() const
{
	return IsolateFilter->GetHiddenTrackPaths().Num() > 0;
}

FText FHideIsolateShowViewModel::GetHideTracksButtonTooltipText() const
{
	FText TooltipText = LOCTEXT("HideTracksButtonToolTip", "Hide selected tracks");

	const TSharedRef<const FInputChord> Chord = FSequencerTrackFilterCommands::Get().HideSelectedTracks->GetFirstValidChord();
	if (Chord->IsValidChord())
	{
		TooltipText = FText::Format(
			LOCTEXT("HideTracksButtonToolTipExtended", "{0} ({1})"), 
			TooltipText, 
			Chord->GetInputText()
			);
	}

	const FText SummaryWithTotal = MakeHiddenTracksCountText_WithTotal(FilterBar->GetFilterData(), *IsolateFilter);
	TooltipText = FText::Format(
		LOCTEXT("HideTracksButtonToolTipExtendedWithTotal", "{0}\n\nUse the Control modifier to reset the hidden track list.\n\n{1}"), 
		TooltipText, 
		SummaryWithTotal
		);

	return TooltipText;
}

FText FHideIsolateShowViewModel::GetIsolateTracksButtonTooltipText() const
{
	FText TooltipText = LOCTEXT("IsolateTracksButtonToolTip", "Isolate selected tracks");

	const TSharedRef<const FInputChord> Chord = FSequencerTrackFilterCommands::Get().IsolateSelectedTracks->GetFirstValidChord();
	if (Chord->IsValidChord())
	{
		TooltipText = FText::Format(
			LOCTEXT("IsolateTracksButtonToolTipExtended", "{0} ({1})"),
			TooltipText,
			Chord->GetInputText()
			);
	}

	const FText SummaryWithTotal = MakeIsolatedTracksCountText_WithTotal(FilterBar->GetFilterData(), *IsolateFilter);
	TooltipText = FText::Format(
		LOCTEXT("IsolateTracksButtonToolTipExtendedWithTotal", "{0}\n\n"
			"Use the Shift modifier to additively isolate.\n"
			"Use the Control modifier to reset the isolated track list.\n\n{1}"
			),
		TooltipText, 
		SummaryWithTotal
		);

	return TooltipText;
}

FText FHideIsolateShowViewModel::GetShowAllTracksButtonTooltipText() const
{
	FText TooltipText = LOCTEXT("ShowAllTracksButtonToolTip", "Show all tracks");

	const TSharedRef<const FInputChord> Chord = FSequencerTrackFilterCommands::Get().ShowAllTracks->GetFirstValidChord();
	if (Chord->IsValidChord())
	{
		TooltipText = FText::Format(
			LOCTEXT("ShowAllTracksButtonToolTipExtended", "{0} ({1})"), 
			TooltipText,
			Chord->GetInputText()
			);
	}

	const FText SummaryText = MakeHiddenAndIsolatedCountText_WithTotal(FilterBar->GetFilterData(), *IsolateFilter);
	TooltipText = FText::Format(
		LOCTEXT("ShowAllTracksButtonToolTipExtendedWithSummary", "{0}\n\n{1}"), 
		TooltipText,
		SummaryText
		);

	return TooltipText;
}

TSet<TWeakViewModelPtr<IOutlinerExtension>> FHideIsolateShowViewModel::GetSelectedTracks() const
{
	const TSet<TWeakViewModelPtr<IOutlinerExtension>> Selection = SelectedTracksAttr.Get();
	return Selection;
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE