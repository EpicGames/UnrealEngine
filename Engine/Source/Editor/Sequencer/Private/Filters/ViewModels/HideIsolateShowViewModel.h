// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "MVVM/ViewModelPtr.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"

class FSequencerTrackFilter_HideIsolate;
class ISequencerFilterBar;
class ISequencerTrackFilters;

namespace UE::Sequencer
{
class FSequencerEditorViewModel;
class IOutlinerExtension;

/** 
 * View model for the SFilterBarIsolateHideShow widget.
 * 
 * It is carefully engineered to be independent of FSequencer references so UI external to Sequencer can potentially create instances.
 * This API is private for now but can be made public more easily in the future.
 */
class FHideIsolateShowViewModel
{
public:
	
	/** Requests UI to view the given view model into view. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FRequestScrollToViewEvent, const TWeakViewModelPtr<IOutlinerExtension>&);
	
	explicit FHideIsolateShowViewModel(
		const TSharedRef<ISequencerTrackFilters>& InFilterBar,
		const TSharedRef<FSequencerTrackFilter_HideIsolate>& InIsolateFilter,
		TAttribute<TSet<TWeakViewModelPtr<IOutlinerExtension>>> InSelectedTracksAttr,
		TAttribute<TSet<TWeakViewModelPtr<IOutlinerExtension>>> InAllTracksAttr
		);
	
	/** Clears all the isolated tracks. */
	void EmptyIsolatedTracks();
	/** Isolates the selected tracks, or all tracks if none are selected. */
	void IsolateSelectedTracks(bool bAddToExisting);
	
	/** Clears all the hidden tracks. */
	void EmptyHiddenTracks();
	/** Hides the selected tracks, or all tracks if none are selected. */
	void HideSelectedTracks(const bool bAddToExisting);
	
	/** Shows all tracks effectively clearing the hidden and isolate states. */
	void ShowAllTracks();
	
	/** @return Whether the filters are muted.  */
	bool AreFiltersMuted() const;
	/** @return Whether any tracks are isolated */
	bool HasIsolatedTracks() const;
	/** @return Whether any tracks are hidden */
	bool HasHiddenTracks() const;
	
	FText GetHideTracksButtonTooltipText() const;
	FText GetIsolateTracksButtonTooltipText() const;
	FText GetShowAllTracksButtonTooltipText() const;
	
	/** @return Broadcasts when an item is supposed to be focused, e.g. scrolled to view. */
	FRequestScrollToViewEvent& OnRequestScrollToView() { return RequestScrollToViewDelegate; }
	
private:
	
	/** The filter bar for which this widget is being displayed. */
	const TSharedRef<ISequencerTrackFilters> FilterBar;
	/** The filter instance for which this widget is being displayed. */
	const TSharedRef<FSequencerTrackFilter_HideIsolate> IsolateFilter;
	
	/** Gets all selected tracks. */
	const TAttribute<TSet<TWeakViewModelPtr<IOutlinerExtension>>> SelectedTracksAttr;
	/** Gets all tracks. */
	const TAttribute<TSet<TWeakViewModelPtr<IOutlinerExtension>>> AllTracksAttr;
	
	/** Broadcasts when an item is supposed to be focused, e.g. scrolled to view. */
	FRequestScrollToViewEvent RequestScrollToViewDelegate;

	/** @return Selected tracks. */
	TSet<TWeakViewModelPtr<IOutlinerExtension>> GetSelectedTracks() const;
};
} // namespace UE::Sequencer

