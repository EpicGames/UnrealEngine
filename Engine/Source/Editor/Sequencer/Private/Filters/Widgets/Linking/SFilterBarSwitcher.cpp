// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFilterBarSwitcher.h"

#include "Sequencer.h"
#include "SFilterLinkStateSwitcher.h"
#include "Filters/Widgets/SSequencerFilterBar.h"
#include "Filters/Linking/Mode/LinkedFilterViewModel.h"

namespace UE::Sequencer
{
void SFilterBarSwitcher::Construct(const FArguments& InArgs)
{
	WeakSequencer = InArgs._Sequencer;
	FilterAreaConfigId = InArgs._FilterAreaConfigId;
	check(WeakSequencer.IsValid());
	
	const TSharedPtr<FLinkedFilterViewModel> LinkedFilterViewModel = InArgs._LinkedFilterViewModel;
	check(LinkedFilterViewModel);
	
	ChildSlot
	[
		SAssignNew(WidgetSwitcher, SFilterLinkStateSwitcher)
		.LinkedFilterViewModel(LinkedFilterViewModel)
		.Visibility(this, &SFilterBarSwitcher::GetFilterBarVisibility)
		.LinkedContent()
		[
			SNew(SSequencerFilterBar, LinkedFilterViewModel->GetLinkedFilterBarImpl())
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("SequencerTrackFilters")))
		]
		.InstancedContent()
		[
			SNew(SSequencerFilterBar, LinkedFilterViewModel->GetInstancedFilterBarImpl())
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("SequencerTrackFilters")))
		]
	];
}

FReply SFilterBarSwitcher::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return WidgetSwitcher->GetActiveWidget()->OnMouseButtonUp(MyGeometry, MouseEvent);
}

EVisibility SFilterBarSwitcher::GetFilterBarVisibility() const
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	USequencerSettings* Settings = Sequencer ? Sequencer->GetSequencerSettings() : nullptr;
	const FSequencerFilterAreaConfig* Config = Settings ? Settings->FindTrackFilterArea(FilterAreaConfigId) : nullptr;
	return Config && Config->IsFilterBarVisible() ? EVisibility::Visible : EVisibility::Collapsed;
}
} // namespace UE::Sequencer
