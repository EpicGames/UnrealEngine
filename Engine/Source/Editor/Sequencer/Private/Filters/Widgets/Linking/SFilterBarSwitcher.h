// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerFilterBar.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::Sequencer
{
class SFilterLinkStateSwitcher;
class FLinkedFilterViewModel;

/** Switches between SSequencerFilterBar instances based on the active ELinkedFilterMode. */
class SFilterBarSwitcher : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SFilterBarSwitcher) {}
		/** 
		 * Used to determine whether the filter bar is visible. 
		 * @see USequencerSettings::FindOrAddTrackFilterArea 
		 */
		SLATE_ARGUMENT(FName, FilterAreaConfigId)
		/** Used to get the USequencerSettings to determine whether the filter bar is visible. */
		SLATE_ARGUMENT(TWeakPtr<FSequencer>, Sequencer)
		
		/** View-model determines the ELinkedFilterMode driving the displayed content. */
		SLATE_ARGUMENT(TSharedPtr<FLinkedFilterViewModel>, LinkedFilterViewModel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	
	//~ Begin SWidget
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End SWidget
	
private:
	
	/** Used to get the USequencerSettings to determine whether the filter bar is visible. */
	TWeakPtr<FSequencer> WeakSequencer;
	/** @see USequencerSettings::FindOrAddTrackFilterArea  */
	FName FilterAreaConfigId;
	
	/** Determines the filter bar that is displayed. */
	TSharedPtr<SFilterLinkStateSwitcher> WidgetSwitcher;
	
	/** @return Checks the settings whether the filter bar should be visible. */
	EVisibility GetFilterBarVisibility() const;
};
} // namespace UE::Sequencer
