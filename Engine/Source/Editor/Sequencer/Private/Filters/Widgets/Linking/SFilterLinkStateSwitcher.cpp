// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFilterLinkStateSwitcher.h"

#include "Filters/Linking/Mode/ILinkedFilterViewModel.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

namespace UE::Sequencer
{
void SFilterLinkStateSwitcher::Construct(const FArguments& InArgs)
{
	LinkedFilterViewModel = InArgs._LinkedFilterViewModel;
	LinkedFilterViewModel->OnFilterModeChanged().AddSP(this, &SFilterLinkStateSwitcher::RefreshContent);
	
	ChildSlot
	[
		SAssignNew(WidgetSwitcher, SWidgetSwitcher)
		+SWidgetSwitcher::Slot() [ InArgs._LinkedContent.Widget ]
		+SWidgetSwitcher::Slot() [ InArgs._InstancedContent.Widget ]
	];
	
	RefreshContent();
}

TSharedPtr<SWidget> SFilterLinkStateSwitcher::GetActiveWidget() const
{
	return WidgetSwitcher->GetActiveWidget();
}

int32 SFilterLinkStateSwitcher::GetActiveIndex() const
{
	static_assert(static_cast<int32>(ELinkedFilterMode::Count) == 2, "Update this switch");
	switch (LinkedFilterViewModel->GetFilterMode())
	{
	case ELinkedFilterMode::Linked: return 0;
	case ELinkedFilterMode::Instanced: return 1;
	default: checkNoEntry(); return 0;
	}
}

void SFilterLinkStateSwitcher::RefreshContent() const
{
	WidgetSwitcher->SetActiveWidgetIndex(GetActiveIndex());
}
} // namespace UE::Sequencer
