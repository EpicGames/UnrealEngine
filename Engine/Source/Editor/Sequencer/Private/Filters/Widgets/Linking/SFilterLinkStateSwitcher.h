// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWidgetSwitcher;

namespace UE::Sequencer
{
class ILinkedFilterViewModel;

/** This widget switches it content based ELinkedFilterMode. */
class SFilterLinkStateSwitcher : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SFilterLinkStateSwitcher) {}
		/** View-model determines the ELinkedFilterMode driving the displayed content. */
		SLATE_ARGUMENT(TSharedPtr<ILinkedFilterViewModel>, LinkedFilterViewModel)

		/** Content to display when in linked mode. */
		SLATE_NAMED_SLOT(FArguments, LinkedContent)
		/** Content to display when in instanced mode. */
		SLATE_NAMED_SLOT(FArguments, InstancedContent)
		
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);
	
	/** @return The active content widget. */
	TSharedPtr<SWidget> GetActiveWidget() const;
	
private:
	
	/** View-model determines the ELinkedFilterMode driving the displayed content. */
	TSharedPtr<ILinkedFilterViewModel> LinkedFilterViewModel;
	
	/** Used to switch between the content. */
	TSharedPtr<SWidgetSwitcher> WidgetSwitcher;
	
	/** @return The widget index that should be active */
	int32 GetActiveIndex() const;
	/** Updates the widget content */
	void RefreshContent() const;
};
} // namespace UE::Sequencer
