// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLinkedFilteringToggleButton.h"

#include "Filters/Linking/Mode/ILinkedFilterViewModel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "SLinkedFilteringToggleButton"

namespace UE::Sequencer
{
void SLinkedFilteringToggleButton::Construct(
	const FArguments& InArgs, const TSharedRef<ILinkedFilterViewModel>& InLinkedFilterViewModel
	)
{
	LinkedFilterViewModel = InLinkedFilterViewModel;
	
	ChildSlot
	[
		SNew(SCheckBox)
		.ToolTipText(this, &SLinkedFilteringToggleButton::GetToolTipText)
		.Style(FAppStyle::Get(), TEXT("Sequencer.LinkedFilterToggleButton"))
		.IsChecked(this, &SLinkedFilteringToggleButton::IsChecked)
		.OnCheckStateChanged(this, &SLinkedFilteringToggleButton::OnButtonClicked)
		[
			SNew(SImage)
			.Image(this, &SLinkedFilteringToggleButton::GetIcon)
		]
	];
}

ECheckBoxState SLinkedFilteringToggleButton::IsChecked() const
{
	return LinkedFilterViewModel->GetFilterMode() == ELinkedFilterMode::Linked
		? ECheckBoxState::Checked
		: ECheckBoxState::Unchecked;
}

FText SLinkedFilteringToggleButton::GetToolTipText() const
{
	static_assert(static_cast<int32>(ELinkedFilterMode::Count) == 2, "Update this switch");
	switch (LinkedFilterViewModel->GetFilterMode())
	{
	case ELinkedFilterMode::Linked: return LOCTEXT("Tooltip.Linked", "Linked filtering.\nChanges made to filters affect all synced UI.");
	case ELinkedFilterMode::Instanced: return LOCTEXT("Tooltip.Instanced", "Instanced filtering.\nChanges made to filters only affect this UI.");
	default: checkNoEntry(); return FText::GetEmpty();
	}
}

void SLinkedFilteringToggleButton::OnButtonClicked(ECheckBoxState CheckBoxState)
{
	const ELinkedFilterMode NewMode = CheckBoxState == ECheckBoxState::Checked
		? ELinkedFilterMode::Linked
		: ELinkedFilterMode::Instanced;
	LinkedFilterViewModel->SetFilterMode(NewMode);
}

const FSlateBrush* SLinkedFilteringToggleButton::GetIcon() const
{
	static_assert(static_cast<int32>(ELinkedFilterMode::Count) == 2, "Update this switch");
	switch (LinkedFilterViewModel->GetFilterMode())
	{
	case ELinkedFilterMode::Linked: return FAppStyle::Get().GetBrush("Sequencer.LinkedFiltering.LinkedIcon");
	case ELinkedFilterMode::Instanced: return FAppStyle::Get().GetBrush("Sequencer.LinkedFiltering.InstancedIcon");;
	default: checkNoEntry(); return nullptr;
	}
}
} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE 