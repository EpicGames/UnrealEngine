// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::Sequencer
{
class ILinkedFilterViewModel;

/** Toggle button for switching between the linked and instanced filter modes. */
class SLinkedFilteringToggleButton : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SLinkedFilteringToggleButton){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<ILinkedFilterViewModel>& InLinkedFilterViewModel);
	
private:
	
	/** Used to toggle between the filter modes. */
	TSharedPtr<ILinkedFilterViewModel> LinkedFilterViewModel;

	ECheckBoxState IsChecked() const;
	FText GetToolTipText() const;
	void OnButtonClicked(ECheckBoxState CheckBoxState);
	const FSlateBrush* GetIcon() const;
};
} // namespace UE::Sequencer
