// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Widgets/SFilterBarClippingHorizontalBox.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/ArrangedChildren.h"
#include "Styling/CoreStyle.h"
#include "Styling/ToolBarStyle.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "SFilterBarClippingHorizontalBox"

void SFilterBarClippingHorizontalBox::Construct(const FArguments& InArgs)
{
	OnWrapButtonClicked = InArgs._OnWrapButtonClicked;
	bIsFocusable = InArgs._IsFocusable;
}

void SFilterBarClippingHorizontalBox::OnArrangeChildren(const FGeometry& InAllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	// If WrapButton hasn't been initialized, that means AddWrapButton() hasn't 
	// been called and this method isn't going to behave properly
	if (!ensure(WeakWrapButton.IsValid()))
	{
		SHorizontalBox::OnArrangeChildren(InAllottedGeometry, ArrangedChildren);
		return;
	}

	const TSharedPtr<SComboButton> WrapButton = WeakWrapButton.Pin();
	constexpr int32 ParentReductionSize = 10; // Arbitrary size to make things look nice

	LastClippedIndex = ClippedIndex;

	NumClippedChildren = 0;
	SHorizontalBox::OnArrangeChildren(InAllottedGeometry, ArrangedChildren);

	// Remove children that are clipped by the allotted geometry
	const int32 NumChildren = ArrangedChildren.Num(); 
	int32 IndexClippedAt = NumChildren;
	for (int32 ChildIndex = NumChildren - 1; ChildIndex >= 0; --ChildIndex)
	{
		const FArrangedWidget& ChildWidget = ArrangedChildren[ChildIndex];

		auto CeilToInt = [](const float InPosition)
		{
			return FMath::CeilToInt(InPosition - KINDA_SMALL_NUMBER);
		};

		// Ceil (Minus a tad for float precision) to ensure contents are not a sub-pixel larger than the box, which will create an unnecessary wrap button
		const FVector2D AbsChildPos = ChildWidget.Geometry.LocalToAbsolute(ChildWidget.Geometry.GetLocalSize());
		const FVector2D AbsParentPos = FVector2D(InAllottedGeometry.AbsolutePosition) + InAllottedGeometry.GetLocalSize() * InAllottedGeometry.Scale;
		const FVector2D ChildPosRounded = FVector2D(CeilToInt(AbsChildPos.X), CeilToInt(AbsChildPos.Y));
		const FVector2D ParentPosRounded = FVector2D(CeilToInt(AbsParentPos.X), CeilToInt(AbsParentPos.Y));

		if (ChildPosRounded.X > ParentPosRounded.X + ParentReductionSize)
		{
			++NumClippedChildren;
			ArrangedChildren.Remove(ChildIndex);
			IndexClippedAt = ChildIndex;
		}
	}

	if (IndexClippedAt == NumChildren)
	{
		WrapButton->SetVisibility(EVisibility::Collapsed);
	}
	else if (ArrangedChildren.Num() > 0)
	{
		WrapButton->SetVisibility(EVisibility::Visible);

		// Further remove any children that the wrap button overlaps with
		for (int32 ChildIndex = IndexClippedAt - 1; ChildIndex >= 0; --ChildIndex)
		{
			const FArrangedWidget& ChildWidget = ArrangedChildren[ChildIndex];

			const int32 ChildRightX = FMath::TruncToInt(ChildWidget.Geometry.AbsolutePosition.X + ChildWidget.Geometry.GetLocalSize().X * ChildWidget.Geometry.Scale);
			const int32 ParentSizeX = FMath::TruncToInt(InAllottedGeometry.GetAbsolutePosition().X + InAllottedGeometry.GetLocalSize().X * InAllottedGeometry.Scale);

			if (ChildRightX > ParentSizeX - ParentReductionSize)
			{
				++NumClippedChildren;
				ArrangedChildren.Remove(ChildIndex);
			}
		}
	}

	const int32 NewChildNum =  ArrangedChildren.Num();
	ClippedIndex = NewChildNum < 0 ? 0 : NewChildNum;
}

TSharedRef<SComboButton> SFilterBarClippingHorizontalBox::CreateWrapButton()
{
	const FToolBarStyle& ToolBarStyle = FCoreStyle::Get().GetWidgetStyle<FToolBarStyle>(TEXT("SlimToolBar"));

	// Always allow this to be focusable to prevent the menu from collapsing during interaction
	const TSharedRef<SComboButton> WrapButton =
		SNew(SComboButton)
		.Visibility(EVisibility::Collapsed)
		.HasDownArrow(false)
		.ButtonStyle(&ToolBarStyle.ButtonStyle)
		.ContentPadding(FMargin(-2.f, 0.f))
		.ToolTipText(LOCTEXT("ExpandFilterBar", "Click to expand the filter bar"))
		.OnGetMenuContent(OnWrapButtonClicked)
		.Cursor(EMouseCursor::Default)
		.IsFocusable(true)
		.ButtonContent()
		[
			SNew(SImage)
			.DesiredSizeOverride(FVector2D(10.f))
			.Image(&ToolBarStyle.WrapButtonStyle.ExpandBrush)
		];

	WeakWrapButton = WrapButton;

	return WrapButton;
}

TSharedRef<SWidget> SFilterBarClippingHorizontalBox::WrapVerticalListWithHeading(const TSharedRef<SWidget>& InWidget, const FPointerEventHandler InMouseButtonUpEvent)
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			// Vertical Filters List Header
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush(TEXT("Brushes.Header")))
			.Padding(FMargin(8.f, 6.f))
			.OnMouseButtonUp(InMouseButtonUpEvent)
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FilterListVerticalHeader", "Filters"))
				.TextStyle(FAppStyle::Get(), TEXT("ButtonText"))
				.Font(FAppStyle::Get().GetFontStyle(TEXT("NormalFontBold")))
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		.VAlign(VAlign_Fill)
		[
			SNew(SBox)
			.MaxDesiredHeight(480.f)
			[
				SNew(SScrollBox)
				+ SScrollBox::Slot()
				[
					InWidget
				]
			]
		];
}

#undef LOCTEXT_NAMESPACE
