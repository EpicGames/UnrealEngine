// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFilterBarIsolateHideShow.h"

#include "Filters/SequencerFilterBar.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "Filters/ViewModels/HideIsolateShowViewModel.h"

#include "Framework/Application/SlateApplication.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SFilterBarIsolateHideShow"

namespace UE::Sequencer
{
void SFilterBarIsolateHideShow::Construct(const FArguments& InArgs, const TSharedRef<FHideIsolateShowViewModel>& InViewModel)
{
	ViewModel = InViewModel;

	constexpr float ButtonContentPadding = 2.f;
	constexpr float ButtonSpacing = 1.f;

	ChildSlot
	[
		SNew(SHorizontalBox)

		// Isolate Selected Button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0.f, 0.f, ButtonSpacing, 0.f)
		[
			SNew(SButton)
			.ContentPadding(ButtonContentPadding)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
			.ToolTipText(InViewModel, &FHideIsolateShowViewModel::GetIsolateTracksButtonTooltipText)
			.IsEnabled(this, &SFilterBarIsolateHideShow::AreFiltersUnmuted)
			.OnClicked(this, &SFilterBarIsolateHideShow::HandleIsolateTracksClick)
			[
				ConstructLayeredImage(TEXT("Sequencer.TrackIsolate")
					, TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(InViewModel, &FHideIsolateShowViewModel::HasIsolatedTracks)))
			]
		]

		// Hide Selected Button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0.f, 0.f, ButtonSpacing, 0.f)
		[
			SNew(SButton)
			.ContentPadding(ButtonContentPadding)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
			.ToolTipText(InViewModel, &FHideIsolateShowViewModel::GetHideTracksButtonTooltipText)
			.IsEnabled(this, &SFilterBarIsolateHideShow::AreFiltersUnmuted)
			.OnClicked(this, &SFilterBarIsolateHideShow::HandleHideTracksClick)
			[
				ConstructLayeredImage(TEXT("Sequencer.TrackHide")
					, TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(InViewModel, &FHideIsolateShowViewModel::HasHiddenTracks)))
			]
		]

		// Show All Button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0.f, 0.f, 0.f, 0.f)
		[
			SNew(SButton)
			.ContentPadding(ButtonContentPadding)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
			.ToolTipText(InViewModel, &FHideIsolateShowViewModel::GetShowAllTracksButtonTooltipText)
			.IsEnabled(this, &SFilterBarIsolateHideShow::AreFiltersUnmuted)
			.OnClicked(this, &SFilterBarIsolateHideShow::HandleShowAllTracksClick)
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(16.f))
				.ColorAndOpacity(this, &SFilterBarIsolateHideShow::GetShowAllTracksButtonTextColor)
				.Image(FAppStyle::Get().GetBrush(TEXT("Sequencer.TrackShow")))
			]
		]
	];
}

TSharedRef<SWidget> SFilterBarIsolateHideShow::ConstructLayeredImage(const FName InBaseImageName, const TAttribute<bool>& InShowBadge)
{
	const TSharedRef<SLayeredImage> LayeredImage = SNew(SLayeredImage)
		.DesiredSizeOverride(FVector2D(16.f))
		.ColorAndOpacity(FStyleColors::Foreground)
		.Image(FAppStyle::Get().GetBrush(InBaseImageName));

	LayeredImage->AddLayer(TAttribute<const FSlateBrush*>::CreateLambda([InShowBadge]() -> const FSlateBrush*
		{
			return InShowBadge.Get(false)
				? FAppStyle::Get().GetBrush(TEXT("Icons.BadgeModified"))
				: nullptr;
		}));

	return LayeredImage;
}

bool SFilterBarIsolateHideShow::AreFiltersUnmuted() const
{
	return !ViewModel->AreFiltersMuted();
}

FReply SFilterBarIsolateHideShow::HandleHideTracksClick()
{
	const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
	const bool bControlDown = ModifierKeys.AreModifersDown(EModifierKey::Control);
	if (bControlDown)
	{
		ViewModel->EmptyHiddenTracks();
	}
	else
	{
		const bool bAddToExisting = !FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Shift);
		ViewModel->HideSelectedTracks(bAddToExisting);
	}

	return FReply::Handled();
}

FReply SFilterBarIsolateHideShow::HandleIsolateTracksClick()
{
	const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
	const bool bControlDown = ModifierKeys.AreModifersDown(EModifierKey::Control);
	if (bControlDown)
	{
		ViewModel->EmptyIsolatedTracks();
	}
	else
	{
		const bool bAddToExisting = FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Shift);
		ViewModel->IsolateSelectedTracks(bAddToExisting);
	}

	return FReply::Handled();
}

FReply SFilterBarIsolateHideShow::HandleShowAllTracksClick()
{
	ViewModel->ShowAllTracks();
	return FReply::Handled();
}

FSlateColor SFilterBarIsolateHideShow::GetShowAllTracksButtonTextColor() const
{
	const bool bHasActiveFilters = ViewModel->HasHiddenTracks() || ViewModel->HasIsolatedTracks();
	return bHasActiveFilters ? FStyleColors::Warning : FStyleColors::Foreground;
}
} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
