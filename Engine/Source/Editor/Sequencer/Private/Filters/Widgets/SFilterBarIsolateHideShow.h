// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ISequencerFilterBar;

namespace UE::Sequencer
{
class FHideIsolateShowViewModel;

/** View for visualizing the FSequencerTrackFilter_HideIsolate filter. */
class SFilterBarIsolateHideShow : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SFilterBarIsolateHideShow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FHideIsolateShowViewModel>& InViewModel);

private:
	
	/** View model for this widget. */
	TSharedPtr<FHideIsolateShowViewModel> ViewModel;
	
	TSharedRef<SWidget> ConstructLayeredImage(const FName InBaseImageName, const TAttribute<bool>& InShowBadge);
	
	/** @return Whether filters are not muted, i.e. whether filtering UI should be enabled. */
	bool AreFiltersUnmuted() const;

	FReply HandleHideTracksClick();
	FReply HandleIsolateTracksClick();
	FReply HandleShowAllTracksClick();

	FSlateColor GetShowAllTracksButtonTextColor() const;
};
}