// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Widgets/SFilterExpressionHelpDialog.h"
#include "Filters/SequencerTextFilterExpressionContext.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"

#define LOCTEXT_NAMESPACE "SFilterExpressionHelpDialog"

FFilterExpressionHelpDialogConfig::FFilterExpressionHelpDialogConfig()
{
	DialogTitle = LOCTEXT("DialogTitle", "Text Filter Expression Help");
	DocumentationLink = TEXT("https://dev.epicgames.com/documentation/en-us/unreal-engine/advanced-search-syntax-in-unreal-engine");
}

const FSlateColor SFilterExpressionHelpDialog::KeyColor = FStyleColors::AccentBlue;
const FSlateColor SFilterExpressionHelpDialog::ValueColor = FStyleColors::AccentOrange;

TMap<FName, TSharedPtr<SFilterExpressionHelpDialog>> SFilterExpressionHelpDialog::DialogInstance;

void SFilterExpressionHelpDialog::Open(FFilterExpressionHelpDialogConfig&& InConfig)
{
	const FName IdentifierName = InConfig.IdentifierName;

	if (DialogInstance.Contains(IdentifierName)
		&& DialogInstance[IdentifierName].IsValid()
		&& DialogInstance[IdentifierName]->IsVisible())
	{
		DialogInstance[IdentifierName]->BringToFront();
		return;
	}

	DialogInstance.Add(IdentifierName, SNew(SFilterExpressionHelpDialog, MoveTemp(InConfig)));

	DialogInstance[IdentifierName]->GetOnWindowClosedEvent().AddLambda([IdentifierName](const TSharedRef<SWindow>& InWindow)
		{
			DialogInstance[IdentifierName].Reset();
		});

	TSharedPtr<SWindow> ParentWindow;
	if (FModuleManager::Get().IsModuleLoaded(TEXT("MainFrame")))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		ParentWindow = MainFrame.GetParentWindow();
	}

	if (ParentWindow.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(DialogInstance[IdentifierName].ToSharedRef(), ParentWindow.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(DialogInstance[IdentifierName].ToSharedRef());
	}
}

bool SFilterExpressionHelpDialog::IsOpen(const FName InName)
{
	return DialogInstance.Contains(InName) && DialogInstance[InName].IsValid();
}

void SFilterExpressionHelpDialog::CloseWindow(const FName InName)
{
	if (DialogInstance.Contains(InName) && DialogInstance[InName].IsValid())
	{
		DialogInstance[InName]->RequestDestroyWindow();
		DialogInstance[InName].Reset();

		DialogInstance.Remove(InName);
	}
}

void SFilterExpressionHelpDialog::Construct(const FArguments& InArgs, FFilterExpressionHelpDialogConfig&& InConfig)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Config = MoveTemp(InConfig);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	SWindow::Construct(SWindow::FArguments()
		.Title(Config.DialogTitle)
		.AutoCenter(EAutoCenter::PrimaryWorkArea)
		.SizingRule(ESizingRule::Autosized)
		.HasCloseButton(true)
		.SupportsMaximize(false)
		.SupportsMinimize(false));

	const TSharedRef<SVerticalBox> ContentWidget =
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.f)
		[
			ConstructDialogHeader()
		]
		+ SVerticalBox::Slot()
		.Padding(5.f)
		[
			SNew(SBox)
			.MaxDesiredWidth(Config.MaxDesiredWidth)
			.MaxDesiredHeight(Config.MaxDesiredHeight)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush(TEXT("Brushes.Recessed")))
				[
					ConstructExpressionWidgetList()
				]
			]
		];

	SetContent(ContentWidget);
}

TSharedRef<SWidget> SFilterExpressionHelpDialog::ConstructDialogHeader()
{
	const TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.Padding(0.f, 0.f, 20.f, 0.f)
		[
			SNew(STextBlock)
			.Font(FCoreStyle::GetDefaultFontStyle(TEXT("Bold"), 11))
			.Text(LOCTEXT("HeaderText", "Text Filter Expressions"))
		];

	if (!Config.DocumentationLink.IsEmpty())
	{
		HorizontalBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(12.f))
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush(TEXT("Icons.Documentation")))
			];

		HorizontalBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(4.f, 0.f, 0.f, 0.f)
			[
				SNew(SHyperlink)
				.Text(LOCTEXT("DocumentationLink", "Documentation"))
				.ToolTipText(FText::Format(LOCTEXT("NavigateToDocumentation", "Open the online documentation ({0})"), FText::FromString(Config.DocumentationLink)))
				.Style(FAppStyle::Get(), TEXT("HoverOnlyHyperlink"))
				.TextStyle(FAppStyle::Get(), TEXT("SmallText"))
				.OnNavigate(this, &SFilterExpressionHelpDialog::OpenDocumentationLink)
			];
	}

	return HorizontalBox;
}

TSharedRef<SWidget> SFilterExpressionHelpDialog::ConstructExpressionWidgetList()
{
	Config.ExpressionHelpEntries.Sort([](const FFilterExpressionHelpEntry& InA, const FFilterExpressionHelpEntry& InB)
		{
			const int32 CountA = InA.Keys.Num();
			if (CountA == 0)
			{
				return false;
			}

			const int32 CountB = InB.Keys.Num();
			if (CountB == 0)
			{
				return true;
			}

			return InA.Keys.Array()[0].LexicalLess(InB.Keys.Array()[0]);
		});

	const TSharedRef<SScrollBox> Container = SNew(SScrollBox);

	bool bIsFirst = true;

	for (const FFilterExpressionHelpEntry& HelpEntry : Config.ExpressionHelpEntries)
	{
		if (bIsFirst)
		{
			bIsFirst = false;
		}
		else
		{
			Container->AddSlot()
				.AutoSize()
				[
					SNew(SSeparator)
				];
		}

		Container->AddSlot()
			.AutoSize()
			.Padding(0.f, 0.f, 5.f, 0.f)
			[
				ConstructExpressionWidget(HelpEntry)
			];
	}

	return Container;
}

TSharedRef<SWidget> SFilterExpressionHelpDialog::ConstructExpressionWidget(const FFilterExpressionHelpEntry& InHelpEntry)
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				ConstructKeysWidget(InHelpEntry.Keys)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.f, 0.f, 1.f, 0.f)
			[
				ConstructValueWidget(InHelpEntry.ValueType)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(15.f, 0.f, 5.f, 5.f)
		[
			SNew(STextBlock)
			.AutoWrapText(true)
			.TextStyle(FAppStyle::Get(), TEXT("NormalText"))
			.Text(InHelpEntry.Description)
		];
}

TSharedRef<SWidget> SFilterExpressionHelpDialog::ConstructKeysWidget(const TSet<FName>& InKeys)
{
	const TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	for (const FName KeyName : InKeys)
	{
		const FString KeyNameString = KeyName.ToString();

		if (HorizontalBox->GetChildren()->Num() > 0)
		{
			HorizontalBox->AddSlot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("KeySeparator", " | "))
				];
		}

		HorizontalBox->AddSlot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.ColorAndOpacity(KeyColor)
				.Text(FText::FromName(KeyName))
			];
	}

	return HorizontalBox;
}

TSharedRef<SWidget> SFilterExpressionHelpDialog::ConstructValueWidget(const ESequencerTextFilterValueType InValueType)
{
	static const FText LessThanText = LOCTEXT("CommaLT", "<");
	static const FText GreaterThanText = LOCTEXT("CommaGT", ">");

	const TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	auto AddValueTypeSlot = [&HorizontalBox](const FText& InText, const TOptional<FSlateColor> InColor = TOptional<FSlateColor>())
	{
		const TSharedRef<STextBlock> TextWidget = SNew(STextBlock)
			.Text(InText);

		if (InColor.IsSet())
		{
			TextWidget->SetColorAndOpacity(InColor.GetValue());
		}

		HorizontalBox->AddSlot()
			.AutoWidth()
			[
				TextWidget
			];
	};

	switch (InValueType)
	{
	case ESequencerTextFilterValueType::String:
		AddValueTypeSlot(LessThanText);
		AddValueTypeSlot(LOCTEXT("StringValue", "String"), ValueColor);
		AddValueTypeSlot(GreaterThanText);
		break;
	case ESequencerTextFilterValueType::Boolean:
		AddValueTypeSlot(LessThanText);
		AddValueTypeSlot(LOCTEXT("TrueValue", "True"), ValueColor);
		AddValueTypeSlot(LOCTEXT("Slash", "/"));
		AddValueTypeSlot(LOCTEXT("FalseValue", "False"), ValueColor);
		AddValueTypeSlot(GreaterThanText);
		break;
	case ESequencerTextFilterValueType::Integer:
		AddValueTypeSlot(LessThanText);
		AddValueTypeSlot(LOCTEXT("IntegerValue", "###"), ValueColor);
		AddValueTypeSlot(GreaterThanText);
		break;
	}

	return HorizontalBox;
}

void SFilterExpressionHelpDialog::OpenDocumentationLink() const
{
	if (!Config.DocumentationLink.IsEmpty())
	{
		FPlatformProcess::LaunchURL(*Config.DocumentationLink, nullptr, nullptr);
	}
}

#undef LOCTEXT_NAMESPACE
