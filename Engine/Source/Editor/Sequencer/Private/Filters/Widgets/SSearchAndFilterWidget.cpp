// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSearchAndFilterWidget.h"

#include "Filters/Menus/SequencerViewOptionsMenu.h"
#include "Filters/SequencerFilterBar.h"
#include "Filters/SequencerFilterBar.h"
#include "Filters/ViewModels/HideIsolateShowViewModel.h"
#include "Filters/Widgets/SFilterBarIsolateHideShow.h"
#include "Filters/Widgets/SSequencerSearchBox.h"
#include "SPositiveActionButton.h"
#include "SSequencerFilterBar.h"
#include "Sequencer.h"
#include "Filters/Utils/FilterViewUtils.h"

#define LOCTEXT_NAMESPACE "SSearchAndFilterRow"

namespace UE::Sequencer
{
void SSearchAndFilterWidget::Construct(
	const FArguments& InArgs, 
	const TSharedRef<ISequencerTrackFilters>& InFilterBar,
	const TSharedRef<FHideIsolateShowViewModel>& InIsolateHideViewModel
	)
{
	FilterBar = InFilterBar;
	OnSearchTextChanged = InArgs._OnSearchTextChanged;
	
	ChildSlot
	[
		SNew(SHorizontalBox)

		// Advanced Search Filter Combo Button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.f, 0.f, 0.f, 0.f)
		[
			MakeAddFilterButton(InFilterBar, InArgs._OnMakeAddFilterMenuContent)
		]

		// Injected content, e.g. linked filter state toggle button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.f, 0.f, 0.f, 0.f)
		[
			InArgs._BeforeSearchBox.Widget
		]

		// Advanced Search Box
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(2.f, 0.f, 0.f, 0.f)
		[
			SAssignNew(SearchBox, SSequencerSearchBox, InFilterBar)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("SequencerFilterSearch")))
			.HintText(LOCTEXT("FilterSearch", "Search..."))
			.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search"))
			.OnTextChanged(this, &SSearchAndFilterWidget::OnOutlinerSearchChanged)
			.OnTextCommitted(this, &SSearchAndFilterWidget::OnOutlinerSearchCommitted)
			.OnSaveSearchClicked(this, &SSearchAndFilterWidget::OnOutlinerSearchSaved)
		]

		// Isolate / Hide / Show
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.f, 0.f, 0.f, 0.f)
		[
			SNew(SFilterBarIsolateHideShow, InIsolateHideViewModel)
		]
	];
}

void SSearchAndFilterWidget::SetSearchText(const FText& InSearchText)
{
	OnSearchTextChanged.ExecuteIfBound(InSearchText);
}

void SSearchAndFilterWidget::OnOutlinerSearchChanged(const FText& InSearchText)
{
	SetSearchText(InSearchText);
}

void SSearchAndFilterWidget::OnOutlinerSearchCommitted(const FText& InSearchText, ETextCommit::Type Arg)
{
	SetSearchText(InSearchText);
}

void SSearchAndFilterWidget::OnOutlinerSearchSaved(const FText& InSearchText)
{
	FCustomTextFilterData CustomTextFilterData;
	CustomTextFilterData.FilterLabel = LOCTEXT("NewFilterName", "New Filter Name");
	CustomTextFilterData.FilterString = InSearchText;
	SSequencerCustomTextFilterDialog::CreateWindow_AddCustomTextFilter(FilterBar.ToSharedRef(), CustomTextFilterData);
}
} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE