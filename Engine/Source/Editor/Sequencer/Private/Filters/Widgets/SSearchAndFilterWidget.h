// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelPtr.h"
#include "SSequencerCustomTextFilterDialog.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class ISequencerTrackFilters;
class SSequencerSearchBox;

namespace UE::Sequencer
{
class FHideIsolateShowViewModel;
class IOutlinerExtension;

/** 
 * Contains widgets for searching and filtering a tree view displaying IOutlinerExtensions. 
 * This view is intentionally engineered to be used independent of SSequencer.
 */
class SSearchAndFilterWidget : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SSearchAndFilterWidget){}
		/** Invoked when the search text changes. */
		SLATE_EVENT(FOnTextChanged, OnSearchTextChanged)

		/** Makes the content for the add filter combo button. */
		SLATE_EVENT(FOnGetContent, OnMakeAddFilterMenuContent)

		/** Slot between "Add filter" combo and search box. Useful for injecting e.g. linked filter toggle button. */
		SLATE_NAMED_SLOT(FArguments, BeforeSearchBox)
	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs, 
		const TSharedRef<ISequencerTrackFilters>& InFilterBar,
		const TSharedRef<FHideIsolateShowViewModel>& InIsolateHideViewModel
		);
	
	/** @return The search box */
	TSharedPtr<SSequencerSearchBox> GetSearchBox() const { return SearchBox; }

private:
	
	/** The filter bar that is being displayed. */
	TSharedPtr<ISequencerTrackFilters> FilterBar;
	
	/** Search box with which the user affects the text filter. */
	TSharedPtr<SSequencerSearchBox> SearchBox;
	
	/** Invoked when the search text changes. */
	FOnTextChanged OnSearchTextChanged;
	
	void SetSearchText(const FText& InSearchText);
	
	void OnOutlinerSearchChanged(const FText& InSearchText);
	void OnOutlinerSearchCommitted(const FText& InSearchText, ETextCommit::Type Arg);
	void OnOutlinerSearchSaved(const FText& InSearchText);
};
} // namespace UE::Sequencer

