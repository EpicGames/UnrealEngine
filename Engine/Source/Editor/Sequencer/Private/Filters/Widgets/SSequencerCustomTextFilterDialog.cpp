// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerCustomTextFilterDialog.h"
#include "Sequencer.h"
#include "SPrimaryButton.h"
#include "Filters/Filters/SequencerTrackFilter_CustomText.h"
#include "Filters/ISequencerTrackFilters.h"
#include "Interfaces/IMainFrameModule.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "SSequencerCustomTextFilterDialog"

TSharedPtr<SSequencerCustomTextFilterDialog> SSequencerCustomTextFilterDialog::DialogInstance;

void SSequencerCustomTextFilterDialog::Construct(const FArguments& InArgs, const TSharedRef<ISequencerTrackFilters>& InFilterBar)
{
	WeakFilterBar = InFilterBar;

	CustomTextFilter = InArgs._CustomTextFilter;
	CustomTextFilterData = InArgs._CustomTextFilterData;

	if (CustomTextFilter.IsValid())
	{
		CustomTextFilterData = CustomTextFilter->CreateCustomTextFilterData();
	}

	InitialFilterSet = FilterSet;
	InitialCustomTextFilterData = CustomTextFilterData;

	const FText WindowTitle = CustomTextFilter.IsValid()
		? LOCTEXT("ModifyCustomTextFilterWindow", "Modify Custom Filter")
		: LOCTEXT("CreateCustomTextFilterWindow", "Create Custom Filter");

	SWindow::Construct(SWindow::FArguments()
		.Title(WindowTitle)
		.HasCloseButton(true)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.SizingRule(ESizingRule::Autosized)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("Brushes.Panel")))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(20.f, 40.f, 20.f, 0)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						ConstructFilterLabelRow()
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						ConstructFilterColorRow()
					]
				]
				+ SVerticalBox::Slot()
				.Padding(20.f, 20.f, 20.f, 0)
				.AutoHeight()
				[
					ConstructFilterStringRow()
				]
				+ SVerticalBox::Slot()
				.Padding(20.f, 40.f, 20.f, 20.f)
				.AutoHeight()
				[
					ConstructButtonRow()
				]
			]
		]);

	OnWindowClosed.BindSP(this, &SSequencerCustomTextFilterDialog::HandleWindowClosed);
}

void SSequencerCustomTextFilterDialog::CreateWindow_AddCustomTextFilter(const TSharedRef<ISequencerTrackFilters>& InFilterBar
	, const FCustomTextFilterData& InCustomTextFilterData
	, const TSharedPtr<SWindow> InParentWindow)
{
	if (DialogInstance.IsValid() && DialogInstance->IsVisible())
	{
		DialogInstance->BringToFront();
		return;
	}

	DialogInstance = SNew(SSequencerCustomTextFilterDialog, InFilterBar)
		.CustomTextFilter(nullptr)
		.CustomTextFilterData(InCustomTextFilterData);

	ShowWindow(DialogInstance.ToSharedRef(), true, InParentWindow);
}

void SSequencerCustomTextFilterDialog::CreateWindow_EditCustomTextFilter(const TSharedRef<ISequencerTrackFilters>& InFilterBar
	, TSharedPtr<FSequencerTrackFilter_CustomText> InCustomTextFilter
	, const TSharedPtr<SWindow> InParentWindow)
{
	if (DialogInstance.IsValid() && DialogInstance->IsVisible())
	{
		DialogInstance->BringToFront();
		return;
	}

	DialogInstance = SNew(SSequencerCustomTextFilterDialog, InFilterBar)
		.CustomTextFilter(InCustomTextFilter);

	ShowWindow(DialogInstance.ToSharedRef(), true, InParentWindow);
}

bool SSequencerCustomTextFilterDialog::IsOpen()
{
	return DialogInstance.IsValid();
}

void SSequencerCustomTextFilterDialog::CloseWindow()
{
	if (DialogInstance.IsValid())
	{
		DialogInstance->RequestDestroyWindow();
		DialogInstance.Reset();
	}
}

void SSequencerCustomTextFilterDialog::ShowWindow(const TSharedRef<SWindow>& InWindowToShow, const bool bInModal, const TSharedPtr<SWindow>& InParentWindow)
{
	if (bInModal)
	{
		TSharedPtr<SWidget> ParentWidget = InParentWindow;

		if (!ParentWidget.IsValid() && FModuleManager::Get().IsModuleLoaded(TEXT("MainFrame")))
		{
			IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
			ParentWidget = MainFrameModule.GetParentWindow();
		}

		FSlateApplication::Get().AddModalWindow(InWindowToShow, ParentWidget);
	}
	else
	{
		if (InParentWindow.IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(InWindowToShow, InParentWindow.ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(InWindowToShow);
		}
	}
}

TSharedRef<SWidget> SSequencerCustomTextFilterDialog::ConstructContentRow(const FText& InLabel, const TSharedRef<SWidget>& InContentWidget)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(120.f)
			[
				SNew(STextBlock)
				.Text(InLabel)
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(560.f)
			[
				InContentWidget
			]
		];
}

TSharedRef<SWidget> SSequencerCustomTextFilterDialog::ConstructFilterLabelRow()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(120)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FilterLabelText", "Filter Label"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(254)
			[
				SAssignNew(FilterLabelTextBox, SEditableTextBox)
				.Text_Lambda([this]()
					{
						return CustomTextFilterData.FilterLabel;
					})
				.OnTextChanged_Lambda([this](const FText& InText)
					{
						CustomTextFilterData.FilterLabel = InText;
					})
			]
		];
}

TSharedRef<SWidget> SSequencerCustomTextFilterDialog::ConstructFilterColorRow()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(40, 0, 0, 0)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("FilterColorText", "Color"))
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(20, 0, 0, 0)
		[
			SNew(SColorBlock)
			.Color_Lambda([this]()
				{
					return CustomTextFilterData.FilterColor;
				})
			.CornerRadius(FVector4(4.f, 4.f, 4.f, 4.f))
			.Size(FVector2D(70.f, 22.f))
			.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
			.OnMouseButtonDown(this, &SSequencerCustomTextFilterDialog::OnColorBlockMouseButtonDown)
		];
}

TSharedRef<SWidget> SSequencerCustomTextFilterDialog::ConstructFilterStringRow()
{
	const TSharedRef<SEditableTextBox> TextStringTextBox = SNew(SEditableTextBox)
		.Text_Lambda([this]()
			{
				return CustomTextFilterData.FilterString;
			})
		.OnTextChanged_Lambda([this](const FText& InText)
			{
				CustomTextFilterData.FilterString = InText;
			});

	return ConstructContentRow(LOCTEXT("TextFilterString", "Text Filter String"), TextStringTextBox);
}

TSharedRef<SWidget> SSequencerCustomTextFilterDialog::ConstructButtonRow()
{
	const TSharedRef<SHorizontalBox> ButtonBox = SNew(SHorizontalBox);

	if (CustomTextFilter.IsValid())
	{
		ButtonBox->AddSlot()
		.FillWidth(1.f)
		.HAlign(HAlign_Right)
		.Padding(0.f, 0.f, 16.f, 0.f)
		[
			SNew(SPrimaryButton)
			.Text(LOCTEXT("ModifyFilterButton", "Save"))
			.OnClicked(this, &SSequencerCustomTextFilterDialog::OnSaveButtonClick)
		];

		ButtonBox->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.Padding(0.f, 0.f, 16.f, 0.f)
		[
			SNew(SButton)
			.Text(LOCTEXT("DeleteButton", "Delete"))
			.OnClicked(this, &SSequencerCustomTextFilterDialog::OnDeleteButtonClick)
		];
	}
	else
	{
		ButtonBox->AddSlot()
		.FillWidth(1.f)
		.HAlign(HAlign_Right)
		.Padding(0.f, 0.f, 16.f, 0.f)
		[
			SNew(SButton)
			.Text(LOCTEXT("CreateAndApplyButton", "Create and Apply"))
			.OnClicked(this, &SSequencerCustomTextFilterDialog::OnCreateButtonClick, true)
		];

		ButtonBox->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.Padding(0.f, 0.f, 16.f, 0.f)
		[
			SNew(SPrimaryButton)
			.Text(LOCTEXT("CreateButton", "Create"))
			.OnClicked(this, &SSequencerCustomTextFilterDialog::OnCreateButtonClick, false)
		];
	}

	// Button to close the dialog box, common to both modes
	ButtonBox->AddSlot()
	.AutoWidth()
	.HAlign(HAlign_Right)
	[
		SNew(SButton)
		.Text(LOCTEXT("CancelButton", "Cancel"))
		.OnClicked(this, &SSequencerCustomTextFilterDialog::OnCancelButtonClick)
	];

	return ButtonBox;
}

bool SSequencerCustomTextFilterDialog::CheckFilterNameValidity() const
{
	const TSharedPtr<ISequencerTrackFilters> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return false;
	}

	USequencerSettings* const SequencerSettings = FilterBar->GetSequencer().GetSequencerSettings();
	if (!IsValid(SequencerSettings))
	{
		return false;
	}

	if (CustomTextFilterData.FilterLabel.IsEmpty())
	{
		FilterLabelTextBox->SetError(LOCTEXT("EmptyFilterLabelError", "Filter Label cannot be empty"));
		return false;
	}

	FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(FilterBar->GetIdentifier(), true);

	const TArray<FCustomTextFilterData> CustomTextFilterDatas = Config.GetCustomTextFilters();

	// Check for duplicate filter labels
	for (const FCustomTextFilterData& Data : CustomTextFilterDatas)
	{
		/* Special Case: If we are editing a filter and don't change the filter label, it will be considered a duplicate of itself!
		 * To prevent this we check against the original filter label if we are in edit mode */
		if (Data.FilterLabel.EqualTo(CustomTextFilterData.FilterLabel)
			&& !(CustomTextFilter.IsValid() && Data.FilterLabel.EqualTo(InitialCustomTextFilterData.FilterLabel)))
		{
			FilterLabelTextBox->SetError(LOCTEXT("DuplicateFilterLabelError", "A filter with this label already exists!"));
			return false;
		}
	}

	return true;
}

FReply SSequencerCustomTextFilterDialog::OnColorBlockMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		const FOnLinearColorValueChanged ColorValueChangedDelegate = FOnLinearColorValueChanged::CreateLambda([this](const FLinearColor& InNewColor)
			{
				CustomTextFilterData.FilterColor = InNewColor;
			});

		FColorPickerArgs PickerArgs = FColorPickerArgs(CustomTextFilterData.FilterColor, ColorValueChangedDelegate);
		PickerArgs.bIsModal = true;
		PickerArgs.ParentWidget = SharedThis(this);

		OpenColorPicker(PickerArgs);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SSequencerCustomTextFilterDialog::OnCreateCustomTextFilter(const bool bInApplyFilter)
{
	const TSharedPtr<ISequencerTrackFilters> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	USequencerSettings* const SequencerSettings = FilterBar->GetSequencer().GetSequencerSettings();
	if (!IsValid(SequencerSettings))
	{
		return;
	}

	const TSharedRef<FSequencerTrackFilter_CustomText> NewCustomTextFilter = MakeShared<FSequencerTrackFilter_CustomText>(*FilterBar.Get());
	NewCustomTextFilter->SetFromCustomTextFilterData(CustomTextFilterData);

	const TSharedPtr<FSequencerTrackFilter> NewFilter = StaticCastSharedPtr<FSequencerTrackFilter>(NewCustomTextFilter->GetFilter());
	if (!NewFilter.IsValid())
	{
		return;
	}

	FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(FilterBar->GetIdentifier(), false);
	Config.AddCustomTextFilter(CustomTextFilterData);
	SequencerSettings->SaveConfig();

	FilterBar->AddCustomTextFilter(NewCustomTextFilter, false);

	if (bInApplyFilter)
	{
		FilterBar->SetTextFilterString(TEXT(""));
		FilterBar->SetFilterActive(NewFilter.ToSharedRef(), true, true);
	}
	else
	{
		FilterBar->SetFilterEnabled(NewFilter.ToSharedRef(), true, true);
	}

	RequestDestroyWindow();
}

void SSequencerCustomTextFilterDialog::OnModifyCustomTextFilter()
{
	const TSharedPtr<ISequencerTrackFilters> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	USequencerSettings* const SequencerSettings = FilterBar->GetSequencer().GetSequencerSettings();
	if (!IsValid(SequencerSettings))
	{
		return;
	}

	const TSharedRef<FSequencerTrackFilter_CustomText> CustomTextFilterRef = CustomTextFilter.ToSharedRef();
	const bool bWasFilterEnabled = FilterBar->IsFilterEnabled(CustomTextFilterRef);
	const bool bWasFilterActive = FilterBar->IsFilterActive(CustomTextFilterRef);
	const FString OldFilterName = CustomTextFilterRef->GetDisplayName().ToString();

	FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(FilterBar->GetIdentifier(), false);
	Config.RemoveCustomTextFilter(OldFilterName);

	if (CustomTextFilter.IsValid())
	{
		CustomTextFilter->SetFromCustomTextFilterData(CustomTextFilterData);
	}

	Config.AddCustomTextFilter(CustomTextFilterData);
	SequencerSettings->SaveConfig();

	FilterBar->RemoveCustomTextFilter(CustomTextFilterRef, false);
	FilterBar->AddCustomTextFilter(CustomTextFilterRef, false);

	const FString NewFilterName = CustomTextFilterData.FilterLabel.ToString();

	if (bWasFilterActive)
	{
		FilterBar->SetFilterActiveByDisplayName(NewFilterName, true, true);
	}
	else if (bWasFilterEnabled)
	{
		FilterBar->SetFilterEnabledByDisplayName(NewFilterName, true, true);
	}

	RequestDestroyWindow();
}

FReply SSequencerCustomTextFilterDialog::OnCreateButtonClick(const bool bInApply)
{
	if (CheckFilterNameValidity())
	{
		OnCreateCustomTextFilter(bInApply);
	}

	return FReply::Handled();
}

FReply SSequencerCustomTextFilterDialog::OnSaveButtonClick()
{
	if (CheckFilterNameValidity())
	{
		OnModifyCustomTextFilter();
	}

	return FReply::Handled();
}

FReply SSequencerCustomTextFilterDialog::OnDeleteButtonClick()
{
	if (!CustomTextFilter.IsValid())
	{
		return FReply::Handled();
	}

	const TSharedPtr<ISequencerTrackFilters> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return FReply::Handled();
	}

	USequencerSettings* const SequencerSettings = FilterBar->GetSequencer().GetSequencerSettings();
	if (!IsValid(SequencerSettings))
	{
		return FReply::Handled();
	}

	FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(FilterBar->GetIdentifier(), false);
	Config.RemoveCustomTextFilter(CustomTextFilterData.FilterLabel.ToString());
	SequencerSettings->SaveConfig();

	FilterBar->RemoveCustomTextFilter(CustomTextFilter.ToSharedRef(), false);

	RequestDestroyWindow();

	return FReply::Handled();
}

FReply SSequencerCustomTextFilterDialog::OnCancelButtonClick()
{
	RequestDestroyWindow();

	return FReply::Handled();
}

void SSequencerCustomTextFilterDialog::HandleWindowClosed(const TSharedRef<SWindow>& InWindow)
{
	DialogInstance.Reset();
}

#undef LOCTEXT_NAMESPACE
