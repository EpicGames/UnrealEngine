// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/CustomTextFilters.h"
#include "Filters/SequencerFilterBarConfig.h"
#include "Widgets/SWindow.h"

class ISequencerTrackFilters;
class FSequencerTrackFilter_CustomText;

class SSequencerCustomTextFilterDialog : public SWindow
{
public:
	static void CreateWindow_AddCustomTextFilter(const TSharedRef<ISequencerTrackFilters>& InFilterBar
		, const FCustomTextFilterData& InCustomTextFilterData = FCustomTextFilterData()
		, const TSharedPtr<SWindow> InParentWindow = nullptr);

	static void CreateWindow_EditCustomTextFilter(const TSharedRef<ISequencerTrackFilters>& InFilterBar
		, TSharedPtr<FSequencerTrackFilter_CustomText> InCustomTextFilter
		, const TSharedPtr<SWindow> InParentWindow = nullptr);

	static bool IsOpen();

	static void CloseWindow();

	SLATE_BEGIN_ARGS(SSequencerCustomTextFilterDialog)
		: _CustomTextFilter(nullptr)
	{}
		SLATE_ARGUMENT(TSharedPtr<FSequencerTrackFilter_CustomText>, CustomTextFilter)
		SLATE_ARGUMENT(FCustomTextFilterData, CustomTextFilterData)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<ISequencerTrackFilters>& InFilterBar);

private:
	static void ShowWindow(const TSharedRef<SWindow>& InWindowToShow, const bool bInModal, const TSharedPtr<SWindow>& InParentWindow = nullptr);

	static TSharedPtr<SSequencerCustomTextFilterDialog> DialogInstance;

	TSharedRef<SWidget> ConstructContentRow(const FText& InLabel, const TSharedRef<SWidget>& InContentWidget);
	TSharedRef<SWidget> ConstructFilterLabelRow();
	TSharedRef<SWidget> ConstructFilterColorRow();
	TSharedRef<SWidget> ConstructFilterStringRow();
	TSharedRef<SWidget> ConstructButtonRow();

	bool CheckFilterNameValidity() const;

	FReply OnColorBlockMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent);

	void OnCreateCustomTextFilter(const bool bInApplyFilter);
	void OnModifyCustomTextFilter();

	FReply OnCreateButtonClick(const bool bInApply);
	FReply OnSaveButtonClick();
	FReply OnDeleteButtonClick();
	FReply OnCancelButtonClick();

	void HandleWindowClosed(const TSharedRef<SWindow>& InWindow);

	TWeakPtr<ISequencerTrackFilters> WeakFilterBar;

	TSharedPtr<FSequencerTrackFilter_CustomText> CustomTextFilter;

	TSharedPtr<SEditableTextBox> FilterLabelTextBox;

	FSequencerFilterSet InitialFilterSet;
	FCustomTextFilterData InitialCustomTextFilterData;

	FSequencerFilterSet FilterSet;
	FCustomTextFilterData CustomTextFilterData;
};
