// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Widgets/SSequencerFilter.h"
#include "Filters/Widgets/SSequencerFilterCheckBox.h"

#define LOCTEXT_NAMESPACE "SSequencerFilter"

void SSequencerFilter::Construct(const FArguments& InArgs)
{
	IsFilterActiveDelegate = InArgs._OnIsFilterActive;

	FilterToggleDelegate = InArgs._OnFilterToggle;
	CtrlClickDelegate = InArgs._OnCtrlClick;
	AltClickDelegate = InArgs._OnAltClick;
	MiddleClickDelegate = InArgs._OnMiddleClick;
	DoubleClickDelegate = InArgs._OnDoubleClick;

	DisplayName = InArgs._DisplayName;
	ToolTipText = InArgs._ToolTipText;
	BlockColor = InArgs._BlockColor;

	GetMenuContentDelegate = InArgs._OnGetMenuContent;

	TSharedPtr<SWidget> ContentWidget;
	FName BrushName;

	switch(InArgs._FilterPillStyle)
	{
	case EFilterPillStyle::Basic:
		{
			ContentWidget = ConstructBasicFilterWidget();
			BrushName = TEXT("FilterBar.BasicFilterButton");
			break;
		}
	case EFilterPillStyle::Default:
	default:
		{
			ContentWidget = ConstructDefaultFilterWidget();
			BrushName = TEXT("FilterBar.FilterButton");
			break;
		}
	}

	ChildSlot
	[
		SAssignNew(ToggleButtonPtr, SSequencerFilterCheckBox)
		.Style(FAppStyle::Get(), BrushName)
		.ToolTipText(ToolTipText)
		.IsChecked(this, &SSequencerFilter::IsChecked)
		.OnCheckStateChanged(this, &SSequencerFilter::OnFilterToggled)
		.CheckBoxContentUsesAutoWidth(false)
		.OnGetMenuContent(this, &SSequencerFilter::GetRightClickMenuContent)
		[
			ContentWidget.ToSharedRef()
		]
	];

	ToggleButtonPtr->SetOnCtrlClick(FOnClicked::CreateSP(this, &SSequencerFilter::OnFilterCtrlClick));
	ToggleButtonPtr->SetOnAltClick(FOnClicked::CreateSP(this, &SSequencerFilter::OnFilterAltClick));
	ToggleButtonPtr->SetOnMiddleButtonClick(FOnClicked::CreateSP(this, &SSequencerFilter::OnFilterMiddleButtonClick));
	ToggleButtonPtr->SetOnDoubleClick(FOnClicked::CreateSP(this, &SSequencerFilter::OnFilterDoubleClick));
}

TSharedRef<SWidget> SSequencerFilter::ConstructBasicFilterWidget()
{
	return SNew(STextBlock)
		.Margin(0.f)
		.TextStyle(FAppStyle::Get(), TEXT("SmallText"))
		.Text(DisplayName);
}

TSharedRef<SWidget> SSequencerFilter::ConstructDefaultFilterWidget()
{
	return SNew(SBorder)
		.Padding(1.f)
		.BorderImage(FAppStyle::Get().GetBrush(TEXT("FilterBar.FilterBackground")))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(8, 16)) // was 22
				.Image(FAppStyle::Get().GetBrush(TEXT("FilterBar.FilterImage")))
				.ColorAndOpacity(this, &SSequencerFilter::GetFilterImageColorAndOpacity)
			]
			+ SHorizontalBox::Slot()
			.Padding(TAttribute<FMargin>(this, &SSequencerFilter::GetFilterNamePadding))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(FCoreStyle::GetDefaultFontStyle(TEXT("Regular"), 8))
				.Text(DisplayName)
				.IsEnabled(this, &SSequencerFilter::IsButtonEnabled)
			]
		];
}

bool SSequencerFilter::IsActive() const
{
	if (IsFilterActiveDelegate.IsBound())
	{
		return IsFilterActiveDelegate.Execute();
	}
	return false;
}

void SSequencerFilter::OnFilterToggled(const ECheckBoxState NewState)
{
	if (FilterToggleDelegate.IsBound())
	{
		FilterToggleDelegate.Execute(NewState);
	}
}

FReply SSequencerFilter::OnFilterCtrlClick()
{
	if (CtrlClickDelegate.IsBound())
	{
		CtrlClickDelegate.Execute();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SSequencerFilter::OnFilterAltClick()
{
	if (AltClickDelegate.IsBound())
	{
		AltClickDelegate.Execute();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SSequencerFilter::OnFilterMiddleButtonClick()
{
	if (MiddleClickDelegate.IsBound())
	{
		MiddleClickDelegate.Execute();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SSequencerFilter::OnFilterDoubleClick()
{
	if (DoubleClickDelegate.IsBound())
	{
		DoubleClickDelegate.Execute();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedRef<SWidget> SSequencerFilter::GetRightClickMenuContent()
{
	if (GetMenuContentDelegate.IsBound())
	{
		return GetMenuContentDelegate.Execute();
	}
	return SNullWidget::NullWidget;
}

ECheckBoxState SSequencerFilter::IsChecked() const
{
	return IsActive() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FSlateColor SSequencerFilter::GetFilterImageColorAndOpacity() const
{
	return BlockColor.Get(FSlateColor());
}

EVisibility SSequencerFilter::GetFilterOverlayVisibility() const
{
	return IsActive() ? EVisibility::Collapsed : EVisibility::HitTestInvisible;
}

FMargin SSequencerFilter::GetFilterNamePadding() const
{
	return ToggleButtonPtr->IsPressed() ? FMargin(3, 1, 3, 0) : FMargin(3, 0, 3, 0);
}

bool SSequencerFilter::IsButtonEnabled() const
{
	if (IsFilterActiveDelegate.IsBound())
	{
		return IsFilterActiveDelegate.Execute();
	}
	return true;
}

#undef LOCTEXT_NAMESPACE
