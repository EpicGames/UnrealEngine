// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Widgets/SSequencerFilterBar.h"
#include "Filters/Filters/SequencerTrackFilter_CustomText.h"
#include "Filters/Menus/SequencerFilterBarContextMenu.h"
#include "Filters/Menus/SequencerTrackFilterContextMenu.h"
#include "Filters/SequencerFilterBar.h"
#include "Filters/Widgets/SFilterBarClippingHorizontalBox.h"
#include "Filters/Widgets/SFilterExpressionHelpDialog.h"
#include "Filters/Widgets/SSequencerFilter.h"
#include "Filters/Widgets/SSequencerSearchBox.h"
#include "MovieScene.h"
#include "Sequencer.h"
#include "SequencerLog.h"
#include "SSequencerCustomTextFilterDialog.h"

#define LOCTEXT_NAMESPACE "SSequencerFilterBar"

SSequencerFilterBar::~SSequencerFilterBar()
{
	if (const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin())
	{
		FilterBar->OnFiltersChanged().RemoveAll(this);
	}

	if (SSequencerCustomTextFilterDialog::IsOpen())
	{
		SSequencerCustomTextFilterDialog::CloseWindow();
	}

	if (TextExpressionHelpDialog.IsValid())
	{
		TextExpressionHelpDialog->RequestDestroyWindow();
		TextExpressionHelpDialog.Reset();
	}
}

void SSequencerFilterBar::Construct(const FArguments& InArgs, const TSharedRef<FSequencerFilterBar>& InFilterBar)
{
	WeakFilterBar = InFilterBar;

	FilterPillStyle = InArgs._FilterPillStyle;

	ContextMenu = MakeShared<FSequencerFilterBarContextMenu>();

	HorizontalContainerWidget = SNew(SFilterBarClippingHorizontalBox)
		.OnWrapButtonClicked(FOnGetContent::CreateSP(this, &SSequencerFilterBar::OnWrapButtonClicked))
		.IsFocusable(false);

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			HorizontalContainerWidget.ToSharedRef()
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			HorizontalContainerWidget->CreateWrapButton()
		]
	];
	
	CreateFilterWidgetsFromConfig();

	InFilterBar->OnFiltersChanged().AddSP(this, &SSequencerFilterBar::OnFiltersChanged);
	InFilterBar->OnMuteFiltersChanged().AddSP(this, &SSequencerFilterBar::OnMuteStateChanged);
	
	OnMuteStateChanged(InFilterBar->AreFiltersMuted());
}

FReply SSequencerFilterBar::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return FReply::Unhandled();
	}

	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		const FWidgetPath* EventPath = MouseEvent.GetEventPath();

		FSlateApplication::Get().PushMenu(AsShared()
			, EventPath ? *EventPath : FWidgetPath()
			, ContextMenu->CreateMenu(FilterBar.ToSharedRef())
			, MouseEvent.GetScreenSpacePosition()
			, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

TSharedPtr<FSequencerFilterBar> SSequencerFilterBar::GetFilterBar() const
{
	return WeakFilterBar.Pin();
}

bool SSequencerFilterBar::HasAnyFilterWidgets() const
{
	return FilterWidgets.Num() > 0;
}

void SSequencerFilterBar::AddWidgetToLayout(const TSharedRef<SWidget>& InWidget)
{
	HorizontalContainerWidget->AddSlot()
		.AutoWidth()
		.Padding(1)
		[
			InWidget
		];
}

void SSequencerFilterBar::RemoveWidgetFromLayout(const TSharedRef<SWidget>& InWidget)
{
	HorizontalContainerWidget->RemoveSlot(InWidget);
}

void SSequencerFilterBar::CreateAndAddFilterWidget(const TSharedRef<FSequencerTrackFilter>& InFilter)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const TSharedRef<FSequencerFilterBar> FilterBarRef = FilterBar.ToSharedRef();

	const TSharedRef<SSequencerFilter> NewFilterWidget = SNew(SSequencerFilter)
		.FilterPillStyle(FilterPillStyle)
		.DisplayName(this, &SSequencerFilterBar::GetFilterDisplayName, InFilter)
		.ToolTipText(InFilter, &FSequencerTrackFilter::GetToolTipText)
		.BlockColor(this, &SSequencerFilterBar::GetFilterBlockColor, InFilter)
		.OnIsFilterActive(FilterBarRef, &FSequencerFilterBar::IsFilterActive, InFilter)
		.OnFilterToggle(this, &SSequencerFilterBar::OnFilterToggle, InFilter)
		.OnAltClick(this, &SSequencerFilterBar::OnFilterAltClick, InFilter)
		.OnMiddleClick(this, &SSequencerFilterBar::OnFilterMiddleClick, InFilter)
		.OnDoubleClick(this, &SSequencerFilterBar::OnFilterDoubleClick, InFilter)
		.OnGetMenuContent(this, &SSequencerFilterBar::OnGetMenuContent, InFilter);

	AddFilterWidget(InFilter, NewFilterWidget);
}

FText SSequencerFilterBar::GetFilterDisplayName(const TSharedRef<FSequencerTrackFilter> InFilter) const
{
	return InFilter->GetDisplayName();
}

FSlateColor SSequencerFilterBar::GetFilterBlockColor(const TSharedRef<FSequencerTrackFilter> InFilter) const
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return FSlateColor();
	}

	if (!FilterBar->IsFilterActive(InFilter))
	{
		return FAppStyle::Get().GetSlateColor(TEXT("Colors.Recessed"));;
	}

	return InFilter->GetColor();
}

void SSequencerFilterBar::OnFilterToggle(const ECheckBoxState InNewState, const TSharedRef<FSequencerTrackFilter> InFilter)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const bool bNewActive = InNewState == ECheckBoxState::Checked;
	FilterBar->SetFilterActive(InFilter, bNewActive, true);
}

void SSequencerFilterBar::OnFilterCtrlClick(const TSharedRef<FSequencerTrackFilter> InFilter)
{
	ActivateAllButThis(false, InFilter);
}

void SSequencerFilterBar::OnFilterAltClick(const TSharedRef<FSequencerTrackFilter> InFilter)
{
	ActivateAllButThis(true, InFilter);
}

void SSequencerFilterBar::OnFilterMiddleClick(const TSharedRef<FSequencerTrackFilter> InFilter)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FilterBar->SetFilterEnabled(InFilter, false, true);
}

void SSequencerFilterBar::OnFilterDoubleClick(const TSharedRef<FSequencerTrackFilter> InFilter)
{
	ActivateAllButThis(false, InFilter);
}

TSharedRef<SWidget> SSequencerFilterBar::OnGetMenuContent(const TSharedRef<FSequencerTrackFilter> InFilter)
{
	FilterContextMenu = MakeShared<FSequencerTrackFilterContextMenu>();
	return FilterContextMenu->CreateMenuWidget(InFilter);
}

void SSequencerFilterBar::ActivateAllButThis(const bool bInActive, const TSharedRef<FSequencerTrackFilter> InFilter)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FilterBar->ActivateAllEnabledFilters(bInActive, {});
	FilterBar->SetFilterActive(InFilter, !bInActive, true);
}

void SSequencerFilterBar::AddFilterWidget(const TSharedRef<FSequencerTrackFilter>& InFilter, const TSharedRef<SSequencerFilter>& InFilterWidget)
{
	FilterWidgets.Add(InFilter, InFilterWidget);
	FilterWidgets.KeySort([](const TSharedRef<FSequencerTrackFilter>& InFilterA
		, const TSharedRef<FSequencerTrackFilter>& InFilterB)
		{
			return InFilterA->GetDisplayName().CompareTo(InFilterB->GetDisplayName()) < 0;
		});

	// Make sure the widgets are rendered in alphabetical order.
	HorizontalContainerWidget->ClearChildren();
	for (const TPair<TSharedRef<FSequencerTrackFilter>, TSharedRef<SSequencerFilter>>& Pair : FilterWidgets)
	{
		AddWidgetToLayout(Pair.Value);
	}
}

void SSequencerFilterBar::RemoveFilterWidget(const TSharedRef<FSequencerTrackFilter>& InFilter)
{
	if (!FilterWidgets.Contains(InFilter))
	{
		return;
	}

	RemoveWidgetFromLayout(FilterWidgets[InFilter]);

	FilterWidgets.Remove(InFilter);
}

void SSequencerFilterBar::RemoveAllFilterWidgets()
{
	for (const TPair<TSharedRef<FSequencerTrackFilter>, TSharedRef<SSequencerFilter>>& FilterPair : FilterWidgets)
	{
		RemoveWidgetFromLayout(FilterPair.Value);
	}

	FilterWidgets.Empty();
}

void SSequencerFilterBar::RemoveAllFilterWidgetsButThis(const TSharedRef<FSequencerTrackFilter>& InFilter)
{
	if (!FilterWidgets.Contains(InFilter))
	{
		return;
	}

	for (const TPair<TSharedRef<FSequencerTrackFilter>, TSharedRef<SSequencerFilter>>& FilterPair : FilterWidgets)
	{
		if (FilterPair.Key == InFilter)
		{
			continue;
		}

		RemoveWidgetFromLayout(FilterPair.Value);
	}

	const TSharedRef<SSequencerFilter> PreviousWidget = FilterWidgets[InFilter];

	FilterWidgets.Empty();

	AddFilterWidget(InFilter, PreviousWidget);
}

void SSequencerFilterBar::OnEnableAllGroupFilters(bool bEnableAll)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	UMovieSceneSequence* const FocusedMovieSequence = FilterBar->GetSequencer().GetFocusedMovieSceneSequence();
	if (!IsValid(FocusedMovieSequence))
	{
		return;
	}
	
	UMovieScene* const FocusedMovieScene = FocusedMovieSequence->GetMovieScene();
	if (!IsValid(FocusedMovieScene))
	{
		return;
	}
	
	for (UMovieSceneNodeGroup* const NodeGroup : FocusedMovieScene->GetNodeGroups())
	{
		NodeGroup->SetEnableFilter(bEnableAll);
	}
}

void SSequencerFilterBar::OnNodeGroupFilterClicked(UMovieSceneNodeGroup* NodeGroup)
{
	if (NodeGroup)
	{
		NodeGroup->SetEnableFilter(!NodeGroup->GetEnableFilter());
	}
}

UWorld* SSequencerFilterBar::GetWorld() const
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return nullptr;
	}

	UObject* const PlaybackContext = FilterBar->GetSequencer().GetPlaybackContext();
	if (!IsValid(PlaybackContext))
	{
		return nullptr;
	}

	return PlaybackContext->GetWorld();
}

void SSequencerFilterBar::OnFiltersChanged(const ESequencerFilterChange InChangeType, const TSharedRef<FSequencerTrackFilter>& InFilter)
{
	switch (InChangeType)
	{
	case ESequencerFilterChange::Enable:
	case ESequencerFilterChange::Activate:
		{
			if (!FilterWidgets.Contains(InFilter))
			{
				CreateAndAddFilterWidget(InFilter);
			}
			break;
		}
	case ESequencerFilterChange::Disable:
		{
			RemoveFilterWidget(InFilter);
			break;
		}
	case ESequencerFilterChange::Deactivate:
		{
			break;
		}
	};
}

void SSequencerFilterBar::OnMuteStateChanged(bool bNewIsMuted)
{
	const bool bIsEnabled = !bNewIsMuted;
	HorizontalContainerWidget->SetEnabled(bIsEnabled);
}

void SSequencerFilterBar::CreateFilterWidgetsFromConfig()
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	USequencerSettings* const SequencerSettings = FilterBar->GetSequencer().GetSequencerSettings();
	check(IsValid(SequencerSettings));

	constexpr bool bSaveConfig = false;
	const FName InstanceIdentifier = FilterBar->GetIdentifier();
	const FSequencerFilterBarConfig& Config = SequencerSettings->FindOrAddTrackFilterBar(InstanceIdentifier, bSaveConfig);

	RemoveAllFilterWidgets();

	const TSet<TSharedRef<FFilterCategory>> DisplayableCategories = FilterBar->GetConfigCategories();
	const auto LoadFilterFromConfig = [this, &Config, &DisplayableCategories](const TSharedRef<FSequencerTrackFilter>& InFilter)
	{
		const TSharedPtr<FFilterCategory> FilterCategory = InFilter->GetCategory();
		if (FilterCategory.IsValid() && !DisplayableCategories.Contains(FilterCategory.ToSharedRef()))
		{
			return;
		}

		const FString FilterName = InFilter->GetDisplayName().ToString();
		if (!Config.IsFilterEnabled(FilterName))
		{
			return;
		}

		if (!FilterWidgets.Contains(InFilter))
		{
			CreateAndAddFilterWidget(InFilter);
		}
	};

	for (const TSharedRef<FSequencerTrackFilter>& Filter : FilterBar->GetCommonFilters())
	{
		LoadFilterFromConfig(Filter);
	}

	for (const TSharedRef<FSequencerTrackFilter_CustomText>& Filter : FilterBar->GetAllCustomTextFilters())
	{
		LoadFilterFromConfig(Filter);
	}
}

TSharedRef<SWidget> SSequencerFilterBar::OnWrapButtonClicked()
{
	const TSharedRef<SVerticalBox> VerticalContainer = SNew(SVerticalBox);

	const int32 NumSlots = HorizontalContainerWidget->NumSlots();
	int32 SlotIndex = HorizontalContainerWidget->GetClippedIndex();
	for (; SlotIndex < NumSlots; ++SlotIndex)
	{
		SHorizontalBox::FSlot& Slot = HorizontalContainerWidget->GetSlot(SlotIndex);

		VerticalContainer->AddSlot()
			.AutoHeight()
			.Padding(1.f)
			[
				Slot.GetWidget()
			];
	}

	const TSharedRef<SBorder> ContainerBorder = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(TEXT("Brushes.Panel")))
		.Padding(0.f, 2.f, 2.f, 2.f)
		[
			VerticalContainer
		];

	return SNew(SBox)
		.Padding(8.f)
		[
			HorizontalContainerWidget->WrapVerticalListWithHeading(ContainerBorder
				, FPointerEventHandler::CreateSP(this, &SSequencerFilterBar::OnMouseButtonUp))
		];
}

#undef LOCTEXT_NAMESPACE
