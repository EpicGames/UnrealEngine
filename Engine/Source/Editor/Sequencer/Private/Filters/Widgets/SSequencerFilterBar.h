// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SBasicFilterBar.h"
#include "Filters/SequencerTrackFilterBase.h"

class FSequencer;
class FSequencerFilterBarContextMenu;
class FSequencerFilterBar;
class FSequencerTrackFilterContextMenu;
class SFilterBarClippingHorizontalBox;
class SFilterExpressionHelpDialog;
class SSequencerFilter;
class SSequencerSearchBox;
class UMovieSceneNodeGroup;
enum class ESequencerFilterChange : uint8;

class SSequencerFilterBar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSequencerFilterBar)
		: _FilterPillStyle(EFilterPillStyle::Default)
		, _UseSectionsForCategories(true)
	{}
		/** Determines how each individual filter pill looks like */
		SLATE_ARGUMENT(EFilterPillStyle, FilterPillStyle)

		/** Whether to use submenus or sections for categories in the filter menu */
		SLATE_ARGUMENT(bool, UseSectionsForCategories)

	SLATE_END_ARGS()

	virtual ~SSequencerFilterBar() override;

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<FSequencerFilterBar>& InFilterBar);

	//~ Begin SWidget
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End SWidget

	TSharedPtr<FSequencerFilterBar> GetFilterBar() const;

	bool HasAnyFilterWidgets() const;

protected:
	void AddWidgetToLayout(const TSharedRef<SWidget>& InWidget);
	void RemoveWidgetFromLayout(const TSharedRef<SWidget>& InWidget);

	void CreateAndAddFilterWidget(const TSharedRef<FSequencerTrackFilter>& InFilter);
	void AddFilterWidget(const TSharedRef<FSequencerTrackFilter>& InFilter, const TSharedRef<SSequencerFilter>& InFilterWidget);
	void RemoveFilterWidget(const TSharedRef<FSequencerTrackFilter>& InFilter);
	void RemoveAllFilterWidgets();
	void RemoveAllFilterWidgetsButThis(const TSharedRef<FSequencerTrackFilter>& InFilter);

	UWorld* GetWorld() const;

	void OnEnableAllGroupFilters(bool bEnableAll);
	void OnNodeGroupFilterClicked(UMovieSceneNodeGroup* NodeGroup);

	/** Refreshes the displayed filters based on the filter change. */
	void OnFiltersChanged(const ESequencerFilterChange InChangeType, const TSharedRef<FSequencerTrackFilter>& InFilter);
	/** Updates the mute state in response to the owning FSequencerTrackFilter changing mute state. */
	void OnMuteStateChanged(bool bNewIsMuted);

	void CreateFilterWidgetsFromConfig();

	TSharedRef<SWidget> OnWrapButtonClicked();

	FText GetFilterDisplayName(const TSharedRef<FSequencerTrackFilter> InFilter) const;
	FSlateColor GetFilterBlockColor(const TSharedRef<FSequencerTrackFilter> InFilter) const;

	void OnFilterToggle(const ECheckBoxState InNewState, const TSharedRef<FSequencerTrackFilter> InFilter);
	void OnFilterCtrlClick(const TSharedRef<FSequencerTrackFilter> InFilter);
	void OnFilterAltClick(const TSharedRef<FSequencerTrackFilter> InFilter);
	void OnFilterMiddleClick(const TSharedRef<FSequencerTrackFilter> InFilter);
	void OnFilterDoubleClick(const TSharedRef<FSequencerTrackFilter> InFilter);

	TSharedRef<SWidget> OnGetMenuContent(const TSharedRef<FSequencerTrackFilter> InFilter);

	void ActivateAllButThis(const bool bInActive, const TSharedRef<FSequencerTrackFilter> InFilter);

	TWeakPtr<FSequencerFilterBar> WeakFilterBar;

	TSharedPtr<SFilterBarClippingHorizontalBox> HorizontalContainerWidget;

	EFilterPillStyle FilterPillStyle = EFilterPillStyle::Default;

	TMap<TSharedRef<FSequencerTrackFilter>, TSharedRef<SSequencerFilter>> FilterWidgets;

	TSharedPtr<SFilterExpressionHelpDialog> TextExpressionHelpDialog;

	TSharedPtr<FSequencerFilterBarContextMenu> ContextMenu;
	TSharedPtr<FSequencerTrackFilterContextMenu> FilterContextMenu;
};
