// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SCheckBox.h"

class SSequencerFilterCheckBox : public SCheckBox
{
public:
	void SetOnCtrlClick(const FOnClicked& InNewCtrlClick)
	{
		OnCtrlClick = InNewCtrlClick;
	}

	void SetOnAltClick(const FOnClicked& InNewAltClick)
	{
		OnAltClick = InNewAltClick;
	}

	void SetOnMiddleButtonClick(const FOnClicked& InNewMiddleButtonClick)
	{
		OnMiddleButtonClick = InNewMiddleButtonClick;
	}

	void SetOnDoubleClick(const FOnClicked& InNewDoubleClick)
	{
		OnDoubleClick = InNewDoubleClick;
	}

protected:
	//~ Begin SWidget

	virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override
	{
		if (InPointerEvent.IsControlDown() && OnCtrlClick.IsBound())
		{
			return OnCtrlClick.Execute();
		}

		if (InPointerEvent.IsAltDown() && OnAltClick.IsBound())
		{
			return OnAltClick.Execute();
		}

		if (InPointerEvent.GetEffectingButton() == EKeys::MiddleMouseButton && OnMiddleButtonClick.IsBound() )
		{
			return OnMiddleButtonClick.Execute();
		}

		SCheckBox::OnMouseButtonUp(InGeometry, InPointerEvent);

		return FReply::Handled().ReleaseMouseCapture();
	}

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override
	{
		if (InPointerEvent.GetEffectingButton() == EKeys::LeftMouseButton && OnDoubleClick.IsBound())
		{
			return OnDoubleClick.Execute();
		}

		return SCheckBox::OnMouseButtonDoubleClick(InGeometry, InPointerEvent);
	}

	//~ End SWidget

	FOnClicked OnCtrlClick;
	FOnClicked OnAltClick;
	FOnClicked OnDoubleClick;
	FOnClicked OnMiddleButtonClick;
};
