// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerFilterSuggestionListRow.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SSequencerFilterSuggestionListRow"

void SSequencerFilterSuggestionListRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
{
	OwnerTable = InOwnerTable;

	check(InArgs._ListItem.IsValid());
	ListEntry = InArgs._ListItem;
	HighlightText = InArgs._HighlightText;

	SMultiColumnTableRow<TSharedPtr<FSequencerFilterSuggestionListEntryBase>>::Construct(FSuperRowType::FArguments(), InOwnerTable);
}

TSharedRef<SWidget> SSequencerFilterSuggestionListRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	switch (ListEntry->GetItemType())
	{
	case ESequencerFilterSuggestionListEntryType::Header:
		{
			const TSharedPtr<FSequencerFilterSuggestionListHeaderEntry> HeaderListItem = StaticCastSharedPtr<FSequencerFilterSuggestionListHeaderEntry>(ListEntry);
			check(HeaderListItem.IsValid());
			return CreateHeaderItem(HeaderListItem, InColumnName);
		}
	case ESequencerFilterSuggestionListEntryType::Suggestion:
		{
			const TSharedPtr<FSequencerFilterSuggestionListEntry> SuggestionListItem = StaticCastSharedPtr<FSequencerFilterSuggestionListEntry>(ListEntry);
			check(SuggestionListItem.IsValid());
			return CreateSuggestionItem(SuggestionListItem, InColumnName);
		}
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SSequencerFilterSuggestionListRow::CreateHeaderItem(const TSharedPtr<FSequencerFilterSuggestionListHeaderEntry>& InHeaderListItem, const FName& InColumnName)
{
	const TSharedRef<SVerticalBox> HeaderBox = SNew(SVerticalBox);

	const bool bIsFirstItem = OwnerTable->GetNumGeneratedChildren() == 0;
	if (!bIsFirstItem)
	{
		HeaderBox->AddSlot()
			.AutoHeight()
			.Padding(0.f, 4.f, 0.f, 2.f) // Add some empty space before the line, and a tiny bit after it
			[
				SNew(SBorder)
				.Padding(FAppStyle::GetMargin(TEXT("Menu.Separator.Padding"))) // We'll use the border's padding to actually create the horizontal line
				.BorderImage(FAppStyle::GetBrush(TEXT("Menu.Separator")))
			];
	}

	HeaderBox->AddSlot()
		.AutoHeight()
		.Padding(3.f, 0.f)
		[
			SNew(STextBlock)
			.Text(InHeaderListItem->Title)
			.TextStyle(FAppStyle::Get(), TEXT("Menu.Heading"))
		];

	return HeaderBox;
}

TSharedRef<SWidget> SSequencerFilterSuggestionListRow::CreateSuggestionItem(const TSharedPtr<FSequencerFilterSuggestionListEntry>& InSuggestionListItem, const FName& InColumnName)
{
	if (InColumnName == TEXT("Suggestion"))
	{
		return SNew(SBox)
			.VAlign(VAlign_Center)
			.Padding(12.f, 1.f, 9.f, 1.f)
			[
				SNew(STextBlock)
				.Text(InSuggestionListItem->Suggestion.DisplayName)
				.ToolTipText(InSuggestionListItem->Suggestion.Description)
				.HighlightText(HighlightText)
			];
	}

	if (InColumnName == TEXT("Description"))
	{
		return SNew(SBox)
			.VAlign(VAlign_Bottom)
			.Padding(3.f, 1.f)
			[
				SNew(STextBlock)
				.Text(InSuggestionListItem->Suggestion.Description)
				.TextStyle(FAppStyle::Get(), TEXT("HintText"))
			];
	}

	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE
