// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/Widgets/SequencerFilterSuggestionListEntry.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Views/SListView.h"

class SSequencerFilterSuggestionListRow : public SMultiColumnTableRow<TSharedPtr<FSequencerFilterSuggestionListEntryBase>>
{
public:
	SLATE_BEGIN_ARGS(SSequencerFilterSuggestionListRow)
	{}
		SLATE_ARGUMENT(TSharedPtr<FSequencerFilterSuggestionListEntryBase>, ListItem)
		SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable);

protected:
	//~ Begin SMultiColumnTableRow
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;
	//~ End SMultiColumnTableRow

	TSharedRef<SWidget> CreateHeaderItem(const TSharedPtr<FSequencerFilterSuggestionListHeaderEntry>& InHeaderListItem, const FName& InColumnName);
	TSharedRef<SWidget> CreateSuggestionItem(const TSharedPtr<FSequencerFilterSuggestionListEntry>& InSuggestionListItem, const FName& InColumnName);

	TSharedPtr<STableViewBase> OwnerTable;

	TSharedPtr<FSequencerFilterSuggestionListEntryBase> ListEntry;

	TAttribute<FText> HighlightText;
};
