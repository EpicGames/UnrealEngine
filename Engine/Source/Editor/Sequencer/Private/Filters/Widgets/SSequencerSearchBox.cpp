// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Widgets/SSequencerSearchBox.h"
#include "Filters/ISequencerFilterBar.h"
#include "Filters/Filters/SequencerTrackFilter_Text.h"
#include "Filters/SequencerTrackFilterCollection.h"
#include "Filters/Widgets/SSequencerFilterSuggestionListRow.h"
#include "Framework/Application/SlateApplication.h"
#include "SequencerSettings.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SListView.h"

class ITableRow;
class STableViewBase;
class SWidget;
struct FGeometry;

#define LOCTEXT_NAMESPACE "SSequencerSearchBox"

/** Case sensitive hashing function for TMap */
template <typename ValueType>
struct FSequencerSearchCategoryKeyMapFuncs : BaseKeyFuncs<ValueType, FText, /*bInAllowDuplicateKeys*/false>
{
	static FORCEINLINE const FString& GetSourceString(const FText& InText)
	{
		const FString* SourceString = FTextInspector::GetSourceString(InText);
		check(SourceString);
		return *SourceString;
	}
	static FORCEINLINE const FText& GetSetKey(const TPair<FText, ValueType>& Element)
	{
		return Element.Key;
	}
	static FORCEINLINE bool Matches(const FText& A, const FText& B)
	{
		return GetSourceString(A).Equals(GetSourceString(B), ESearchCase::CaseSensitive);
	}
	static FORCEINLINE uint32 GetKeyHash(const FText& Key)
	{
		return FLocKey::ProduceHash(GetSourceString(Key));
	}
};

void SSequencerSearchBox::Construct(const FArguments& InArgs, const TWeakPtr<ISequencerFilterBar>& InWeakFilterBar)
{
	WeakFilterBar = InWeakFilterBar;

	OnTextChanged = InArgs._OnTextChanged;
	OnTextCommitted = InArgs._OnTextCommitted;
	OnKeyDownHandler = InArgs._OnKeyDownHandler;
	PossibleSuggestions = InArgs._PossibleSuggestions;
	OnSuggestionFilter = InArgs._OnSuggestionFilter;
	OnSuggestionChosen = InArgs._OnSuggestionChosen;
	PreCommittedText = InArgs._InitialText.Get();
	bMustMatchPossibleSuggestions = InArgs._MustMatchPossibleSuggestions.Get();

	if (!OnSuggestionFilter.IsBound())
	{
		OnSuggestionFilter.BindSP(this, &SSequencerSearchBox::DefaultSuggestionFilterImpl);
	}

	if (!OnSuggestionChosen.IsBound())
	{
		OnSuggestionChosen.BindSP(this, &SSequencerSearchBox::DefaultSuggestionChosenImpl);
	}

	ChildSlot
		[
			SAssignNew(SuggestionMenuAnchor, SMenuAnchor)
			.Placement(InArgs._SuggestionListPlacement)
			[
				/** Use SFilterSearchBox internally to add the ability to show search history and save searches as filters */
				SAssignNew(SearchBox, SFilterSearchBox)
				.InitialText(InArgs._InitialText)
				.HintText(InArgs._HintText)
				.ShowSearchHistory(InArgs._ShowSearchHistory)
				.DelayChangeNotificationsWhileTyping(InArgs._DelayChangeNotificationsWhileTyping)
				.OnTextChanged(this, &SSequencerSearchBox::HandleTextChanged)
				.OnTextCommitted(this, &SSequencerSearchBox::HandleTextCommitted)
				.OnKeyDownHandler(this, &SSequencerSearchBox::HandleKeyDown)
				.OnSaveSearchClicked(InArgs._OnSaveSearchClicked)
			]
			.MenuContent(GetSuggestionListMenuContent())
		];
	
	const TSharedPtr<ISequencerFilterBar> FilterBarPin = WeakFilterBar.Pin();
	check(FilterBarPin);
	FilterBarPin->OnTextFilterTextChanged().AddSP(this, &SSequencerSearchBox::OnFilterTextChanged);
	FilterBarPin->OnMuteFiltersChanged().AddSP(this, &SSequencerSearchBox::OnMuteStateChanged);
	
	OnFilterTextChanged();
	OnMuteStateChanged(FilterBarPin->AreFiltersMuted());
}

TSharedRef<SWidget> SSequencerSearchBox::GetSuggestionListMenuContent()
{
	const USequencerSettings* const SequencerSettings = GetDefault<USequencerSettings>();
	check(SequencerSettings);

	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(TEXT("Menu.Background")))
		.Padding(FMargin(2))
		[
			SNew(SBox)
			.MinDesiredWidth(180)
			.MinDesiredHeight(16)
			.MaxDesiredHeight(260)
			[
				SAssignNew(SuggestionListView, SListView<TSharedPtr<FSequencerFilterSuggestionListEntryBase>>)
				.ListItemsSource(&SuggestionItems)
				.SelectionMode(ESelectionMode::Single) // Ideally the mouse over would not highlight while keyboard controls the UI
				.OnGenerateRow(this, &SSequencerSearchBox::MakeSuggestionListItemWidget)
				.OnSelectionChanged(this, &SSequencerSearchBox::OnSelectionChanged)
				.ScrollbarDragFocusCause(EFocusCause::SetDirectly) // Use SetDirect so that clicking the scrollbar doesn't close the suggestions list
				.HeaderRow
				(
					SNew(SHeaderRow)
					.Visibility(EVisibility::Collapsed)
					+ SHeaderRow::Column("Suggestion")
					.DefaultLabel(LOCTEXT("SuggestionColumnHeader", "Suggestion"))
					.FixedWidth(180.f)
					.VAlignCell(VAlign_Center)
					.HAlignCell(HAlign_Left)
					+ SHeaderRow::Column("Description")
					.DefaultLabel(LOCTEXT("DescriptionColumnHeader", "Description"))
					.FillWidth(1.f)
					.VAlignCell(VAlign_Center)
					.HAlignCell(HAlign_Left)
				)
			]
		];
}

FText SSequencerSearchBox::GetText() const
{
	return SearchBox->GetText();
}

void SSequencerSearchBox::SetText(const TAttribute<FText>& InNewText)
{
	SearchBox->SetText(InNewText);
	PreCommittedText = InNewText.Get();

	// Set the cursor interaction location to the end of the newly inserted suggestion
	if (LastChosenCursorOffset != INDEX_NONE)
	{
		const FTextLocation CursorLocation = FTextLocation(0, LastChosenCursorOffset);
		SearchBox->SelectText(CursorLocation, CursorLocation);

		LastChosenCursorOffset = INDEX_NONE;
	}
}

void SSequencerSearchBox::SetError(const FText& InError)
{
	SearchBox->SetError(InError);
}

void SSequencerSearchBox::SetError(const FString& InError)
{
	SearchBox->SetError(InError);
}

FReply SSequencerSearchBox::OnPreviewKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (SuggestionMenuAnchor->IsOpen() && InKeyEvent.GetKey() == EKeys::Escape)
	{
		// Clear any selection first to prevent the currently selection being set in the text box
		SuggestionListView->ClearSelection();
		SuggestionMenuAnchor->SetIsOpen(false, false);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SSequencerSearchBox::HandleKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (!SearchBox->GetText().IsEmpty()
		&& (InKeyEvent.GetKey() == EKeys::Up || InKeyEvent.GetKey() == EKeys::Down))
	{
		if (!SuggestionMenuAnchor->IsOpen())
		{
			SuggestionMenuAnchor->SetIsOpen(true, false);
		}

		const bool bSelectingUp = InKeyEvent.GetKey() == EKeys::Up;
		const TSharedPtr<FSequencerFilterSuggestionListEntryBase> SelectedSuggestion = GetSelectedSuggestion();

		int32 TargetIdx = INDEX_NONE;
		if (SelectedSuggestion.IsValid())
		{
			const int32 SelectionDirection = bSelectingUp ? -1 : 1;

			// Select the next non-header suggestion, based on the direction of travel
			TargetIdx = SuggestionItems.IndexOfByKey(SelectedSuggestion);
			if (SuggestionItems.IsValidIndex(TargetIdx))
			{
				do
				{
					TargetIdx += SelectionDirection;
				}
				while (SuggestionItems.IsValidIndex(TargetIdx) && SuggestionItems[TargetIdx]->IsHeader());
			}
		}
		else if (!bSelectingUp && SuggestionItems.Num() > 0)
		{
			// Nothing selected and pressed down, select the first non-header suggestion
			TargetIdx = 0;
			while (SuggestionItems.IsValidIndex(TargetIdx) && SuggestionItems[TargetIdx]->IsHeader())
			{
				TargetIdx += 1;
			}
		}

		if (SuggestionItems.IsValidIndex(TargetIdx))
		{
			SuggestionListView->SetSelection(SuggestionItems[TargetIdx]);
			SuggestionListView->RequestScrollIntoView(SuggestionItems[TargetIdx]);
		}

		return FReply::Handled();
	}

	if (OnKeyDownHandler.IsBound())
	{
		return OnKeyDownHandler.Execute(InGeometry, InKeyEvent);
	}

	return FReply::Unhandled();
}

bool SSequencerSearchBox::SupportsKeyboardFocus() const
{
	return SearchBox->SupportsKeyboardFocus();
}

bool SSequencerSearchBox::HasKeyboardFocus() const
{
	// Since keyboard focus is forwarded to our editable text, we will test it instead
	return SearchBox->HasKeyboardFocus();
}

FReply SSequencerSearchBox::OnFocusReceived(const FGeometry& InGeometry, const FFocusEvent& InFocusEvent)
{
	// Forward keyboard focus to our editable text widget
	return SearchBox->OnFocusReceived(InGeometry, InFocusEvent);
}

void SSequencerSearchBox::HandleTextChanged(const FText& InNewText)
{
	LastCursorLocation = SearchBox->GetSelection().GetEnd();

	OnTextChanged.ExecuteIfBound(InNewText);

	UpdateSuggestionList();
}

void SSequencerSearchBox::HandleTextCommitted(const FText& InNewText, const ETextCommit::Type InCommitType)
{
	TSharedPtr<FSequencerFilterSuggestionListEntryBase> SelectedSuggestion = GetSelectedSuggestion();

	FText CommittedText;
	if (SelectedSuggestion.IsValid() && !SelectedSuggestion->IsHeader() && InCommitType != ETextCommit::OnCleared)
	{
		CommittedText = OnSuggestionChosen.Execute(InNewText, SelectedSuggestion->AsSuggestionEntry()->Suggestion.Suggestion);
	}
	else
	{
		if (InCommitType == ETextCommit::OnCleared)
		{
			// Clear text when escape is pressed then commit an empty string
			CommittedText = FText::GetEmpty();
		}
		else if (bMustMatchPossibleSuggestions)
		{
			const FString NewTextStr = InNewText.ToString();
			const bool bIsSuggestion = PossibleSuggestions.Get().ContainsByPredicate(
				[NewTextStr](const FSequencerFilterSuggestion& InSuggestion)
				{
					return InSuggestion.Suggestion == NewTextStr;
				});
			if (bIsSuggestion)
			{
				CommittedText = InNewText;
			}
			else
			{
				// commit the original text if we have to match a suggestion
				CommittedText = PreCommittedText;
			}
		}
		else
		{
			// otherwise, set the typed text
			CommittedText = InNewText;
		}
	}

	// Set the text and execute the delegate
	bDisableOpeningSuggestions = true;
	SetText(CommittedText);
	OnTextCommitted.ExecuteIfBound(CommittedText, InCommitType);

	if (InCommitType != ETextCommit::Default)
	{
		// Clear the suggestion box if the user has navigated away or set their own text
		SuggestionMenuAnchor->SetIsOpen(false, false);
	}
}

void SSequencerSearchBox::OnSelectionChanged(TSharedPtr<FSequencerFilterSuggestionListEntryBase> InNewValue, const ESelectInfo::Type InSelectInfo)
{
	// If the user clicked directly on an item to select it, then accept the choice and close the window
	if (InSelectInfo == ESelectInfo::OnMouseClick && !InNewValue->IsHeader())
	{
		const FText NewText = OnSuggestionChosen.Execute(SearchBox->GetText(), InNewValue->AsSuggestionEntry()->Suggestion.Suggestion);
		SetText(NewText);

		SuggestionMenuAnchor->SetIsOpen(false, false);
		FocusEditBox();
	}
}

TSharedRef<ITableRow> SSequencerSearchBox::MakeSuggestionListItemWidget(const TSharedPtr<FSequencerFilterSuggestionListEntryBase> InSuggestion, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SSequencerFilterSuggestionListRow, InOwnerTable)
		.ListItem(InSuggestion)
		.HighlightText(this, &SSequencerSearchBox::GetHighlightText);
}

FText SSequencerSearchBox::GetHighlightText() const
{
	return SuggestionHighlightText;
}

void SSequencerSearchBox::UpdateSuggestionList()
{
	FText SearchText = SearchBox->GetText();

	SuggestionItems.Reset();
	SuggestionHighlightText = FText::GetEmpty();

	if (!SearchText.IsEmpty())
	{
		typedef TMap<FText
			, TArray<TSharedPtr<FSequencerFilterSuggestionListEntryBase>>
			, FDefaultSetAllocator
			, FSequencerSearchCategoryKeyMapFuncs<TArray<TSharedPtr<FSequencerFilterSuggestionListEntryBase>>>> FCategorizedSuggestionsMap;

		FParsedExpression ParsedExpression;
		ExtractSearchFilterTerms(SearchText, ParsedExpression);

		// Get the potential suggestions
		TArray<FSequencerFilterSuggestion> FilteredSuggestions;
		if (PossibleSuggestions.IsBound())
		{
			FilteredSuggestions = PossibleSuggestions.Get();
		}
		else
		{
			if (!ParsedExpression.Key.IsSet() && ParsedExpression.Value.IsSet())
			{
				DefaultKeySuggestions(SearchText.ToString(), FilteredSuggestions);
			}
			else if (ParsedExpression.Key.IsSet())
			{
				DefaultValueSuggestions(ParsedExpression.Key.GetValue(), FilteredSuggestions);
			}

			if (ParsedExpression.Value.IsSet())
			{
				SearchText = FText::FromString(ParsedExpression.Value.GetValue());
			}
		}

		// Run them through the filter
		OnSuggestionFilter.Execute(SearchText, FilteredSuggestions, SuggestionHighlightText);

		// Split the suggestions list into categories
		FCategorizedSuggestionsMap CategorizedSuggestions;
		for (const FSequencerFilterSuggestion& Suggestion : FilteredSuggestions)
		{
			TArray<TSharedPtr<FSequencerFilterSuggestionListEntryBase>>& CategorySuggestions = CategorizedSuggestions.FindOrAdd(Suggestion.CategoryName);

			const TSharedRef<FSequencerFilterSuggestionListEntry> NewSuggestionEntry = MakeShared<FSequencerFilterSuggestionListEntry>();
			NewSuggestionEntry->Suggestion = Suggestion;

			CategorySuggestions.Add(NewSuggestionEntry);
		}

		// Rebuild the flat list in categorized groups
		// If there is only one category, and that category is empty (undefined), then skip adding the category headers
		const bool bSkipCategoryHeaders = CategorizedSuggestions.Num() == 1 && CategorizedSuggestions.Contains(FText::GetEmpty());
		for (const TPair<FText, TArray<TSharedPtr<FSequencerFilterSuggestionListEntryBase>>>& CategorySuggestionsPair : CategorizedSuggestions)
		{
			if (!bSkipCategoryHeaders)
			{
				const FText CategoryDisplayName = CategorySuggestionsPair.Key.IsEmpty()
					? LOCTEXT("UndefinedCategory", "Undefined")
					: CategorySuggestionsPair.Key;

				const TSharedRef<FSequencerFilterSuggestionListHeaderEntry> NewHeaderEntry = MakeShared<FSequencerFilterSuggestionListHeaderEntry>(CategoryDisplayName);

				SuggestionItems.Add(NewHeaderEntry);
			}
			SuggestionItems.Append(CategorySuggestionsPair.Value);
		}
	}

	if (!bDisableOpeningSuggestions && SuggestionItems.Num() > 0 && HasKeyboardFocus())
	{
		SuggestionMenuAnchor->SetIsOpen(true, false);
	}
	else
	{
		SuggestionMenuAnchor->SetIsOpen(false, false);
	}

	bDisableOpeningSuggestions = false;

	SuggestionListView->RequestListRefresh();
}

void SSequencerSearchBox::FocusEditBox()
{
	FWidgetPath WidgetToFocusPath;
	FSlateApplication::Get().GeneratePathToWidgetUnchecked(SearchBox.ToSharedRef(), WidgetToFocusPath);
	FSlateApplication::Get().SetKeyboardFocus(WidgetToFocusPath, EFocusCause::SetDirectly);
}

TSharedPtr<FSequencerFilterSuggestionListEntryBase> SSequencerSearchBox::GetSelectedSuggestion() const
{
	TSharedPtr<FSequencerFilterSuggestionListEntryBase> SelectedSuggestion;

	if (SuggestionMenuAnchor->IsOpen())
	{
		const TArray<TSharedPtr<FSequencerFilterSuggestionListEntryBase>>& SelectedSuggestionList = SuggestionListView->GetSelectedItems();
		if (SelectedSuggestionList.Num() > 0)
		{
			// Selection mode is Single, so there should only be one suggestion at the most
			SelectedSuggestion = SelectedSuggestionList[0];
		}
	}

	return SelectedSuggestion;
}

void SSequencerSearchBox::SetOnSaveSearchHandler(SFilterSearchBox::FOnSaveSearchClicked InOnSaveSearchHandler)
{
	SearchBox->SetOnSaveSearchHandler(InOnSaveSearchHandler);
}

void SSequencerSearchBox::ExtractSearchFilterTerms(const FText& InSearchText, FParsedExpression& OutParsedPair) const
{
	const TSharedPtr<ISequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const FTextFilterExpressionEvaluator& TextFilterExpressionEvaluator = FilterBar->GetTextFilterExpressionEvaluator();
	const TArray<FExpressionToken>& ExpressionTokens = TextFilterExpressionEvaluator.GetFilterExpressionTokens();
	if (ExpressionTokens.Num() == 0)
	{
		return;
	}

	const FString SearchString = InSearchText.ToString();

	OutParsedPair.SuggestionIndex = SearchString.Len();

	const int32 CaretLocationIndex = LastCursorLocation.GetOffset();
	const int32 CaretTokenIndex = FindTokenIndex(ExpressionTokens, CaretLocationIndex);

	if (CaretTokenIndex == INDEX_NONE)
	{
		return;
	}

	// Inspect the tokens to see what the last part of the search term was
	// If it was a key->value pair then we'll use that to control what kinds of results we show
	// For anything else we just use the text from the last token as our filter term to allow incremental auto-complete
	const FExpressionToken& LastToken = ExpressionTokens[CaretTokenIndex];

	const int32 SecondToLastIndex = CaretTokenIndex - 1;
	const int32 ThirdToLastIndex = CaretTokenIndex - 2;

	// If the last token is a text token, then consider it as a value and walk back to see if we also have a key
	if (LastToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
	{
		OutParsedPair.Value = LastToken.Context.GetString();
		OutParsedPair.SuggestionIndex = LastToken.Context.GetCharacterIndex();

		if (ExpressionTokens.IsValidIndex(ThirdToLastIndex))
		{
			// Check if the second from last token is a operator (=, !=, <, >, etc)
			const FExpressionToken& ComparisonToken = ExpressionTokens[SecondToLastIndex];
			if (IsOperatorToken(ComparisonToken))
			{
				const FExpressionToken& KeyToken = ExpressionTokens[ThirdToLastIndex];
				if (KeyToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
				{
					OutParsedPair.Key = KeyToken.Context.GetString();
					OutParsedPair.SuggestionIndex = KeyToken.Context.GetCharacterIndex();
				}
			}
			else if (IsLogicalOperatorToken(ComparisonToken))
			{
				const FExpressionToken& KeyToken = ExpressionTokens[SecondToLastIndex];
				if (KeyToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
				{
					OutParsedPair.Key = KeyToken.Context.GetString();
					OutParsedPair.SuggestionIndex = LastToken.Context.GetCharacterIndex();
				}
			}
		}
	}
	// If the last token is a comparison operator, then walk back and see if we have a key
	else if (IsOperatorToken(LastToken))
	{
		if (ExpressionTokens.IsValidIndex(SecondToLastIndex))
		{
			const FExpressionToken& KeyToken = ExpressionTokens[SecondToLastIndex];
			if (KeyToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
			{
				OutParsedPair.Key = KeyToken.Context.GetString();
				OutParsedPair.Value = TEXT("");
				OutParsedPair.SuggestionIndex = LastToken.Context.GetCharacterIndex();
			}
		}
	}
}

void SSequencerSearchBox::DefaultSuggestionFilterImpl(const FText& InSearchText, TArray<FSequencerFilterSuggestion>& OutPossibleSuggestions, FText& OutSuggestionHighlightText)
{
	OutPossibleSuggestions.RemoveAll([SearchStr = InSearchText.ToString()](const FSequencerFilterSuggestion& InSuggestion)
		{
			return !InSuggestion.Suggestion.Contains(SearchStr);
		});

	OutSuggestionHighlightText = InSearchText;
}

FText SSequencerSearchBox::DefaultSuggestionChosenImpl(const FText& InSearchText, const FString& InSuggestion)
{
	const FTextFilterExpressionEvaluator* const TextFilterExpressionEvaluator = GetTextFilterExpressionEvaluator();
	if (!TextFilterExpressionEvaluator)
	{
		return InSearchText;
	}

	const TArray<FExpressionToken>& ExpressionTokens = TextFilterExpressionEvaluator->GetFilterExpressionTokens();
	const int32 CaretLocationIndex = LastCursorLocation.GetOffset();
	const int32 CaretTokenIndex = FindTokenIndex(ExpressionTokens, CaretLocationIndex);
	const FExpressionToken& CaretToken = ExpressionTokens[CaretTokenIndex];
	const int32 SuggestionInsertionIndex = CaretToken.Context.GetCharacterIndex();
	const FString CaretTokenString = CaretToken.Context.GetString();

	// Replace the value with the suggestion
	FString SearchString = InSearchText.ToString();
	SearchString.RemoveAt(SuggestionInsertionIndex, CaretTokenString.Len(), EAllowShrinking::No);
	SearchString.InsertAt(SuggestionInsertionIndex, InSuggestion);

	// Set the cursor interaction location to the end of the newly inserted suggestion
	LastChosenCursorOffset = SuggestionInsertionIndex + InSuggestion.Len();

	return FText::FromString(SearchString);
}

void SSequencerSearchBox::DefaultKeySuggestions(const FString& InKeyValue, TArray<FSequencerFilterSuggestion>& OutPossibleSuggestions)
{
	const TSharedPtr<ISequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const TArray<TSharedRef<ISequencerTextFilterExpressionContext>>& FilterExpressions = FilterBar->GetTextFilterExpressionContexts();

	for (const TSharedRef<ISequencerTextFilterExpressionContext>& Expression : FilterExpressions)
	{
		const TArray<FName> Keys = Expression->GetKeys().Array();
		for (const FName Key : Keys)
		{
			FSequencerFilterSuggestion NewSuggestion;
			NewSuggestion.Suggestion = Key.ToString(); 
			NewSuggestion.DisplayName = FText::FromName(Key);
			NewSuggestion.Description = Expression->GetDescription();

			OutPossibleSuggestions.Add(NewSuggestion);
		}
	}

	// Add keys for all common filters to support suggestions for these text expressions
	FilterBar->GetCommonFilterKeySuggestions(OutPossibleSuggestions);

	OutPossibleSuggestions.Sort([](const FSequencerFilterSuggestion& InA, const FSequencerFilterSuggestion& InB)
		{
			return InA.DisplayName.CompareTo(InB.DisplayName) < 0;
		});
}

void SSequencerSearchBox::DefaultValueSuggestions(const FString& InKeyValue, TArray<FSequencerFilterSuggestion>& OutPossibleSuggestions)
{
	const TSharedPtr<ISequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const TArray<TSharedRef<ISequencerTextFilterExpressionContext>> FilterExpressions = FilterBar->GetTextFilterExpressionContexts();

	FSequencerTextFilterKeyword FoundKeyword;
	TSharedPtr<ISequencerTextFilterExpressionContext> FoundExpression;

	for (const TSharedRef<ISequencerTextFilterExpressionContext>& Expression : FilterExpressions)
	{
		for (const FName Key : Expression->GetKeys())
		{
			if (InKeyValue.Equals(Key.ToString(), ESearchCase::IgnoreCase))
			{
				FoundExpression = Expression;
				goto end;
			}
		}
	}

end:
	if (FoundExpression.IsValid())
	{
		if (FoundExpression->GetValueType() == ESequencerTextFilterValueType::Boolean)
		{
			FSequencerFilterSuggestion::MakeBooleanSuggestions(OutPossibleSuggestions);
		}
		else
		{
			const TArray<FSequencerTextFilterKeyword> ValueKeywords = FoundExpression->GetValueKeywords();
			for (const FSequencerTextFilterKeyword& ValueKeyword : ValueKeywords)
			{
				FSequencerFilterSuggestion NewSuggestion;
				NewSuggestion.Suggestion = ValueKeyword.Keyword;
				NewSuggestion.DisplayName = FText::FromString(ValueKeyword.Keyword);
				NewSuggestion.Description = ValueKeyword.Description;

				OutPossibleSuggestions.Add(NewSuggestion);
			}
		}
	}
	else
	{
		// Add keys for all common filters to support suggestions for these text expressions
		FilterBar->GetCommonFilterValueSuggestions(InKeyValue, OutPossibleSuggestions);
	}

	OutPossibleSuggestions.Sort([](const FSequencerFilterSuggestion& InA, const FSequencerFilterSuggestion& InB)
		{
			return InA.DisplayName.CompareTo(InB.DisplayName) < 0;
		});
}

void SSequencerSearchBox::OnFilterTextChanged()
{
	const TSharedPtr<ISequencerFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar)
	{
		return;
	}

	// If the filter was changed externally, keep the widget in sync.
	// Skip SetText if the change was instantiated by this widget: when focused and calling SetText it can cause the cursor location to change.
	const FString FilterString = FilterBar->GetTextFilterString();
	const bool bHasChanged = GetText().ToString() != FilterString;
	if (bHasChanged)
	{
		const FText NewText = FText::FromString(FilterString);
		SetText(NewText);
	}
}

void SSequencerSearchBox::OnMuteStateChanged(bool bIsMuted)
{
	SetEnabled(!bIsMuted);
}

bool SSequencerSearchBox::IsOperatorToken(const FExpressionToken& InToken)
{
	return InToken.Node.Cast<TextFilterExpressionParser::FEqual>()
		|| InToken.Node.Cast<TextFilterExpressionParser::FNotEqual>()
		|| InToken.Node.Cast<TextFilterExpressionParser::FLess>()
		|| InToken.Node.Cast<TextFilterExpressionParser::FLessOrEqual>()
		|| InToken.Node.Cast<TextFilterExpressionParser::FGreater>()
		|| InToken.Node.Cast<TextFilterExpressionParser::FGreaterOrEqual>();
};

bool SSequencerSearchBox::IsLogicalOperatorToken(const FExpressionToken& InToken)
{
	return InToken.Node.Cast<TextFilterExpressionParser::FAnd>()
		|| InToken.Node.Cast<TextFilterExpressionParser::FOr>();
};

int32 SSequencerSearchBox::FindTokenIndex(const TArray<FExpressionToken>& ExpressionTokens, const int32 InIndexToFind)
{
	for (int32 Index = ExpressionTokens.Num() - 1; Index > 0; --Index)
	{
		const int32 CharacterIndex = ExpressionTokens[Index].Context.GetCharacterIndex();
		const int32 TokenLength = ExpressionTokens[Index].Context.GetString().Len();
		if (InIndexToFind > CharacterIndex)// && InIndexToFind <= CharacterIndex + TokenLength)
		{
			return Index;
		}
	}
	return ExpressionTokens.IsEmpty() ? INDEX_NONE : 0;
}

const FTextFilterExpressionEvaluator* SSequencerSearchBox::GetTextFilterExpressionEvaluator() const
{
	if (const TSharedPtr<ISequencerFilterBar> FilterBar = WeakFilterBar.Pin())
	{
		return &FilterBar->GetTextFilterExpressionEvaluator();
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
