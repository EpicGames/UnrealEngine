// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerFilterSuggestion.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

enum class ESequencerFilterSuggestionListEntryType : uint8
{
	Header,
	Suggestion
};

class FSequencerFilterSuggestionListEntryBase : public TSharedFromThis<FSequencerFilterSuggestionListEntryBase>
{
public:
	FSequencerFilterSuggestionListEntryBase(const ESequencerFilterSuggestionListEntryType InItemType)
		: ItemType(InItemType)
	{}

	ESequencerFilterSuggestionListEntryType GetItemType() const
	{
		return ItemType;
	}

	bool IsHeader() const
	{
		return ItemType == ESequencerFilterSuggestionListEntryType::Header;
	}

	TSharedPtr<class FSequencerFilterSuggestionListHeaderEntry> AsHeaderEntry()
	{
		return StaticCastSharedRef<FSequencerFilterSuggestionListHeaderEntry>(SharedThis(this));
	}

	TSharedPtr<class FSequencerFilterSuggestionListEntry> AsSuggestionEntry()
	{
		return StaticCastSharedRef<FSequencerFilterSuggestionListEntry>(SharedThis(this));
	}

protected:
	ESequencerFilterSuggestionListEntryType ItemType;
};


class FSequencerFilterSuggestionListHeaderEntry : public FSequencerFilterSuggestionListEntryBase
{
public:
	FSequencerFilterSuggestionListHeaderEntry(const FText& InTitle)
		: FSequencerFilterSuggestionListEntryBase(ESequencerFilterSuggestionListEntryType::Header)
		, Title(InTitle)
	{}

	FText Title;
};


class FSequencerFilterSuggestionListEntry : public FSequencerFilterSuggestionListEntryBase
{
public:
	FSequencerFilterSuggestionListEntry()
		: FSequencerFilterSuggestionListEntryBase(ESequencerFilterSuggestionListEntryType::Suggestion)
	{}

	FSequencerFilterSuggestion Suggestion;
};
