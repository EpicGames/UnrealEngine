// Copyright Epic Games, Inc. All Rights Reserved.

#include "IKeyArea.h"
#include "ISequencerDecorationEditor.h"
#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SNullWidget.h"
#include "ISequencerChannelInterface.h"
#include "SequencerClipboardReconciler.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "CurveModel.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "CurveEditorSettings.h"
#include "CurveEditorTypes.h"

IKeyArea::IKeyArea(TWeakPtr<ISequencerSection> InSection, FMovieSceneChannelHandle InChannel)
	: TreeSerialNumber(0)
	, ChannelHandle(InChannel)
{
	Reinitialize(InSection, InChannel);
}

void IKeyArea::Reinitialize(TWeakPtr<ISequencerSection> InSection, FMovieSceneChannelHandle InChannel)
{
	WeakSection = InSection;
	ChannelHandle = InChannel;
	Color = FLinearColor::White;

	if (const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData())
	{
		Color = MetaData->Color;
		ChannelName = MetaData->Name;
		DisplayText = MetaData->DisplayText;
	}

	UMovieSceneSection*       SectionObject = InSection.Pin()->GetSectionObject();
	UMovieScenePropertyTrack* PropertyTrack = SectionObject->GetTypedOuter<UMovieScenePropertyTrack>();
	if (PropertyTrack && PropertyTrack->GetPropertyPath() != NAME_None)
	{
		PropertyBindings = FTrackInstancePropertyBindings(PropertyTrack->GetPropertyName(), PropertyTrack->GetPropertyPath().ToString());
	}
}

FMovieSceneChannel* IKeyArea::ResolveChannel() const
{
	return ChannelHandle.Get();
}

UMovieSceneSection* IKeyArea::GetOwningSection() const
{
	TSharedPtr<ISequencerSection> SectionInterface = WeakSection.Pin();
	return SectionInterface ? SectionInterface->GetSectionObject() : nullptr;;
}

UObject* IKeyArea::GetOwningObject() const
{
	const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();
	UObject* OwningObject = MetaData ? MetaData->WeakOwningObject.Get() : nullptr;
	if (OwningObject)
	{
		return OwningObject;
	}

	return GetOwningSection();
}

TSharedPtr<ISequencerSection> IKeyArea::GetSectionInterface() const
{
	return WeakSection.Pin();
}

FName IKeyArea::GetName() const
{
	return ChannelName;
}

void IKeyArea::SetName(FName InName)
{
	ChannelName = InName;
}

ISequencerChannelInterface* IKeyArea::FindChannelEditorInterface() const
{
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	ISequencerChannelInterface* EditorInterface = SequencerModule.FindChannelEditorInterface(ChannelHandle.GetChannelTypeName());
	ensureMsgf(EditorInterface, TEXT("No channel interface found for type '%s'. Did you forget to call ISequencerModule::RegisterChannelInterface<ChannelType>()?"), *ChannelHandle.GetChannelTypeName().ToString());
	return EditorInterface;
}

FKeyHandle IKeyArea::AddOrUpdateKey(FFrameNumber Time, const FGuid& ObjectBindingID, ISequencer& InSequencer)
{
	ISequencerChannelInterface* EditorInterface = FindChannelEditorInterface();
	FMovieSceneChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection* Section = GetOwningSection();
	check(Section != nullptr);

	// Decoration-owned channels route through the decoration editor for keying.
	auto [DecorationEditor, Decoration] = InSequencer.FindDecorationOwner(ChannelHandle, Section);
	if (DecorationEditor)
	{
		return DecorationEditor->AddOrUpdateKey(*Decoration, *Section, ChannelHandle, Time, ObjectBindingID, InSequencer);
	}

	UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>();
	if (!Track)
	{
		return FKeyHandle();
	}

	// If track does not allow auto expanding sections, check if key time is within bounds. We include exclusive end frame to allow keyframes to visually align with section rendering
	if (!Track->bAutoExpandToKeyframes && (Time < Section->GetInclusiveStartFrame() || Time > Section->GetExclusiveEndFrame()))
	{
		return FKeyHandle();
	}

	if (const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData())
	{
		Time -= MetaData->GetOffsetTime(Section);
	}

	// The extended editor data may be null, but is passed to the interface regardless
	const void* RawExtendedData = ChannelHandle.GetExtendedEditorData();

	if (EditorInterface && Channel)
	{
		FTrackInstancePropertyBindings* BindingsPtr = PropertyBindings.IsSet() ? &PropertyBindings.GetValue() : nullptr;
		return EditorInterface->AddOrUpdateKey_Raw(Channel, Section, RawExtendedData, Time, InSequencer, ObjectBindingID, BindingsPtr);
	}

	return FKeyHandle();
}

void IKeyArea::DeleteKeys(TArrayView<const FKeyHandle> InHandles, FFrameNumber InTime)
{
	ISequencerChannelInterface* EditorInterface = FindChannelEditorInterface();
	FMovieSceneChannel* Channel = ChannelHandle.Get();

	if (Channel)
	{
		EditorInterface->DeleteKeys_Raw(Channel, InHandles, InTime);
	}
}

FKeyHandle IKeyArea::DuplicateKey(FKeyHandle InKeyHandle) const
{
	FKeyHandle NewHandle = FKeyHandle::Invalid();

	if (FMovieSceneChannel* Channel = ChannelHandle.Get())
	{
		Channel->DuplicateKeys(TArrayView<const FKeyHandle>(&InKeyHandle, 1), TArrayView<FKeyHandle>(&NewHandle, 1));
	}

	return NewHandle;
}

void IKeyArea::SetKeyTimes(TArrayView<const FKeyHandle> InKeyHandles, TArrayView<const FFrameNumber> InKeyTimes) const
{
	check(InKeyHandles.Num() == InKeyTimes.Num());

	FMovieSceneChannel*               Channel  = ChannelHandle.Get();
	UMovieSceneSection*               Section  = GetOwningSection();
	const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();

	if (!Channel || !MetaData || !Section)
	{
		return;
	}

	FFrameNumber KeyOffset = MetaData->GetOffsetTime(Section);
	if (KeyOffset != 0)
	{
		// Need to copy the array to offset
		TArray<FFrameNumber> OffsetKeyTimes;
		OffsetKeyTimes.SetNumUninitialized(InKeyTimes.Num());
		for (int32 i = 0; i < InKeyTimes.Num(); ++i)
		{
			OffsetKeyTimes[i] = InKeyTimes[i] - KeyOffset;
		}

		Channel->SetKeyTimes(InKeyHandles, OffsetKeyTimes);
	}
	else
	{
		Channel->SetKeyTimes(InKeyHandles, InKeyTimes);
	}
}

void IKeyArea::GetKeyTimes(TArrayView<const FKeyHandle> InKeyHandles, TArrayView<FFrameNumber> OutTimes) const
{
	FMovieSceneChannel*               Channel  = ChannelHandle.Get();
	UMovieSceneSection*               Section  = GetOwningSection();
	const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();

	if (!Channel || !MetaData || !Section)
	{
		return;
	}

	Channel->GetKeyTimes(InKeyHandles, OutTimes);

	FFrameNumber KeyOffset = MetaData->GetOffsetTime(Section);
	if (KeyOffset != 0)
	{
		for (int32 i = 0; i < OutTimes.Num(); ++i)
		{
			OutTimes[i] += KeyOffset;
		}
	}
}

void IKeyArea::GetKeyInfo(TArray<FKeyHandle>* OutHandles, TArray<FFrameNumber>* OutTimes, const TRange<FFrameNumber>& WithinRange) const
{
	FMovieSceneChannel*               Channel  = ChannelHandle.Get();
	UMovieSceneSection*               Section  = GetOwningSection();
	const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();

	if (!Channel || !MetaData || !Section)
	{
		return;
	}

	FFrameNumber KeyOffset = MetaData->GetOffsetTime(Section);

	TRange<FFrameNumber> OffsetRange = WithinRange;
	if (KeyOffset != 0)
	{
		if (OffsetRange.HasLowerBound())
		{
			OffsetRange.SetLowerBoundValue(OffsetRange.GetLowerBoundValue() - KeyOffset);
		}
		if (OffsetRange.HasUpperBound())
		{
			OffsetRange.SetUpperBoundValue(OffsetRange.GetUpperBoundValue() - KeyOffset);
		}
	}

	Channel->GetKeys(OffsetRange, OutTimes, OutHandles);

	if (OutTimes && KeyOffset != 0)
	{
		const int32 Num = OutTimes->Num();
		for (int32 i = 0; i < Num; ++i)
		{
			(*OutTimes)[i] += KeyOffset;
		}
	}
}

TSharedPtr<FStructOnScope> IKeyArea::GetKeyStruct(FKeyHandle KeyHandle) const
{
	ISequencerChannelInterface* EditorInterface = FindChannelEditorInterface();
	if (EditorInterface)
	{
		return EditorInterface->GetKeyStruct_Raw(ChannelHandle, KeyHandle);
	}
	return nullptr;
}

int32 IKeyArea::DrawExtra(const FSequencerChannelPaintArgs& PaintArgs, int32 LayerId) const
{
	ISequencerChannelInterface* EditorInterface = FindChannelEditorInterface();
	if (EditorInterface)
	{
		FMovieSceneChannel* Channel = ChannelHandle.Get();
		const UMovieSceneSection* OwningSection = GetOwningSection();
		return EditorInterface->DrawExtra_Raw(Channel, OwningSection, PaintArgs, LayerId);
	}
	return LayerId;
}

void IKeyArea::DrawKeys(TArrayView<const FKeyHandle> InKeyHandles, TArrayView<FKeyDrawParams> OutKeyDrawParams)
{
	check(InKeyHandles.Num() == OutKeyDrawParams.Num());

	ISequencerChannelInterface* EditorInterface = FindChannelEditorInterface();
	FMovieSceneChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection* OwningSection = GetOwningSection();

	if (EditorInterface && Channel && OwningSection)
	{
		return EditorInterface->DrawKeys_Raw(Channel, InKeyHandles, OwningSection, OutKeyDrawParams);
	}
}

bool IKeyArea::CanCreateKeyEditor() const
{
	ISequencerChannelInterface* EditorInterface = FindChannelEditorInterface();
	const FMovieSceneChannel* Channel = ChannelHandle.Get();

	return EditorInterface && Channel && EditorInterface->CanCreateKeyEditor_Raw(Channel);
}

TSharedRef<SWidget> IKeyArea::CreateKeyEditor(TWeakPtr<ISequencer> Sequencer, const FGuid& ObjectBindingID)
{
	using namespace UE::Sequencer;

	ISequencerChannelInterface* EditorInterface = FindChannelEditorInterface();
	UMovieSceneSection* OwningSection = GetOwningSection();

	TWeakPtr<FTrackInstancePropertyBindings> PropertyBindingsPtr;
	if (PropertyBindings.IsSet())
	{
		PropertyBindingsPtr = TSharedPtr<FTrackInstancePropertyBindings>(AsShared(), &PropertyBindings.GetValue());
	}

	const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();
	if (EditorInterface && OwningSection && MetaData)
	{
		// Let decoration editors customize key editors for their channels
		UObject* OwningObject = MetaData->WeakOwningObject.Get();
		if (OwningObject && OwningObject != OwningSection)
		{
			if (TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin())
			{
				if (ISequencerDecorationEditor* DecorationEditor = SequencerPtr->FindDecorationEditor(OwningObject->GetClass()))
				{
					if (TSharedPtr<SWidget> CustomEditor = DecorationEditor->CreateKeyEditor(*OwningObject, *OwningSection, ChannelHandle, Sequencer, ObjectBindingID))
					{
						return CustomEditor.ToSharedRef();
					}
				}
			}
		}

		FCreateKeyEditorParams Params{
			OwningSection,
			OwningObject,
			Sequencer.Pin().ToSharedRef(),
			ObjectBindingID,
			PropertyBindingsPtr
		};
		return EditorInterface->CreateKeyEditor_Raw(ChannelHandle, Params);
	}
	return SNullWidget::NullWidget;
}

void IKeyArea::CopyKeys(FMovieSceneClipboardBuilder& ClipboardBuilder, TArrayView<const FKeyHandle> KeyMask) const
{
	ISequencerChannelInterface* EditorInterface = FindChannelEditorInterface();
	FMovieSceneChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection* OwningSection = GetOwningSection();
	const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();

	if (EditorInterface && Channel && OwningSection && MetaData)
	{
		TGuardValue<FFrameNumber> GuardKeyOffset(ClipboardBuilder.KeyOffset, ClipboardBuilder.KeyOffset + MetaData->GetOffsetTime(OwningSection));

		EditorInterface->CopyKeys_Raw(Channel, OwningSection, ChannelName, ClipboardBuilder, KeyMask);
	}
}

TArray<FKeyHandle> IKeyArea::PasteKeys(const FMovieSceneClipboardKeyTrack& KeyTrack, const FMovieSceneClipboardEnvironment& SrcEnvironment, const FSequencerPasteEnvironment& DstEnvironment)
{
	TArray<FKeyHandle> PastedKeys;

	ISequencerChannelInterface* EditorInterface = FindChannelEditorInterface();
	FMovieSceneChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection* OwningSection = GetOwningSection();
	UObject*            OwningObject = GetOwningObject();
	const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();

	if (EditorInterface && Channel && OwningSection && OwningObject && MetaData)
	{
		if (OwningSection->IsReadOnly())
		{
			return PastedKeys;
		}

		OwningObject->Modify();

		EditorInterface->PasteKeys_Raw(Channel, OwningSection, KeyTrack, SrcEnvironment, DstEnvironment, PastedKeys);

		FFrameNumber KeyOffset = MetaData->GetOffsetTime(OwningSection);
		if (KeyOffset != 0)
		{
			TArray<FFrameNumber> KeyTimes;
			KeyTimes.SetNumUninitialized(PastedKeys.Num());

			Channel->GetKeyTimes(PastedKeys, KeyTimes);
			for (int32 i = 0; i < KeyTimes.Num(); ++i)
			{
				KeyTimes[i] += KeyOffset;
			}
			Channel->SetKeyTimes(PastedKeys, KeyTimes);
		}
	}

	return PastedKeys;
}

FText GetOwningObjectBindingName(UMovieSceneTrack* InTrack, ISequencer& InSequencer)
{
	check(InTrack);

	UMovieSceneSequence* FocusedSequence = InSequencer.GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = FocusedSequence->GetMovieScene();

	FGuid PossessableGuid;
	if (MovieScene->FindTrackBinding(*InTrack, PossessableGuid))
	{
		return MovieScene->GetObjectDisplayName(PossessableGuid);
	}

	// Couldn't find an owning track, so must not be nestled inside of something!
	return FText();
}

namespace UE
{
namespace MovieScene
{
	void CleanUpCurveEditorFormatStringInline(FString& InStr)
	{
		for (int32 Index = 0; Index < InStr.GetCharArray().Num() - 1; Index++)
		{
			TCHAR CurrentChar = InStr.GetCharArray()[Index];
			TCHAR NextChar = InStr.GetCharArray()[Index + 1];
			if (CurrentChar == '.' && NextChar == '.')
			{
				const int32 NumToRemove = 1;
				InStr.RemoveAt(Index, NumToRemove, EAllowShrinking::No);
				Index--;
			}
		}

		// We can still end up with a "." at the end using the above logic
		if (InStr.GetCharArray()[InStr.GetCharArray().Num() - 1] == '.')
		{
			InStr.RemoveAt(InStr.GetCharArray().Num() - 1);
		}
	}
}}

TUniquePtr<FCurveModel> IKeyArea::CreateCurveEditorModel(TSharedRef<ISequencer> InSequencer) const
{
	ISequencerChannelInterface* EditorInterface = FindChannelEditorInterface();
	UMovieSceneSection* OwningSection = GetOwningSection();
	if (EditorInterface && OwningSection && ChannelHandle.Get() != nullptr)
	{
		const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();

		UE::Sequencer::FCreateCurveEditorModelParams Params = {
			OwningSection,
			MetaData->WeakOwningObject.Get(),
			InSequencer
		};

		TUniquePtr<FCurveModel> CurveModel = EditorInterface->CreateCurveEditorModel_Raw(ChannelHandle, Params);
		if (CurveModel.IsValid())
		{
			// Build long, short and context names for this curve to maximize information shown in the Curve Editor UI.
			UMovieSceneTrack* OwningTrack = OwningSection->GetTypedOuter<UMovieSceneTrack>();
			FText OwningTrackName;
			FText ObjectBindingName;

			if (OwningTrack)
			{
				OwningTrackName = OwningTrack->GetDisplayName();
				
				// This track might be inside an object binding and we'd like to prepend the object binding's name for more context.
				ObjectBindingName = GetOwningObjectBindingName(OwningTrack, *InSequencer);
			}
			
			// This is a FText (even though the end result is a FString) because all the incoming data sources are FText
			// And it's probably cheaper to convert one FText->FString instead of 4 shorter ones.
			FFormatNamedArguments FormatArgs;

			// Not all tracks have all the information so we need to format it differently depending on how many are valid.
			FormatArgs.Add(TEXT("ObjectBindingName"), ObjectBindingName);
			FormatArgs.Add(TEXT("OwningTrackName"), OwningTrackName);
			FormatArgs.Add(TEXT("GroupName"), MetaData->Group);
			FormatArgs.Add(TEXT("DisplayName"), DisplayText);

			FText FormatText = NSLOCTEXT("SequencerIKeyArea", "CurveLongDisplayNameFormat", "{ObjectBindingName}.{OwningTrackName}.{GroupName}.{DisplayName}");
			FString FormattedText = FText::Format(FormatText, FormatArgs).ToString();

			UE::MovieScene::CleanUpCurveEditorFormatStringInline(FormattedText);

			FText LongDisplayName = FText::FromString(FormattedText);
			const FText ShortDisplayName = DisplayText;
			FString IntentName = MetaData->IntentName.ToString();
			if (IntentName.IsEmpty())
			{
				IntentName = MetaData->Group.IsEmptyOrWhitespace() ? DisplayText.ToString() : FString::Format(TEXT("{0}.{1}"), { MetaData->Group.ToString(), DisplayText.ToString() });
			}

			FText LongIntentNameFormat = MetaData->LongIntentNameFormat;
			if (LongIntentNameFormat.IsEmpty())
			{
				LongIntentNameFormat = NSLOCTEXT("SequencerIKeyArea", "LongIntentNameFormat", "{ObjectBindingName}.{GroupName}.{DisplayName}");
			}
			
			FormatArgs.Add(TEXT("IntentName"), FText::FromString(IntentName));


			FString LongIntentName = FText::Format(LongIntentNameFormat, FormatArgs).ToString();
			UE::MovieScene::CleanUpCurveEditorFormatStringInline(LongIntentName);


			CurveModel->SetShortDisplayName(DisplayText);
			CurveModel->SetLongDisplayName(LongDisplayName);
			CurveModel->SetIntentionName(IntentName);
			CurveModel->SetLongIntentionName(LongIntentName);
			CurveModel->SetChannelName(MetaData->Name);
			if (Color.IsSet())
			{
				CurveModel->SetColor(Color.GetValue());
			}

			//Use editor preference color if it's been set, this function is const so we can't set just Color optional
			const UCurveEditorSettings* Settings = GetDefault<UCurveEditorSettings>();
			UObject* Object = nullptr;
			FString Name;
			CurveModel->GetCurveColorObjectAndName(&Object, Name);
			if (Object)
			{
				TOptional<FLinearColor> SettingColor = Settings->GetCustomColor(Object->GetClass(), Name);
				if (SettingColor.IsSet())
				{
					CurveModel->SetColor(SettingColor.GetValue());
				}
			}
		}
		return CurveModel;
	}

	return nullptr;
}

bool IKeyArea::ShouldShowCurve() const
{
	ISequencerChannelInterface* EditorInterface = FindChannelEditorInterface();
	FMovieSceneChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection* OwningSection = GetOwningSection();
	if (EditorInterface && Channel && OwningSection)
	{
		return EditorInterface->ShouldShowCurve_Raw(Channel, OwningSection);
	}

	return false;
}

EMovieSceneKeyInterpolation IKeyArea::GetInterpolationMode(FFrameNumber InTime, EMovieSceneKeyInterpolation DefaultMode) const
{
	ISequencerChannelInterface* EditorInterface = FindChannelEditorInterface();
	FMovieSceneChannel* Channel = ChannelHandle.Get();
	if (EditorInterface && Channel)
	{
		return EditorInterface->GetInterpolationMode_Raw(Channel, InTime, DefaultMode);
	}

	return DefaultMode;
}
