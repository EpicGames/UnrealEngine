// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISequencer.h"

#include "AnimatedRange.h"
#include "Camera/CameraComponent.h"
#include "Capabilities/CameraCutViewTargetCacheCapability.h"
#include "Misc/AssertionMacros.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "ITimeSlider.h"
#include "SequencerUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ISequencer)

TWeakObjectPtr<UCameraComponent> ISequencer::GetLastEvaluatedCameraCut() const
{
	const TSharedPtr<const UE::MovieScene::FSharedPlaybackState> PlaybackState = FindSharedPlaybackState();
	const UE::MovieScene::FCameraCutViewTargetCacheCapability* Capability = PlaybackState
	   ? PlaybackState->FindCapability<UE::MovieScene::FCameraCutViewTargetCacheCapability>() : nullptr;
	return Capability ? Capability->LastViewTargetCamera.Get() : nullptr;
}

FAnimatedRange ISequencer::GetViewRange() const
{
	return FAnimatedRange();
}

FFrameRate ISequencer::GetRootTickResolution() const
{
	UMovieSceneSequence* RootSequence = GetRootMovieSceneSequence();
	if (RootSequence)
	{
		return RootSequence->GetMovieScene()->GetTickResolution();
	}

	ensureMsgf(false, TEXT("No valid sequence found."));
	return FFrameRate();
}

FFrameRate ISequencer::GetRootDisplayRate() const
{
	UMovieSceneSequence* RootSequence = GetRootMovieSceneSequence();
	if (RootSequence)
	{
		return RootSequence->GetMovieScene()->GetDisplayRate();
	}

	ensureMsgf(false, TEXT("No valid sequence found."));
	return FFrameRate();
}

FFrameRate ISequencer::GetFocusedTickResolution() const
{
	UMovieSceneSequence* FocusedSequence = GetFocusedMovieSceneSequence();
	if (FocusedSequence)
	{
		return FocusedSequence->GetMovieScene()->GetTickResolution();
	}

	ensureMsgf(false, TEXT("No valid sequence found."));
	return FFrameRate();
}

FFrameRate ISequencer::GetFocusedDisplayRate() const
{
	UMovieSceneSequence* FocusedSequence = GetFocusedMovieSceneSequence();
	if (FocusedSequence)
	{
		return FocusedSequence->GetMovieScene()->GetDisplayRate();
	}

	ensureMsgf(false, TEXT("No valid sequence found."));
	return FFrameRate();
}

FGuid ISequencer::CreateBinding(UObject& InObject, const FString& InName)
{
	UE::Sequencer::FCreateBindingParams BindingParams;
	BindingParams.BindingNameOverride = InName;
	return CreateBinding(InObject, BindingParams);
}

FGuid ISequencer::CreateBinding(UMovieSceneSequence* InSequence, UObject* InObject)
{
	if (GetFocusedMovieSceneSequence() == InSequence && InObject)
	{
		UE::Sequencer::FCreateBindingParams BindingParams;
		BindingParams.bAllowCustomBinding = true;
		return CreateBinding(*InObject, BindingParams);
	}
	return IMovieScenePlayer::CreateBinding(InSequence, InObject);
}
