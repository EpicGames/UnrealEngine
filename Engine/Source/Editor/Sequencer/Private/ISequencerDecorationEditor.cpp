// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISequencerDecorationEditor.h"

#include "ISequencerModule.h"
#include "ISequencerChannelInterface.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneSection.h"
#include "MovieSceneSignedObject.h"

FKeyHandle ISequencerDecorationEditor::AddOrUpdateKey(
	UObject& Decoration,
	UMovieSceneSection& Section,
	FMovieSceneChannelHandle ChannelHandle,
	FFrameNumber Time,
	const FGuid& ObjectBindingID,
	ISequencer& InSequencer)
{
	FMovieSceneChannel* Channel = ChannelHandle.Get();
	if (!Channel)
	{
		return FKeyHandle();
	}

	// Apply time offset from channel metadata
	if (const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData())
	{
		Time -= MetaData->GetOffsetTime(&Section);
	}

	// Find the channel editor interface for this channel type
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	ISequencerChannelInterface* EditorInterface = SequencerModule.FindChannelEditorInterface(ChannelHandle.GetChannelTypeName());

	if (EditorInterface)
	{
		if (UMovieSceneSignedObject* SignedDecoration = Cast<UMovieSceneSignedObject>(&Decoration))
		{
			SignedDecoration->Modify();
		}
		const void* RawExtendedData = ChannelHandle.GetExtendedEditorData();
		return EditorInterface->AddOrUpdateKey_Raw(Channel, &Section, RawExtendedData, Time, InSequencer, ObjectBindingID, nullptr);
	}

	return FKeyHandle();
}
