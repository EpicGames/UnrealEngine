// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISequencerNumericTypeInterface.h"
#include "PropertyHandle.h"

namespace UE::Sequencer
{

int32 FSequencerNumericTypeInterface::GetRelevancyScore(const ISequencer& Sequencer, TSharedPtr<IPropertyHandle> Property) const
{
	if (Property)
	{
		const FString& MetaUIDisplayAsString = Property->GetMetaData(TEXT("UIFrameDisplayAs"));

		const bool bIsDurationProperty = MetaUIDisplayAsString.Contains(TEXT("Duration"), ESearchCase::IgnoreCase);

		// Don't use non-duration intents for duration properties and visa-versa
		if ((Intent == ENumericIntent::Duration) == bIsDurationProperty)
		{
			return 100;
		}
	}

	// By default, prefer position type interfaces.
	return Intent == ENumericIntent::Position ? 10 : 0;
}

} // namespace UE::Sequencer