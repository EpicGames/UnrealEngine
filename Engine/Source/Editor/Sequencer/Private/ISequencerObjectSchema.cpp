// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISequencerObjectSchema.h"
#include "Internationalization/Text.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"

namespace UE::Sequencer
{

bool operator>(const FObjectSchemaRelevancy& A, const FObjectSchemaRelevancy& B)
{
	if (A.Class != nullptr)
	{
		if (B.Class == nullptr)
		{
			return true;
		}
		if (A.Class == B.Class)
		{
			return A.Priority > B.Priority;
		}
		return A.Class->IsChildOf(B.Class);
	}
	else if (B.Class != nullptr)
	{
		return false;
	}
	return A.Priority > B.Priority;
}

TMap<const IObjectSchema*, TArray<UObject*>> IObjectSchema::ComputeRelevancy(TArrayView<UObject* const> InObjects)
{
	struct FRelevancyData
	{
		const IObjectSchema* MostRelevantSchema = nullptr;
		FObjectSchemaRelevancy Relevancy;
	};

	TMap<const IObjectSchema*, TArray<UObject*>> Result;

	ISequencerModule& SequencerModule = FModuleManager::GetModuleChecked<ISequencerModule>("Sequencer");
	TArrayView<const TSharedPtr<UE::Sequencer::IObjectSchema>> Schemas = SequencerModule.GetObjectSchemas();

	for (UObject* Object : InObjects)
	{
		const IObjectSchema* MostRelevantSchema = nullptr;

		FObjectSchemaRelevancy BestRelevancy;
		for (const TSharedPtr<UE::Sequencer::IObjectSchema>& Schema : Schemas)
		{
			FObjectSchemaRelevancy Relevancy = Schema->GetRelevancy(Object);
			if (Relevancy > BestRelevancy)
			{
				MostRelevantSchema = Schema.Get();
				BestRelevancy = Relevancy;
			}
		}

		if (MostRelevantSchema)
		{
			Result.FindOrAdd(MostRelevantSchema).Add(Object);
		}
	}

	return Result;
}

FText IObjectSchema::GetPrettyName(const UObject* Object) const
{
	return FText::FromString(Object->GetName());
}

UObject* IObjectSchema::GetRedirectedBoundObject(UObject* Object) const
{
	return Object;
}

} // namespace UE::Sequencer