// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISequencerSection.h"

#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Decorations/IMovieSceneChannelDecoration.h"
#include "HAL/PlatformCrt.h"
#include "IKeyArea.h"
#include "ISectionLayoutBuilder.h"
#include "Math/NumericLimits.h"
#include "Math/Range.h"
#include "Math/RangeBound.h"
#include "Math/UnrealMathSSE.h"
#include "MovieSceneSection.h"
#include "SequencerSectionPainter.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "ISequencerChannelInterface.h"
#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"
#include "MVVM/ViewModels/ViewDensity.h"
#include "Widgets/SOverlay.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ISequencerSection)

struct FMovieSceneChannel;

/** Data pertaining to a group of channels */
struct FGroupData
{
	FGroupData(FText InGroupText, FGetMovieSceneTooltipText InGetGroupTooltipTextDelegate)
		: GroupText(InGroupText)
		, GetGroupTooltipTextDelegate(InGetGroupTooltipTextDelegate)
		, SortOrder(-1)
		, bSortEmptyGroupsLast(true)
	{}

	void AddChannel(ISequencerSection::FChannelData&& InChannel)
	{
		if (InChannel.MetaData.SortOrder < SortOrder)
		{
			SortOrder = InChannel.MetaData.SortOrder;
		}
		bSortEmptyGroupsLast = InChannel.MetaData.bSortEmptyGroupsLast;

		Channels.Add(MoveTemp(InChannel));
	}

	/** Text to display for the group */
	FText GroupText;
	
	/** Getter for text to display for the group tooltip */
	FGetMovieSceneTooltipText GetGroupTooltipTextDelegate;

	/** Sort order of the group */
	uint32 SortOrder;

	/** By default if a channel has no FText::Group specified, we put it last, by setting this to false we use SortIndex instead */
	bool bSortEmptyGroupsLast;

	/** Array of channels within this group */
	TArray<ISequencerSection::FChannelData, TInlineAllocator<4>> Channels;
};

void ISequencerSection::GenerateSectionLayout( ISectionLayoutBuilder& LayoutBuilder )
{
	using namespace UE::Sequencer;

	UMovieSceneSection* Section = GetSectionObject();
	if (!Section)
	{
		return;
	}

	// Group channels by their group name
	TMap<FName, FGroupData> GroupToChannelsMap;

	FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
	for (const FMovieSceneChannelEntry& Entry : Section->GetChannelProxy().GetAllEntries())
	{
		const FName ChannelTypeName = Entry.GetChannelTypeName();

		// One editor data ptr per channel
		TArrayView<FMovieSceneChannel* const>        Channels    = Entry.GetChannels();
		TArrayView<const FMovieSceneChannelMetaData> AllMetaData = Entry.GetMetaData();

		for (int32 Index = 0; Index < Channels.Num(); ++Index)
		{
			FMovieSceneChannelHandle Channel = ChannelProxy.MakeHandle(ChannelTypeName, Index);

			const FMovieSceneChannelMetaData& MetaData = AllMetaData[Index];
			if (MetaData.bEnabled)
			{
				// Skip channels owned by decorations - they're handled by FDecorationOutlinerModel
				if (UObject* OwningObject = MetaData.WeakOwningObject.Get())
				{
					if (OwningObject->Implements<UMovieSceneChannelDecoration>())
					{
						continue;
					}
				}

				FName GroupName = *MetaData.Group.ToString();

				FGroupData* ExistingGroup = GroupToChannelsMap.Find(GroupName);
				if (!ExistingGroup)
				{
					FText GroupDisplayName = FText::FromString(MetaData.GetPropertyMetaData(FCommonChannelData::GroupDisplayName));
					if (GroupDisplayName.IsEmpty())
					{
						GroupDisplayName = FText::FromName(GroupName);
					}
					ExistingGroup = &GroupToChannelsMap.Add(GroupName, FGroupData(GroupDisplayName, MetaData.GetGroupTooltipTextDelegate));
				}

				ExistingGroup->AddChannel(FChannelData{ Channel, MetaData });
			}
		}
	}

	if (GroupToChannelsMap.Num() == 0)
	{
		return;
	}


	ISequencerModule* SequencerModule = &FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");

	auto ChannelFactory = [this, SequencerModule](FName InChannelName, const FSectionModel& InSection, const FMovieSceneChannelHandle& InChannel)
	{
		TSharedPtr<FChannelModel> ChannelModel = this->ConstructChannelModel(InChannelName, InChannel);
		if (!ChannelModel)
		{
			ISequencerChannelInterface* EditorInterface = SequencerModule->FindChannelEditorInterface(InChannel.GetChannelTypeName());
			if (EditorInterface)
			{
				ChannelModel = EditorInterface->CreateChannelModel_Raw(InChannel, InSection, InChannelName);
			}
		}

		return ChannelModel;
	};

	// Collapse single channels to the top level track node if allowed
	if (GroupToChannelsMap.Num() == 1)
	{
		const TTuple<FName, FGroupData>& Pair = *GroupToChannelsMap.CreateIterator();
		if (Pair.Value.Channels.Num() == 1 && Pair.Value.Channels[0].MetaData.bCanCollapseToTrack)
		{
			LayoutBuilder.SetTopLevelChannel(Pair.Value.Channels[0].Channel, ChannelFactory);
			return;
		}
	}

	// Sort the channels in each group by its sort order and name
	TArray<FName, TInlineAllocator<6>> SortedGroupNames;
	for (TPair<FName, FGroupData>& Pair : GroupToChannelsMap)
	{
		SortedGroupNames.Add(Pair.Key);

		// Sort by sort order then name
		Pair.Value.Channels.Sort([](const FChannelData& A, const FChannelData& B){
			if (A.MetaData.SortOrder == B.MetaData.SortOrder)
			{
				return A.MetaData.Name.LexicalLess(B.MetaData.Name);
			}
			return A.MetaData.SortOrder < B.MetaData.SortOrder;
		});
	}

	// Sort groups by the lowest sort order in each group
	auto SortPredicate = [&GroupToChannelsMap](FName A, FName B)
	{
		if (A.IsNone())
		{
			const bool bSortEmptyGroupsLast = GroupToChannelsMap.FindChecked(A).bSortEmptyGroupsLast;
			if(bSortEmptyGroupsLast)
			{ 
				return false;
			}
		}
		else if (B.IsNone())
		{
			const bool bSortEmptyGroupsLast = GroupToChannelsMap.FindChecked(B).bSortEmptyGroupsLast;
			if (bSortEmptyGroupsLast)
			{
				return true;
			}
		}


		const int32 SortOrderA = GroupToChannelsMap.FindChecked(A).SortOrder;
		const int32 SortOrderB = GroupToChannelsMap.FindChecked(B).SortOrder;
		return SortOrderA < SortOrderB;
	};
	SortedGroupNames.Sort(SortPredicate);

	// Create key areas for each group name
	for (FName GroupName : SortedGroupNames)
	{
		FGroupData& ChannelData = GroupToChannelsMap.FindChecked(GroupName);

		if (!GroupName.IsNone())
		{
			auto Factory = [this, &ChannelData](FName InCategoryName, const FText& InDisplayText)
			{
				return this->ConstructCategoryModel(InCategoryName, InDisplayText, ChannelData.Channels);
			};

			LayoutBuilder.PushCategory(GroupName, ChannelData.GroupText, ChannelData.GetGroupTooltipTextDelegate, Factory);
		}

		for (const FChannelData& ChannelAndData : ChannelData.Channels)
		{
			LayoutBuilder.AddChannel(ChannelAndData.Channel, ChannelFactory);
		}

		if (!GroupName.IsNone())
		{
			LayoutBuilder.PopCategory();
		}
	}
}

void ISequencerSection::CreateViewWidgets(const UE::Sequencer::FCreateSectionViewWidgetParams& Params)
{
	TSharedRef<SWidget> LegacyGeneratedWidget = GenerateSectionWidget();
	if (LegacyGeneratedWidget != SNullWidget::NullWidget)
	{
		Params.Overlay->AddSlot(UE::Sequencer::FCreateSectionViewWidgetParams::DefaultWidgetOrder)
		[
			LegacyGeneratedWidget
		];
	}
}

float ISequencerSection::GetSectionHeight() const
{
	return SequencerSectionConstants::DefaultSectionHeight;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
float ISequencerSection::GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const
{
	// Call the deprecated method
	float Height = GetSectionHeight();
	if (Height != SequencerSectionConstants::DefaultSectionHeight)
	{
		// Override the uniform height for some sections
		return Height;
	}
	return ViewDensity.UniformHeight.Get(Height);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void ISequencerSection::ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeFrameNumber)
{
	UMovieSceneSection* SectionObject = GetSectionObject();
	if (ResizeMode == ESequencerSectionResizeMode::SSRM_LeadingEdge)
	{
		FFrameNumber MaxFrame = SectionObject->HasEndFrame() ? SectionObject->GetExclusiveEndFrame()-1 : TNumericLimits<int32>::Max();
		ResizeFrameNumber = FMath::Min( ResizeFrameNumber, MaxFrame );

		SectionObject->SetRange(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(ResizeFrameNumber), SectionObject->GetRange().GetUpperBound()));
	}
	else
	{
		FFrameNumber MinFrame = SectionObject->HasStartFrame() ? SectionObject->GetInclusiveStartFrame() : TNumericLimits<int32>::Lowest();
		ResizeFrameNumber = FMath::Max( ResizeFrameNumber, MinFrame );

		SectionObject->SetRange(TRange<FFrameNumber>(SectionObject->GetRange().GetLowerBound(), TRangeBound<FFrameNumber>::Exclusive(ResizeFrameNumber)));
	}
}

int32 FSequencerSection::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	return Painter.PaintSectionBackground();
}
