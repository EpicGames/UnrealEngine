// Copyright Epic Games, Inc. All Rights Reserved.

#include "KeyPropertyParams.h"

#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "PropertyHandle.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "SequencerCommonHelpers.h"

FCanKeyPropertyParams::FCanKeyPropertyParams(const UClass* InObjectClass, const FPropertyPath& InPropertyPath)
	: ObjectClass(InObjectClass)
	, PropertyPath(InPropertyPath)
{
}

FCanKeyPropertyParams::FCanKeyPropertyParams(const UClass* InObjectClass, const IPropertyHandle& InPropertyHandle)
	: ObjectClass(InObjectClass)
	, PropertyPath(SequencerHelpers::PropertyHandleToPropertyPath(InPropertyHandle))
{
}

const UStruct* FCanKeyPropertyParams::FindPropertyOwner(const FProperty* ForProperty) const
{
	check(ForProperty);

	bool bFoundProperty = false;
	for (int32 Index = PropertyPath.GetNumProperties() - 1; Index >= 0; --Index)
	{
		const FProperty* Property = PropertyPath.GetPropertyInfo(Index).Property.Get();
		if (!bFoundProperty)
		{
			bFoundProperty = Property == ForProperty;
			if (bFoundProperty)
			{
				return Property->GetOwnerStruct();
			}
		}
		else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(Property))
		{
			return StructProperty->Struct;
		}
	}
	return ObjectClass;
}

const UStruct* FCanKeyPropertyParams::FindPropertyContainer(const FProperty* ForProperty) const
{
	check(ForProperty);

	bool bFoundProperty = false;
	for (int32 Index = PropertyPath.GetNumProperties() - 1; Index >= 0; --Index)
	{
		const FProperty* Property = PropertyPath.GetPropertyInfo(Index).Property.Get();
		if (!bFoundProperty)
		{
			bFoundProperty = Property == ForProperty;
			if (bFoundProperty)
			{
				return ObjectClass;
			}
		}
		else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(Property))
		{
			return StructProperty->Struct;
		}
	}
	return ObjectClass;
}

FKeyPropertyParams::FKeyPropertyParams(TArray<UObject*> InObjectsToKey, const FPropertyPath& InPropertyPath, ESequencerKeyMode InKeyMode)
	: ObjectsToKey(InObjectsToKey)
	, PropertyPath(InPropertyPath)
	, KeyMode(InKeyMode)

{
}

FKeyPropertyParams::FKeyPropertyParams(TArray<UObject*> InObjectsToKey, const IPropertyHandle& InPropertyHandle, ESequencerKeyMode InKeyMode)
	: ObjectsToKey(InObjectsToKey)
	, PropertyPath(InObjectsToKey.Num() > 0 ? SequencerHelpers::PropertyHandleToPropertyPath(InPropertyHandle) : FPropertyPath())
	, KeyMode(InKeyMode)
{
}

FPropertyChangedParams::FPropertyChangedParams(TArray<UObject*> InObjectsThatChanged, const FPropertyPath& InPropertyPath, const FPropertyPath& InStructPathToKey, ESequencerKeyMode InKeyMode)
	: ObjectsThatChanged(InObjectsThatChanged)
	, PropertyPath(InPropertyPath)
	, StructPathToKey(InStructPathToKey)
	, KeyMode(InKeyMode)
{
}

TPair<const FProperty*, UE::MovieScene::FSourcePropertyValue> FPropertyChangedParams::GetPropertyAndValue() const
{
	using namespace UE::MovieScene;

	check(ObjectsThatChanged[0]);

	void* ContainerPtr = ObjectsThatChanged[0];
	const FProperty* Property = nullptr;
	for (int32 i = 0; i < PropertyPath.GetNumProperties(); i++)
	{
		const FPropertyInfo& PropertyInfo = PropertyPath.GetPropertyInfo(i);
		Property = PropertyInfo.Property.Get();
		if (!Property)
		{
			return TPair<const FProperty*, FSourcePropertyValue>();
		}

		if (const FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property))
		{
			// Sometimes property paths have the array property twice, first with no array index,
			// then a second so we skip over this property if that's the case
			int32 ArrayIndex = FMath::Max(0, PropertyInfo.ArrayIndex);
			if (PropertyInfo.ArrayIndex == INDEX_NONE && i < PropertyPath.GetNumProperties()-1) //-V1051
			{
				const FPropertyInfo& InnerPropertyInfo = PropertyPath.GetPropertyInfo(i+1);
				const FProperty* InnerProperty = InnerPropertyInfo.Property.Get();
				if (InnerProperty && InnerProperty->GetOwner<FProperty>() == ArrayProp)
				{
					ArrayIndex = InnerPropertyInfo.ArrayIndex;
					++i;
				}
			}

			FScriptArrayHelper ParentArrayHelper(ArrayProp, ArrayProp->ContainerPtrToValuePtr<void>(ContainerPtr));
			if (!ParentArrayHelper.IsValidIndex(ArrayIndex))
			{
				return TPair<const FProperty*, FSourcePropertyValue>(nullptr, FSourcePropertyValue());
			}
			ContainerPtr = ParentArrayHelper.GetRawPtr(ArrayIndex);
		}
		else if (const FStructProperty* StructProp = CastField<FStructProperty>(Property))
		{
			if (StructProp->Struct == FInstancedStruct::StaticStruct())
			{
				FInstancedStruct* InstancedStruct = Property->ContainerPtrToValuePtr<FInstancedStruct>(ContainerPtr);
				ContainerPtr = InstancedStruct->GetMutableMemory();
			}
			else
			{
				ContainerPtr = Property->ContainerPtrToValuePtr<void>(ContainerPtr);
			}
		}
		else
		{
			ContainerPtr = Property->ContainerPtrToValuePtr<void>(ContainerPtr);
		}
	}


	// Bool property values are stored in a bit field so using a straight cast of the pointer to get their value does not
	// work.  Instead use the actual property to get the correct value.
	if (const FBoolProperty* BoolProperty = CastField<const FBoolProperty>(Property))
	{
		return MakeTuple(BoolProperty, FSourcePropertyValue::FromValue(BoolProperty->GetPropertyValue(ContainerPtr)));
	}
	// Object properties might have various different types of storage, but we always expose them as a raw ptr
	else if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property))
	{
		return MakeTuple(ObjectProperty, FSourcePropertyValue::FromValue(ObjectProperty->GetObjectPropertyValue(ContainerPtr)));
	}

	check(Property);
	return MakeTuple(Property, FSourcePropertyValue::FromAddress(ContainerPtr, *Property));
}

UE::MovieScene::FSourcePropertyValue FPropertyChangedParams::GetPropertyValue() const
{
	return GetPropertyAndValue().Value;
}

FString FPropertyChangedParams::GetPropertyPathString() const
{
	return PropertyPath.ToString(TEXT("."));
}
