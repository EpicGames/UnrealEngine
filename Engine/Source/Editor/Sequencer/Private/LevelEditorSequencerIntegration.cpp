// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditorSequencerIntegration.h"

#include "SequencerEdMode.h"
#include "Styling/SlateIconFinder.h"
#include "PropertyHandle.h"
#include "IDetailKeyframeHandler.h"
#include "IDetailTreeNode.h"
#include "GameDelegates.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "PropertyEditorModule.h"
#include "ILevelEditor.h"
#include "IAssetViewport.h"
#include "LevelEditor.h"
#include "SLevelViewport.h"
#include "MovieSceneSpawnableAnnotation.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailsView.h"
#include "ISequencer.h"
#include "ISequencerTrackEditor.h"
#include "KeyPropertyParams.h"
#include "Engine/Selection.h"
#include "Sequencer.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "EditorModeManager.h"
#include "SequencerCommands.h"
#include "Sections/MovieSceneSubSection.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "MovieSceneSequenceID.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "SequencerSettings.h"
#include "SequencerInfoColumn.h"
#include "SequencerSpawnableColumn.h"
#include "LevelEditorViewport.h"
#include "Modules/ModuleManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PropertyEditorModule.h"
#include "SceneOutlinerFwd.h"
#include "SceneOutlinerModule.h"
#include "SceneOutlinerPublicTypes.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "UObject/ObjectKey.h"
#include "UObject/ObjectSaveContext.h"
#include "UnrealEdGlobals.h"
#include "UnrealEdMisc.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorSupportDelegates.h"
#include "Subsystems/UnrealEditorSubsystem.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelEditorSequencerIntegration)

#define LOCTEXT_NAMESPACE "LevelEditorSequencerIntegration"


struct FPilotedSpawnable
{
	TWeakPtr<SLevelViewport> WeakLevelViewport;
	FLevelViewportActorLock PreviousActorLock;
	FMovieSceneSpawnableAnnotation Annotation;
};

class FDetailKeyframeHandlerWrapper : public IDetailKeyframeHandler
{
public:

	void Add(TWeakPtr<ISequencer> InSequencer)
	{
		Sequencers.Add(InSequencer);
	}

	void Remove(TWeakPtr<ISequencer> InSequencer)
	{
		Sequencers.Remove(InSequencer);
	}

	virtual bool IsPropertyKeyable(const UClass* InObjectClass, const IPropertyHandle& InPropertyHandle) const
	{
		FCanKeyPropertyParams CanKeyPropertyParams(InObjectClass, InPropertyHandle);

		for (const TWeakPtr<ISequencer>& WeakSequencer : Sequencers)
		{
			TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
			if (Sequencer.IsValid() && Sequencer->CanKeyProperty(CanKeyPropertyParams) && !Sequencer->IsReadOnly())
			{
				return true;
			}
		}
		return false;
	}

	virtual bool IsPropertyKeyingEnabled() const
	{
		for (const TWeakPtr<ISequencer>& WeakSequencer : Sequencers)
		{
			TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
			if (Sequencer.IsValid() && Sequencer->GetFocusedMovieSceneSequence() && Sequencer->GetAllowEditsMode() != EAllowEditsMode::AllowLevelEditsOnly)
			{
				return true;
			}
		}
		return false;
	}

	virtual bool IsPropertyAnimated(const IPropertyHandle& PropertyHandle, UObject *ParentObject) const
	{
		
		for (const TWeakPtr<ISequencer>& WeakSequencer : Sequencers)
		{
			TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
			if (Sequencer.IsValid() && Sequencer->GetFocusedMovieSceneSequence())
			{
				constexpr bool bCreateHandleIfMissing = false;
				FGuid ObjectHandle = Sequencer->GetHandleToObject(ParentObject, bCreateHandleIfMissing);
				if (ObjectHandle.IsValid()) 
				{
					UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
					FProperty* Property = PropertyHandle.GetProperty();
					TSharedRef<FPropertyPath> PropertyPath = FPropertyPath::CreateEmpty();
					PropertyPath->AddProperty(FPropertyInfo(Property));
					FName PropertyName(*PropertyPath->ToString(TEXT(".")));
					TSubclassOf<UMovieSceneTrack> TrackClass; //use empty @todo find way to get the UMovieSceneTrack from the Property type.
					return MovieScene->FindTrack(TrackClass, ObjectHandle, PropertyName) != nullptr;
				}
				
				return false;
			}
		}
		return false;
	}
	virtual void OnKeyPropertyClicked(const IPropertyHandle& KeyedPropertyHandle)
	{
		TArray<UObject*> Objects;
		KeyedPropertyHandle.GetOuterObjects( Objects );

		TArray<UObject*> EachObject;
		EachObject.SetNum(1);
		for (const TWeakPtr<ISequencer>& WeakSequencer : Sequencers)
		{
			TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
			if (Sequencer.IsValid())
			{
				for (UObject* Object : Objects)
				{
					EachObject[0] = Object;
					FKeyPropertyParams KeyPropertyParams(EachObject, KeyedPropertyHandle, ESequencerKeyMode::ManualKeyForced);
					Sequencer->KeyProperty(KeyPropertyParams);
				}
			}
		}
	}

	virtual EPropertyKeyedStatus GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const override
	{
		EPropertyKeyedStatus KeyedStatus = EPropertyKeyedStatus::NotKeyed;
		for (const TWeakPtr<ISequencer>& WeakSequencer : Sequencers)
		{
			if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
			{
				EPropertyKeyedStatus NewKeyedStatus = Sequencer->GetPropertyKeyedStatus(PropertyHandle);
				KeyedStatus = FMath::Max(KeyedStatus, NewKeyedStatus);
			}
		}
		return KeyedStatus;
	}

private:
	TArray<TWeakPtr<ISequencer>> Sequencers;
};

void FLevelEditorSequencerUpdateGizmoTickFunction::ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	if (bSequencerEvaluated)
	{
		GUnrealEd->UpdatePivotLocationForSelection();
		bSequencerEvaluated = false;
	}
}

FString FLevelEditorSequencerUpdateGizmoTickFunction::DiagnosticMessage()
{
	return TEXT("[SequencerUpdateGizmoPosition]");
}

FName FLevelEditorSequencerUpdateGizmoTickFunction::DiagnosticContext(bool bDetailed)
{
	return FName(TEXT("SequencerUpdateGizmoPosition"));
}

static const FName DetailsTabIdentifiers[] = { "LevelEditorSelectionDetails", "LevelEditorSelectionDetails2", "LevelEditorSelectionDetails3", "LevelEditorSelectionDetails4" };

FLevelEditorSequencerIntegration::FLevelEditorSequencerIntegration()
	: bDeferUpdates(false)
{
	KeyFrameHandler = MakeShared<FDetailKeyframeHandlerWrapper>();
}

FLevelEditorSequencerIntegration::~FLevelEditorSequencerIntegration() = default;

FLevelEditorSequencerIntegration& FLevelEditorSequencerIntegration::Get()
{
	static FLevelEditorSequencerIntegration Singleton;
	return Singleton;
}

void FLevelEditorSequencerIntegration::IterateAllSequencers(TFunctionRef<void(FSequencer&, const FLevelEditorSequencerIntegrationOptions& Options)> It) const
{
	for (const FSequencerAndOptions& SequencerAndOptions : BoundSequencers)
	{
		TSharedPtr<FSequencer> Pinned = SequencerAndOptions.Sequencer.Pin();
		if (Pinned.IsValid())
		{
			It(*Pinned, SequencerAndOptions.Options);
		}
	}
}

void FLevelEditorSequencerIntegration::Initialize(const FLevelEditorSequencerIntegrationOptions& Options)
{
	AcquiredResources.Release();

	// Register for saving the level so that the state of the scene can be restored before saving and updated after saving.
	{
		FDelegateHandle Handle = FEditorDelegates::PreSaveWorldWithContext.AddRaw(this, &FLevelEditorSequencerIntegration::OnPreSaveWorld);
		AcquiredResources.Add([=]{ FEditorDelegates::PreSaveWorldWithContext.Remove(Handle); });
	}
	{
		FDelegateHandle Handle = FEditorDelegates::PostSaveWorldWithContext.AddRaw(this, &FLevelEditorSequencerIntegration::OnPostSaveWorld);
		AcquiredResources.Add([=]{ FEditorDelegates::PostSaveWorldWithContext.Remove(Handle); });
	}
	{
		FDelegateHandle Handle = FEditorDelegates::PreSaveExternalActors.AddRaw(this, &FLevelEditorSequencerIntegration::OnPreSaveExternalActors);
		AcquiredResources.Add([=]{ FEditorDelegates::PreSaveExternalActors.Remove(Handle); });
	}
	{
		FDelegateHandle Handle = FEditorDelegates::PostSaveExternalActors.AddRaw(this, &FLevelEditorSequencerIntegration::OnPostSaveExternalActors);
		AcquiredResources.Add([=]{ FEditorDelegates::PostSaveExternalActors.Remove(Handle); });
	}
	{
		FDelegateHandle Handle = FEditorDelegates::OnPreAssetValidation.AddRaw(this, &FLevelEditorSequencerIntegration::OnPreAssetValidation);
		AcquiredResources.Add([=] { FEditorDelegates::OnPreAssetValidation.Remove(Handle); });
	}
	{
		FDelegateHandle Handle = FEditorDelegates::OnPostAssetValidation.AddRaw(this, &FLevelEditorSequencerIntegration::OnPostAssetValidation);
		AcquiredResources.Add([=] { FEditorDelegates::OnPostAssetValidation.Remove(Handle); });
	}
	{
		FDelegateHandle Handle = FEditorDelegates::PreBeginPIE.AddRaw(this, &FLevelEditorSequencerIntegration::OnPreBeginPIE);
		AcquiredResources.Add([=]{ FEditorDelegates::PreBeginPIE.Remove(Handle); });
	}
	{
		FDelegateHandle Handle = FEditorDelegates::EndPIE.AddRaw(this, &FLevelEditorSequencerIntegration::OnEndPIE);
		AcquiredResources.Add([=]{ FEditorDelegates::EndPIE.Remove(Handle); });
	}
	{
		FDelegateHandle Handle = FGameDelegates::Get().GetEndPlayMapDelegate().AddRaw(this, &FLevelEditorSequencerIntegration::OnEndPlayMap);
		AcquiredResources.Add([=]{ FGameDelegates::Get().GetEndPlayMapDelegate().Remove(Handle); });
	}
	{
		FDelegateHandle Handle = FWorldDelegates::LevelAddedToWorld.AddRaw(this, &FLevelEditorSequencerIntegration::OnLevelAdded);
		AcquiredResources.Add([=]{ FWorldDelegates::LevelAddedToWorld.Remove(Handle); });
	}
	{
		FDelegateHandle Handle = FWorldDelegates::LevelRemovedFromWorld.AddRaw(this, &FLevelEditorSequencerIntegration::OnLevelRemoved);
		AcquiredResources.Add([=]{ FWorldDelegates::LevelRemovedFromWorld.Remove(Handle); });
	}
	{
		FDelegateHandle Handle = FEditorDelegates::NewCurrentLevel.AddRaw(this, &FLevelEditorSequencerIntegration::OnNewCurrentLevel);
		AcquiredResources.Add([=]{ FEditorDelegates::NewCurrentLevel.Remove(Handle); });
	}
	{
		FDelegateHandle Handle = FEditorDelegates::OnMapOpened.AddRaw(this, &FLevelEditorSequencerIntegration::OnMapOpened);
		AcquiredResources.Add([=]{ FEditorDelegates::OnMapOpened.Remove(Handle); });
	}
	{
		FDelegateHandle Handle = FEditorDelegates::OnNewActorsDropped.AddRaw(this, &FLevelEditorSequencerIntegration::OnNewActorsDropped);
		AcquiredResources.Add([=]{ FEditorDelegates::OnNewActorsDropped.Remove(Handle); });
	}
	{
		FDelegateHandle Handle = USelection::SelectionChangedEvent.AddRaw( this, &FLevelEditorSequencerIntegration::OnActorSelectionChanged );
		AcquiredResources.Add([=]{ USelection::SelectionChangedEvent.Remove(Handle); });
	}

	{
		FDelegateHandle Handle = FCoreDelegates::OnActorLabelChanged.AddRaw(this, &FLevelEditorSequencerIntegration::OnActorLabelChanged);
		AcquiredResources.Add([=]{ FCoreDelegates::OnActorLabelChanged.Remove(Handle); });
	}

	// Menus need to be registered in a callback to make sure the system is ready for them.
	{
		UToolMenus::RegisterStartupCallback(
			FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FLevelEditorSequencerIntegration::RegisterMenus));
		AcquiredResources.Add([this]{ 	// Clean up menu things
			UToolMenus::UnRegisterStartupCallback(this);
			UToolMenus::UnregisterOwner(this);
		});
	}

	BindDetailHandler(Options);

	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>("LevelEditor");

		FDelegateHandle TabContentChanged = LevelEditorModule.OnTabContentChanged().AddRaw(this, &FLevelEditorSequencerIntegration::OnTabContentChanged);
		FDelegateHandle MapChanged = LevelEditorModule.OnMapChanged().AddRaw(this, &FLevelEditorSequencerIntegration::OnMapChanged);
		AcquiredResources.Add(
			[=]{
				FLevelEditorModule* LevelEditorModulePtr = FModuleManager::Get().GetModulePtr<FLevelEditorModule>("LevelEditor");
				if (LevelEditorModulePtr)
				{
					LevelEditorModulePtr->OnTabContentChanged().Remove(TabContentChanged);
					LevelEditorModulePtr->OnMapChanged().Remove(MapChanged);
				}
			}
		);
	}

	bool bForceRefresh = Options.bForceRefreshDetails;
	UpdateDetails(bForceRefresh);
}

void RenameBindingRecursive(FSequencer* Sequencer, UMovieScene* MovieScene, FMovieSceneSequenceIDRef SequenceID, const FMovieSceneSequenceHierarchy* Hierarchy, AActor* ChangedActor)
{
	check(MovieScene);

	// Iterate all this movie scene's spawnables, renaming as appropriate
	for (int32 Index = 0; Index < MovieScene->GetSpawnableCount(); ++Index)
	{
		FGuid ThisGuid = MovieScene->GetSpawnable(Index).GetGuid();

		for (TWeakObjectPtr<> WeakObject : Sequencer->FindBoundObjects(ThisGuid, SequenceID))
		{
			AActor* Actor = Cast<AActor>(WeakObject.Get());
			if (Actor && Actor == ChangedActor)
			{
				MovieScene->Modify();
				MovieScene->GetSpawnable(Index).SetName(ChangedActor->GetActorLabel());
			}
		}
	}
	for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
	{
		FGuid ThisGuid = MovieScene->GetPossessable(Index).GetGuid();

		// If there is only one binding, set the name of the possessable
		TArrayView<TWeakObjectPtr<>> BoundObjects = Sequencer->FindBoundObjects(ThisGuid, SequenceID);
		if (BoundObjects.Num() == 1)
		{
			AActor* Actor = Cast<AActor>(BoundObjects[0].Get());
			if (Actor && Actor == ChangedActor)
			{
				MovieScene->Modify();
				MovieScene->GetPossessable(Index).SetName(ChangedActor->GetActorLabel());
			}
		}
	}

	if (Hierarchy)
	{
		// Recurse into child nodes
		if (const FMovieSceneSequenceHierarchyNode* Node = Hierarchy->FindNode(SequenceID))
		{
			for (FMovieSceneSequenceIDRef ChildID : Node->Children)
			{
				const FMovieSceneSubSequenceData* SubData = Hierarchy->FindSubData(ChildID);
				if (SubData)
				{
					UMovieSceneSequence* SubSequence   = SubData->GetSequence();
					UMovieScene*         SubMovieScene = SubSequence ? SubSequence->GetMovieScene() : nullptr;

					if (SubMovieScene)
					{
						RenameBindingRecursive(Sequencer, SubMovieScene, ChildID, Hierarchy, ChangedActor);
					}
				}
			}
		}
	}
}

void FLevelEditorSequencerIntegration::OnActorLabelChanged(AActor* ChangedActor)
{
	// Sync up the spawnable or possessable name with the actor label if it is changed in editor (but not in PIE or for preview actors)
	if (ChangedActor == nullptr || ChangedActor->bIsEditorPreviewActor)
	{
		return;
	}

	const bool bPIEWorld = ChangedActor->GetWorld() != nullptr && ChangedActor->GetWorld()->IsPlayInEditor();
	if (bPIEWorld)
	{
		return;
	}

	for (const FSequencerAndOptions& SequencerAndOptions : BoundSequencers)
	{
		if (!SequencerAndOptions.Options.bSyncBindingsToActorLabels)
		{
			continue;
		}
		
		TSharedPtr<FSequencer> Pinned = SequencerAndOptions.Sequencer.Pin();
		if (Pinned.IsValid())
		{
			FMovieSceneRootEvaluationTemplateInstance& RootInstance = Pinned->GetEvaluationTemplate();
			const FMovieSceneSequenceHierarchy*        Hierarchy    = RootInstance.GetCompiledDataManager()->FindHierarchy(RootInstance.GetCompiledDataID());

			UMovieSceneSequence* RootSequence = Pinned->GetRootMovieSceneSequence();
			UMovieScene*         MovieScene   = RootSequence ? RootSequence->GetMovieScene() : nullptr;

			if (MovieScene)
			{
				RenameBindingRecursive(Pinned.Get(), MovieScene, MovieSceneSequenceID::Root, Hierarchy, ChangedActor);
			}
		}
	}
}

void FLevelEditorSequencerIntegration::OnPreSaveWorld(class UWorld* World, FObjectPreSaveContext ObjectSaveContext)
{
	RestoreToSavedState(World);
}

void FLevelEditorSequencerIntegration::OnPostSaveWorld(class UWorld* World, FObjectPostSaveContext ObjectSaveContext)
{
	ResetToAnimatedState(World);
}

void FLevelEditorSequencerIntegration::OnPreSaveExternalActors(UWorld* World)
{
	RestoreToSavedState(World);
}

void FLevelEditorSequencerIntegration::OnPostSaveExternalActors(UWorld* World)
{
	ResetToAnimatedState(World);
}

void FLevelEditorSequencerIntegration::OnPreAssetValidation()
{
	// Asset validation doesn't have a world context, so we'll just use the editor world.
	UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();
	if(UnrealEditorSubsystem && UnrealEditorSubsystem->GetEditorWorld())
	{
		RestoreToSavedState(UnrealEditorSubsystem->GetEditorWorld());
	}
}

void FLevelEditorSequencerIntegration::OnPostAssetValidation()
{
	UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();
	if(UnrealEditorSubsystem && UnrealEditorSubsystem->GetEditorWorld())
	{
		ResetToAnimatedState(UnrealEditorSubsystem->GetEditorWorld());
	}
}

void FLevelEditorSequencerIntegration::OnNewCurrentLevel()
{
	auto IsSequenceEditor = [](const FSequencerAndOptions& In) { return In.Sequencer.IsValid() && In.Options.bActivateSequencerEdMode; };
	if (BoundSequencers.FindByPredicate(IsSequenceEditor))
	{
		ActivateSequencerEditorMode();
	}
}

void FLevelEditorSequencerIntegration::OnMapOpened(const FString& Filename, bool bLoadAsTemplate)
{
	auto IsSequenceEditor = [](const FSequencerAndOptions& In) { return In.Sequencer.IsValid() && In.Options.bActivateSequencerEdMode; };
	if (BoundSequencers.FindByPredicate(IsSequenceEditor))
	{
		ActivateSequencerEditorMode();
	}
}

void FLevelEditorSequencerIntegration::OnLevelAdded(ULevel* InLevel, UWorld* InWorld)
{
	IterateAllSequencers(
		[](FSequencer& In, const FLevelEditorSequencerIntegrationOptions& Options)
		{
			if (Options.bRequiresLevelEvents)
			{
				In.GetEvaluationState()->ClearObjectCaches(In);
			}
		}
	);
}

void FLevelEditorSequencerIntegration::OnLevelRemoved(ULevel* InLevel, UWorld* InWorld)
{
	IterateAllSequencers(
		[](FSequencer& In, const FLevelEditorSequencerIntegrationOptions& Options)
		{
			if (Options.bRequiresLevelEvents)
			{
				In.GetEvaluationState()->ClearObjectCaches(In);
			}
		}
	);
}

void FLevelEditorSequencerIntegration::OnActorSelectionChanged( UObject* )
{
	IterateAllSequencers(
		[](FSequencer& In, const FLevelEditorSequencerIntegrationOptions& Options)
		{
			if (Options.bRequiresActorEvents)
			{
				if (In.GetSequencerSettings()->GetShowSelectedNodesOnly())
				{
					In.RefreshTree();
				}

				In.ExternalSelectionHasChanged();
			}
		}
	);
}

void FLevelEditorSequencerIntegration::OnNewActorsDropped(const TArray<UObject*>& DroppedObjects, const TArray<AActor*>& DroppedActors)
{
	IterateAllSequencers(
		[&](FSequencer& In, const FLevelEditorSequencerIntegrationOptions& Options)
		{
			if (Options.bRequiresActorEvents)
			{
				In.OnNewActorsDropped(DroppedObjects, DroppedActors);
			}
		}
	);
}

static TAutoConsoleVariable<bool> CVarEnableHitProxyInvalidationDuringEvaluation(
	TEXT("LevelSequence.EnableHitProxyInvalidationDuringEvaluation"),
	false,
	TEXT("Whether Sequencer evaluation should cause the viewport's hit proxies to be invalidated. If enabled, this causes FPS drops while moving the mouse in the viewport while playing back Sequencer.")
	);

namespace UE::Sequencer
{
static void ReRenderLevelViewports()
{
	// Request a single real-time frame to be rendered to ensure that we tick the world and update the viewport
	// We only do this on level viewports instead of GetAllViewportClients to avoid needlessly redrawing Cascade,
	// Blueprint, and other editors that have a 3d viewport.
	for (FEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && !LevelVC->IsRealtime())
		{
			LevelVC->RequestRealTimeFrames(1);
		}
	}
}

static void InvalidateLevelViewports(bool bInvalidateHitProxies)
{
	for (FEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC)
		{
			LevelVC->Invalidate(true, bInvalidateHitProxies);
		}
	}
}
}

void FLevelEditorSequencerIntegration::OnSequencerEvaluated()
{
	RestoreSpawnablePilotData();

	// Redraw if not in PIE/simulate
	const bool bIsInPIEOrSimulate = GEditor->PlayWorld != NULL || GEditor->bIsSimulatingInEditor;
	if (bIsInPIEOrSimulate)
	{
		return;
	}
	
	UE::Sequencer::ReRenderLevelViewports();
	
	// Summary: When Sequencer plays back, moving the mouse in the viewport used to cause a drop in FPS. See CL description.
	const bool bInvalidateHitProxies = !bDeferUpdates;
	UE::Sequencer::InvalidateLevelViewports(bInvalidateHitProxies);

	if (!bDeferUpdates)
	{
		UpdateDetails();
	}

	// If realtime is off, this needs to be called to update the pivot location when scrubbing.
	UpdateGizmoTickFunction.bSequencerEvaluated = true;
}

void FLevelEditorSequencerIntegration::OnBeginDeferUpdates()
{
	bDeferUpdates = true;
}

void FLevelEditorSequencerIntegration::OnEndDeferUpdates()
{
	bDeferUpdates = false;
	UpdateDetails();

	constexpr bool bInvalidateHitProxies = true;
	UE::Sequencer::InvalidateLevelViewports(bInvalidateHitProxies);
}

bool FLevelEditorSequencerIntegration::IsBindingVisible(const FMovieSceneBinding& InBinding)
{
	// If nothing selected, show all nodes
	if (GEditor->GetSelectedActorCount() == 0)
	{
		return true;
	}

	// Disregard if not a level sequence (ie. a control rig sequence)	
	for (FSequencerAndOptions& SequencerAndOptions : BoundSequencers)
	{
		TSharedPtr<FSequencer> Pinned = SequencerAndOptions.Sequencer.Pin();
		if (Pinned.IsValid())
		{
			if (UMovieSceneSequence* RootSequence = Pinned->GetRootMovieSceneSequence())
			{
				if (RootSequence->GetClass()->GetName() != TEXT("LevelSequence"))
				{
					return true;
				}
				else
				{
					TArrayView<TWeakObjectPtr<>> Objects = Pinned->FindObjectsInCurrentSequence(InBinding.GetObjectGuid());
					for (TWeakObjectPtr<> Object : Objects)
					{
						if (AActor* Actor = Cast<AActor>(Object.Get()))
						{
							if (GEditor->GetSelectedActors()->IsSelected(Actor))
							{
								return true;
							}
						}
					}
				}
			}
		}
	}

	return false;
}

void FLevelEditorSequencerIntegration::OnMovieSceneBindingsChanged()
{
	for (FSequencerAndOptions& SequencerAndOptions : BoundSequencers)
	{
		SequencerAndOptions.BindingData.Get().bActorBindingsDirty = true;
	}
}

void FLevelEditorSequencerIntegration::OnMovieSceneDataChanged(EMovieSceneDataChangeType DataChangeType)
{
	if (DataChangeType == EMovieSceneDataChangeType::MovieSceneStructureItemAdded ||
		DataChangeType == EMovieSceneDataChangeType::MovieSceneStructureItemRemoved ||
		DataChangeType == EMovieSceneDataChangeType::MovieSceneStructureItemsChanged ||
		DataChangeType == EMovieSceneDataChangeType::RefreshAllImmediately ||
		DataChangeType == EMovieSceneDataChangeType::ActiveMovieSceneChanged)
	{
		UpdateDetails();
	}
}

void FLevelEditorSequencerIntegration::OnAllowEditsModeChanged(EAllowEditsMode AllowEditsMode)
{
	UpdateDetails(true);
}

void FLevelEditorSequencerIntegration::UpdateDetails(bool bForceRefresh)
{
	bool bNeedsRefresh = bForceRefresh;

	for (FSequencerAndOptions& SequencerAndOptions : BoundSequencers)
	{
		TSharedPtr<FSequencer> Pinned = SequencerAndOptions.Sequencer.Pin();
		if (Pinned.IsValid())
		{
			SequencerAndOptions.BindingData.Get().bPropertyBindingsDirty = true;
		
			if (Pinned.Get()->GetAllowEditsMode() == EAllowEditsMode::AllowLevelEditsOnly)
			{
				bNeedsRefresh = true;
			}
		}
	}

	if (bNeedsRefresh)
	{
		FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		
		for (const FName& DetailsTabIdentifier : DetailsTabIdentifiers)
		{
			TSharedPtr<IDetailsView> DetailsView = EditModule.FindDetailView(DetailsTabIdentifier);
			if(DetailsView.IsValid())
			{
				DetailsView->ForceRefresh();
			}
		}
	}
}


void FLevelEditorSequencerIntegration::ActivateSequencerEditorMode()
{
	// Release the sequencer mode if we already enabled it
	DeactivateSequencerEditorMode();

	// Activate the default mode in case FEditorModeTools::Tick isn't run before here. 
	// This can be removed once a general fix for UE-143791 has been implemented.
	GLevelEditorModeTools().ActivateDefaultMode();

	FEditorModeID ModeID = TEXT("SequencerToolsEditMode");
	GLevelEditorModeTools().ActivateMode(ModeID);

	GLevelEditorModeTools().ActivateMode( FSequencerEdMode::EM_SequencerMode );
	FSequencerEdMode* SequencerEdMode = (FSequencerEdMode*)GLevelEditorModeTools().GetActiveMode(FSequencerEdMode::EM_SequencerMode);

	for (const FSequencerAndOptions& SequencerAndOptions : BoundSequencers)
	{
		TSharedPtr<FSequencer> Pinned = SequencerAndOptions.Sequencer.Pin();
		if (Pinned.IsValid())
		{
			SequencerEdMode->AddSequencer(Pinned);
		}
	}
}


void FLevelEditorSequencerIntegration::DeactivateSequencerEditorMode()
{
	const FEditorModeID ModeID = TEXT("SequencerToolsEditMode");
	
	if (GLevelEditorModeTools().IsModeActive(ModeID))
	{
		GLevelEditorModeTools().DeactivateMode(ModeID);
	}
	if (GLevelEditorModeTools().IsModeActive(FSequencerEdMode::EM_SequencerMode))
	{
		GLevelEditorModeTools().DeactivateMode(FSequencerEdMode::EM_SequencerMode);
	}
}



void FLevelEditorSequencerIntegration::OnPreBeginPIE(bool bIsSimulating)
{
	IterateAllSequencers(
		[](FSequencer& In, const FLevelEditorSequencerIntegrationOptions& Options)
		{
			if (Options.bRequiresLevelEvents)
			{
				In.OnPlaybackContextChanged();
				In.RestorePreAnimatedState();
				In.GetEvaluationState()->ClearObjectCaches(In);
				In.RequestEvaluate();
			}
		}
	);
}

void FLevelEditorSequencerIntegration::OnEndPlayMap()
{
	bool bAddRestoreCallback = false;
	const FText SystemDisplayName = LOCTEXT("RealtimeOverrideMessage_Sequencer", "Sequencer");
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC)
		{
			// If the Sequencer was opened during PIE, we didn't make the viewport realtime. Now that PIE has ended,
			// we can add our override.
			if (LevelVC->IsPerspective() && LevelVC->AllowsCinematicControl() && !LevelVC->HasRealtimeOverride(SystemDisplayName))
			{
				const bool bShouldBeRealtime = true;
				LevelVC->AddRealtimeOverride(bShouldBeRealtime, SystemDisplayName);
				bAddRestoreCallback = true;
			}
		}
	}
	if (bAddRestoreCallback)
	{
		AcquiredResources.Add([this] { this->RestoreRealtimeViewports(); });
	}

	IterateAllSequencers(
		[](FSequencer& In, const FLevelEditorSequencerIntegrationOptions& Options)
		{
			if (Options.bRequiresLevelEvents)
			{
				// Update and clear any stale bindings 
				In.OnPlaybackContextChanged();
				In.GetEvaluationState()->ClearObjectCaches(In);
				In.ForceEvaluate();
			}
		}
	);
}

void FLevelEditorSequencerIntegration::OnEndPIE(bool bIsSimulating)
{
	OnEndPlayMap();
}

void FindActorInSequencesRecursive(AActor* InActor, FSequencer& Sequencer, FMovieSceneSequenceIDRef SequenceID, TArray<TPair<FMovieSceneSequenceID, FSequencer*> >& FoundInSequences)
{
	FMovieSceneRootEvaluationTemplateInstance& RootInstance = Sequencer.GetEvaluationTemplate();

	// Find the sequence that corresponds to the sequence ID
	UMovieSceneSequence* Sequence = RootInstance.GetSequence(SequenceID);
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;


	const FMovieSceneSequenceHierarchy* Hierarchy = RootInstance.GetCompiledDataManager()->FindHierarchy(RootInstance.GetCompiledDataID());

	// Recurse into child nodes
	const FMovieSceneSequenceHierarchyNode* Node = Hierarchy ? Hierarchy->FindNode(SequenceID) : nullptr;
	if (Node)
	{
		for (FMovieSceneSequenceIDRef ChildID : Node->Children)
		{
			FindActorInSequencesRecursive(InActor, Sequencer, ChildID, FoundInSequences);
		}
	}

	if (MovieScene)
	{
		FString SequenceName = Sequence->GetDisplayName().ToString();

		// Search all possessables
		for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
		{
			FGuid ThisGuid = MovieScene->GetPossessable(Index).GetGuid();

			for (TWeakObjectPtr<> WeakObject : Sequencer.FindBoundObjects(ThisGuid, SequenceID))
			{
				AActor* Actor = Cast<AActor>(WeakObject.Get());
				if (Actor == InActor)
				{
					FoundInSequences.Add(TPair<FMovieSceneSequenceID, FSequencer*>(SequenceID, &Sequencer));
					return;
				}
			}
		}

		// Search all spawnables
		for (int32 Index = 0; Index < MovieScene->GetSpawnableCount(); ++Index)
		{
			FGuid ThisGuid = MovieScene->GetSpawnable(Index).GetGuid();

			for (TWeakObjectPtr<> WeakObject : Sequencer.FindBoundObjects(ThisGuid, SequenceID))
			{
				AActor* Actor = Cast<AActor>(WeakObject.Get());
				if (Actor == InActor)
				{
					FoundInSequences.Add(TPair<FMovieSceneSequenceID, FSequencer*>(SequenceID, &Sequencer));
					return;
				}
			}
		}
	}
}

void FLevelEditorSequencerIntegration::RegisterMenus()
{
	// Allows cleanup when module unloads.
	FToolMenuOwnerScoped OwnerScoped(this);

	{
		UToolMenu* LevelEditorMenu = UToolMenus::Get()->ExtendMenu("LevelEditor");
		FToolMenuSection& UEToolsSection = LevelEditorMenu->FindOrAddSection("ActorUETools");
		UEToolsSection.AddSubMenu(
			"BrowseToActorSubMenu",
			LOCTEXT("BrowseToActorInSequencer", "Browse to Actor in Sequencer"),
			FText(),
			FNewToolMenuDelegate::CreateRaw(this, &FLevelEditorSequencerIntegration::MakeBrowseToSelectedActorSubMenu),
			false,
			FSlateIcon("LevelSequenceEditorStyle", "LevelSequenceEditor.Tabs.Sequencer")
		);
	}

	{
		UToolMenu* ActorContextMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.ActorContextMenu");
		FToolMenuSection& ActorTypeToolsSection = ActorContextMenu->FindOrAddSection("ActorTypeTools");
		ActorTypeToolsSection.AddSubMenu(
			"BrowseToActorSubMenu",
			LOCTEXT("BrowseToActorInSequencer", "Browse to Actor in Sequencer"),
			FText(),
			FNewToolMenuDelegate::CreateRaw(this, &FLevelEditorSequencerIntegration::MakeBrowseToSelectedActorSubMenu),
			false,
			FSlateIcon("LevelSequenceEditorStyle", "LevelSequenceEditor.Tabs.Sequencer")
		);
	}
}

void FLevelEditorSequencerIntegration::MakeBrowseToSelectedActorSubMenu(UToolMenu* Menu)
{
	AActor* Actor = nullptr;
	TArray<TPair<FMovieSceneSequenceID, FSequencer*> > FoundInSequences;

	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		// We are interested in the (unique) assets backing the actor, or else the actor
		// itself if it is not asset backed (such as UDynamicMesh).
		Actor = static_cast<AActor*>(*It);

		IterateAllSequencers(
			[&](FSequencer& In, const FLevelEditorSequencerIntegrationOptions& Options)
			{
				FindActorInSequencesRecursive(Actor, In, MovieSceneSequenceID::Root, FoundInSequences);
			});

		if (Actor)
		{
			break;
		}
	};

	if (!Actor || !FoundInSequences.Num())
	{
		return;
	}

	FToolMenuSection& Section = Menu->AddSection("BrowseToActorSection");
	for (const TPair<FMovieSceneSequenceID, FSequencer*>& Sequence : FoundInSequences)
	{
		UMovieSceneSequence* MovieSceneSequence = nullptr;
		if (Sequence.Key == MovieSceneSequenceID::Root)
		{
			MovieSceneSequence = Sequence.Value->GetRootMovieSceneSequence();
		}
		else
		{
			UMovieSceneSubSection* SubSection = Sequence.Value->FindSubSection(Sequence.Key);
			MovieSceneSequence = SubSection ? SubSection->GetSequence() : nullptr;
		}
		
		if (MovieSceneSequence)
		{
			FText ActorName = FText::Format(LOCTEXT("ActorNameSingular", "\"{0}\""), FText::FromString(Actor->GetActorLabel()));
			FUIAction AddMenuAction(FExecuteAction::CreateLambda([this, Actor, Sequence]() {this->BrowseToSelectedActor(Actor, Sequence.Value, Sequence.Key); }));
			FText MenuName = FText::Format(LOCTEXT("BrowseToSelectedActorText", "Browse to {0} in {1}"), ActorName, MovieSceneSequence->GetDisplayName());
			Section.AddMenuEntry(FName(*MenuName.ToString()), MenuName, FText(), FSlateIcon(), AddMenuAction);
		}
	}
}

void FLevelEditorSequencerIntegration::ActivateDetailHandler()
{
	// Add sequencer detail keyframe handler
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	for (const FName& DetailsTabIdentifier : DetailsTabIdentifiers)
	{
		TSharedPtr<IDetailsView> DetailsView = EditModule.FindDetailView(DetailsTabIdentifier);
		if(DetailsView.IsValid())
		{
			DetailsView->SetKeyframeHandler(KeyFrameHandler);
			DetailsView->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateRaw(this, &FLevelEditorSequencerIntegration::IsPropertyReadOnly));
		}
	}
}

void FLevelEditorSequencerIntegration::BindDetailHandler(const FLevelEditorSequencerIntegrationOptions& Options)
{
	static const FName DetailHandlerName("DetailHandler");
	// NOTE: this should already have been done in Initialize
	AcquiredResources.Release(DetailHandlerName);

	// bind keyframe handler
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDelegateHandle OnPropertyEditorOpenedHandle = EditModule.OnPropertyEditorOpened().AddLambda([this]()
	{
		ActivateDetailHandler();
	});

	// run the handler for any open details panels.
	ActivateDetailHandler();
	
	// unbind keyframe handler
	auto DeactivateDetailKeyframeHandler = [this, OnPropertyEditorOpenedHandle]()
	{		
		if (FPropertyEditorModule* EditModulePtr = FModuleManager::Get().GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			EditModulePtr->OnPropertyEditorOpened().Remove(OnPropertyEditorOpenedHandle);

			for (const FName& DetailsTabIdentifier : DetailsTabIdentifiers)
			{
				TSharedPtr<IDetailsView> DetailsView = EditModulePtr->FindDetailView(DetailsTabIdentifier);
				if (DetailsView.IsValid())
				{
					if (DetailsView->GetKeyframeHandler() == KeyFrameHandler)
					{
						DetailsView->SetKeyframeHandler(nullptr);
					}

					DetailsView->GetIsPropertyReadOnlyDelegate().Unbind();
				}
			}
		}
	};
	AcquiredResources.Add(DetailHandlerName, DeactivateDetailKeyframeHandler);

	static const FName DetailHandlerRefreshName("DetailHandlerRefresh");
	// NOTE: this should already have been done in Initialize
	AcquiredResources.Release(DetailHandlerRefreshName);
	
	if (Options.bForceRefreshDetails)
	{
		auto RefreshDetailHandler = []()
		{
			if (FPropertyEditorModule* EditModulePtr = FModuleManager::Get().GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
			{
				for (const FName& DetailsTabIdentifier : DetailsTabIdentifiers)
				{
					TSharedPtr<IDetailsView> DetailsView = EditModulePtr->FindDetailView(DetailsTabIdentifier);
					if (DetailsView.IsValid())
					{
						DetailsView->ForceRefresh();
					}
				}
			}
		};
		AcquiredResources.Add(DetailHandlerRefreshName, RefreshDetailHandler);
	}
}

void FLevelEditorSequencerIntegration::BrowseToSelectedActor(AActor* Actor, FSequencer* Sequencer, FMovieSceneSequenceID SequenceID)
{
	Sequencer->PopToSequenceInstance(MovieSceneSequenceID::Root);

	if (SequenceID != MovieSceneSequenceID::Root)
	{
		Sequencer->FocusSequenceInstance(*Sequencer->FindSubSection(SequenceID));
	}

	Sequencer->SelectObject(Sequencer->FindObjectId(*Actor, SequenceID));
}

namespace FaderConstants
{	
	/** The opacity when we are hovered */
	const float HoveredOpacity = 1.0f;
	/** The opacity when we are not hovered */
	const float NonHoveredOpacity = 0.75f;
	/** The amount of time spent actually fading in or out */
	const float FadeTime = 0.15f;
}

/** Wrapper widget allowing us to fade widgets in and out on hover state */
class SFader : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SFader)
		: _Content()
	{}

	SLATE_DEFAULT_SLOT(FArguments, Content)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		FadeInSequence = FCurveSequence(0.0f, FaderConstants::FadeTime);
		FadeOutSequence = FCurveSequence(0.0f, FaderConstants::FadeTime);
		FadeOutSequence.JumpToEnd();

		SetHover(false);

		SBorder::Construct(SBorder::FArguments()
			.BorderImage(FCoreStyle::Get().GetBrush("NoBorder"))
			.Padding(0.0f)
			.VAlign(VAlign_Center)
			.ColorAndOpacity(this, &SFader::GetColorAndOpacity)
			.Content()
			[
				InArgs._Content.Widget
			]);
	}

	FLinearColor GetColorAndOpacity() const
	{
		FLinearColor Color = FLinearColor::White;
	
		if(FadeOutSequence.IsPlaying() || !IsHovered())
		{
			Color.A = FMath::Lerp(FaderConstants::HoveredOpacity, FaderConstants::NonHoveredOpacity, FadeOutSequence.GetLerp());
		}
		else
		{
			Color.A = FMath::Lerp(FaderConstants::NonHoveredOpacity, FaderConstants::HoveredOpacity, FadeInSequence.GetLerp());
		}

		return Color;
	}

	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if(!FSlateApplication::Get().IsUsingHighPrecisionMouseMovment())
		{
			SetHover(true);
			if(FadeOutSequence.IsPlaying())
			{
				// Fade out is already playing so just force the fade in curve to the end so we don't have a "pop" 
				// effect from quickly resetting the alpha
				FadeInSequence.JumpToEnd();
			}
			else
			{
				FadeInSequence.Play(AsShared());
			}
		}
	}

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		if(!FSlateApplication::Get().IsUsingHighPrecisionMouseMovment())
		{
			SetHover(false);
			FadeOutSequence.Play(AsShared());
		}
	}

private:
	/** Curve sequence for fading out the widget */
	FCurveSequence FadeOutSequence;
	/** Curve sequence for fading in the widget */
	FCurveSequence FadeInSequence;
};

TSharedRef< ISceneOutlinerColumn > FLevelEditorSequencerIntegration::CreateSequencerInfoColumn( ISceneOutliner& SceneOutliner ) const
{
	//@todo only supports the first bound sequencer
	check(BoundSequencers.Num() > 0);
	check(BoundSequencers[0].Sequencer.IsValid());

	return MakeShareable( new Sequencer::FSequencerInfoColumn( SceneOutliner, *BoundSequencers[0].Sequencer.Pin(), BoundSequencers[0].BindingData.Get() ) );
}

TSharedRef< ISceneOutlinerColumn > FLevelEditorSequencerIntegration::CreateSequencerSpawnableColumn( ISceneOutliner& SceneOutliner ) const
{
	//@todo only supports the first bound sequencer
	check(BoundSequencers.Num() > 0);
	check(BoundSequencers[0].Sequencer.IsValid());

	return MakeShareable( new Sequencer::FSequencerSpawnableColumn() );
}

void FLevelEditorSequencerIntegration::AttachOutlinerColumn()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	
	FSceneOutlinerColumnInfo SpawnColumnInfo(ESceneOutlinerColumnVisibility::Visible, 11, 
	FCreateSceneOutlinerColumn::CreateRaw( this, &FLevelEditorSequencerIntegration::CreateSequencerSpawnableColumn),
	true, TOptional<float>(), LOCTEXT("SpawnableColumnName", "Spawnable"));

	FSceneOutlinerColumnInfo SequencerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 15, 
		FCreateSceneOutlinerColumn::CreateRaw( this, &FLevelEditorSequencerIntegration::CreateSequencerInfoColumn), 
		true, TOptional<float>(), LOCTEXT("SequencerColumnName", "Sequencer"));

	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");

	// First we register the columns as default columns so they show up in any new Outliners that are opened
	SceneOutlinerModule.RegisterDefaultColumnType< Sequencer::FSequencerSpawnableColumn >(SpawnColumnInfo);
	SceneOutlinerModule.RegisterDefaultColumnType< Sequencer::FSequencerInfoColumn >(SequencerColumnInfo);

	// Then we go through all currently open Outliners and add the column manually
	if(TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor())
	{
		TArray<TWeakPtr<ISceneOutliner>> Outliners = LevelEditor->GetAllSceneOutliners();

		for(TWeakPtr<ISceneOutliner> Outliner : Outliners)
		{
			if(TSharedPtr<ISceneOutliner> OutlinerPinned = Outliner.Pin())
			{
				OutlinerPinned->AddColumn(Sequencer::FSequencerSpawnableColumn::GetID(), SpawnColumnInfo);
				OutlinerPinned->AddColumn(Sequencer::FSequencerInfoColumn::GetID(), SequencerColumnInfo);
			}
		}
	}
}

void FLevelEditorSequencerIntegration::DetachOutlinerColumn()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");

	// First we remove the column from any active Outliners
	if(TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor())
	{
		TArray<TWeakPtr<ISceneOutliner>> Outliners = LevelEditor->GetAllSceneOutliners();

		for(TWeakPtr<ISceneOutliner> Outliner : Outliners)
		{
			if(TSharedPtr<ISceneOutliner> OutlinerPinned = Outliner.Pin())
			{
				OutlinerPinned->RemoveColumn(Sequencer::FSequencerSpawnableColumn::GetID());
				OutlinerPinned->RemoveColumn(Sequencer::FSequencerInfoColumn::GetID());
			}
		}
	}

	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");

	// Then we unregister the column type so it isn't added to any Outliners that are opened in the future	
	SceneOutlinerModule.UnRegisterColumnType< Sequencer::FSequencerSpawnableColumn >();
	SceneOutlinerModule.UnRegisterColumnType< Sequencer::FSequencerInfoColumn >();
}

void FLevelEditorSequencerIntegration::ActivateRealtimeViewports()
{
	// If PIE is running, the viewport will already be rendering the scene in realtime as part of the 
	// normal game loop. If we set it to realtime, the editor would render it a second time each frame.
	if (GEditor->IsPlaySessionInProgress())
	{
		return;
	}

	for (const FSequencerAndOptions& SequencerAndOptions : BoundSequencers)
	{
		TSharedPtr<FSequencer> Pinned = SequencerAndOptions.Sequencer.Pin();
		if (Pinned.IsValid())
		{
			if (!Pinned.Get()->GetSequencerSettings()->ShouldActivateRealtimeViewports())
			{
				return;
			}
		}
	}

	for(FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC)
		{
			// If there is a director group, set the perspective viewports to realtime automatically.
			if (LevelVC->IsPerspective() && LevelVC->AllowsCinematicControl())
			{				
				const bool bShouldBeRealtime = true;
				LevelVC->AddRealtimeOverride(bShouldBeRealtime, LOCTEXT("RealtimeOverrideMessage_Sequencer", "Sequencer"));
			}
		}
	}

	AcquiredResources.Add([this]{ this->RestoreRealtimeViewports(); });
}

void FLevelEditorSequencerIntegration::RestoreRealtimeViewports()
{
	// Undo any weird settings to editor level viewports.

	// We don't care if our cinematic viewports still have our override or not because we just want to make sure nobody has
	// it anymore. It could happen that a viewport doesn't have it if that viewport is an actual Cinematic Viewport, for instance.
	const bool bCheckMissingOverride = false;

	if (GEditor)
	{
		for(FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
		{
			if (LevelVC)
			{
				// Turn off realtime when exiting.
				if( LevelVC->IsPerspective() && LevelVC->AllowsCinematicControl() )
				{				
					LevelVC->RemoveRealtimeOverride(LOCTEXT("RealtimeOverrideMessage_Sequencer", "Sequencer"), bCheckMissingOverride);
				}
			}
		}
	}
}

void FLevelEditorSequencerIntegration::RestoreToSavedState(UWorld* World)
{
	BackupSpawnablePilotData();

	// Restore the saved state so that the level save can save that instead of the animated state.
	IterateAllSequencers(
		[World](FSequencer& In, const FLevelEditorSequencerIntegrationOptions& Options)
		{
			if (Options.bRequiresLevelEvents)
			{
				for (const TSharedPtr<ISequencerTrackEditor>& TrackEditor : In.GetTrackEditors())
				{
					TrackEditor->OnPreSaveWorld(World);
				}
				In.RestorePreAnimatedState();
			}
		}
	);
}

void FLevelEditorSequencerIntegration::ResetToAnimatedState(UWorld* World)
{
	// Reset the time after saving so that an update will be triggered to put objects back to their animated state.
	IterateAllSequencers(
		[World](FSequencer& In, const FLevelEditorSequencerIntegrationOptions& Options)
		{
			if (Options.bRequiresLevelEvents)
			{
				In.ForceEvaluate();

				for (const TSharedPtr<ISequencerTrackEditor>& TrackEditor : In.GetTrackEditors())
				{
					TrackEditor->OnPostSaveWorld(World);
				}
			}
		}
	);
}

void FLevelEditorSequencerIntegration::OnTabContentChanged()
{

}

void FLevelEditorSequencerIntegration::OnMapChanged(UWorld* World, EMapChangeType MapChangeType)
{
	if (MapChangeType == EMapChangeType::TearDownWorld)
	{
		IterateAllSequencers(
			[=](FSequencer& In, const FLevelEditorSequencerIntegrationOptions& Options)
		{
			if (Options.bRequiresLevelEvents)
			{
				In.OnPlaybackContextChanged();
				In.RestorePreAnimatedState();
				In.GetEvaluationState()->ClearObjectCaches(In);

				// Notify data changed to enqueue an evaluate
				In.NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::Unknown);
			}
		}
		);
	}
}

void FLevelEditorSequencerIntegration::BackupSpawnablePilotData()
{
	if (PilotedSpawnables.Num() != 0)
	{
		return;
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid())
	{
		for (TSharedPtr<SLevelViewport> LevelViewport : LevelEditor->GetViewports())
		{
			const FLevelViewportActorLock& ActorLock = static_cast<FLevelEditorViewportClient&>(LevelViewport->GetAssetViewportClient()).GetActorLock();
			if (AActor* LockedActor = ActorLock.GetLockedActor())
			{
				TOptional<FMovieSceneSpawnableAnnotation> SpawnableAnnotation = FMovieSceneSpawnableAnnotation::Find(ActorLock.GetLockedActor());
				if (SpawnableAnnotation)
				{
					FPilotedSpawnable Pilot;
					Pilot.WeakLevelViewport = LevelViewport;
					Pilot.PreviousActorLock = ActorLock;
					Pilot.Annotation = SpawnableAnnotation.GetValue();
					PilotedSpawnables.Add(Pilot);
				}
			}
		}
	}
}

void FLevelEditorSequencerIntegration::RestoreSpawnablePilotData()
{
	if (PilotedSpawnables.Num() == 0)
	{
		return;
	}

	for (const FPilotedSpawnable& PilotData : PilotedSpawnables)
	{
		TSharedPtr<SLevelViewport> LevelViewport = PilotData.WeakLevelViewport.Pin();
		if (LevelViewport && static_cast<FLevelEditorViewportClient&>(LevelViewport->GetAssetViewportClient()).GetActorLock().GetLockedActor() == nullptr)
		{
			// Find the new spawnable
			IterateAllSequencers(
				[&PilotData, LevelViewport](FSequencer& In, const FLevelEditorSequencerIntegrationOptions& Options)
				{
					for (TWeakObjectPtr<> WeakObject : In.FindBoundObjects(PilotData.Annotation.ObjectBindingID, PilotData.Annotation.SequenceID))
					{
						if (AActor* Actor = Cast<AActor>(WeakObject.Get()))
						{
							// Update the actor lock using the previous settings, but with the new actor
							FLevelViewportActorLock ActorLock = PilotData.PreviousActorLock;
							ActorLock.LockedActor = Actor;
							static_cast<FLevelEditorViewportClient&>(LevelViewport->GetAssetViewportClient()).SetActorLock(ActorLock);
							break;
						}
					}
				}
			);
		}
	}

	PilotedSpawnables.Empty();
}


void FLevelEditorSequencerIntegration::AddSequencer(TSharedRef<ISequencer> InSequencer, const FLevelEditorSequencerIntegrationOptions& Options)
{
	if (!BoundSequencers.Num())
	{
		Initialize(Options);
	}
	if (UObject* PlaybackContext = InSequencer->GetSharedPlaybackState()->GetPlaybackContext())
	{
		if (UWorld* World = PlaybackContext->GetWorld())
		{
			if (World->PersistentLevel)
			{
				if (!UpdateGizmoTickFunction.IsTickFunctionRegistered())
				{
					UpdateGizmoTickFunction.TickGroup = TG_PostUpdateWork;
					UpdateGizmoTickFunction.bCanEverTick = true;
					UpdateGizmoTickFunction.bStartWithTickEnabled = true;
					UpdateGizmoTickFunction.RegisterTickFunction(World->PersistentLevel);
				}
			}
		}
	}

	KeyFrameHandler->Add(InSequencer);

	auto DerivedSequencerPtr = StaticCastSharedRef<FSequencer>(InSequencer);
	BoundSequencers.Add(FSequencerAndOptions{ DerivedSequencerPtr, Options, FAcquiredResources(), MakeShareable(new FLevelEditorSequencerBindingData) });

	{
		TWeakPtr<ISequencer> WeakSequencer = InSequencer;

		// Set up a callback for when this sequencer changes its time to redraw any non-realtime viewports
		FDelegateHandle EvalHandle = InSequencer->OnGlobalTimeChanged().AddRaw(this, &FLevelEditorSequencerIntegration::OnSequencerEvaluated);

		// Set up a callback for when this sequencer changes to update the sequencer data mapping
		FDelegateHandle BindingsHandle = InSequencer->OnMovieSceneBindingsChanged().AddRaw(this, &FLevelEditorSequencerIntegration::OnMovieSceneBindingsChanged);
		FDelegateHandle DataHandle = InSequencer->OnMovieSceneDataChanged().AddRaw(this, &FLevelEditorSequencerIntegration::OnMovieSceneDataChanged);
		FDelegateHandle AllowEditsModeHandle = InSequencer->GetSequencerSettings()->GetOnAllowEditsModeChanged().AddRaw(this, &FLevelEditorSequencerIntegration::OnAllowEditsModeChanged);

		FDelegateHandle PlayHandle = InSequencer->OnPlayEvent().AddRaw(this, &FLevelEditorSequencerIntegration::OnBeginDeferUpdates);
		FDelegateHandle StopHandle = InSequencer->OnStopEvent().AddRaw(this, &FLevelEditorSequencerIntegration::OnEndDeferUpdates);
		FDelegateHandle BeginScrubbingHandle = InSequencer->OnBeginScrubbingEvent().AddRaw(this, &FLevelEditorSequencerIntegration::OnBeginDeferUpdates);
		FDelegateHandle EndScrubbingHandle = InSequencer->OnEndScrubbingEvent().AddRaw(this, &FLevelEditorSequencerIntegration::OnEndDeferUpdates);

		InSequencer->OnGetIsBindingVisible().BindRaw(this, &FLevelEditorSequencerIntegration::IsBindingVisible);

		BoundSequencers.Last().AcquiredResources.Add(
			[=]
			{
				TSharedPtr<ISequencer> Pinned = WeakSequencer.Pin();
				if (Pinned.IsValid())
				{
					Pinned->OnGlobalTimeChanged().Remove(EvalHandle);
					Pinned->OnMovieSceneBindingsChanged().Remove(BindingsHandle);
					Pinned->OnMovieSceneDataChanged().Remove(DataHandle);
					Pinned->GetSequencerSettings()->GetOnAllowEditsModeChanged().Remove(AllowEditsModeHandle);
					Pinned->OnPlayEvent().Remove(PlayHandle);
					Pinned->OnStopEvent().Remove(StopHandle);
					Pinned->OnBeginScrubbingEvent().Remove(BeginScrubbingHandle);
					Pinned->OnEndScrubbingEvent().Remove(EndScrubbingHandle);
				}
			}
		);
	}

	FSequencerEdMode* SequencerEdMode = (FSequencerEdMode*)(GLevelEditorModeTools().GetActiveMode(FSequencerEdMode::EM_SequencerMode));
	if (SequencerEdMode)
	{
		SequencerEdMode->AddSequencer(DerivedSequencerPtr);
	}
	else if (Options.bActivateSequencerEdMode)
	{
		ActivateSequencerEditorMode();
	}

	ActivateRealtimeViewports();
	if (Options.bAttachOutlinerColumns)
	{
		AttachOutlinerColumn();
	}
	OnSequencersChanged.Broadcast();
}

void FLevelEditorSequencerIntegration::OnSequencerReceivedFocus(TSharedRef<ISequencer> InSequencer)
{
	FSequencerEdMode* SequencerEdMode = (FSequencerEdMode*)(GLevelEditorModeTools().GetActiveMode(FSequencerEdMode::EM_SequencerMode));
	if (SequencerEdMode)
	{
		SequencerEdMode->OnSequencerReceivedFocus(StaticCastSharedRef<FSequencer>(InSequencer));
	}
}

void FLevelEditorSequencerIntegration::RemoveSequencer(TSharedRef<ISequencer> InSequencer)
{
	// Remove any instances of this sequencer in the array of bound sequencers, along with its resources
	BoundSequencers.RemoveAll(
		[=](const FSequencerAndOptions& In)
		{
			return In.Sequencer == InSequencer;
		}
	);

	FSequencerEdMode* SequencerEdMode = (FSequencerEdMode*)(GLevelEditorModeTools().GetActiveMode(FSequencerEdMode::EM_SequencerMode));
	if (SequencerEdMode)
	{
		SequencerEdMode->RemoveSequencer(StaticCastSharedRef<FSequencer>(InSequencer));
	}

	KeyFrameHandler->Remove(InSequencer);

	bool bHasValidSequencer = false;
	bool bHasSequencerEditor = false;
	bool bHasOutlinerColumns = false;
	for (const FSequencerAndOptions& In : BoundSequencers)
	{
		if (In.Sequencer.IsValid())
		{
			bHasValidSequencer = true;
			bHasSequencerEditor = bHasSequencerEditor || In.Options.bActivateSequencerEdMode;
			bHasOutlinerColumns = bHasOutlinerColumns || In.Options.bAttachOutlinerColumns;
		}
	}
	if (!bHasValidSequencer)
	{
		AcquiredResources.Release();
	}
	if (!bHasSequencerEditor)
	{
		DeactivateSequencerEditorMode();
	}
	if (!bHasOutlinerColumns)
	{
		DetachOutlinerColumn();
	}

	OnSequencersChanged.Broadcast();
}

TArray<TWeakPtr<ISequencer>> FLevelEditorSequencerIntegration::GetSequencers()
{
	TArray<TWeakPtr<ISequencer>> SequencerPtrs;
	SequencerPtrs.Reserve(BoundSequencers.Num());
	for (FSequencerAndOptions& SequencerAndOption : BoundSequencers)
	{
		SequencerPtrs.Add(SequencerAndOption.Sequencer);
	}

	return SequencerPtrs;
}

void AddActorsToBindingsMapRecursive(FSequencer& Sequencer, UMovieSceneSequence* Sequence, FMovieSceneSequenceIDRef SequenceID, const FMovieSceneSequenceHierarchy* Hierarchy, TMap<FObjectKey, FString>& ActorBindingsMap)
{
	if (UMovieScene* MovieScene = Sequence->GetMovieScene())
	{
		FString SequenceName = Sequence->GetDisplayName().ToString();

		// Search all possessables
		for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
		{
			FGuid ThisGuid = MovieScene->GetPossessable(Index).GetGuid();

			for (TWeakObjectPtr<> WeakObject : Sequencer.FindBoundObjects(ThisGuid, SequenceID))
			{
				AActor* Actor = Cast<AActor>(WeakObject.Get());
				if (Actor != nullptr)
				{
					FObjectKey ActorKey(Actor);

					if (ActorBindingsMap.Contains(ActorKey))
					{
						ActorBindingsMap[ActorKey] = ActorBindingsMap[ActorKey] + TEXT(", ") + SequenceName;
					}
					else
					{
						ActorBindingsMap.Add(ActorKey, SequenceName);
					}
				}
			}
		}

		// Search all spawnables
		for (int32 Index = 0; Index < MovieScene->GetSpawnableCount(); ++Index)
		{
			FGuid ThisGuid = MovieScene->GetSpawnable(Index).GetGuid();

			for (TWeakObjectPtr<> WeakObject : Sequencer.FindBoundObjects(ThisGuid, SequenceID))
			{
				AActor* Actor = Cast<AActor>(WeakObject.Get());
				if (Actor != nullptr)
				{
					FObjectKey ActorKey(Actor);

					if (ActorBindingsMap.Contains(ActorKey))
					{
						ActorBindingsMap[ActorKey] = ActorBindingsMap[ActorKey] + TEXT(", ") + SequenceName;
					}
					else
					{
						ActorBindingsMap.Add(ActorKey, SequenceName);
					}
				}
			}
		}
	}

	if (Hierarchy)
	{
		// Recurse into child nodes
		if (const FMovieSceneSequenceHierarchyNode* Node = Hierarchy->FindNode(SequenceID))
		{
			for (FMovieSceneSequenceIDRef ChildID : Node->Children)
			{
				const FMovieSceneSubSequenceData* SubData     = Hierarchy->FindSubData(ChildID);
				UMovieSceneSequence*              SubSequence = SubData ? SubData->GetSequence() : nullptr;

				if (SubSequence)
				{
					AddActorsToBindingsMapRecursive(Sequencer, SubSequence, ChildID, Hierarchy, ActorBindingsMap);
				}
			}
		}
	}
}

void AddPropertiesToBindingsMap(TWeakPtr<FSequencer> Sequencer, UMovieSceneSequence* Sequence, FMovieSceneSequenceIDRef SequenceID, TMap<FObjectKey, TArray<FString> >& PropertyBindingsMap)
{
	const UMovieScene* MovieScene = Sequence->GetMovieScene();

	for (FMovieSceneBinding Binding : MovieScene->GetBindings())
	{
		for (UMovieSceneTrack* Track : Binding.GetTracks())
		{
			if (Track->IsA(UMovieScenePropertyTrack::StaticClass()))
			{
				UMovieScenePropertyTrack* PropertyTrack = Cast<UMovieScenePropertyTrack>(Track);
				FName PropertyName = PropertyTrack->GetPropertyName();
				FString PropertyPath = PropertyTrack->GetPropertyPath().ToString();

				// Find the property for the given actor
				for (TWeakObjectPtr<> WeakObject : Sequencer.Pin()->FindBoundObjects(Binding.GetObjectGuid(), SequenceID))
				{
					if (WeakObject.IsValid())
					{
						FObjectKey ObjectKey(WeakObject.Get());

						if (!PropertyBindingsMap.Contains(ObjectKey))
						{
							PropertyBindingsMap.Add(ObjectKey);
						}

						PropertyBindingsMap[ObjectKey].Add(PropertyPath);
					}
				}
			}
		}
	}
}

FString FLevelEditorSequencerBindingData::GetLevelSequencesForActor(TWeakPtr<FSequencer> Sequencer, const AActor* InActor)
{
	if (bActorBindingsDirty)
	{
		UpdateActorBindingsData(Sequencer);
	}

	FObjectKey ActorKey(InActor);

	if (ActorBindingsMap.Contains(ActorKey))
	{
		return ActorBindingsMap[ActorKey];
	}
	return FString();
}
	
bool FLevelEditorSequencerBindingData::GetIsPropertyBound(TWeakPtr<FSequencer> Sequencer, const struct FPropertyAndParent& InPropertyAndParent)
{
	if (bPropertyBindingsDirty)
	{
		UpdatePropertyBindingsData(Sequencer);
	}

	for (auto Object : InPropertyAndParent.Objects)
	{
		FObjectKey ObjectKey(Object.Get());

		if (PropertyBindingsMap.Contains(ObjectKey))
		{
			return PropertyBindingsMap[ObjectKey].Contains(InPropertyAndParent.Property.GetName());
		}
	}

	return false;
}

void FLevelEditorSequencerBindingData::UpdateActorBindingsData(TWeakPtr<FSequencer> InSequencer)
{
	static bool bIsReentrant = false;

	TSharedPtr<FSequencer> Pinned = InSequencer.Pin();
	if( !bIsReentrant && Pinned.IsValid() )
	{
		ActorBindingsMap.Empty();
	
		// Finding the bound objects can cause bindings to be evaluated and changed, causing this to be invoked again
		TGuardValue<bool> ReentrantGuard(bIsReentrant, true);

		FMovieSceneRootEvaluationTemplateInstance& RootInstance = Pinned->GetEvaluationTemplate();
		const FMovieSceneSequenceHierarchy*        Hierarchy    = RootInstance.GetCompiledDataManager()->FindHierarchy(RootInstance.GetCompiledDataID());

		UMovieSceneSequence* RootSequence = Pinned->GetRootMovieSceneSequence();
		
		if (RootSequence)
		{
			AddActorsToBindingsMapRecursive(*Pinned, RootSequence, MovieSceneSequenceID::Root, Hierarchy, ActorBindingsMap);
		}

		bActorBindingsDirty = false;

		ActorBindingsDataChanged.Broadcast();
	}
}


void FLevelEditorSequencerBindingData::UpdatePropertyBindingsData(TWeakPtr<FSequencer> InSequencer)
{
	static bool bIsReentrant = false;

	if( !bIsReentrant )
	{
		PropertyBindingsMap.Empty();
	
		// Finding the bound objects can cause bindings to be evaluated and changed, causing this to be invoked again
		TGuardValue<bool> ReentrantGuard(bIsReentrant, true);

		AddPropertiesToBindingsMap(InSequencer, InSequencer.Pin()->GetRootMovieSceneSequence(), MovieSceneSequenceID::Root, PropertyBindingsMap);

		FMovieSceneRootEvaluationTemplateInstance& RootInstance = InSequencer.Pin()->GetEvaluationTemplate();
		const FMovieSceneSequenceHierarchy* Hierarchy = RootInstance.GetCompiledDataManager()->FindHierarchy(RootInstance.GetCompiledDataID());
		if (Hierarchy)
		{
			for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
			{
				UMovieSceneSequence* Sequence = Pair.Value.GetSequence();
				if (Sequence)
				{
					AddPropertiesToBindingsMap(InSequencer, Sequence, Pair.Key, PropertyBindingsMap);
				}
			}
		}

		bPropertyBindingsDirty = false;

		PropertyBindingsDataChanged.Broadcast();
	}
}

bool FLevelEditorSequencerIntegration::IsPropertyReadOnly(const FPropertyAndParent& InPropertyAndParent)
{
	for (const FSequencerAndOptions& SequencerAndOptions : BoundSequencers)
	{
		TSharedPtr<FSequencer> Pinned = SequencerAndOptions.Sequencer.Pin();
		if (Pinned.IsValid())
		{
			if (Pinned.Get()->GetAllowEditsMode() == EAllowEditsMode::AllowLevelEditsOnly)
			{
				if (SequencerAndOptions.BindingData.Get().GetIsPropertyBound(SequencerAndOptions.Sequencer, InPropertyAndParent))
				{
					return true;
				}
			}
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
