// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/CurveEditorExtension.h"

#include "CurveEditor/SequencerCurveEditorApp.h"
#include "Filters/Linking/Mode/ILinkedFilterViewModel.h"
#include "FrameNumberDetailsCustomization.h"
#include "Framework/Docking/TabManager.h"
#include "Menus/SequencerToolbarUtils.h"
#include "Misc/TransportControlsHelper.h"
#include "Modification/Resolution/CurveMetaDataIdentifiers.h"
#include "Modification/Resolution/CurveModelLookUpInfo.h"
#include "Modification/Utils/ScopedSelectionChange.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "SCurveEditorPanel.h"
#include "SCurveKeyDetailPanel.h"
#include "Sequencer.h"
#include "Toolkits/IToolkitHost.h"
#include "Tree/SCurveEditorTree.h"

#define LOCTEXT_NAMESPACE "SequencerCurveEditorExtension"

namespace UE
{
namespace Sequencer
{
const FName FCurveEditorExtension::CurveEditorTabName = FName(TEXT("SequencerGraphEditor"));

FCurveEditorExtension::FCurveEditorExtension()
{}

FCurveEditorExtension::~FCurveEditorExtension()
{}

TSharedPtr<ISequencer> FCurveEditorExtension::GetSequencer() const
{
	const TSharedPtr<FSequencerEditorViewModel> OwnerModel = WeakOwnerModel.Pin();
	return OwnerModel.IsValid() ? OwnerModel->GetSequencerImpl() : nullptr;
}

USequencerSettings* FCurveEditorExtension::GetSequencerSettings() const
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	return Sequencer.IsValid() ? Sequencer->GetSequencerSettings() : nullptr;
}

TSharedPtr<FTabManager> FCurveEditorExtension::GetTabManager() const
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	const TSharedPtr<IToolkitHost> ToolkitHost = Sequencer.IsValid() ? Sequencer->GetToolkitHost() : nullptr;
	return ToolkitHost.IsValid() ? ToolkitHost->GetTabManager() : nullptr;
}

void FCurveEditorExtension::OnCreated(TSharedRef<FViewModel> InWeakOwner)
{
	ensureMsgf(!WeakOwnerModel.Pin().IsValid(), TEXT("This extension was already created!"));
	WeakOwnerModel = InWeakOwner->CastThisShared<FSequencerEditorViewModel>();
}

void FCurveEditorExtension::CreateCurveEditor(const FTimeSliderArgs& TimeSliderArgs)
{
	const TSharedPtr<FSequencerEditorViewModel> OwnerModel = WeakOwnerModel.Pin();
	if (!ensure(OwnerModel))
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = OwnerModel->GetSequencerImpl();
	if (!ensure(Sequencer))
	{
		return;
	}

	USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!ensure(SequencerSettings))
	{
		return;
	}

	// If they've said they want to support the curve editor then they need to provide a toolkit host
	// so that we know where to spawn our tab into.
	const TSharedPtr<IToolkitHost> ToolkitHost = Sequencer->GetToolkitHost();
	if (!ensure(ToolkitHost.IsValid()))
	{
		return;
	}

	const TSharedPtr<FTabManager> TabManager = GetTabManager();
	if (!ensure(TabManager.IsValid()))
	{
		return;
	}

	CurveEditorApp = MakePimpl<FSequencerCurveEditorApp>(Sequencer.ToSharedRef(), TimeSliderArgs);
}

TSharedPtr<FCurveEditor> FCurveEditorExtension::GetCurveEditor() const
{
	return CurveEditorApp ? CurveEditorApp->GetCurveEditor() : nullptr;
}

void FCurveEditorExtension::OpenCurveEditor()
{
	if (CurveEditorApp)
	{
		CurveEditorApp->OpenCurveEditor();
	}
}

bool FCurveEditorExtension::IsCurveEditorOpen() const
{
	return CurveEditorApp && CurveEditorApp->IsCurveEditorOpen();
}

void FCurveEditorExtension::CloseCurveEditor()
{
	if (CurveEditorApp)
	{
		CurveEditorApp->CloseCurveEditor();
	}
}

TSharedPtr<SCurveEditorTree> FCurveEditorExtension::GetCurveEditorTreeView() const
{
	return CurveEditorApp ? CurveEditorApp->GetCurveEditorTreeView() : nullptr;
}

void FCurveEditorExtension::RequestSyncSelection()
{
	if (CurveEditorApp)
	{
		CurveEditorApp->RequestSyncSelection();
	}
}


EVisibility FCurveEditorExtension::GetPopoutTransportControlsVisibility() const
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return EVisibility::Collapsed;
	}

	const bool bShowTransportControls = GetDefault<UCurveEditorSettings>()->GetShowTransportControls();

	return IsPopoutTransportControlsVisible(Sequencer.ToSharedRef(), bShowTransportControls)
		? EVisibility::Visible : EVisibility::Collapsed;
}

} // namespace Sequencer
} // namespace UE

#undef LOCTEXT_NAMESPACE
