// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/CurveEditorIntegrationExtension.h"

#include "CurveEditor.h"
#include "CurveEditor/SequencerCurveEditorApp.h"
#include "MVVM/CurveEditorExtension.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "SequencerSettings.h"

namespace UE
{
namespace Sequencer
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS

FCurveEditorIntegrationExtension::FCurveEditorIntegrationExtension()
{
}

void FCurveEditorIntegrationExtension::OnCreated(TSharedRef<FViewModel> InWeakOwner)
{
	ensureMsgf(!WeakOwnerModel.Pin().IsValid(), TEXT("This extension was already created!"));
	WeakOwnerModel = InWeakOwner->CastThisShared<FSequenceModel>();
}

void FCurveEditorIntegrationExtension::OnHierarchyChanged()
{
	UpdateCurveEditor();
}

FCurveEditorExtension* FCurveEditorIntegrationExtension::GetCurveEditorExtension()
{
	if (TSharedPtr<FSequenceModel> OwnerModel = WeakOwnerModel.Pin())
	{
		if (TSharedPtr<FSequencerEditorViewModel> EditorViewModel = OwnerModel->GetEditor())
		{
			return EditorViewModel->CastDynamic<FCurveEditorExtension>();
		}
	}
	return nullptr;
}

FSequencerCurveEditorApp* FCurveEditorIntegrationExtension::GetCurveEditorIntegration() const
{
	TSharedPtr<FSequenceModel> OwnerModel = WeakOwnerModel.Pin();
	const TSharedPtr<ISequencer> Sequencer = OwnerModel ? OwnerModel->GetSequencer() : nullptr;
	return Sequencer ? FSequencerCurveEditorApp::Get(*Sequencer) : nullptr;
}

void FCurveEditorIntegrationExtension::UpdateCurveEditor()
{
	if (FSequencerCurveEditorApp* CurveEditorApp = GetCurveEditorIntegration())
	{
		CurveEditorApp->UpdateCurveEditor();
	}
}

void FCurveEditorIntegrationExtension::ResetCurveEditor()
{
	if (FSequencerCurveEditorApp* CurveEditorApp = GetCurveEditorIntegration())
	{
		CurveEditorApp->ResetCurveEditor();
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
} // namespace Sequencer
} // namespace UE

