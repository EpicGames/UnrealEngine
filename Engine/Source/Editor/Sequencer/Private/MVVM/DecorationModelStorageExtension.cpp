// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/DecorationModelStorageExtension.h"

#include "MVVM/ViewModels/DecorationOutlinerModel.h"
#include "MVVM/ViewModels/SectionOutlinerModel.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "Decorations/IMovieSceneChannelDecoration.h"
#include "Decorations/IMovieSceneSectionProviderDecoration.h"
#include "Decorations/MovieSceneDecorationContainer.h"
#include "ISequencer.h"
#include "ISequencerDecorationEditor.h"

namespace UE::Sequencer
{

FDecorationModelStorageExtension::FDecorationModelStorageExtension()
{
}

void FDecorationModelStorageExtension::OnReinitialize()
{
	for (auto It = DecorationToModel.CreateIterator(); It; ++It)
	{
		if (It.Key().ResolveObjectPtr() == nullptr || It.Value().Pin().Get() == nullptr)
		{
			It.RemoveCurrent();
		}
	}
	DecorationToModel.Compact();
}

TSharedPtr<FDecorationOutlinerModel> FDecorationModelStorageExtension::FindOrCreateModel(
	UObject* Decoration,
	UMovieSceneDecorationContainerObject* Container,
	UMovieSceneTrack* ParentTrack,
	bool& bOutNewlyCreated)
{
	bOutNewlyCreated = false;

	if (!Decoration || !Container || !ParentTrack)
	{
		return nullptr;
	}

	FObjectKey DecorationKey(Decoration);
	if (TSharedPtr<FDecorationOutlinerModel> Existing = DecorationToModel.FindRef(DecorationKey).Pin())
	{
		return Existing;
	}

	TSharedPtr<FDecorationOutlinerModel> NewModel = MakeShared<FDecorationOutlinerModel>(ParentTrack, Container, Decoration);
	DecorationToModel.Add(DecorationKey, NewModel);
	bOutNewlyCreated = true;
	return NewModel;
}

TSharedPtr<FDecorationOutlinerModel> FDecorationModelStorageExtension::FindModel(
	const UObject* Decoration) const
{
	FObjectKey DecorationKey(Decoration);
	return DecorationToModel.FindRef(DecorationKey).Pin();
}

void FDecorationModelStorageExtension::SyncDecorationModels(
	UMovieSceneDecorationContainerObject* Container,
	UMovieSceneTrack* ParentTrack,
	FViewModelChildren& OutlinerChildren,
	ISequencer* InSequencer)
{
	if (!Container || !ParentTrack)
	{
		return;
	}

	// Build the set of currently visible decorations on this container
	TSet<UObject*> VisibleDecorations;
	for (const TObjectPtr<UObject>& Decoration : Container->GetDecorations())
	{
		if (!Decoration)
		{
			continue;
		}

		const bool bHasDecorationEditor = InSequencer
			&& InSequencer->FindDecorationEditor(Decoration->GetClass()) != nullptr;

		if (Cast<IMovieSceneChannelDecoration>(Decoration.Get()) ||
			Cast<IMovieSceneSectionProviderDecoration>(Decoration.Get()) ||
			bHasDecorationEditor)
		{
			VisibleDecorations.Add(Decoration.Get());
		}
	}

	// Create or update models for each visible decoration
	for (UObject* Decoration : VisibleDecorations)
	{
		bool bNewlyCreated = false;
		TSharedPtr<FDecorationOutlinerModel> Model = FindOrCreateModel(Decoration, Container, ParentTrack, bNewlyCreated);
		if (Model)
		{
			OutlinerChildren.AddChild(Model);
			// New models get OnConstruct via the framework (SetSharedData). For existing models,
			// only reconstruct if the decoration's state actually changed.
			if (!bNewlyCreated && Model->NeedsReconstruct())
			{
				Model->OnConstruct();
			}
		}
	}

	// Collect stale decoration models before removing (can't modify list during iteration)
	TArray<TSharedPtr<FDecorationOutlinerModel>> StaleModels;
	for (TSharedPtr<FDecorationOutlinerModel> DecorationModel : OutlinerChildren.IterateSubList<FDecorationOutlinerModel>())
	{
		if (!DecorationModel)
		{
			continue;
		}

		// Only remove models that belong to this container
		if (DecorationModel->GetDecorationContainer().Get() != Container)
		{
			continue;
		}

		UObject* Decoration = DecorationModel->GetDecoration();
		if (!Decoration || !VisibleDecorations.Contains(Decoration))
		{
			StaleModels.Add(DecorationModel);
		}
	}

	for (const TSharedPtr<FDecorationOutlinerModel>& StaleModel : StaleModels)
	{
		StaleModel->RemoveFromParent();
	}

	// Clean up stale decoration models on section outliners. When decorations are
	// removed from a section, the event fires before the decoration is actually
	// removed from the container, so event-driven cleanup can't detect the change.
	// This deferred check catches it during the subsequent rebuild.
	for (TSharedPtr<FSectionOutlinerModel> SectionOutliner : OutlinerChildren.IterateSubList<FSectionOutlinerModel>())
	{
		if (!SectionOutliner)
		{
			continue;
		}

		UMovieSceneSection* Section = SectionOutliner->GetSection();
		if (!Section)
		{
			continue;
		}

		// Collect stale models from section outliner children
		StaleModels.Reset();
		FViewModelChildren SectionOutlinerChildren = SectionOutliner->GetChildList(EViewModelListType::Outliner);
		for (TSharedPtr<FDecorationOutlinerModel> DecorationModel : SectionOutlinerChildren.IterateSubList<FDecorationOutlinerModel>())
		{
			if (!DecorationModel)
			{
				continue;
			}

			UObject* Decoration = DecorationModel->GetDecoration();
			if (!Decoration)
			{
				StaleModels.Add(DecorationModel);
				continue;
			}

			// Check if this decoration is still on the section
			bool bStillExists = false;
			for (const TObjectPtr<UObject>& SectionDecoration : Section->GetDecorations())
			{
				if (SectionDecoration.Get() == Decoration)
				{
					bStillExists = true;
					break;
				}
			}

			if (!bStillExists)
			{
				StaleModels.Add(DecorationModel);
			}
		}

		for (const TSharedPtr<FDecorationOutlinerModel>& StaleModel : StaleModels)
		{
			StaleModel->RemoveFromParent();
		}
	}
}

} // namespace UE::Sequencer
