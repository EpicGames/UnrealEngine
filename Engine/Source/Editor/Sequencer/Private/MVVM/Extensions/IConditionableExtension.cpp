// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Extensions/IConditionableExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MovieSceneSection.h"

namespace UE::Sequencer
{

	ECachedConditionState FConditionStateCacheExtension::ComputeFlagsForModel(const FViewModelPtr& ViewModel)
	{
		ECachedConditionState& ParentFlags = IndividualItemFlags.Last();
		ECachedConditionState ThisModelFlags = ECachedConditionState::None;

		if (EnumHasAnyFlags(ParentFlags, ECachedConditionState::HasCondition | ECachedConditionState::ParentHasCondition))
		{
			ThisModelFlags |= ECachedConditionState::ParentHasCondition;

			if (EnumHasAnyFlags(ParentFlags, ECachedConditionState::ConditionEvaluatingTrue | ECachedConditionState::ParentHasConditionEvaluatingTrue))
			{
				ThisModelFlags |= ECachedConditionState::ParentHasConditionEvaluatingTrue;
			}
		}

		if (TViewModelPtr<IConditionableExtension> Conditionable = ViewModel.ImplicitCast())
		{
			EConditionableConditionState ConditionState = Conditionable->GetConditionState();
			if (ConditionState != EConditionableConditionState::None)
			{
				ThisModelFlags |= ECachedConditionState::HasCondition;
			}

			// Special case- if we're a track or track row, and our section has a condition, mark that.
			// This allows us to surface on the track row level that a section on that row has a condition

			int32 RowIndex = INDEX_NONE;
			if (TViewModelPtr<ITrackRowExtension> TrackRowExtension = ViewModel.ImplicitCast())
			{
				RowIndex = TrackRowExtension->GetRowIndex();
			}

			if (TViewModelPtr<ISectionOwnerExtension> SectionOwner = ViewModel.ImplicitCast())
			{
				for (UMovieSceneSection* Section : SectionOwner->GetSections())
				{
					if (Section 
						&& (RowIndex == INDEX_NONE || Section->GetRowIndex() == RowIndex)
						&& Section->ConditionContainer.Condition)
					{
						ThisModelFlags |= ECachedConditionState::SectionHasCondition;
					}
				}
			}

			if (ConditionState == EConditionableConditionState::HasConditionEvaluatingTrue || ConditionState == EConditionableConditionState::HasConditionEditorForceTrue)
			{
				ThisModelFlags |= ECachedConditionState::ConditionEvaluatingTrue;
				if (ConditionState == EConditionableConditionState::HasConditionEditorForceTrue)
				{
					ThisModelFlags |= ECachedConditionState::EditorForceTrue;
				}
			}
		}

		return ThisModelFlags;
	}

	void FConditionStateCacheExtension::PostComputeChildrenFlags(const FViewModelPtr& ViewModel, ECachedConditionState& OutThisModelFlags, ECachedConditionState& OutPropagateToParentFlags)
	{
		// --------------------------------------------------------------------
		// Handle condition state propagation
		const bool bHasCondition = EnumHasAnyFlags(OutThisModelFlags, ECachedConditionState::HasCondition);
		const bool bSectionHasCondition = EnumHasAnyFlags(OutThisModelFlags, ECachedConditionState::SectionHasCondition);
		const bool bChildrenHaveCondition = EnumHasAnyFlags(OutThisModelFlags, ECachedConditionState::ChildHasCondition);

		if (bHasCondition || bSectionHasCondition || bChildrenHaveCondition)
		{
			OutPropagateToParentFlags |= ECachedConditionState::ChildHasCondition;
		}
	}

} // namespace UE::Sequencer

