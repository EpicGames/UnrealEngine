// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/FolderModelStorageExtension.h"
#include "MVVM/TrackModelStorageExtension.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/FolderModel.h"
#include "MVVM/ViewModels/TrackModel.h"

#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MovieSceneFolder.h"

namespace UE
{
namespace Sequencer
{

FFolderModelStorageExtension::FFolderModelStorageExtension()
{
}

TSharedPtr<FFolderModel> FFolderModelStorageExtension::FindModelForFolder(UMovieSceneFolder* Folder) const
{
	return FolderToModel.FindRef(Folder).Pin();
}

TSharedPtr<FFolderModel> FFolderModelStorageExtension::CreateModelForFolder(UMovieSceneFolder* Folder, TSharedPtr<FViewModel> DesiredParent)
{
	if (!DesiredParent)
	{
		DesiredParent = OwnerModel->AsShared();
	}

	TOptional<FViewModelChildren> Children = DesiredParent->GetChildList(EViewModelListType::Outliner);
	ensureMsgf(Children.IsSet(), TEXT("Attempting to create a folder within something that is not able to contain outliner items"));

	FObjectKey FolderAsKey(Folder);
	if (TSharedPtr<FFolderModel> FolderModel = FolderToModel.FindRef(FolderAsKey).Pin())
	{
		if (Children.IsSet())
		{
			Children->AddChild(FolderModel);
		}
		return FolderModel;
	}

	TSharedPtr<FFolderModel> NewModel = MakeShared<FFolderModel>(Folder);

	// IMPORTANT: We always add the model to the map before calling initialize
	// So that any code that runs inside Initialize is still able to find this
	FolderToModel.Add(FolderAsKey, NewModel);

	if (Children.IsSet())
	{
		Children->AddChild(NewModel);
	}

	return NewModel;
}

void FFolderModelStorageExtension::OnCreated(TSharedRef<FViewModel> InWeakOwner)
{
	OwnerModel = InWeakOwner->CastThis<FSequenceModel>();
}

void FFolderModelStorageExtension::OnReinitialize()
{
	Unlink();

	UMovieSceneSequence* MovieSceneSequence = OwnerModel->GetSequence();
	UMovieScene* MovieScene = MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr;
	if (MovieScene)
	{
		MovieScene->EventHandlers.Link(this);

		for (UMovieSceneFolder* RootFolder : MovieScene->GetRootFolders())
		{
			OnRootFolderAdded(RootFolder);
		}
	}

	for (auto It = FolderToModel.CreateIterator(); It; ++It)
	{
		if (It.Key().ResolveObjectPtr() == nullptr || It.Value().Pin().Get() == nullptr)
		{
			It.RemoveCurrent();
		}
	}
	FolderToModel.Compact();
}

void FFolderModelStorageExtension::OnRootFolderAdded(UMovieSceneFolder* InFolder)
{
	CreateModelForFolder(InFolder);
}

void FFolderModelStorageExtension::OnRootFolderRemoved(UMovieSceneFolder* InFolder)
{
	FObjectKey FolderAsKey(InFolder);

	TSharedPtr<FFolderModel> Model = FolderToModel.FindRef(FolderAsKey).Pin();
	if (Model)
	{
		Model->RemoveFromParent();
	}
}

} // namespace Sequencer
} // namespace UE

