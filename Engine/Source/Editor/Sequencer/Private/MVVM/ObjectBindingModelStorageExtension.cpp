// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ObjectBindingModelStorageExtension.h"

#include "Containers/Array.h"
#include "EventHandlers/ISequenceDataEventHandler.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/PossessableModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/SpawnableModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "Misc/AssertionMacros.h"
#include "MovieScene.h"
#include "MovieSceneBinding.h"
#include "MovieSceneSequence.h"
#include "Templates/TypeHash.h"

class UMovieSceneTrack;
struct FMovieScenePossessable;
struct FMovieSceneSpawnable;

namespace UE
{
namespace Sequencer
{

class FPlaceholderObjectBindingModel
	: public FViewModel
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE(FPlaceholderObjectBindingModel, FViewModel);

	FPlaceholderObjectBindingModel()
		: OutlinerChildList(EViewModelListType::Outliner)
	{
		RegisterChildList(&OutlinerChildList);
	}

private:
	FViewModelListHead OutlinerChildList;
};

UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FPlaceholderObjectBindingModel);

FObjectBindingModelStorageExtension::FObjectBindingModelStorageExtension()
{
}

TSharedPtr<FObjectBindingModel> FObjectBindingModelStorageExtension::CreateModelForObjectBinding(const FMovieSceneBinding& Binding)
{
	if (TSharedPtr<FObjectBindingModel> ExistingModel = FindModelForObjectBinding(Binding.GetObjectGuid()))
	{
		return ExistingModel;
	}

	UMovieScene* MovieScene = OwnerModel->GetMovieScene();
	check(MovieScene);

	const FMovieScenePossessable* Possessable = MovieScene->FindPossessable(Binding.GetObjectGuid());
	const FMovieSceneSpawnable*   Spawnable   = MovieScene->FindSpawnable(Binding.GetObjectGuid());

	TSharedPtr<FObjectBindingModel> NewModel;
	TSharedPtr<FSpawnableModel>     SpawnableModel;
	TSharedPtr<FPossessableModel>   PossessableModel;
	if (Possessable)
	{
		PossessableModel = MakeShared<FPossessableModel>(OwnerModel, Binding, *Possessable);
		NewModel = PossessableModel;
	}
	else if (Spawnable)
	{
		SpawnableModel = MakeShared<FSpawnableModel>(OwnerModel, Binding, *Spawnable);
		NewModel = SpawnableModel;
	}
	else
	{
		NewModel = MakeShared<FObjectBindingModel>(OwnerModel, Binding);
	}

	// IMPORTANT: We always add the model to the map before calling initialize
	// So that any code that runs inside Initialize is still able to find this
	ObjectBindingToModel.Add(Binding.GetObjectGuid(), NewModel);

	return NewModel;
}

TSharedPtr<FObjectBindingModel> FObjectBindingModelStorageExtension::FindModelForObjectBinding(const FGuid& InObjectBindingID) const
{
	return ObjectBindingToModel.FindRef(InObjectBindingID).Pin();
}

TSharedPtr<FViewModel> FObjectBindingModelStorageExtension::CreatePlaceholderForObjectBinding(const FGuid& ObjectID)
{
	TSharedPtr<FViewModel> Placeholder = FindPlaceholderForObjectBinding(ObjectID);
	if (Placeholder)
	{
		return Placeholder;
	}

	TSharedPtr<FPlaceholderObjectBindingModel> NewPlaceholder = MakeShared<FPlaceholderObjectBindingModel>();
	ObjectBindingToPlaceholder.Add(ObjectID, NewPlaceholder);

	FViewModelChildren RootChildren = OwnerModel->GetChildList(EViewModelListType::Outliner);
	RootChildren.AddChild(NewPlaceholder);

	return NewPlaceholder;
}

TSharedPtr<FViewModel> FObjectBindingModelStorageExtension::FindPlaceholderForObjectBinding(const FGuid& InObjectBindingID) const
{
	return ObjectBindingToPlaceholder.FindRef(InObjectBindingID).Pin();
}

void FObjectBindingModelStorageExtension::OnCreated(TSharedRef<FViewModel> InWeakOwner)
{
	OwnerModel = InWeakOwner->CastThis<FSequenceModel>();
}

void FObjectBindingModelStorageExtension::OnReinitialize()
{
	Unlink();

	UMovieSceneSequence* MovieSceneSequence = OwnerModel->GetSequence();
	UMovieScene* MovieScene = MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr;
	if (MovieScene)
	{
		MovieScene->EventHandlers.Link(this);

		FViewModelChildren RootChildren = OwnerModel->GetChildList(EViewModelListType::Outliner);

		for (const FMovieSceneBinding& Binding : ((const UMovieScene*)MovieScene)->GetBindings())
		{
			// Need to create a new one for this object binding
			TSharedPtr<FViewModel> Model = GetOrCreateModelForBinding(Binding.GetObjectGuid());
			if (!Model->IsConstructed())
			{
				RootChildren.AddChild(Model->GetRoot());
			}
		}
	}

	Compact();
}

void FObjectBindingModelStorageExtension::Compact()
{
	for (auto It = ObjectBindingToPlaceholder.CreateIterator(); It; ++It)
	{
		if (It.Value().Pin().Get() == nullptr)
		{
			It.RemoveCurrent();
		}
	}
	for (auto It = ObjectBindingToModel.CreateIterator(); It; ++It)
	{
		if (It.Value().Pin().Get() == nullptr)
		{
			It.RemoveCurrent();
		}
	}

	ObjectBindingToModel.Compact();
	ObjectBindingToPlaceholder.Compact();
}

TSharedPtr<FViewModel> FObjectBindingModelStorageExtension::GetOrCreateModelForBinding(const FGuid& Binding)
{
	UMovieScene* MovieScene = OwnerModel->GetMovieScene();
	check(MovieScene);
	
	// Find the parent binding. Note that if GetOrCreateModelForBinding is being called from inside a loop,
	// This call makes the loop tend towards O(n^2) the greater percentage of parent bindings there are
	if (const FMovieSceneBinding* ObjectBinding = MovieScene->FindBinding(Binding))
	{
		return GetOrCreateModelForBinding(*ObjectBinding);
	}

	// Placeholders are always created at the root
	return CreatePlaceholderForObjectBinding(Binding);
}

TSharedPtr<FViewModel> FObjectBindingModelStorageExtension::GetOrCreateModelForBinding(const FMovieSceneBinding& Binding)
{
	TSharedPtr<FObjectBindingModel> ObjectModel = CreateModelForObjectBinding(Binding);

	// Set up parentage
	FGuid DesiredParent = ObjectModel->GetDesiredParentBinding();
	if (DesiredParent.IsValid())
	{
		TSharedPtr<FViewModel> Parent = GetOrCreateModelForBinding(DesiredParent);

		// If our parent doesn't have a parent, it will probably get destructed at the end of this
		// function because nothing is holding onto it, so we add it to the root to keep it alive
		if (Parent->GetParent() == nullptr)
		{
			OwnerModel->GetChildList(EViewModelListType::Outliner).AddChild(Parent);
		}

		Parent->GetChildList(EViewModelListType::Outliner).AddChild(ObjectModel);
	}

	TSharedPtr<FViewModel> Placeholder = FindPlaceholderForObjectBinding(Binding.GetObjectGuid());
	if (Placeholder)
	{
		// We previously encountered this GUID but there was no binding for it,
		// so convert the placeholder to a proper binding model
		ObjectBindingToPlaceholder.Remove(Binding.GetObjectGuid());

		FViewModelChildren PlaceholderChildren = Placeholder->GetChildList(EViewModelListType::Outliner);
		FViewModelChildren RealChildren        = ObjectModel->GetChildList(EViewModelListType::Outliner);

		PlaceholderChildren.MoveChildrenTo(RealChildren);

		// We're done with the placeholder now
		Placeholder->RemoveFromParent();
	}

	return ObjectModel;
}

void FObjectBindingModelStorageExtension::OnBindingAdded(const FMovieSceneBinding& Binding)
{
	TSharedPtr<FViewModel> Model = GetOrCreateModelForBinding(Binding);

	if (!Model->IsConstructed())
	{
		OwnerModel->GetChildList(EViewModelListType::Outliner)
			.AddChild(Model);
	}
}

void FObjectBindingModelStorageExtension::OnBindingRemoved(const FGuid& ObjectBindingID)
{
	TSharedPtr<FObjectBindingModel> Model = ObjectBindingToModel.FindRef(ObjectBindingID).Pin();

	ObjectBindingToModel.Remove(ObjectBindingID);

	if (Model)
	{
		Model->RemoveFromParent();
	}
}

void FObjectBindingModelStorageExtension::OnTrackAddedToBinding(UMovieSceneTrack* Track, const FGuid& ObjectBindingID)
{
	TSharedPtr<FObjectBindingModel> Model = ObjectBindingToModel.FindRef(ObjectBindingID).Pin();
	if (Model && Track && Track->ParentTrack == nullptr)
	{
		Model->AddTrack(Track);
	}
}

void FObjectBindingModelStorageExtension::OnTrackRemovedFromBinding(UMovieSceneTrack* Track, const FGuid& ObjectBindingID)
{
	TSharedPtr<FObjectBindingModel> Model = ObjectBindingToModel.FindRef(ObjectBindingID).Pin();
	if (Model)
	{
		Model->RemoveTrack(Track);
	}
}

void FObjectBindingModelStorageExtension::OnBindingParentChanged(const FGuid& ObjectBindingID, const FGuid& NewParent)
{
	TSharedPtr<FObjectBindingModel> Model = ObjectBindingToModel.FindRef(ObjectBindingID).Pin();
	if (Model)
	{
		Model->SetParentBindingID(NewParent);

		TSharedPtr<FViewModel> ParentModel = GetOrCreateModelForBinding(NewParent);
		if (ParentModel)
		{
			ParentModel->GetChildList(EViewModelListType::Outliner)
				.AddChild(Model);
		}
	}
}



} // namespace Sequencer
} // namespace UE

