// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/PinEditorExtension.h"

#include "MovieScene.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/IPinnableExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "ScopedTransaction.h"
#include "Sequencer.h"

namespace UE::Sequencer
{

FPinEditorExtension::FPinEditorExtension()
{

}

void FPinEditorExtension::OnCreated(TSharedRef<FViewModel> InWeakOwner)
{
	ensureMsgf(!WeakOwnerModel.Pin().IsValid(), TEXT("This extension was already created!"));
	WeakOwnerModel = InWeakOwner->CastThisShared<FSequencerEditorViewModel>();
}

bool FPinEditorExtension::IsNodePinned(TWeakViewModelPtr<IOutlinerExtension> InWeakOutlinerExtension) const
{
	TViewModelPtr<IOutlinerExtension> OutlinerExtension = InWeakOutlinerExtension.Pin();
	if (OutlinerExtension.IsValid())
	{
		if (TViewModelPtr<IPinnableExtension> PinnableExtension = OutlinerExtension.ImplicitCast())
		{
			return PinnableExtension->IsPinned();
		}
	}
	return false;
}

bool FPinEditorExtension::IsNodePinnable(TWeakViewModelPtr<IOutlinerExtension> InWeakOutlinerExtension) const
{
	TSharedPtr<FViewModel> Item = InWeakOutlinerExtension.Pin();
	if (Item)
	{
		return Item->GetHierarchicalDepth() == 1;
	}
	return false;
}

void FPinEditorExtension::SetNodePinned(TWeakViewModelPtr<IOutlinerExtension> InWeakOutlinerExtension, const bool bInIsPinned)
{
	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "SetNodePinned", "Set Node Pinned"));

	TViewModelPtr<IOutlinerExtension> OutlinerItem = InWeakOutlinerExtension.Pin();
	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = WeakOwnerModel.Pin();
	TSharedPtr<FSequencer> Sequencer = EditorViewModel ? EditorViewModel->GetSequencerImpl() : nullptr;

	if (OutlinerItem && Sequencer)
	{
		TSharedPtr<FOutlinerViewModel> Outliner = EditorViewModel->GetOutliner();
		Outliner->UnpinAllNodes();

		UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();

		if (MovieScene->IsReadOnly())
		{
			return;
		}

		MovieScene->Modify();

		FMovieSceneEditorData& EditorData = MovieScene->GetEditorData();

		if (OutlinerItem->GetSelectionState() == EOutlinerSelectionState::SelectedDirectly)
		{
			// pin all selected
			for (FViewModelPtr Node : Sequencer->GetViewModel()->GetSelection()->Outliner)
			{
				PinItem(EditorData, Node, bInIsPinned);
			}
		}
		else
		{
			// pin only passed in outliner extension
			TSharedPtr<FViewModel> Item = OutlinerItem;
			PinItem(EditorData, Item, bInIsPinned);
		}
	}
}

void FPinEditorExtension::PinItem(FMovieSceneEditorData& InEditorData, TSharedPtr<FViewModel> InItem, const bool bInIsPinned)
{
	TSharedPtr<IPinnableExtension> PinnableParent = InItem->FindAncestorOfType<IPinnableExtension>(true);

	if (PinnableParent)
	{
		PinnableParent->SetPinned(bInIsPinned);

		if (bInIsPinned)
		{
			InEditorData.PinnedNodes.AddUnique(IOutlinerExtension::GetPathName(InItem));
		}
		else
		{
			InEditorData.PinnedNodes.RemoveSingle(IOutlinerExtension::GetPathName(InItem));
		}
	}
}

} // namespace UE::Sequencer

