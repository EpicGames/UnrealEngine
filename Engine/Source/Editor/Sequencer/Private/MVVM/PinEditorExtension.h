// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/Extensions/DynamicExtensionContainer.h"

struct FMovieSceneEditorData;

namespace UE::Sequencer
{

class FSequencerEditorViewModel;
class IOutlinerExtension;

/**
* Extension that handles the pinned state of items.
*/
class FPinEditorExtension : public IDynamicExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID(FPinEditorExtension)

	FPinEditorExtension();

	/** Called when the extension is created on a data model. */
	virtual void OnCreated(TSharedRef<FViewModel> InWeakOwner) override;

	/** Returns whether a given item is pinned. */
	bool IsNodePinned(TWeakViewModelPtr<IOutlinerExtension> InWeakOutlinerExtension) const;

	/** Returns whether or not this item is at the top of the hierarchy. */
	bool IsNodePinnable(TWeakViewModelPtr<IOutlinerExtension> InWeakOutlinerExtension) const;

	/**
	* Modifies an item and it's child items to be pinned.
	* This operation applies to all selected items if the modified item is selected.
	* @param bInIsPinned - Pin state to set the item to
	* @param InWeakOutlinerExtension - Item to pin
	*/
	void SetNodePinned(TWeakViewModelPtr<IOutlinerExtension> InWeakOutlinerExtension, const bool bInIsPinned);

private:

	/** Helper function for modifying the pin state of individual items */
	void PinItem(FMovieSceneEditorData& InEditorData, TSharedPtr<FViewModel> InItem, const bool bInIsPinned);

private:

	/** The sequencer editor we are extending */
	TWeakPtr<FSequencerEditorViewModel> WeakOwnerModel;
};

} // namespace UE::Sequencer

