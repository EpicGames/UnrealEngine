// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/SectionModelStorageExtension.h"

#include "ISequencer.h"
#include "ISequencerDecorationEditor.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MovieSceneSection.h"
#include "ISequencerCoreModule.h"
#include "Modules/ModuleManager.h"

namespace UE::Sequencer
{

namespace SectionModelStorage
{

	ISequencerCoreModule& GetSequencerCoreModule()
	{
		static ISequencerCoreModule* Module = &FModuleManager::Get().LoadModuleChecked<ISequencerCoreModule>("SequencerCore");
		return *Module;
	}

}// namespace SectionModelStorage

FSectionModelStorageExtension::FSectionModelStorageExtension()
{
}

void FSectionModelStorageExtension::OnCreated(TSharedRef<FViewModel> InOwner)
{
	WeakOwner = InOwner;
}

void FSectionModelStorageExtension::OnReinitialize()
{
	for (auto It = SectionToModel.CreateIterator(); It; ++It)
	{
		if (It.Key().ResolveObjectPtr() == nullptr || It.Value().Pin().Get() == nullptr)
		{
			It.RemoveCurrent();
		}
	}
	SectionToModel.Compact();
}

TSharedPtr<FSectionModel> FSectionModelStorageExtension::CreateModelForSection(UMovieSceneSection* InSection, TSharedRef<ISequencerSection> SectionInterface)
{
	FObjectKey SectionAsKey(InSection);
	if (TSharedPtr<FSectionModel> Existing = SectionToModel.FindRef(SectionAsKey).Pin())
	{
		return Existing;
	}

	TSharedPtr<FSectionModel> SectionModel;

	if (TSharedPtr<FViewModel> ViewModel = SectionModelStorage::GetSequencerCoreModule().FactoryNewModel(InSection))
	{
		SectionModel = ViewModel->CastThisShared<FSectionModel>();
		ensureMsgf(SectionModel, TEXT("Section model type for Section Object was not an FSectionModel! %s (type: %s)"), *InSection->GetPathName(), *InSection->GetClass()->GetName());
	}
	
	if (!SectionModel)
	{
		SectionModel = MakeShared<FSectionModel>();
		SectionModel->InitializeObject(InSection);
	}

	SectionModel->InitializeSection(SectionInterface);

	SectionToModel.Add(SectionAsKey, SectionModel);

	NotifyDecorationEditors(InSection);

	return SectionModel;
}

TSharedPtr<FSectionModel> FSectionModelStorageExtension::FindModelForSection(const UMovieSceneSection* InSection) const
{
	FObjectKey SectionAsKey(InSection);
	return SectionToModel.FindRef(SectionAsKey).Pin();
}

void FSectionModelStorageExtension::NotifyDecorationEditors(UMovieSceneSection* InSection)
{
	if (!InSection)
	{
		return;
	}

	TSharedPtr<FViewModel> Owner = WeakOwner.Pin();
	if (!Owner)
	{
		return;
	}

	FSequenceModel* SequenceModel = Owner->CastThis<FSequenceModel>();
	if (!SequenceModel)
	{
		return;
	}

	TWeakPtr<ISequencer> WeakSequencer = SequenceModel->GetSequencer();
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	if (!SequencerPtr)
	{
		return;
	}

	for (TObjectPtr<UObject> Decoration : InSection->GetDecorations())
	{
		if (Decoration)
		{
			if (ISequencerDecorationEditor* Editor = SequencerPtr->FindDecorationEditor(Decoration->GetClass()))
			{
				Editor->OnSectionInterfaceCreated(*InSection, *Decoration, WeakSequencer);
			}
		}
	}
}

} // namespace UE::Sequencer

