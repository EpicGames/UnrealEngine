// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Selection/Selection.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Extensions/ISelectableExtension.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "Channels/MovieSceneChannel.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "SequencerCommonHelpers.h"

namespace UE::Sequencer
{

void FKeySelection::Deselect(FKeyHandle InKey)
{
	SequencerHelpers::RemoveDuplicateKeys(*this, MakeArrayView(&InKey, 1));

	TUniqueFragmentSelectionSet<FKeyHandle, FChannelModel>::Deselect(InKey);
}

void FKeySelection::Empty()
{
	SequencerHelpers::RemoveDuplicateKeys(*this, MakeArrayView(GetSelected().Array()));

	TUniqueFragmentSelectionSet<FKeyHandle, FChannelModel>::Empty();
}

bool FTrackAreaSelection::OnSelectItem(const FWeakViewModelPtr& WeakViewModel)
{
	if (TSharedPtr<FViewModel> ViewModel = WeakViewModel.Pin())
	{
		ISelectableExtension* Selectable = ViewModel->CastThis<ISelectableExtension>();
		if (Selectable && Selectable->IsSelectable() == ESelectionIntent::Never)
		{
			return false;
		}

		return true;
	}

	return false;
}

FSequencerSelection::FSequencerSelection()
{
	AddSelectionSet(&Outliner);
	AddSelectionSet(&TrackArea);
	AddSelectionSet(&KeySelection);
	AddSelectionSet(&MarkedFrames);
}

void FSequencerSelection::Initialize(TViewModelPtr<FEditorViewModel> InViewModel)
{
	FViewModelPtr RootModel = InViewModel->GetRootModel();
	if (RootModel)
	{
		FSimpleMulticastDelegate& HierarchyChanged = RootModel->GetSharedData()->SubscribeToHierarchyChanged(RootModel);
		HierarchyChanged.AddSP(this, &FSequencerSelection::OnHierarchyChanged);
	}
}

void FSequencerSelection::Empty()
{
	FSelectionEventSuppressor EventSuppressor = SuppressEvents();

	Outliner.Empty();
	TrackArea.Empty();
	KeySelection.Empty();
	MarkedFrames.Empty();
}

void FSequencerSelection::PreSelectionSetChangeEvent(FSelectionBase* InSelectionSet)
{
	if (InSelectionSet == &Outliner)
	{
		// Empty the track area selection when selecting anything on the outliner
		if (!TrackArea.HasPendingChanges() && !KeySelection.HasPendingChanges())
		{
			TrackArea.Empty();
			KeySelection.Empty();
		}
	}
}

void FSequencerSelection::PreBroadcastChangeEvent()
{
	// Repopulate the nodes with keys or sections set

	// First off reset the selection states from the previous set
	for (TWeakViewModelPtr<IOutlinerExtension> WeakOldNode : NodesWithKeysOrSections)
	{
		TViewModelPtr<IOutlinerExtension> OldNode = WeakOldNode.Pin();
		if (OldNode)
		{
			OldNode->ToggleSelectionState(EOutlinerSelectionState::HasSelectedKeys | EOutlinerSelectionState::HasSelectedTrackAreaItems, false);

			OldNode = OldNode.AsModel()->FindAncestorOfType<IOutlinerExtension>();
			while (OldNode)
			{
				OldNode->ToggleSelectionState(EOutlinerSelectionState::DescendentHasSelectedTrackAreaItems | EOutlinerSelectionState::DescendentHasSelectedKeys, false);
				OldNode = OldNode.AsModel()->FindAncestorOfType<IOutlinerExtension>();
			}
		}
	}

	// Reset the selection set
	NodesWithKeysOrSections.Reset();

	// Gather selection states from selected track area items
	for (FViewModelPtr TrackAreaModel : TrackArea)
	{
		TViewModelPtr<IOutlinerExtension> ParentOutlinerNode = TrackAreaModel->FindAncestorOfType<IOutlinerExtension>();
		if (ParentOutlinerNode)
		{
			ParentOutlinerNode->ToggleSelectionState(EOutlinerSelectionState::HasSelectedTrackAreaItems, true);
			NodesWithKeysOrSections.Add(ParentOutlinerNode);

			ParentOutlinerNode = ParentOutlinerNode.AsModel()->FindAncestorOfType<IOutlinerExtension>();
			while (ParentOutlinerNode)
			{
				ParentOutlinerNode->ToggleSelectionState(EOutlinerSelectionState::DescendentHasSelectedTrackAreaItems, true);
				ParentOutlinerNode = ParentOutlinerNode.AsModel()->FindAncestorOfType<IOutlinerExtension>();
			}
		}

	}

	// Gather selection states from selected keys
	{
		TSet<TViewModelPtr<FChannelModel>> Channels;
		for (const FKeyHandle& Key : KeySelection)
		{
			if (TViewModelPtr<FChannelModel> Channel = KeySelection.GetModelForKey(Key))
			{
				Channels.Add(Channel);
			}
		}

		TSet<TViewModelPtr<IOutlinerExtension>> ParentOutlinerNodes;
		ParentOutlinerNodes.Reserve(Channels.Num());
		for (const TViewModelPtr<FChannelModel>& Channel : Channels)
		{
			if (TViewModelPtr<IOutlinerExtension> ParentOutlinerNode = Channel ? Channel->GetLinkedOutlinerItem() : nullptr)
			{
				ParentOutlinerNodes.Add(ParentOutlinerNode);			
			}
		}

		NodesWithKeysOrSections.Reserve(NodesWithKeysOrSections.Num() + ParentOutlinerNodes.Num());
		for (TViewModelPtr<IOutlinerExtension> ParentOutlinerNode: ParentOutlinerNodes)
		{
			ParentOutlinerNode->ToggleSelectionState(EOutlinerSelectionState::HasSelectedKeys, true);
			NodesWithKeysOrSections.Add(ParentOutlinerNode);

			ParentOutlinerNode = ParentOutlinerNode.AsModel()->FindAncestorOfType<IOutlinerExtension>();
			while (ParentOutlinerNode)
			{
				ParentOutlinerNode->ToggleSelectionState(EOutlinerSelectionState::DescendentHasSelectedKeys, true);
				ParentOutlinerNode = ParentOutlinerNode.AsModel()->FindAncestorOfType<IOutlinerExtension>();
			}
		}
	}

	FSelectionEventSuppressor EventSuppressor = SuppressEvents();
	FOutlinerSelection OutlinerCopy = Outliner;

	// Select any outliner nodes that don't have keys or sections selected
	for (TViewModelPtr<IOutlinerExtension> OutlinerItem : Outliner)
	{
		bool bFound = false;
		bool bAnyIndirectSelection = false;

		const TSet<TViewModelPtr<IObjectBindingExtension>> ObjectBindingAncestors = OutlinerItem.AsModel()->GetAncestorsOfType<IObjectBindingExtension>(true).Populate<TSet>();


		for (TViewModelPtr<IOutlinerExtension> IndirectItem : IterateIndirectOutlinerSelection())
		{
			bAnyIndirectSelection = true;

			if (IndirectItem == OutlinerItem)
			{
				bFound = true;
				break;
			}

			for (TViewModelPtr<IObjectBindingExtension> IndirectObjectBindingItem : IndirectItem.AsModel()->GetAncestorsOfType<IObjectBindingExtension>(true))
			{
				if (ObjectBindingAncestors.Contains(IndirectObjectBindingItem))
				{
					bFound = true;
					break;
				}
			}

			if (bFound)
			{
				break;
			}
		}

		if (bAnyIndirectSelection && !bFound)
		{
			OutlinerCopy.Deselect(OutlinerItem);
		}
	}

	Outliner = OutlinerCopy;
}

FIndirectOutlinerSelectionIterator FSequencerSelection::IterateIndirectOutlinerSelection() const
{
	return FIndirectOutlinerSelectionIterator{ &NodesWithKeysOrSections };
}

TSet<FGuid> FSequencerSelection::GetBoundObjectsGuids()
{
	TSet<FGuid> OutGuids;

	for (const TWeakViewModelPtr<IOutlinerExtension>& WeakModel : NodesWithKeysOrSections)
	{
		FViewModelPtr Model = WeakModel.Pin();
		if (Model)
		{
			TSharedPtr<IObjectBindingExtension> ObjectBinding = Model->FindAncestorOfType<IObjectBindingExtension>(true);
			if (ObjectBinding)
			{
				OutGuids.Add(ObjectBinding->GetObjectGuid());
			}
		}
	}

	if (OutGuids.IsEmpty())
	{
		for (FViewModelPtr Model : Outliner)
		{
			TSharedPtr<IObjectBindingExtension> ObjectBinding = Model->FindAncestorOfType<IObjectBindingExtension>(true);
			if (ObjectBinding)
			{
				OutGuids.Add(ObjectBinding->GetObjectGuid());
			}
		}
	}

	return OutGuids;
}

TSet<UMovieSceneSection*> FSequencerSelection::GetSelectedSections() const
{
	TSet<UMovieSceneSection*> SelectedSections;
	SelectedSections.Reserve(TrackArea.Num());

	for (TViewModelPtr<FSectionModel> Model : TrackArea.Filter<FSectionModel>())
	{
		if (UMovieSceneSection* Section = Model->GetSection())
		{
			SelectedSections.Add(Section);
		}
	}

	return SelectedSections;
}

TSet<UMovieSceneTrack*> FSequencerSelection::GetSelectedTracks() const
{
	TSet<UMovieSceneTrack*> SelectedTracks;
	SelectedTracks.Reserve(TrackArea.Num());

	for (TViewModelPtr<ITrackExtension> TrackExtension : Outliner.Filter<ITrackExtension>())
	{
		if (UMovieSceneTrack* Track = TrackExtension->GetTrack())
		{
			SelectedTracks.Add(Track);
		}
	}

	for (TViewModelPtr<ITrackRowExtension> TrackRowExtension : Outliner.Filter<ITrackRowExtension>())
	{
		if (UMovieSceneTrack* Track = TrackRowExtension->GetParentTrack())
		{
			SelectedTracks.Add(Track);
		}
	}

	return SelectedTracks;
}

TSet<TPair<UMovieSceneTrack*, int32>> FSequencerSelection::GetSelectedTrackRows() const
{
	TSet<TPair<UMovieSceneTrack*, int32>> SelectedTrackRows;
	SelectedTrackRows.Reserve(TrackArea.Num());

	// Only add a 'track row' as selected if either we have a row-level model selected (FTrackRowModel or layer model),
	// or else we have a track selected and there's only a single track row, and the track allows multiple rows.
	for (TViewModelPtr<ITrackRowExtension> TrackRowExtension : Outliner.Filter<ITrackRowExtension>())
	{
		if (UMovieSceneTrack* Track = TrackRowExtension->GetParentTrack())
		{
			SelectedTrackRows.Add(TPair<UMovieSceneTrack*, int32>(Track, TrackRowExtension->GetRowIndex()));
		}
	}

	for (TViewModelPtr<ITrackExtension> TrackExtension : Outliner.Filter<ITrackExtension>())
	{
		if (UMovieSceneTrack* Track = TrackExtension->GetTrack())
		{
			if (Track->SupportsMultipleRows() && Track->GetMaxRowIndex() == 0)
			{
				SelectedTrackRows.Add(TPair<UMovieSceneTrack*, int32>(Track, 0));
			}
		}
	}

	TSet<UMovieSceneSection*> SelectedSections = GetSelectedSections();
	for (UMovieSceneSection* Section : SelectedSections)
	{
		if (UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>())
		{
			if (Track->SupportsMultipleRows())
			{
				SelectedTrackRows.Add(TPair<UMovieSceneTrack*, int32>(Track, Section->GetRowIndex()));
			}
		}
	}

	return SelectedTrackRows;
}


void FSequencerSelection::OnHierarchyChanged()
{
	// This is an esoteric hack that ensures we re-synchronize external (ie Actor)
	// selection when models are removed from the tree. Doing so ensures that
	// FSequencer::SynchronizeExternalSelectionWithSequencerSelection is called within
	// the scope of GIsTransacting being true, which prevents that function from creating new
	// transactions for the selection synchronization. This is important because otherwise
	// the undo/redo stack gets wiped by actor selections when undoing if the selection is
	// not identical
	RevalidateSelection();
}

void FSequencerSelection::RevalidateSelection()
{
	FSelectionEventSuppressor EventSuppressor = SuppressEvents();

	KeySelection.RemoveByPredicate(
		[this](FKeyHandle Key)
		{
			TViewModelPtr<FChannelModel> Channel = this->KeySelection.GetModelForKey(Key);
			return !Channel || Channel->GetSection() == nullptr;
		}
	);

	TrackArea.RemoveByPredicate(
		[this](const FWeakViewModelPtr& Key)
		{
			return Key.Pin() == nullptr;
		}
	);

	Outliner.RemoveByPredicate(
		[this](const TWeakViewModelPtr<IOutlinerExtension>& Key)
		{
			return Key.Pin() == nullptr;
		}
	);
}

} // namespace UE::Sequencer