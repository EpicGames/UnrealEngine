// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MVVM/ViewModels/LayerBarModel.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/FolderModel.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/OutlinerItemModel.h"
#include "MVVM/ViewModels/SpawnableModel.h"
#include "MVVM/ViewModels/PossessableModel.h"

#include "MVVM/Extensions/ICurveEditorTreeItemExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Extensions/ITrackRowExtension.h"
#include "MVVM/Extensions/ISectionOwnerExtension.h"
#include "MVVM/Extensions/ITopLevelChannelHolderExtension.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/ISnappableExtension.h"
#include "MVVM/Extensions/IBindingLifetimeExtension.h"
#include "MVVM/Extensions/IConditionableExtension.h"
#include "MVVM/Extensions/ITimeDomainExtension.h"
#include "MVVM/Extensions/IClockExtension.h"
#include "MVVM/Extensions/ISectionDragOwnerExtension.h"

#include "MVVM/ViewModels/SequencerOutlinerViewModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/SequencerTrackAreaViewModel.h"

#include "MVVM/CurveEditorExtension.h"
#include "MVVM/CurveEditorIntegrationExtension.h"
#include "MVVM/FolderModelStorageExtension.h"
#include "MVVM/ObjectBindingModelStorageExtension.h"
#include "MVVM/PinEditorExtension.h"
#include "MVVM/TrackModelStorageExtension.h"
#include "MVVM/TrackRowModelStorageExtension.h"
#include "MVVM/DecorationModelStorageExtension.h"
#include "MVVM/SectionModelStorageExtension.h"


namespace UE
{
namespace Sequencer
{

// Model types
UE_SEQUENCER_DEFINE_CASTABLE(FBindingLifetimeOverlayModel);
UE_SEQUENCER_DEFINE_CASTABLE(FCategoryGroupModel);
UE_SEQUENCER_DEFINE_CASTABLE(FCategoryModel);
UE_SEQUENCER_DEFINE_CASTABLE(FChannelGroupModel);
UE_SEQUENCER_DEFINE_CASTABLE(FChannelGroupOutlinerModel);
UE_SEQUENCER_DEFINE_CASTABLE(FChannelModel);
UE_SEQUENCER_DEFINE_CASTABLE(FFolderModel);
UE_SEQUENCER_DEFINE_CASTABLE(FLayerBarModel);
UE_SEQUENCER_DEFINE_CASTABLE(FOutlinerItemModel);
UE_SEQUENCER_DEFINE_CASTABLE(FEvaluableOutlinerItemModel);
UE_SEQUENCER_DEFINE_CASTABLE(FObjectBindingModel);
UE_SEQUENCER_DEFINE_CASTABLE(FPossessableModel);
UE_SEQUENCER_DEFINE_CASTABLE(FSectionModel);
UE_SEQUENCER_DEFINE_CASTABLE(FSequenceModel);
UE_SEQUENCER_DEFINE_CASTABLE(FSpawnableModel);
UE_SEQUENCER_DEFINE_CASTABLE(FTrackModel);
UE_SEQUENCER_DEFINE_CASTABLE(FTrackRowModel);

// Interface types
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ITrackExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ITrackRowExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ISectionOwnerExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ITopLevelChannelHolderExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IConditionableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IClockExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ISectionDragOwnerExtension);

// View model types
UE_SEQUENCER_DEFINE_CASTABLE(FSequencerEditorViewModel);
UE_SEQUENCER_DEFINE_CASTABLE(FSequencerOutlinerViewModel);
UE_SEQUENCER_DEFINE_CASTABLE(FSequencerTrackAreaViewModel);

// Extension types
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FCurveEditorExtension);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FCurveEditorIntegrationExtension);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FFolderModelStorageExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FObjectBindingModelStorageExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FPinEditorExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FDecorationModelStorageExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FSectionModelStorageExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FTrackModelStorageExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FTrackRowModelStorageExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ICurveEditorTreeItemExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IObjectBindingExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ISnappableExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(ITimeDomainExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IBindingLifetimeExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FOutlinerCacheExtension);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FConditionStateCacheExtension);

} // namespace Sequencer
} // namespace UE

