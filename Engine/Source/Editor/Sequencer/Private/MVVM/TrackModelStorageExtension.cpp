// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/TrackModelStorageExtension.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MovieSceneFolder.h"

namespace UE
{
namespace Sequencer
{

FTrackModelStorageExtension::FTrackModelStorageExtension(const TArray<FOnCreateTrackModel>& InTrackModelCreators)
	: TrackModelCreators(InTrackModelCreators)
{
}

void FTrackModelStorageExtension::OnCreated(TSharedRef<FViewModel> InWeakOwner)
{
	OwnerModel = InWeakOwner->CastThis<FSequenceModel>();
	ensure(OwnerModel);
}

void FTrackModelStorageExtension::OnReinitialize()
{
	Unlink();

	UMovieSceneSequence* MovieSceneSequence = OwnerModel->GetSequence();
	UMovieScene* MovieScene = MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr;
	if (MovieScene)
	{
		MovieScene->EventHandlers.Link(this);

		TSet<UMovieSceneTrack*> TracksInFolders;

		TArray<UMovieSceneFolder*> AllFolders;
		GetMovieSceneFoldersRecursive(MovieScene->GetRootFolders(), AllFolders);
		for (UMovieSceneFolder* Folder : AllFolders)
		{
			for (UMovieSceneTrack* Track : Folder->GetChildTracks())
			{
				TracksInFolders.Add(Track);
			}
		}

		for (UMovieSceneTrack* Track : MovieScene->GetTracks())
		{
			if (!TracksInFolders.Contains(Track))
			{
				OnTrackAdded(Track);
			}
		}
		if (UMovieSceneTrack* Track = MovieScene->GetCameraCutTrack())
		{
			if (!TracksInFolders.Contains(Track))
			{
				OnTrackAdded(Track);
			}
		}
	}

	for (auto It = TrackToModel.CreateIterator(); It; ++It)
	{
		if (It.Key().ResolveObjectPtr() == nullptr || It.Value().Pin().Get() == nullptr)
		{
			It.RemoveCurrent();
		}
	}
	TrackToModel.Compact();
}

TSharedPtr<FTrackModel> FTrackModelStorageExtension::CreateModelForTrack(UMovieSceneTrack* InTrack, TSharedPtr<FViewModel> DesiredParent)
{
	if (!DesiredParent)
	{
		DesiredParent = OwnerModel->AsShared();
	}

	TOptional<FViewModelChildren> Children = DesiredParent->GetChildList(EViewModelListType::Outliner);
	ensureMsgf(Children.IsSet(), TEXT("Attempting to create a folder within something that is not able to contain outliner items"));

	FObjectKey TrackAsKey(InTrack);

	TSharedPtr<FTrackModel> TrackModel = TrackToModel.FindRef(TrackAsKey).Pin().ImplicitCast();
	if (TrackModel)
	{
		// If we already have a track model, just ensure it's added to the correct parent
		if (Children.IsSet())
		{
			Children->AddChild(TrackModel);
		}

		return TrackModel;
	}

	// Create a new track model
	for (const FOnCreateTrackModel& Delegate : TrackModelCreators)
	{
		TrackModel = Delegate.Execute(InTrack);
		if (TrackModel.IsValid())
		{
			break;
		}
	}
	
	if (!TrackModel.IsValid())
	{
		TrackModel = MakeShared<FTrackModel>(InTrack);
	}

	// IMPORTANT: We always add the model to the map before calling initialize
	// So that any code that runs inside Initialize is still able to find this
	TrackToModel.Add(TrackAsKey, TrackModel);

	if (Children.IsSet())
	{
		Children->AddChild(TrackModel);
	}

	return TrackModel;
}

TSharedPtr<FTrackModel> FTrackModelStorageExtension::FindModelForTrack(UMovieSceneTrack* InTrack) const
{
	FObjectKey TrackAsKey(InTrack);
	TWeakViewModelPtr<ITrackExtension> WeakModel = TrackToModel.FindRef(TrackAsKey);
	TViewModelPtr<ITrackExtension> Model = WeakModel.Pin();
	return Model.ImplicitCast();
}

TViewModelPtr<ITrackExtension> FTrackModelStorageExtension::FindTrackExtensionForTrack(UMovieSceneTrack* InTrack) const
{
	FObjectKey TrackAsKey(InTrack);
	return TrackToModel.FindRef(TrackAsKey).Pin();
}

void FTrackModelStorageExtension::RegisterTrackExtensionForTrack(UMovieSceneTrack* InTrack, TViewModelPtr<ITrackExtension> InModel)
{
	FObjectKey TrackAsKey(InTrack);
	TrackToModel.Add(TrackAsKey, InModel);
}

void FTrackModelStorageExtension::OnTrackAdded(UMovieSceneTrack* InTrack)
{
	CreateModelForTrack(InTrack);
}

void FTrackModelStorageExtension::OnTrackRemoved(UMovieSceneTrack* InTrack)
{
	FObjectKey TrackAsKey(InTrack);

	if (TViewModelPtr<ITrackExtension> Model = TrackToModel.FindRef(TrackAsKey).Pin())
	{
		if (TSharedPtr<FViewModel> ViewModel = Model.AsModel())
		{
			ViewModel->RemoveFromParent();
		}
	}

	TrackToModel.Remove(TrackAsKey);
}

} // namespace Sequencer
} // namespace UE

