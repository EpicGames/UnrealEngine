// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/TrackRowModelStorageExtension.h"

#include "HAL/PlatformCrt.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "MovieSceneTrack.h"
#include "Templates/TypeHash.h"

namespace UE
{
namespace Sequencer
{

FTrackRowModelStorageExtension::FTrackRowModelStorageExtension()
{
}

void FTrackRowModelStorageExtension::OnCreated(TSharedRef<FViewModel> InWeakOwner)
{
	OwnerModel = InWeakOwner->CastThis<FSequenceModel>();
}

void FTrackRowModelStorageExtension::OnReinitialize()
{
	for (auto It = TrackToModel.CreateIterator(); It; ++It)
	{
		if (It.Key().Key.ResolveObjectPtr() == nullptr || It.Value().Pin().Get() == nullptr)
		{
			It.RemoveCurrent();
		}
	}
	TrackToModel.Compact();
}

TSharedPtr<FTrackRowModel> FTrackRowModelStorageExtension::CreateModelForTrackRow(UMovieSceneTrack* InTrack, int32 InRowIndex, TSharedPtr<FViewModel> DesiredParent)
{
	KeyType Key{ FObjectKey(InTrack), InRowIndex };
	if (TSharedPtr<FTrackRowModel> Existing = TrackToModel.FindRef(Key).Pin())
	{
		return Existing;
	}

	TSharedPtr<FTrackRowModel> NewModel = MakeShared<FTrackRowModel>(InTrack, InRowIndex);

	// IMPORTANT: We always add the model to the map before calling initialize
	// So that any code that runs inside Initialize is still able to find this
	TrackToModel.Add(Key, NewModel);

	if (!DesiredParent)
	{
		DesiredParent = OwnerModel->AsShared();
	}

	TOptional<FViewModelChildren> Children = DesiredParent->GetChildList(EViewModelListType::Outliner);
	if (ensureMsgf(Children.IsSet(), TEXT("Attempting to create a folder within something that is not able to contain outliner items")))
	{
		Children->AddChild(NewModel);
	}

	NewModel->Initialize();

	return NewModel;
}

TSharedPtr<FTrackRowModel> FTrackRowModelStorageExtension::FindModelForTrackRow(UMovieSceneTrack* InTrack, int32 InRowIndex) const
{
	KeyType Key{ FObjectKey(InTrack), InRowIndex };
	return TrackToModel.FindRef(Key).Pin();
}

} // namespace Sequencer
} // namespace UE

