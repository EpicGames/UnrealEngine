// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/BindingLifetimeOverlayModel.h"

#include "Containers/ArrayView.h"
#include "HAL/Platform.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/Views/STrackLane.h"
#include "Math/RangeBound.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "MovieSceneTimeHelpers.h"
#include "Templates/TypeHash.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SSequencerBindingLifetimeOverlay.h"
#include "Sequencer.h"
#include "AnimatedRange.h"

namespace UE
{
	namespace Sequencer
	{

		FBindingLifetimeOverlayModel::FBindingLifetimeOverlayModel(TWeakPtr<FViewModel> LayerRoot
			, TWeakPtr<FSequencerEditorViewModel> InEditorViewModel
			, TViewModelPtr<IBindingLifetimeExtension> InBindingLifetimeTrack)
			: WeakEditorViewModel(InEditorViewModel)
			, BindingLifetimeTrack(InBindingLifetimeTrack)
		{
		}

		FBindingLifetimeOverlayModel::~FBindingLifetimeOverlayModel()
		{
		}

		TSharedPtr<ITrackLaneWidget> FBindingLifetimeOverlayModel::CreateTrackLaneView(const FCreateTrackLaneViewParams& InParams)
		{
			return SNew(SSequencerBindingLifetimeOverlay, InParams.OwningTrackLane->GetTrackAreaView(), InParams.Editor->CastThisSharedChecked<FSequencerEditorViewModel>(), SharedThis(this))
					.Visibility(EVisibility::HitTestInvisible);
		}

		FTrackLaneVirtualAlignment FBindingLifetimeOverlayModel::ArrangeVirtualTrackLaneView() const
		{
			TSharedPtr<FSequencerEditorViewModel> EditorViewModel = WeakEditorViewModel.Pin();
			TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();

			FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();

			TRange<FFrameNumber> ViewRange(TickResolution.AsFrameNumber(Sequencer->GetViewRange().GetLowerBoundValue()), TickResolution.AsFrameNumber(Sequencer->GetViewRange().GetUpperBoundValue()));


			return FTrackLaneVirtualAlignment::Proportional(ViewRange, 1.f);
		}

		const TArray<FFrameNumberRange>& FBindingLifetimeOverlayModel::GetInverseLifetimeRange() const
		{
			const static TArray<FFrameNumberRange> EmptyLifetimeRange;
			if (BindingLifetimeTrack)
			{
				return BindingLifetimeTrack->GetInverseLifetimeRange();
			}
			return EmptyLifetimeRange;
		}
	} // namespace Sequencer
} // namespace UE

