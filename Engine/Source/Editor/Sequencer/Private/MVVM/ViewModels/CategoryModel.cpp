// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/CategoryModel.h"

#include "HAL/PlatformCrt.h"
#include "ISequencerSection.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/SequencerModelUtils.h"
#include "MVVM/Views/SChannelView.h"
#include "MVVM/Views/SOutlinerItemViewBase.h"
#include "MVVM/Views/SSequencerKeyNavigationButtons.h"
#include "MVVM/Views/SOutlinerTrackColorPicker.h"
#include "MVVM/Views/SOutlinerTrackPreserveRatio.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "MovieSceneTrack.h"
#include "SequencerCoreFwd.h"
#include "SequencerSectionPainter.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "MVVM/Views/STrackLane.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackModel.h"

class SWidget;

#define LOCTEXT_NAMESPACE "SequencerCategoryModel"

namespace UE::Sequencer
{

FCategoryModel::FCategoryModel(FName InCategoryName)
	: Children(EViewModelListType::Generic)
	, CategoryName(InCategoryName)
{
	RegisterChildList(&Children);
}

bool FCategoryModel::IsAnimated() const
{
	for (TSharedPtr<FChannelModel> ChannelModel : GetDescendantsOfType<FChannelModel>())
	{
		if (ChannelModel.IsValid() && ChannelModel->IsAnimated())
		{
			return true;
		}
	}
	return false;
}

FOutlinerSizing FCategoryModel::GetDesiredSizing() const
{
	return FOutlinerSizing(15.f + 2.f*2.f);
}

TSharedPtr<ITrackLaneWidget> FCategoryModel::CreateTrackLaneView(const FCreateTrackLaneViewParams& InParams)
{
	return SNew(SChannelView, SharedThis(this), InParams.OwningTrackLane->GetTrackAreaView())
		.KeyBarColor(this, &FCategoryModel::GetKeyBarColor);
}

FTrackLaneVirtualAlignment FCategoryModel::ArrangeVirtualTrackLaneView() const
{
	TSharedPtr<ITrackLaneExtension> Parent = FindAncestorOfType<ITrackLaneExtension>();
	return Parent ? Parent->ArrangeVirtualTrackLaneView() : FTrackLaneVirtualAlignment();
}

FLinearColor FCategoryModel::GetKeyBarColor() const
{
	TViewModelPtr<ITrackExtension> Track = FindAncestorOfType<ITrackExtension>();
	UMovieSceneTrack* TrackObject = Track ? Track->GetTrack() : nullptr;
	if (TrackObject)
	{
		FLinearColor Tint = FSequencerSectionPainter::BlendColor(TrackObject->GetColorTint());
		return Tint.CopyWithNewOpacity(1.f);
	}
	return FColor(160, 160, 160);
}

FCategoryGroupModel::FCategoryGroupModel(FName InCategoryName, const FText& InDisplayText, FGetMovieSceneTooltipText InGetGroupTooltipTextDelegate)
	: CategoryName(InCategoryName)
	, DisplayText(InDisplayText)
	, GetGroupTooltipTextDelegate(InGetGroupTooltipTextDelegate)
{
	SetIdentifier(InCategoryName);
}

FCategoryGroupModel::~FCategoryGroupModel()
{
}

bool FCategoryGroupModel::IsAnimated() const
{
	for (const TWeakViewModelPtr<FCategoryModel>& WeakCategory : Categories)
	{
		TSharedPtr<FCategoryModel> Category = WeakCategory.Pin();
		if (Category && Category->IsAnimated())
		{
			return true;
		}
	}
	return false;
}

void FCategoryGroupModel::AddCategory(TWeakViewModelPtr<FCategoryModel> InCategory)
{
	if (!Categories.Contains(InCategory))
	{
		Categories.Add(InCategory);
	}
}

TArrayView<const TWeakViewModelPtr<FCategoryModel>> FCategoryGroupModel::GetCategories() const
{
	return Categories;
}

TSharedPtr<FCategoryModel> FCategoryGroupModel::CreateTrackAreaCategory()
{
	TSharedPtr<FCategoryModel> Category = MakeShared<FCategoryModel>(CategoryName);
	AddCategory(Category);
	Category->SetLinkedOutlinerItem(SharedThis(this));
	return Category;
}

FOutlinerSizing FCategoryGroupModel::RecomputeSizing()
{
	FOutlinerSizing MaxSizing;

	for (TWeakViewModelPtr<FCategoryModel> WeakCategory : Categories)
	{
		if (TSharedPtr<FCategoryModel> Category = WeakCategory.Pin())
		{
			FOutlinerSizing Desired = Category->GetDesiredSizing();
			MaxSizing.Accumulate(Desired);
		}
	}

	if (ComputedSizing != MaxSizing)
	{
		ComputedSizing = MaxSizing;

		for (const TWeakViewModelPtr<FCategoryModel>& WeakCategory : Categories)
		{
			if (TSharedPtr<FCategoryModel> Category = WeakCategory.Pin())
			{
				Category->SetComputedSizing(MaxSizing);
			}
		}
	}

	return MaxSizing;
}

FOutlinerSizing FCategoryGroupModel::GetOutlinerSizing() const
{
	if (EnumHasAnyFlags(ComputedSizing.Flags, EOutlinerSizingFlags::DynamicSizing))
	{
		const_cast<FCategoryGroupModel*>(this)->RecomputeSizing();
	}

	FOutlinerSizing FinalSizing = ComputedSizing;
	if (!EnumHasAnyFlags(ComputedSizing.Flags, EOutlinerSizingFlags::CustomHeight))
	{
		if (TSharedPtr<FSequencerEditorViewModel> Editor = GetEditor())
		{
			FViewDensityInfo Density = Editor->GetViewDensity();
			FinalSizing.Height = Density.UniformHeight.Get(FinalSizing.Height);
		}
	}
	return FinalSizing;
}

FText FCategoryGroupModel::GetLabel() const
{
	return GetDisplayText();
}

FText FCategoryGroupModel::GetLabelToolTipText() const
{
	if (GetGroupTooltipTextDelegate.IsBound())
	{
		if (TViewModelPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>())
		{
			if (TSharedPtr<FSequencerEditorViewModel> SequencerModel = SequenceModel->GetEditor())
			{
				FMovieSceneSequenceID SequenceID = SequenceModel->GetSequenceID();
				IMovieScenePlayer* Player = SequencerModel->GetSequencer().Get();
				if (Player)
				{
					if (TViewModelPtr<FObjectBindingModel> ObjectBindingModel = FindAncestorOfType<FObjectBindingModel>())
					{
						FGuid ObjectBindingID = ObjectBindingModel->GetObjectGuid();

						return GetGroupTooltipTextDelegate.Execute(Player, ObjectBindingID, SequenceID);
					}
				}
			}
		}
	}
	return FText();
}

FSlateFontInfo FCategoryGroupModel::GetLabelFont() const
{
	return IsAnimated()
		? FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.ItalicFont")
		: FOutlinerItemModel::GetLabelFont();
}

TSharedPtr<SWidget> FCategoryGroupModel::CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName)
{
	TViewModelPtr<FSequencerEditorViewModel> Editor = InParams.Editor->CastThisShared<FSequencerEditorViewModel>();
	if (!Editor)
	{
		return SNullWidget::NullWidget;
	}

	if (InColumnName == FCommonOutlinerNames::Label)
	{
		return SNew(SOutlinerItemViewBase, SharedThis(this), InParams.Editor, InParams.TreeViewRow);
	}

	if (InColumnName == FCommonOutlinerNames::Edit)
	{
		return SNew(SOutlinerTrackPreserveRatio, SharedThis(this), InParams.Editor);
	}

	if (InColumnName == FCommonOutlinerNames::KeyFrame)
	{
		EKeyNavigationButtons Buttons = EKeyNavigationButtons::AddKey;

		return SNew(SSequencerKeyNavigationButtons, SharedThis(this), Editor->GetSequencer())
			.Buttons(Buttons);
	}

	if (InColumnName == FCommonOutlinerNames::Nav)
	{
		EKeyNavigationButtons Buttons = InParams.TreeViewRow->IsColumnVisible(FCommonOutlinerNames::KeyFrame)
			? EKeyNavigationButtons::NavOnly
			: EKeyNavigationButtons::All;

		return SNew(SSequencerKeyNavigationButtons, SharedThis(this), Editor->GetSequencer())
			.Buttons(Buttons);
	}

	if (InColumnName == FCommonOutlinerNames::Nav)
	{
		return SNew(SSequencerKeyNavigationButtons, SharedThis(this), Editor->GetSequencer());
	}

	if (InColumnName == FCommonOutlinerNames::ColorPicker)
	{
		return SNew(SOutlinerTrackColorPicker, SharedThis(this), InParams.Editor);
	}

	return nullptr;
}

FTrackAreaParameters FCategoryGroupModel::GetTrackAreaParameters() const
{
	FTrackAreaParameters Parameters;
	Parameters.LaneType = ETrackAreaLaneType::Inline;
	return Parameters;
}

FViewModelVariantIterator FCategoryGroupModel::GetTrackAreaModelList() const
{
	return &Categories;
}

bool FCategoryGroupModel::CanDelete(FText* OutErrorMessage) const
{
	return true;
}

void FCategoryGroupModel::Delete()
{
	TArray<FName> PathFromTrack;
	TViewModelPtr<ITrackExtension> Track = GetParentTrackNodeAndNamePath(CastThisShared<IOutlinerExtension>(), PathFromTrack);
	check(Track);

	Track->GetTrack()->Modify();

	for (const FViewModelPtr& Category : GetTrackAreaModelList())
	{
		if (TViewModelPtr<FSectionModel> Section = Category->FindAncestorOfType<FSectionModel>())
		{
			Section->GetSectionInterface()->RequestDeleteCategory(PathFromTrack);
		}
	}
}

void FCategoryGroupModel::OnRecycle()
{
	Categories.Empty();
}

void FCategoryGroupModel::BuildSidebarMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();
	if (!Sequencer.IsValid())
	{
		return;
	}

	if (const TViewModelPtr<ISectionOwnerExtension> SectionOwner = FindAncestorOfType<ISectionOwnerExtension>())
	{
		TArray<TWeakObjectPtr<>> WeakSectionObjects;
		Algo::Transform(SectionOwner->GetSections(), WeakSectionObjects, [](UMovieSceneSection* const InSection)
			{
				return InSection;
			});
		SequencerHelpers::BuildEditSectionMenu(Sequencer, WeakSectionObjects, MenuBuilder, false);
	}

	FOutlinerItemModel::BuildSidebarMenu(MenuBuilder);
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

