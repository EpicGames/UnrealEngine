// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/ChannelModel.h"

#include "Channels/IMovieSceneChannelOverrideProvider.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneSectionChannelOverrideRegistry.h"
#include "Channels/IMovieSceneChannelOwner.h"
#include "CurveModel.h"
#include "DetailsViewArgs.h"
#include "HAL/PlatformCrt.h"
#include "IKeyArea.h"
#include "ISequencerChannelInterface.h"
#include "ISequencerSection.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/SequencerModelUtils.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/Views/SOutlinerView.h"
#include "MVVM/Views/SOutlinerItemViewBase.h"
#include "MVVM/Views/SSequencerKeyNavigationButtons.h"
#include "MVVM/Views/SOutlinerTrackColorPicker.h"
#include "MVVM/Views/STrackLane.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneTrack.h"
#include "MVVM/Views/SChannelView.h"
#include "PropertyEditorModule.h"
#include "ISequencerDecorationEditor.h"
#include "SKeyAreaEditorSwitcher.h"
#include "SequencerCoreFwd.h"
#include "SequencerSectionPainter.h"
#include "SequencerSettings.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Algo/AnyOf.h"

class ISequencer;
class SWidget;

#define LOCTEXT_NAMESPACE "SequencerChannelModel"

namespace UE::Sequencer
{


struct FDynamicChannelMuteExtension : IDynamicExtension, IMutableExtension
{
	TWeakObjectPtr<> WeakOwner;
	FName ChannelName;

	using Implements = TImplements<IDynamicExtension, IMutableExtension>;

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID(FDynamicChannelMuteExtension)

	FDynamicChannelMuteExtension(TWeakObjectPtr<> InWeakOwner, FName InChannelName)
		: WeakOwner(InWeakOwner)
		, ChannelName(InChannelName)
	{}

	bool IsMuted() const override
	{
		IMovieSceneChannelOwner* ChannelOwner = Cast<IMovieSceneChannelOwner>(WeakOwner.Get());
		return ChannelOwner && ChannelOwner->IsMuted(ChannelName);
	}
	void SetIsMuted(bool bIsMuted) override
	{
		IMovieSceneChannelOwner* ChannelOwner = Cast<IMovieSceneChannelOwner>(WeakOwner.Get());
		if (ChannelOwner)
		{
			ChannelOwner->SetIsMuted(ChannelName, bIsMuted);
		}
	}
};
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FDynamicChannelMuteExtension);

struct FDynamicChannelGroupMuteExtension : IDynamicExtension, IMutableExtension
{
	TWeakViewModelPtr<FChannelGroupModel> WeakChannelGroup;

	using Implements = TImplements<IDynamicExtension, IMutableExtension>;

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID(FDynamicChannelGroupMuteExtension)

	FDynamicChannelGroupMuteExtension(TWeakViewModelPtr<FChannelGroupModel> InWeakChannelGroup)
		: WeakChannelGroup(InWeakChannelGroup)
	{}

	bool IsInheritable() const override
	{
		return false;
	}

	bool IsMuted() const override
	{
		TViewModelPtr<FChannelGroupModel> ChannelGroup = WeakChannelGroup.Pin();
		if (!ChannelGroup)
		{
			return false;
		}

		for (const TWeakViewModelPtr<FChannelModel>& WeakChannel : ChannelGroup->GetChannels())
		{
			TViewModelPtr<IMutableExtension> Mutable = WeakChannel.ImplicitPin();
			if (Mutable && Mutable->IsMuted())
			{
				return true;
			}
		}
		return false;
	}

	void SetIsMuted(bool bIsMuted) override
	{
		TViewModelPtr<FChannelGroupModel> ChannelGroup = WeakChannelGroup.Pin();
		if (ChannelGroup)
		{
			for (const TWeakViewModelPtr<FChannelModel>& WeakChannel : ChannelGroup->GetChannels())
			{
				TViewModelPtr<IMutableExtension> Mutable = WeakChannel.ImplicitPin();
				if (Mutable)
				{
					Mutable->SetIsMuted(bIsMuted);
				}
			}
		}
	}
};
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FDynamicChannelGroupMuteExtension);


FChannelModel::FChannelModel(FName InChannelName, TWeakPtr<ISequencerSection> InSection, FMovieSceneChannelHandle InChannel)
	: KeyArea(MakeShared<IKeyArea>(InSection, InChannel))
	, ChannelName(InChannelName)
{
}

FChannelModel::~FChannelModel()
{
}

bool FChannelModel::IsAnimated() const
{
	if (KeyArea)
	{
		FMovieSceneChannel* Channel = KeyArea->ResolveChannel();
		return Channel && Channel->GetNumKeys() > 0;
	}
	return false;
}

void FChannelModel::Initialize(TWeakPtr<ISequencerSection> InSection, FMovieSceneChannelHandle InChannel)
{
	if (!KeyArea)
	{
		KeyArea = MakeShared<IKeyArea>(InSection, InChannel);
	}
	else
	{
		KeyArea->Reinitialize(InSection, InChannel);
	}

	const FMovieSceneChannelMetaData* ChannelMetaData = InChannel.GetMetaData();
	IMovieSceneChannelOwner*          ChannelOwner    = Cast<IMovieSceneChannelOwner>(KeyArea->GetOwningObject());
	if (ChannelMetaData && ChannelOwner && ChannelOwner->GetCapabilities(KeyArea->GetName()).bSupportsMute)
	{
		AddDynamicExtension<FDynamicChannelMuteExtension>(KeyArea->GetOwningObject(), ChannelMetaData->Name);
	}
	else
	{
		RemoveDynamicExtension<FDynamicChannelMuteExtension>();
	}
}

FMovieSceneChannel* FChannelModel::GetChannel() const
{
	check(KeyArea);
	return KeyArea->ResolveChannel();
}

UMovieSceneSection* FChannelModel::GetSection() const
{
	return KeyArea->GetOwningSection();
}

UObject* FChannelModel::GetOwningObject() const
{
	return KeyArea->GetOwningObject();
}

FOutlinerSizing FChannelModel::GetDesiredSizing() const
{
	FOutlinerSizing Sizing(15.f);

	if (KeyArea->ShouldShowCurve())
	{
		TViewModelPtr<FSequenceModel> Sequence = FindAncestorOfType<FSequenceModel>();
		if (Sequence)
		{
			Sizing.Height = Sequence->GetSequencer()->GetSequencerSettings()->GetKeyAreaHeightWithCurves();
			Sizing.Flags  |= EOutlinerSizingFlags::CustomHeight;
		}
	}

	return Sizing;
}

TSharedPtr<ITrackLaneWidget> FChannelModel::CreateTrackLaneView(const FCreateTrackLaneViewParams& InParams)
{
	ISequencerChannelInterface* EditorInterface = KeyArea->FindChannelEditorInterface();
	if (EditorInterface)
	{
		TSharedPtr<STrackAreaLaneView> CustomWidget = EditorInterface->CreateChannelView_Raw(KeyArea->GetChannel(), SharedThis(this), InParams);
		if (CustomWidget)
		{
			return CustomWidget;
		}
	}

	return SNew(SChannelView, SharedThis(this), InParams.OwningTrackLane->GetTrackAreaView())
		.KeyBarColor(this, &FChannelModel::GetKeyBarColor);
}

FTrackLaneVirtualAlignment FChannelModel::ArrangeVirtualTrackLaneView() const
{
	TSharedPtr<FSectionModel> Section = FindAncestorOfType<FSectionModel>();
	if (Section)
	{
		return FTrackLaneVirtualAlignment::Proportional(Section->GetRange(), 1.f);
	}
	return FTrackLaneVirtualAlignment::Proportional(TRange<FFrameNumber>(), 1.f);
}

void FChannelModel::OnRecycle()
{
	KeyArea = nullptr;
}

bool FChannelModel::UpdateCachedKeys(TSharedPtr<FCachedKeys>& OutCachedKeys) const
{
	struct FSequencerCachedKeys : FCachedKeys
	{
		FSequencerCachedKeys(const FChannelModel* InChannel)
		{
			Update(InChannel);
		}

		bool Update(const FChannelModel* InChannel)
		{
			UMovieSceneSection* Section = InChannel->GetSection();

			// Check both the section signature and the owning object signature.
			// For decoration channels, the owning object (decoration) changes when
			// keys are modified, but the host section's signature may not update
			// since it's not in the decoration's outer chain.
			FGuid OwnerSignature;
			if (const TSharedPtr<IKeyArea>& ChannelKeyArea = InChannel->GetKeyArea())
			{
				if (const FMovieSceneChannelMetaData* MetaData = ChannelKeyArea->GetChannel().GetMetaData())
				{
					if (UMovieSceneSignedObject* SignedOwner = Cast<UMovieSceneSignedObject>(MetaData->WeakOwningObject.Get()))
					{
						if (SignedOwner != Section)
						{
							OwnerSignature = SignedOwner->GetSignature();
						}
					}
				}
			}

			const bool bSectionChanged = !Section || !CachedSignature.IsValid() || Section->GetSignature() != CachedSignature;
			const bool bOwnerChanged = OwnerSignature.IsValid() && OwnerSignature != CachedOwnerSignature;

			if (bSectionChanged || bOwnerChanged)
			{
				CachedSignature = Section && InChannel->GetKeyArea() && InChannel->GetKeyArea()->ResolveChannel() ? Section->GetSignature() : FGuid();
				CachedOwnerSignature = OwnerSignature;

				KeyTimes.Reset();
				KeyHandles.Reset();

				TArray<FFrameNumber> KeyFrames;
				InChannel->GetKeyArea()->GetKeyInfo(&KeyHandles, &KeyFrames);

				KeyTimes.SetNumUninitialized(KeyFrames.Num());
				for (int32 Index = 0; Index < KeyFrames.Num(); ++Index)
				{
					KeyTimes[Index] = KeyFrames[Index];
				}
				return true;
			}

			return false;
		}

		FGuid CachedSignature;
		FGuid CachedOwnerSignature;
	};

	if (OutCachedKeys)
	{
		return StaticCastSharedPtr<FSequencerCachedKeys>(OutCachedKeys)->Update(this);
	}
	else
	{
		OutCachedKeys = MakeShared<FSequencerCachedKeys>(this);
		return true;
	}
}

bool FChannelModel::GetFixedExtents(double& OutFixedMin, double& OutFixedMax) const
{
	TViewModelPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	if (!SequenceModel)
	{
		return false;
	}

	FString KeyAreaName = KeyArea->GetName().ToString();
	if (SequenceModel->GetSequencer()->GetSequencerSettings()->HasKeyAreaCurveExtents(KeyAreaName))
	{
		SequenceModel->GetSequencer()->GetSequencerSettings()->GetKeyAreaCurveExtents(KeyAreaName, OutFixedMin, OutFixedMax);
		return true;
	}

	return false;
}

int32 FChannelModel::CustomPaint(const FSequencerChannelPaintArgs& PaintArgs, int32 LayerId) const
{
	return KeyArea->DrawExtra(PaintArgs, LayerId);
}

void FChannelModel::DrawKeys(TArrayView<const FKeyHandle> InKeyHandles, TArrayView<FKeyDrawParams> OutKeyDrawParams)
{
	KeyArea->DrawKeys(InKeyHandles, OutKeyDrawParams);
}

TUniquePtr<FCurveModel> FChannelModel::CreateCurveModel()
{
	if (TViewModelPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>())
	{
		return KeyArea->CreateCurveEditorModel(SequenceModel->GetSequencer().ToSharedRef());
	}

	return nullptr;
}

FLinearColor FChannelModel::GetKeyBarColor() const
{
	if (TViewModelPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>())
	{
		if (SequenceModel->GetSequencer()->GetSequencerSettings()->GetShowChannelColors())
		{
			TOptional<FLinearColor> ChannelColor = KeyArea->GetColor();
			if (ChannelColor)
			{
				return ChannelColor.GetValue();
			}
		}
	}

	TViewModelPtr<ITrackExtension> Track = FindAncestorOfType<ITrackExtension>();
	UMovieSceneTrack* TrackObject = Track ? Track->GetTrack() : nullptr;
	if (TrackObject)
	{
		FLinearColor Tint = FSequencerSectionPainter::BlendColor(TrackObject->GetColorTint()).LinearRGBToHSV();

		// If this is a top level chanel, draw using the fill color
		FViewModelPtr OutlinerItem = GetLinkedOutlinerItem();
		if (OutlinerItem && !OutlinerItem->IsA<FChannelGroupModel>())
		{
			Tint.G *= .5f;
			Tint.B = FMath::Max(.03f, Tint.B*.1f);
		}

		return Tint.HSVToLinearRGB().CopyWithNewOpacity(1.f);
	}
	return FColor(160, 160, 160);
}

FChannelGroupModel::FChannelGroupModel(FName InChannelName, const FText& InDisplayText)
	: ChannelsSerialNumber(0)
	, ChannelName(InChannelName)
	, DisplayText(InDisplayText)
{
}
FChannelGroupModel::FChannelGroupModel(FName InChannelName, const FText& InDisplayText, const FText& InTooltipText)
	: ChannelsSerialNumber(0)
	, ChannelName(InChannelName)
	, DisplayText(InDisplayText)
	, GetTooltipTextDelegate(FGetMovieSceneTooltipText::CreateLambda([InTooltipText](...) {return InTooltipText; }))
{
}

FChannelGroupModel::FChannelGroupModel(FName InChannelName, const FText& InDisplayText, FGetMovieSceneTooltipText InGetTooltipTextDelegate)
	: ChannelsSerialNumber(0)
	, ChannelName(InChannelName)
	, DisplayText(InDisplayText)
	, GetTooltipTextDelegate(InGetTooltipTextDelegate)
{
}

FChannelGroupModel::~FChannelGroupModel()
{
}

bool FChannelGroupModel::IsAnimated() const
{
	for (TWeakViewModelPtr<FChannelModel> WeakChannel : Channels)
	{
		if (TSharedPtr<FChannelModel> Channel = WeakChannel.Pin())
		{
			if (Channel->IsAnimated())
			{
				return true;
			}
		}
	}
	return false;
}

void FChannelGroupModel::AddChannel(TWeakViewModelPtr<FChannelModel> InChannel)
{
	if (!Channels.Contains(InChannel))
	{
		Channels.Add(InChannel);
		++ChannelsSerialNumber;
	}
}

uint32 FChannelGroupModel::GetChannelsSerialNumber() const
{
	const_cast<FChannelGroupModel*>(this)->CleanupChannels();
	return ChannelsSerialNumber;
}

void FChannelGroupModel::OnChannelOverridesChanged()
{
	++ChannelsSerialNumber;
}

void FChannelGroupModel::CleanupChannels()
{
	const int32 NumRemoved = Channels.RemoveAll([](TWeakViewModelPtr<FChannelModel> Item) { return !Item.Pin().IsValid(); });
	if (NumRemoved > 0)
	{
		++ChannelsSerialNumber;

		const bool bShouldBeMutable = Algo::AnyOf(Channels, [](TWeakViewModelPtr<FChannelModel> In)
			{
				TViewModelPtr<IMutableExtension> Mutable = In.ImplicitPin();
				return Mutable.IsValid();
			}
		);

		const bool bIsMutable = CastDynamic<IMutableExtension>() != nullptr;
		if (!bShouldBeMutable && bIsMutable)
		{
			RemoveDynamicExtension<FDynamicChannelGroupMuteExtension>();
		}
	}
}

FText FChannelGroupModel::GetTooltipText() const
{
	if (GetTooltipTextDelegate.IsBound())
	{
		if (TViewModelPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>())
		{
			if (TSharedPtr<FSequencerEditorViewModel> SequencerModel = SequenceModel->GetEditor())
			{
				FMovieSceneSequenceID SequenceID = SequenceModel->GetSequenceID();
				IMovieScenePlayer* Player = SequencerModel->GetSequencer().Get();
				if (Player)
				{
					if (TViewModelPtr<FObjectBindingModel> ObjectBindingModel = FindAncestorOfType<FObjectBindingModel>())
					{
						FGuid ObjectBindingID = ObjectBindingModel->GetObjectGuid();

						return GetTooltipTextDelegate.Execute(Player, ObjectBindingID, SequenceID);
					}
				}
			}
		}
	}
	return FText();
}

TArrayView<const TWeakViewModelPtr<FChannelModel>> FChannelGroupModel::GetChannels() const
{
	return Channels;
}

TSharedPtr<IKeyArea> FChannelGroupModel::GetKeyArea(TSharedPtr<FSectionModel> InOwnerSection) const
{
	return GetKeyArea(InOwnerSection->GetSection());
}

TSharedPtr<IKeyArea> FChannelGroupModel::GetKeyArea(const UMovieSceneSection* InOwnerSection) const
{
	TSharedPtr<FChannelModel> Channel = GetChannel(InOwnerSection);
	return Channel ? Channel->GetKeyArea() : nullptr;
}

TSharedPtr<FChannelModel> FChannelGroupModel::GetChannel(int32 Index) const
{
	return Channels[Index].Pin();
}

TSharedPtr<FChannelModel> FChannelGroupModel::GetChannel(TSharedPtr<FSectionModel> InOwnerSection) const
{
	return GetChannel(InOwnerSection->GetSection());
}

TSharedPtr<FChannelModel> FChannelGroupModel::GetChannel(const UMovieSceneSection* InOwnerSection) const
{
	for (TWeakViewModelPtr<FChannelModel> WeakChannel : Channels)
	{
		if (TSharedPtr<FChannelModel> Channel = WeakChannel.Pin())
		{
			if (Channel->GetSection() == InOwnerSection)
			{
				return Channel;
			}
		}
	}
	return nullptr;
}

TArray<TSharedRef<IKeyArea>> FChannelGroupModel::GetAllKeyAreas() const
{
	TArray<TSharedRef<IKeyArea>> KeyAreas;
	KeyAreas.Reserve(Channels.Num());
	for (TWeakViewModelPtr<FChannelModel> WeakChannel : Channels)
	{
		if (TSharedPtr<FChannelModel> Channel = WeakChannel.Pin())
		{
			if (Channel->GetKeyArea())
			{
				KeyAreas.Add(Channel->GetKeyArea().ToSharedRef());
			}
		}
	}
	return KeyAreas;
}

FTrackAreaParameters FChannelGroupModel::GetTrackAreaParameters() const
{
	FTrackAreaParameters Parameters;
	Parameters.LaneType = ETrackAreaLaneType::Inline;
	return Parameters;
}

FViewModelVariantIterator FChannelGroupModel::GetTrackAreaModelList() const
{
	return &Channels;
}

void FChannelGroupModel::OnRecycle()
{
	Channels.Empty();
}

void FChannelGroupModel::CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels)
{
	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	if (!SequenceModel)
	{
		return;
	}

	TSharedPtr<ISequencer> Sequencer = SequenceModel->GetSequencer();
	if (!Sequencer)
	{
		return;
	}

	for (TWeakViewModelPtr<FChannelModel> WeakChannel : GetChannels())
	{
		if (TSharedPtr<FChannelModel> Channel = WeakChannel.Pin())
		{
			TUniquePtr<FCurveModel> NewCurve = Channel->GetKeyArea()->CreateCurveEditorModel(Sequencer.ToSharedRef());
			if (NewCurve.IsValid())
			{
				OutCurveModels.Add(MoveTemp(NewCurve));
			}
		}
	}
}

TOptional<FString> FChannelGroupModel::GetUniquePathName() const
{
	TStringBuilder<256> StringBuilder;
	IOutlinerExtension::GetPathName(*this, StringBuilder);
	FString PathName(StringBuilder.ToString());
	TOptional<FString> FullPathName = PathName;
	return FullPathName;
}

bool FChannelGroupModel::HasCurves() const
{
	for (const TSharedRef<IKeyArea>& KeyArea : GetAllKeyAreas())
	{
		const ISequencerChannelInterface* EditorInterface = KeyArea->FindChannelEditorInterface();
		if (EditorInterface && EditorInterface->SupportsCurveEditorModels_Raw(KeyArea->GetChannel()))
		{
			return true;
		}
	}
	return false;
}

void FChannelGroupModel::BuildChannelOverrideMenu(FMenuBuilder& MenuBuilder)
{
	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	FChannelGroupOverrideHelper::BuildChannelOverrideMenu(MenuBuilder, SequenceModel);
}

bool FChannelGroupOverrideHelper::GetChannelsFromOutlinerSelection(TSharedPtr<FSequenceModel> SequenceModel, TArray<TViewModelPtr<FChannelModel>>& OutChannelModels)
{
	// Sanity checks.
	if (!SequenceModel)
	{
		return false;
	}

	TSharedPtr<ISequencer> Sequencer = SequenceModel->GetSequencer();
	if (!Sequencer)
	{
		return false;
	}

	// The selected items could either be channel groups, or tracks/track-rows with top-level channel groups that
	// don't explicitly show up in the outliner (they are view-models that live under the track or row, but the
	// user interface shows them as the track or row itself).
	for (FViewModelPtr ViewModel : Sequencer->GetViewModel()->GetSelection()->Outliner)
	{
		if (TViewModelPtr<FChannelGroupModel> ChannelGroup = ViewModel.ImplicitCast())
		{
			const int32 NumChannels = ChannelGroup->GetChannels().Num();
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
				if (TSharedPtr<FChannelModel> Channel = ChannelGroup->GetChannel(ChannelIndex))
				{
					OutChannelModels.Add(Channel);
				}
			}
		}
		else
		{
			const EViewModelListType ListType = FTrackModel::GetTopLevelChannelGroupType();
			TOptional<FViewModelChildren> TopLevelChannels = ViewModel->FindChildList(ListType);
			if (TopLevelChannels.IsSet())
			{
				for (auto It(TopLevelChannels->IterateSubList<FChannelGroupModel>()); It; ++It)
				{
					for (TWeakViewModelPtr<FChannelModel> WeakChannel : It->GetChannels())
					{
						if (TViewModelPtr<FChannelModel> Channel = WeakChannel.Pin())
						{
							OutChannelModels.Add(Channel);
						}
					}
				}
			}
		}
	}

	return true;
}

void FChannelGroupOverrideHelper::BuildChannelOverrideMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FSequenceModel> SequenceModel)
{
	TArray<TViewModelPtr<FChannelModel>> Channels;
	if (!GetChannelsFromOutlinerSelection(SequenceModel, Channels))
	{
		return;
	}

	// Gather up information:
	// - Candidates for overriding each of the channel types found in each group.
	// - Already overriden channels.
	int32 NumOverridableChannels = 0;
	TMap<UClass*, TArray<UMovieSceneChannelOverrideContainer*>> ExistingOverrides;
	TArray<UMovieSceneChannelOverrideContainer::FOverrideCandidates> AllCandidateOverrides;
	for (TViewModelPtr Channel : Channels)
	{
		UMovieSceneSection* Section = Channel->GetSection();
		if (!Section)
		{
			continue;
		}

		IMovieSceneChannelOverrideProvider* OverrideProvider = Cast<IMovieSceneChannelOverrideProvider>(Section);
		if (!OverrideProvider)
		{
			continue;
		}

		++NumOverridableChannels;

		UMovieSceneSectionChannelOverrideRegistry* OverrideRegistry = OverrideProvider->GetChannelOverrideRegistry(false);
		UE::MovieScene::FChannelOverrideProviderTraitsHandle OverrideTraits = OverrideProvider->GetChannelOverrideProviderTraits();

		UMovieSceneChannelOverrideContainer* OverrideContainer = OverrideRegistry ? 
			OverrideRegistry->GetChannel(Channel->GetChannelName()) : nullptr;
		if (OverrideContainer)
		{
			ExistingOverrides.FindOrAdd(OverrideContainer->GetClass()).Add(OverrideContainer);
		}

		const FName DefaultChannelTypeName = OverrideTraits->GetDefaultChannelTypeName(Channel->GetChannelName());
		if (!DefaultChannelTypeName.IsNone())
		{
			UMovieSceneChannelOverrideContainer::FOverrideCandidates OverrideCandidates;
			UMovieSceneChannelOverrideContainer::GetOverrideCandidates(DefaultChannelTypeName, OverrideCandidates);

			AllCandidateOverrides.Add(MoveTemp(OverrideCandidates));
		}
	}

	// Add menu entries for overriding channels, with only types that are common among all sections.
	TMap<TSubclassOf<UMovieSceneChannelOverrideContainer>, int32> CandidateCounts;
	for (const UMovieSceneChannelOverrideContainer::FOverrideCandidates& CandidateOverrides : AllCandidateOverrides)
	{
		for (TSubclassOf<UMovieSceneChannelOverrideContainer> CandidateOverride : CandidateOverrides)
		{
			++CandidateCounts.FindOrAdd(CandidateOverride);
		}
	}

	UMovieSceneChannelOverrideContainer::FOverrideCandidates CommonCandidateOverrides;
	for (const TTuple<TSubclassOf<UMovieSceneChannelOverrideContainer>, int32>& CandidateCount : CandidateCounts)
	{
		if (CandidateCount.Value == NumOverridableChannels)
		{
			CommonCandidateOverrides.Add(CandidateCount.Key);
		}
	}

	if (CommonCandidateOverrides.Num() > 0 || ExistingOverrides.Num() > 0)
	{
		MenuBuilder.BeginSection("ChannelOverrides", LOCTEXT("ChannelGroupOverrideMenuSectionName", "Channel Overrides"));

		if (CommonCandidateOverrides.Num() == 1)
		{
			BuildChannelOverrideMenu(MenuBuilder, SequenceModel, CommonCandidateOverrides);
		}
		else if (CommonCandidateOverrides.Num() > 1)
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("OverrideChannelChoiceLabel", "Override with..."),
				LOCTEXT("OverrideChannelChoiceTooltip", "Overrides all channels with the specified channel"),
				FNewMenuDelegate::CreateStatic(&FChannelGroupOverrideHelper::BuildChannelOverrideMenu, SequenceModel, CommonCandidateOverrides));
		}

		if (ExistingOverrides.Num() > 0)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("RemoveOverrideChannelLabel", "Remove Override"),
				LOCTEXT("RemoveOverrideChannelTooltip", "Removes the channel override and revert back to the default channel type"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateStatic(&FChannelGroupOverrideHelper::RemoveChannelOverrides, SequenceModel),
					FCanExecuteAction()));
		}

		MenuBuilder.EndSection();
	}
}

void FChannelGroupOverrideHelper::BuildChannelOverrideMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FSequenceModel> SequenceModel, UMovieSceneChannelOverrideContainer::FOverrideCandidates OverrideCandidates)
{
	for (TSubclassOf<UMovieSceneChannelOverrideContainer> OverrideCandidate : OverrideCandidates)
	{
		MenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("OverrideChannelLabel", "Override with {0}"),
				OverrideCandidate->GetDisplayNameText()),
			FText::Format(LOCTEXT("OverrideChannelTooltip", "Overrides all channels with {0}"),
				OverrideCandidate->GetDisplayNameText()),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&FChannelGroupOverrideHelper::OverrideChannels, SequenceModel, OverrideCandidate),
				FCanExecuteAction()));
	}
}

void FChannelGroupOverrideHelper::OverrideChannels(TSharedPtr<FSequenceModel> SequenceModel, TSubclassOf<UMovieSceneChannelOverrideContainer> OverrideClass)
{
	TArray<TViewModelPtr<FChannelModel>> Channels;
	if (!GetChannelsFromOutlinerSelection(SequenceModel, Channels))
	{
		return;
	}

	bool bAnyChannelsChanged = false;
	TSet<TSharedPtr<FChannelGroupModel>> ChangedChannelGroups;
	FScopedTransaction Transaction(LOCTEXT("OverrideChannels", "Override Channels"));

	for (TViewModelPtr<FChannelModel> Channel : Channels)
	{
		UMovieSceneSection* Section = Channel->GetSection();
		IMovieSceneChannelOverrideProvider* OverrideProvider = Cast<IMovieSceneChannelOverrideProvider>(Section);
		if (!OverrideProvider)
		{
			continue;
		}

		if (!Section->TryModify())
		{
			continue;
		}

		UMovieSceneSectionChannelOverrideRegistry* OverrideRegistry = OverrideProvider->GetChannelOverrideRegistry(true);
		check(OverrideRegistry);

		UMovieSceneChannelOverrideContainer* OverrideChannelContainer = NewObject<UMovieSceneChannelOverrideContainer>(
				OverrideRegistry, OverrideClass, NAME_None, RF_Transactional);
		OverrideChannelContainer->InitializeOverride(Channel->GetChannel());
		OverrideRegistry->AddChannel(Channel->GetChannelName(), OverrideChannelContainer);	

		OverrideProvider->OnChannelOverridesChanged();

		bAnyChannelsChanged = true;
		if (TSharedPtr<FChannelGroupModel> ParentGroupModel = Channel->CastParent<FChannelGroupModel>())
		{
			ChangedChannelGroups.Add(ParentGroupModel);
		}
	}

	for (TSharedPtr<FChannelGroupModel> ChangedChannelGroup : ChangedChannelGroups)
	{
		ChangedChannelGroup->OnChannelOverridesChanged();
	}

	if (!bAnyChannelsChanged)
	{
		Transaction.Cancel();
	}

	SequenceModel->GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

void FChannelGroupOverrideHelper::RemoveChannelOverrides(TSharedPtr<FSequenceModel> SequenceModel)
{
	TArray<TViewModelPtr<FChannelModel>> Channels;
	if (!GetChannelsFromOutlinerSelection(SequenceModel, Channels))
	{
		return;
	}

	bool bAnyChannelsChanged = false;
	TSet<TSharedPtr<FChannelGroupModel>> ChangedChannelGroups;
	FScopedTransaction Transaction(LOCTEXT("RemoveChannelOverrides", "Remove Override Channels"));

	for (TViewModelPtr<FChannelModel> Channel : Channels)
	{
		UMovieSceneSection* Section = Channel->GetSection();
		IMovieSceneChannelOverrideProvider* OverrideProvider = Cast<IMovieSceneChannelOverrideProvider>(Section);
		if (!OverrideProvider)
		{
			continue;
		}

		UMovieSceneSectionChannelOverrideRegistry* OverrideRegistry = OverrideProvider->GetChannelOverrideRegistry(false);
		if (!OverrideRegistry)
		{
			continue;
		}

		if (!Section->TryModify())
		{
			continue;
		}

		OverrideRegistry->RemoveChannel(Channel->GetChannelName());

		OverrideProvider->OnChannelOverridesChanged();

		bAnyChannelsChanged = true;
		if (TSharedPtr<FChannelGroupModel> ParentGroupModel = Channel->CastParent<FChannelGroupModel>())
		{
			ChangedChannelGroups.Add(ParentGroupModel);
		}
	}

	for (TSharedPtr<FChannelGroupModel> ChangedChannelGroup : ChangedChannelGroups)
	{
		ChangedChannelGroup->OnChannelOverridesChanged();
	}

	if (!bAnyChannelsChanged)
	{
		Transaction.Cancel();
	}

	SequenceModel->GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

FChannelGroupOutlinerModel::FChannelGroupOutlinerModel(FName InChannelName, const FText& InDisplayText, FGetMovieSceneTooltipText InGetTooltipTextDelegate)
	: TOutlinerModelMixin<FChannelGroupModel>(InChannelName, InDisplayText, InGetTooltipTextDelegate)
{
	SetIdentifier(InChannelName);
}

FChannelGroupOutlinerModel::~FChannelGroupOutlinerModel()
{}

void FChannelGroupOutlinerModel::OnUpdated()
{
	WeakCommonChannelModel = nullptr;

	// If all channels are the same type, assign the common channel
	//    model for edit interactions and context menus
	{
		TViewModelPtr<FChannelModel> CommonChannel;
		for (TWeakViewModelPtr<FChannelModel> WeakChannel : Channels)
		{
			if (TViewModelPtr<FChannelModel> Channel = WeakChannel.Pin())
			{
				if (!CommonChannel)
				{
					CommonChannel = Channel;
				}
				else if (CommonChannel->GetTypeTable().GetTypeID() != Channel->GetTypeTable().GetTypeID())
				{
					CommonChannel = nullptr;
					break;
				}
			}
		}
		WeakCommonChannelModel = CommonChannel;
	}


	const bool bShouldBeMutable = Algo::AnyOf(Channels,
		[](TWeakViewModelPtr<FChannelModel> In)
		{
			TViewModelPtr<IMutableExtension> Mutable = In.ImplicitPin();
			return Mutable.IsValid();
		}
	);

	const bool bIsMutable = CastDynamic<IMutableExtension>() != nullptr;

	if (bShouldBeMutable && !bIsMutable)
	{
		AddDynamicExtension<FDynamicChannelGroupMuteExtension>(SharedThis(this));
	}
	else if (!bShouldBeMutable && bIsMutable)
	{
		RemoveDynamicExtension<FDynamicChannelGroupMuteExtension>();
	}
}

FOutlinerSizing FChannelGroupOutlinerModel::RecomputeSizing()
{
	FOutlinerSizing MaxSizing;

	for (TWeakViewModelPtr<FChannelModel> WeakChannel : Channels)
	{
		if (TSharedPtr<FChannelModel> Channel = WeakChannel.Pin())
		{
			FOutlinerSizing Desired = Channel->GetDesiredSizing();
			MaxSizing.Accumulate(Desired);
		}
	}

	if (ComputedSizing != MaxSizing)
	{
		ComputedSizing = MaxSizing;

		for (TWeakViewModelPtr<FChannelModel> WeakChannel : Channels)
		{
			if (TSharedPtr<FChannelModel> Channel = WeakChannel.Pin())
			{
				Channel->SetComputedSizing(MaxSizing);
			}
		}
	}

	return MaxSizing;
}

FOutlinerSizing FChannelGroupOutlinerModel::GetOutlinerSizing() const
{
	if (EnumHasAnyFlags(ComputedSizing.Flags, EOutlinerSizingFlags::DynamicSizing))
	{
		const_cast<FChannelGroupOutlinerModel*>(this)->RecomputeSizing();
	}

	FOutlinerSizing FinalSizing = ComputedSizing;
	if (!EnumHasAnyFlags(ComputedSizing.Flags, EOutlinerSizingFlags::CustomHeight))
	{
		if (TSharedPtr<FSequencerEditorViewModel> Editor = GetEditor())
		{
			FViewDensityInfo Density = Editor->GetViewDensity();
			FinalSizing.Height = Density.UniformHeight.Get(FinalSizing.Height);
		}
	}
	return FinalSizing;
}

TSharedPtr<SWidget> FChannelGroupOutlinerModel::CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName)
{
	TViewModelPtr<FSequencerEditorViewModel> Editor = InParams.Editor->CastThisShared<FSequencerEditorViewModel>();
	if (!Editor)
	{
		return SNullWidget::NullWidget;
	}

	// Ask a common channel to populate the outliner column view first
	if (TViewModelPtr<FChannelModel> CommonChannel = WeakCommonChannelModel.Pin())
	{
		TSharedPtr<SWidget> Widget = CommonChannel->CreateOutlinerViewForColumn(InParams, InColumnName);
		if (Widget)
		{
			return Widget;
		}
	}

	if (InColumnName == FCommonOutlinerNames::Label)
	{
		return SNew(SOutlinerItemViewBase, SharedThis(this), InParams.Editor, InParams.TreeViewRow);
	}

	if (InColumnName == FCommonOutlinerNames::Edit)
	{
		return SNew(SKeyAreaEditorSwitcher, SharedThis(this), Editor)
			.Visibility(this, &FChannelGroupOutlinerModel::GetKeyEditorVisibility);
	}

	// Helper to check if any channel in this group can be keyed
	auto CanAnyChannelBeKeyed = [this]()
	{
		for (const TWeakViewModelPtr<FChannelModel>& WeakChannel : GetChannels())
		{
			if (TSharedPtr<FChannelModel> Channel = WeakChannel.Pin())
			{
				if (TSharedPtr<IKeyArea> KeyArea = Channel->GetKeyArea())
				{
					const FMovieSceneChannelMetaData* MetaData = KeyArea->GetChannel().GetMetaData();
					if (MetaData && MetaData->bCanBeKeyed)
					{
						return true;
					}
				}
			}
		}
		return false;
	};

	// Check if any channel's decoration editor provides a custom keyframe widget
	// (e.g., bone match channel replaces "add key" with a dialog launcher).
	// Used by both KeyFrame and Nav columns since the add-key button can appear in either.
	auto TryCreateDecorationKeyFrameWidget = [this, &Editor]() -> TSharedPtr<SWidget>
	{
		for (const TWeakViewModelPtr<FChannelModel>& WeakChannel : GetChannels())
		{
			if (TSharedPtr<FChannelModel> Channel = WeakChannel.Pin())
			{
				if (TSharedPtr<IKeyArea> KeyArea = Channel->GetKeyArea())
				{
					const FMovieSceneChannelMetaData* MetaData = KeyArea->GetChannel().GetMetaData();
					UObject* OwningObject = MetaData ? MetaData->WeakOwningObject.Get() : nullptr;
					UMovieSceneSection* Section = KeyArea->GetOwningSection();

					if (OwningObject && Section && OwningObject != Section)
					{
						if (TSharedPtr<ISequencer> SequencerPtr = Editor->GetSequencer())
						{
							if (ISequencerDecorationEditor* DecorationEditor = SequencerPtr->FindDecorationEditor(OwningObject->GetClass()))
							{
								FGuid ObjectBindingID;
								if (TViewModelPtr<IObjectBindingExtension> ObjectBinding = FindAncestorOfType<IObjectBindingExtension>())
								{
									ObjectBindingID = ObjectBinding->GetObjectGuid();
								}

								if (TSharedPtr<SWidget> CustomWidget = DecorationEditor->CreateKeyFrameWidget(
									*OwningObject, *Section, KeyArea->GetChannel(), Editor->GetSequencer(), ObjectBindingID,
									SharedThis(this)))
								{
									return CustomWidget;
								}
							}
						}

						// Fall through to standard key navigation.
					}
				}
			}
		}
		return nullptr;
	};

	if (InColumnName == FCommonOutlinerNames::KeyFrame)
	{
		if (TSharedPtr<SWidget> CustomWidget = TryCreateDecorationKeyFrameWidget())
		{
			return CustomWidget;
		}

		if (!CanAnyChannelBeKeyed())
		{
			return nullptr;
		}

		EKeyNavigationButtons Buttons = EKeyNavigationButtons::AddKey;
		return SNew(SSequencerKeyNavigationButtons, SharedThis(this), Editor->GetSequencer())
			.Buttons(Buttons);
	}

	if (InColumnName == FCommonOutlinerNames::Nav)
	{
		if (TSharedPtr<SWidget> CustomWidget = TryCreateDecorationKeyFrameWidget())
		{
			return CustomWidget;
		}

		if (!CanAnyChannelBeKeyed())
		{
			return nullptr;
		}

		EKeyNavigationButtons Buttons = InParams.TreeViewRow->IsColumnVisible(FCommonOutlinerNames::KeyFrame)
			? EKeyNavigationButtons::NavOnly
			: EKeyNavigationButtons::All;

		return SNew(SSequencerKeyNavigationButtons, SharedThis(this), Editor->GetSequencer())
			.Buttons(Buttons);
	}

	if (InColumnName == FCommonOutlinerNames::ColorPicker)
	{
		return SNew(SOutlinerTrackColorPicker, SharedThis(this), InParams.Editor);
	}

	return nullptr;
}

EVisibility FChannelGroupOutlinerModel::GetKeyEditorVisibility() const
{
	return GetChannels().Num() == 0 ? EVisibility::Collapsed : EVisibility::Visible;
}

FText FChannelGroupOutlinerModel::GetLabel() const
{
	return GetDisplayText();
}

FSlateFontInfo FChannelGroupOutlinerModel::GetLabelFont() const
{
	return IsAnimated()
		? FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.ItalicFont")
		: FOutlinerItemModelMixin::GetLabelFont();
}

FText FChannelGroupOutlinerModel::GetLabelToolTipText() const
{
	return GetTooltipText();
}

bool FChannelGroupOutlinerModel::HasCurves() const
{
	return FChannelGroupModel::HasCurves();
}

TOptional<FString> FChannelGroupOutlinerModel::GetUniquePathName() const
{
	return FChannelGroupModel::GetUniquePathName();
}

bool FChannelGroupOutlinerModel::CanDelete(FText* OutErrorMessage) const
{
	return true;
}

void FChannelGroupOutlinerModel::Delete()
{
	TArray<FName> PathFromTrack;
	TViewModelPtr<ITrackExtension> Track = GetParentTrackNodeAndNamePath(SharedThis(this), PathFromTrack);

	Track->GetTrack()->Modify();

	for (const FViewModelPtr& Channel : GetTrackAreaModelList())
	{
		if (TViewModelPtr<FSectionModel> Section = Channel->FindAncestorOfType<FSectionModel>())
		{
			Section->GetSectionInterface()->RequestDeleteKeyArea(PathFromTrack);
		}
	}
}

void FChannelGroupOutlinerModel::CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels)
{
	FChannelGroupModel::CreateCurveModels(OutCurveModels);
}

void FChannelGroupOutlinerModel::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	if (TViewModelPtr<FChannelModel> CommonChannel = WeakCommonChannelModel.Pin())
	{
		CommonChannel->BuildContextMenu(MenuBuilder, TViewModelPtr<FChannelGroupOutlinerModel>(this));
	}

	FOutlinerItemModelMixin::BuildContextMenu(MenuBuilder);

	BuildChannelOverrideMenu(MenuBuilder);
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

