// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/DecorationOutlinerModel.h"

#include "ISequencer.h"
#include "ISequencerDecorationEditor.h"
#include "ISequencerSection.h"
#include "SequencerLog.h"
#include "MovieSceneSection.h"
#include "SequencerCommonHelpers.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Decorations/IMovieSceneChannelDecoration.h"
#include "Decorations/IMovieSceneSectionProviderDecoration.h"
#include "Decorations/MovieSceneDecorationContainer.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/TrackModelLayoutBuilder.h"
#include "MVVM/ViewModels/ViewDensity.h"
#include "SSequencer.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "MVVM/Views/ViewUtilities.h"

#define LOCTEXT_NAMESPACE "DecorationOutlinerModel"

namespace UE::Sequencer
{
UE_SEQUENCER_DEFINE_CASTABLE(FDecorationOutlinerModel);

FDecorationOutlinerModel::FDecorationOutlinerModel(UMovieSceneTrack* InParentTrack, UMovieSceneDecorationContainerObject* InDecorationContainer, UObject* InDecoration)
	: WeakParentTrack(InParentTrack)
	, WeakDecorationContainer(InDecorationContainer)
	, WeakDecoration(InDecoration)
{
	RegisterChildList(&DecoratorList);
	RegisterChildList(&SectionModelList);
}

FDecorationOutlinerModel::~FDecorationOutlinerModel()
{
	if (UMovieSceneSignedObject* SignedObject = Cast<UMovieSceneSignedObject>(WeakDecoration.Get()))
	{
		SignedObject->OnSignatureChanged().Remove(SignatureChangedHandle);
	}
}

void FDecorationOutlinerModel::OnDecorationSignatureChanged()
{
	bDecorationSignatureChanged = true;
}

bool FDecorationOutlinerModel::NeedsReconstruct() const
{
	UObject* Decoration = WeakDecoration.Get();
	if (!Decoration)
	{
		return false;
	}

	// Consume all dirty flags in a single pass to avoid leaving stale flags
	// when a decoration implements both interfaces.
	bool bNeedsReconstruct = false;

	if (IMovieSceneChannelDecoration* ChannelDecoration = Cast<IMovieSceneChannelDecoration>(Decoration))
	{
		bNeedsReconstruct |= ChannelDecoration->ConsumeDecorationChannelProxyDirty();
	}

	if (IMovieSceneSectionProviderDecoration* SectionProviderDecoration = Cast<IMovieSceneSectionProviderDecoration>(Decoration))
	{
		bNeedsReconstruct |= SectionProviderDecoration->ConsumeStructureChanged();
	}

	return bNeedsReconstruct;
}

void FDecorationOutlinerModel::OnConstruct()
{
	UObject* Decoration = WeakDecoration.Get();
	if (!WeakParentTrack.IsValid() || !WeakDecorationContainer.IsValid() || !Decoration)
	{
		return;
	}

	// Early out if we don't have shared data yet - this means we're being called
	// during AddChild() before we're fully integrated into the hierarchy.
	// The explicit OnConstruct() call after AddChild() will do the real work.
	if (!GetSharedData())
	{
		return;
	}

	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	if (!SequenceModel)
	{
		return;
	}

	FSectionModelStorageExtension* SectionModelStorage = SequenceModel->CastDynamic<FSectionModelStorageExtension>();
	if (!SectionModelStorage)
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = SequenceModel->GetSequencer();
	if (!Sequencer)
	{
		return;
	}

	TrackEditor = Sequencer->GetTrackEditor(WeakParentTrack.Get());
	FViewModelChildren OutlinerChildren = GetTopLevelChannels();

	FScopedViewModelListHead RecycledModels(AsShared(), EViewModelListType::Recycled);
	OutlinerChildren.MoveChildrenTo<IRecyclableExtension>(RecycledModels.GetChildren(), IRecyclableExtension::CallOnRecycle);
	FViewModelChildren RecycledChildren = RecycledModels.GetChildren();

	// Remove old channel and category models from their parent section models before destroying proxies.
	for (const TWeakPtr<FViewModel>& WeakModel : TrackedSectionChildModels)
	{
		if (TSharedPtr<FViewModel> OldModel = WeakModel.Pin())
		{
			OldModel->RemoveFromParent();
		}
	}
	TrackedSectionChildModels.Empty();

	// Clear old channel proxy before rebuilding
	ChannelDecorationProxy.Reset();

	// Clear section model list
	FViewModelChildren SectionChildren = GetChildrenForList(&SectionModelList);
	SectionChildren.Empty();

	// Handle section-provider decorations: host sections directly in this model's track
	// area without creating intermediate FSectionOutlinerModel wrappers.
	// Skip this path if the decoration also implements IMovieSceneChannelDecoration,
	// since the channel path handles both section and track containers.
	if (IMovieSceneSectionProviderDecoration* SectionProvider = Cast<IMovieSceneSectionProviderDecoration>(Decoration);
		SectionProvider && !Cast<IMovieSceneChannelDecoration>(Decoration))
	{
		ISequencerDecorationEditor* DecorationEditor = Sequencer->FindDecorationEditor(Decoration->GetClass());
		if (!DecorationEditor)
		{
			UE_LOGF(LogSequencer, Warning, "No decoration editor registered for decoration class %ls. Sections will not be displayed.", *Decoration->GetClass()->GetName());
			return;
		}

		// Create layout builder once — calling RefreshLayout per section accumulates
		// channels from all sections, matching how FTrackModel handles multi-section tracks.
		FTrackModelLayoutBuilder LayoutBuilder(AsShared());

		for (UMovieSceneSection* Section : SectionProvider->GetSections())
		{
			if (!Section)
			{
				continue;
			}
			const TSharedPtr<ISequencerSection> SectionInterface = DecorationEditor->MakeSectionInterface(*Section, *WeakParentTrack.Get(), WeakParentTrack.Get()->FindObjectBindingGuid());
			if (!SectionInterface)
			{
				continue;
			}
			const TSharedPtr<FSectionModel> SectionModel = SectionModelStorage->CreateModelForSection(Section, SectionInterface.ToSharedRef());

			// Add section model to the track area list (renders the section in the track area)
			SectionChildren.AddChild(SectionModel);
			SectionModel->SetLinkedOutlinerItem(CastViewModel<IOutlinerExtension>(AsShared()));

			LayoutBuilder.RefreshLayout(SectionModel);
		}
	}
	// Handle channel decorations
	else if (IMovieSceneChannelDecoration* ChannelDecoration = Cast<IMovieSceneChannelDecoration>(Decoration))
	{
		if (!TrackEditor)
		{
			return;
		}

		UMovieSceneSection* ParentSection = Cast<UMovieSceneSection>(WeakDecorationContainer.Get());
		TSharedPtr<FSectionModel> SectionModel;
		FMovieSceneChannelProxy* ChannelProxyPtr = nullptr;

		if (ParentSection)
		{
			// Section-level decoration: channels are hosted by the parent section,
			// with a separate proxy owned by this model.
			TSharedRef<ISequencerSection> NewSectionInterface = TrackEditor->MakeSectionInterface(
				*ParentSection, *WeakParentTrack.Get(), WeakParentTrack.Get()->FindObjectBindingGuid());
			SectionModel = SectionModelStorage->CreateModelForSection(ParentSection, NewSectionInterface);

			FMovieSceneChannelProxyData ChannelProxyData;
			ChannelDecoration->PopulateChannelProxy(ChannelProxyData);
			ChannelDecorationProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(ChannelProxyData));
			ChannelProxyPtr = ChannelDecorationProxy.Get();
		}
		else
		{
			// Track-level decoration: get the host section from the decoration's
			// IMovieSceneSectionProviderDecoration and use it for channel hosting.
			if (IMovieSceneSectionProviderDecoration* HostSectionProvider = Cast<IMovieSceneSectionProviderDecoration>(Decoration))
			{
				TArrayView<TObjectPtr<UMovieSceneSection>> Sections = HostSectionProvider->GetSections();
				if (Sections.Num() > 0)
				{
					ParentSection = Sections[0];
				}
			}
			if (!ParentSection)
			{
				return;
			}

			TSharedRef<ISequencerSection> NewSectionInterface = MakeShared<FSequencerSection>(*ParentSection);
			SectionModel = SectionModelStorage->CreateModelForSection(ParentSection, NewSectionInterface);
			SectionChildren.AddChild(SectionModel);
			SectionModel->SetLinkedOutlinerItem(CastViewModel<IOutlinerExtension>(AsShared()));

			ParentSection->InvalidateChannelProxy();
			ChannelProxyPtr = &ParentSection->GetChannelProxy();
		}

		FMovieSceneChannelProxy& ChannelProxy = *ChannelProxyPtr;
		TSharedPtr<ISequencerSection> SectionInterface = SectionModel->GetSectionInterface();

		// Collect all channels with their metadata for sorting
		struct FChannelSortData
		{
			FMovieSceneChannelHandle Handle;
			const FMovieSceneChannelMetaData* MetaData;
		};
		TArray<FChannelSortData> SortedChannels;

		for (const FMovieSceneChannelEntry& Entry : ChannelProxy.GetAllEntries())
		{
			TArrayView<const FMovieSceneChannelMetaData> AllMetaData = Entry.GetMetaData();

			for (int32 ChannelIndex = 0; ChannelIndex < AllMetaData.Num(); ++ChannelIndex)
			{
				FChannelSortData SortData;
				SortData.Handle = ChannelProxy.MakeHandle(Entry.GetChannelTypeName(), ChannelIndex);
				SortData.MetaData = &AllMetaData[ChannelIndex];
				SortedChannels.Add(SortData);
			}
		}

		// Sort by SortOrder
		SortedChannels.Sort([](const FChannelSortData& A, const FChannelSortData& B)
		{
			return A.MetaData->SortOrder < B.MetaData->SortOrder;
		});

		// Channels with a non-empty Group are placed under a collapsible
		// FCategoryGroupModel sub-group. Channels with no Group are added
		// directly under this decoration model.
		TMap<FName, TSharedPtr<FCategoryGroupModel>> CategoryGroupModels;
		TMap<FName, TSharedPtr<FCategoryModel>> CategoryModels;
		TMap<FName, TSharedPtr<FViewModel>> CategoryGroupTails;

		TSharedPtr<FViewModel> ChannelTail;
		for (const FChannelSortData& ChannelData : SortedChannels)
		{
			const FMovieSceneChannelMetaData& MetaData = *ChannelData.MetaData;
			FMovieSceneChannelHandle ChannelHandle = ChannelData.Handle;

			FName ChannelName = MetaData.Name;

			const bool bIsSubGroup = !MetaData.Group.IsEmpty();
			const FName GroupName = bIsSubGroup ? FName(*MetaData.Group.ToString()) : NAME_None;

			// Find or create the category group model for sub-grouped channels
			TSharedPtr<FCategoryGroupModel> CategoryGroup;
			if (bIsSubGroup)
			{
				TSharedPtr<FCategoryGroupModel>& ExistingCategoryGroup = CategoryGroupModels.FindOrAdd(GroupName);

				if (!ExistingCategoryGroup)
				{
					ExistingCategoryGroup = MakeShared<FCategoryGroupModel>(GroupName, MetaData.Group, MetaData.GetGroupTooltipTextDelegate);
					OutlinerChildren.InsertChild(ExistingCategoryGroup, ChannelTail);
					ChannelTail = ExistingCategoryGroup;

					// Create a track-area FCategoryModel so the category group can
					// render aggregated keys when collapsed in the outliner.
					TSharedPtr<FCategoryModel> CategoryModel = ExistingCategoryGroup->CreateTrackAreaCategory();
					SectionModel->GetChildList(FTrackModel::GetTopLevelChannelType()).AddChild(CategoryModel);
					CategoryModels.Add(GroupName, CategoryModel);
					TrackedSectionChildModels.Add(CategoryModel);
				}
				CategoryGroup = ExistingCategoryGroup;
			}

			// Search for an existing channel group model in the appropriate parent
			TSharedPtr<FChannelGroupOutlinerModel> ChannelGroupModel;
			if (bIsSubGroup)
			{
				FViewModelChildren CategoryChildren = CategoryGroup->GetChildList(EViewModelListType::Outliner);
				for (TSharedPtr<FChannelGroupOutlinerModel> ExistingGroup : CategoryChildren.IterateSubList<FChannelGroupOutlinerModel>())
				{
					if (ExistingGroup && ExistingGroup->GetChannelName() == ChannelName)
					{
						ChannelGroupModel = ExistingGroup;
						break;
					}
				}
			}
			else
			{
				for (TSharedPtr<FChannelGroupOutlinerModel> ExistingGroup : OutlinerChildren.IterateSubList<FChannelGroupOutlinerModel>())
				{
					if (ExistingGroup && ExistingGroup->GetChannelName() == ChannelName)
					{
						ChannelGroupModel = ExistingGroup;
						break;
					}
				}
			}

			if (!ChannelGroupModel)
			{
				ChannelGroupModel = MakeShared<FChannelGroupOutlinerModel>(ChannelName, MetaData.DisplayText, MetaData.GetTooltipTextDelegate);
			}
			else
			{
				// Clear old channels when reusing an existing group to avoid stale handles
				ChannelGroupModel->OnRecycle();
			}

			if (bIsSubGroup)
			{
				TSharedPtr<FViewModel>& GroupTail = CategoryGroupTails.FindOrAdd(GroupName);
				CategoryGroup->GetChildList(EViewModelListType::Outliner).InsertChild(ChannelGroupModel, GroupTail);
				GroupTail = ChannelGroupModel;
			}
			else
			{
				OutlinerChildren.InsertChild(ChannelGroupModel, ChannelTail);
				ChannelTail = ChannelGroupModel;
			}

			TSharedPtr<FChannelModel> ChannelModel = MakeShared<FChannelModel>(ChannelName, SectionInterface, ChannelHandle);
			ChannelGroupModel->AddChannel(ChannelModel);
			ChannelModel->SetLinkedOutlinerItem(ChannelGroupModel.ToSharedRef());
			ChannelModel->Initialize(SectionInterface, ChannelHandle);

			// Parent under the track-area category model if this channel belongs to
			// a sub-group, otherwise add directly to the section model.
			if (bIsSubGroup)
			{
				CategoryModels[GroupName]->GetChildList(EViewModelListType::Generic).AddChild(ChannelModel);
			}
			else
			{
				SectionModel->GetChildList(FTrackModel::GetTopLevelChannelType()).AddChild(ChannelModel);
			}

			TrackedSectionChildModels.Add(ChannelModel);
		}

		// Recompute sizing for category groups now that their categories are populated
		for (const auto& Pair : CategoryGroupModels)
		{
			Pair.Value->RecomputeSizing();
		}
	}
}

/*------------------------------------------------------------------------------------
	FOutlinerItemModel
 -----------------------------------------------------------------------------------*/
void FDecorationOutlinerModel::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UObject* Decoration = WeakDecoration.Get();
	if (Decoration && Decoration->Implements<UMovieSceneSectionProviderDecoration>())
	{
		const TArray<TWeakObjectPtr<>> TrackAreaModels = SequencerHelpers::GetSectionObjectsFromTrackAreaModels(GetTrackAreaModelList());
		SequencerHelpers::BuildEditSectionMenu(Sequencer, TrackAreaModels, MenuBuilder, true);
	}

	FOutlinerItemModel::BuildContextMenu(MenuBuilder);
}

void FDecorationOutlinerModel::BuildSidebarMenu(FMenuBuilder& MenuBuilder)
{
	BuildContextMenu(MenuBuilder);
}

/*------------------------------------------------------------------------------------
	IOutlinerExtension
 -----------------------------------------------------------------------------------*/
FOutlinerSizing FDecorationOutlinerModel::GetOutlinerSizing() const
{
	const TSharedPtr<FSequencerEditorViewModel> Editor = GetEditor();
	const FViewDensityInfo Density = Editor ? Editor->GetViewDensity() : FViewDensityInfo();
	float Height = Density.UniformHeight.Get(SequencerLayoutConstants::SectionAreaDefaultHeight);
	if (auto It = SectionModelList.Iterate<FSectionModel>().begin(); It != SectionModelList.Iterate<FSectionModel>().end())
	{
		Height = (*It)->GetSectionInterface()->GetSectionHeight(Density);
	}
	return FOutlinerSizing(Height);
}

TSharedRef<SWidget> FDecorationOutlinerModel::BuildAddDecorationSubMenu(TSharedPtr<FEditorViewModel> Editor) const
{
	FMenuBuilder MenuBuilder(true, nullptr);
	const TObjectPtr<UObject> Decoration = WeakDecoration.Get();

	const TSharedPtr<FSequencerEditorViewModel> SequencerEditor = Editor->CastThisShared<FSequencerEditorViewModel>();
	if (!SequencerEditor)
	{
		return MenuBuilder.MakeWidget();
	}

	FGuid ObjectBindingID = WeakParentTrack.IsValid() ? WeakParentTrack->FindObjectBindingGuid() : FGuid();

	if (Decoration)
	{
		UMovieSceneDecorationContainerObject* DecorationContainer = Decoration->GetTypedOuter<UMovieSceneDecorationContainerObject>();
		SequencerHelpers::BuildDecorationMenu(MenuBuilder, DecorationContainer, ObjectBindingID, SequencerEditor->GetSequencer());
	}

	return  MenuBuilder.MakeWidget();
}

TSharedPtr<SWidget> FDecorationOutlinerModel::CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName)
{
	if (InColumnName == FCommonOutlinerNames::Add)
	{
		return MakeAddButton(
			LOCTEXT("AddModifier", "Modifier"),
			FOnGetContent::CreateSP(
				this,
				&FDecorationOutlinerModel::BuildAddDecorationSubMenu,
				InParams.Editor
				),
				InParams.Editor
			);
	}
	if (InColumnName == FCommonOutlinerNames::Edit)
	{
		if (UObject* Decoration = WeakDecoration.Get())
		{
			TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
			TWeakPtr<ISequencer> WeakSequencer = SequenceModel ? SequenceModel->GetSequencer() : nullptr;
			if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
			{
				if (ISequencerDecorationEditor* DecorationEditor = SequencerPtr->FindDecorationEditor(Decoration->GetClass()))
				{
					TSharedPtr<SWidget> Widget = DecorationEditor->CreateOutlinerWidget(*Decoration, WeakSequencer);
					if (Widget)
					{
						return Widget;
					}
				}
			}
		}
	}
	if (TrackEditor)
	{
		const FBuildColumnWidgetParams Params(SharedThis(this), InParams);
		return TrackEditor->BuildOutlinerColumnWidget(Params, InColumnName);
	}
	return nullptr;
}

FText FDecorationOutlinerModel::GetLabel() const
{
	if (UObject* Decoration = WeakDecoration.Get())
	{
		return Decoration->GetClass()->GetDisplayNameText();
	}
	return FText::GetEmpty();
}

const FSlateBrush* FDecorationOutlinerModel::GetIconBrush() const
{
	if (UObject* Decoration = WeakDecoration.Get())
	{
		TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
		if (SequenceModel)
		{
			if (TSharedPtr<ISequencer> SequencerPtr = SequenceModel->GetSequencer())
			{
				if (ISequencerDecorationEditor* DecorationEditor = SequencerPtr->FindDecorationEditor(Decoration->GetClass()))
				{
					return DecorationEditor->GetIconBrush();
				}
			}
		}
	}
	return nullptr;
}

FViewModelChildren FDecorationOutlinerModel::GetSectionModels()
{
	return GetChildrenForList(&SectionModelList);
}

FViewModelChildren FDecorationOutlinerModel::GetTopLevelChannels()
{
	return GetChildrenForList(&DecoratorList);
}

/*------------------------------------------------------------------------------------
	ITrackAreaExtension
------------------------------------------------------------------------------------*/

FTrackAreaParameters FDecorationOutlinerModel::GetTrackAreaParameters() const
{
	FTrackAreaParameters Parameters;
	// Use Nested when the decoration has its own section so child channel lanes
	// register as children, enabling section background painting behind them.
	const bool bHasOwnSection = SectionModelList.Iterate<FSectionModel>().begin() != SectionModelList.Iterate<FSectionModel>().end();
	Parameters.LaneType = bHasOwnSection ? ETrackAreaLaneType::Nested : ETrackAreaLaneType::Inline;
	return Parameters;
}

FViewModelVariantIterator FDecorationOutlinerModel::GetTrackAreaModelList() const
{
	return &SectionModelList;
}

FViewModelVariantIterator FDecorationOutlinerModel::GetTopLevelChildTrackAreaModels() const
{
	return &DecoratorList;
}

/*------------------------------------------------------------------------------------
	IDeletableExtension
------------------------------------------------------------------------------------*/

bool FDecorationOutlinerModel::CanDelete(FText* OutErrorMessage) const
{
	return WeakDecoration.IsValid() && WeakDecorationContainer.IsValid();
}

void FDecorationOutlinerModel::Delete()
{
	UMovieSceneDecorationContainerObject* DecorationContainer = WeakDecorationContainer.Get();
	UMovieSceneTrack* ParentTrack = WeakParentTrack.Get();
	UObject* Decoration = WeakDecoration.Get();

	if (!DecorationContainer || !ParentTrack || !Decoration)
	{
		return;
	}

	ParentTrack->Modify();
	DecorationContainer->Modify();

	DecorationContainer->RemoveDecoration(Decoration->GetClass());
}

/*-----------------------------------------------------------------------------
	ILockableExtension
-----------------------------------------------------------------------------*/

ELockableLockState FDecorationOutlinerModel::GetLockState() const
{
	int32 NumSections = 0;
	int32 NumLockedSections = 0;

	for (const TViewModelPtr<FSectionModel>& Section : SectionModelList.Iterate<FSectionModel>())
	{
		++NumSections;

		const UMovieSceneSection* SectionObject = Section->GetSection();
		if (SectionObject && SectionObject->IsLocked())
		{
			++NumLockedSections;
		}
	}

	if (NumSections == 0 || NumLockedSections == 0)
	{
		return ELockableLockState::None;
	}

	return NumLockedSections == NumSections ? ELockableLockState::Locked : ELockableLockState::PartiallyLocked;
}

void FDecorationOutlinerModel::SetIsLocked(bool bIsLocked)
{
	for (const TViewModelPtr<FSectionModel>& Section : SectionModelList.Iterate<FSectionModel>())
	{
		UMovieSceneSection* SectionObject = Section->GetSection();
		if (SectionObject && SectionObject->IsLocked() != bIsLocked)
		{
			SectionObject->Modify();
			SectionObject->SetIsLocked(bIsLocked);
		}
	}
}


UObject* FDecorationOutlinerModel::GetDecoration() const
{
	return WeakDecoration.Get();
}

bool FDecorationOutlinerModel::IsMuted() const
{
	UMovieSceneDecorationContainerObject* Container = WeakDecorationContainer.Get();
	UObject* Decoration = WeakDecoration.Get();
	if (Container && Decoration)
	{
		return Container->IsDecorationMuted(Decoration);
	}
	return false;
}

void FDecorationOutlinerModel::SetIsMuted(bool bIsMuted)
{
	UMovieSceneDecorationContainerObject* Container = WeakDecorationContainer.Get();
	UObject* Decoration = WeakDecoration.Get();
	if (!Container || !Decoration)
	{
		return;
	}

	Container->SetDecorationMuted(Decoration, bIsMuted);

	if (IMovieSceneChannelDecoration* ChannelDecoration = Cast<IMovieSceneChannelDecoration>(Decoration))
	{
		ChannelDecoration->InvalidateDecorationChannelProxy();
	}

	if (IMovieSceneSectionProviderDecoration* SectionProvider = Cast<IMovieSceneSectionProviderDecoration>(Decoration))
	{
		SectionProvider->MarkStructureChanged();
	}

	// Update the hierarchical caches so mute/deactive dimming is reflected immediately
	if (TSharedPtr<FSharedViewModelData> CachedSharedData = GetSharedData())
	{
		if (FOutlinerCacheExtension* OutlinerCache = CachedSharedData->CastThis<FOutlinerCacheExtension>())
		{
			OutlinerCache->UpdateCachedFlags();
		}
	}

	if (TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor())
	{
		if (TSharedPtr<ISequencer> Sequencer = EditorViewModel->GetSequencer())
		{
			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		}
	}
}

bool FDecorationOutlinerModel::IsDeactivated() const
{
	UMovieSceneDecorationContainerObject* Container = WeakDecorationContainer.Get();
	UObject* Decoration = WeakDecoration.Get();
	if (Container && Decoration)
	{
		return Container->IsDecorationDisabled(Decoration);
	}
	return false;
}

void FDecorationOutlinerModel::SetIsDeactivated(bool bInIsDeactivated)
{
	UMovieSceneDecorationContainerObject* Container = WeakDecorationContainer.Get();
	UObject* Decoration = WeakDecoration.Get();
	if (!Container || !Decoration)
	{
		return;
	}

	Container->SetDecorationDisabled(Decoration, bInIsDeactivated);

	if (IMovieSceneChannelDecoration* ChannelDecoration = Cast<IMovieSceneChannelDecoration>(Decoration))
	{
		ChannelDecoration->InvalidateDecorationChannelProxy();
	}

	if (IMovieSceneSectionProviderDecoration* SectionProvider = Cast<IMovieSceneSectionProviderDecoration>(Decoration))
	{
		SectionProvider->MarkStructureChanged();
	}

	// Update the hierarchical caches so mute/deactive dimming is reflected immediately
	if (TSharedPtr<FSharedViewModelData> CachedSharedData = GetSharedData())
	{
		if (FOutlinerCacheExtension* OutlinerCache = CachedSharedData->CastThis<FOutlinerCacheExtension>())
		{
			OutlinerCache->UpdateCachedFlags();
		}
	}

	if (TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor())
	{
		if (TSharedPtr<ISequencer> Sequencer = EditorViewModel->GetSequencer())
		{
			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		}
	}
}

bool FDecorationOutlinerModel::IsDimmed() const
{
	UMovieSceneDecorationContainerObject* Container = WeakDecorationContainer.Get();
	UObject* Decoration = WeakDecoration.Get();
	if (Container && Decoration)
	{
		return !Container->IsDecorationActive(Decoration);
	}
	return false;
}

}

#undef LOCTEXT_NAMESPACE
