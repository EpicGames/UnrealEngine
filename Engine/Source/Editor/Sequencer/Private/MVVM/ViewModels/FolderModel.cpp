// Copyright Epic Games, Inc. All Rights Reserved.


#include "MVVM/ViewModels/FolderModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/LayerBarModel.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/Extensions/IRecyclableExtension.h"
#include "MVVM/Views/SOutlinerItemViewBase.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/OutlinerViewModelDragDropOp.h"
#include "MVVM/TrackModelStorageExtension.h"
#include "MVVM/ObjectBindingModelStorageExtension.h"
#include "MVVM/FolderModelStorageExtension.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"

#include "Sequencer.h"
#include "SequencerNodeTree.h"
#include "SequencerSettings.h"
#include "SequencerUtilities.h"
#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Colors/SColorPicker.h"
#include "SequencerOutlinerItemDragDropOp.h"
#include "MovieScene.h"
#include "MovieSceneBinding.h"
#include "MovieSceneFolder.h"
#include "Decorations/MovieSceneMuteSoloDecoration.h"

#include "DragAndDrop/ActorDragDropOp.h"
#include "ScopedTransaction.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "SequencerFolderModel"

namespace UE::Sequencer
{

TMap<TWeakObjectPtr<UMovieSceneFolder>, FColor> FFolderModel::InitialFolderColors;
bool FFolderModel::bFolderPickerWasCancelled = false;

FFolderModel::FFolderModel(UMovieSceneFolder* Folder)
	: TrackAreaList(EViewModelListType::TrackArea)
	, WeakFolder(Folder)
	, OwnerModel(nullptr)
{
	RegisterChildList(&TrackAreaList);
}

FFolderModel::~FFolderModel()
{
}

UMovieSceneFolder* FFolderModel::GetFolder() const
{
	return WeakFolder.Get();
}

void FFolderModel::OnConstruct()
{
	UMovieSceneFolder* Folder = WeakFolder.Get();

	OwnerModel = FindAncestorOfType<FSequenceModel>().Get();

	check(Folder && OwnerModel);

	SetIdentifier(Folder->GetFolderName());

	if (!IsLinked())
	{
		Folder->EventHandlers.Link(this);
	}

	RepopulateChildren();
}

void FFolderModel::RepopulateChildren()
{
	UMovieSceneFolder* Folder = WeakFolder.Get();
	if (!Folder)
	{
		DiscardAllChildren();
		return;
	}

	FTrackModelStorageExtension*         TrackStorage         = OwnerModel->CastDynamic<FTrackModelStorageExtension>();
	FObjectBindingModelStorageExtension* ObjectBindingStorage = OwnerModel->CastDynamic<FObjectBindingModelStorageExtension>();
	FFolderModelStorageExtension*        FolderStorage        = OwnerModel->CastDynamic<FFolderModelStorageExtension>();

	TSharedPtr<FFolderModel> This = SharedThis(this);

	FViewModelChildren TrackAreaChildren = GetChildrenForList(&TrackAreaList);
	FViewModelChildren OutlinerChildren  = GetChildrenForList(&OutlinerChildList);

	// Only need to every create a layer bar once
	if (!LayerBar)
	{
		TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
		TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();

		if (Sequencer->GetSequencerSettings()->GetShowLayerBars())
		{
			LayerBar = MakeShared<FLayerBarModel>(This);
			LayerBar->SetLinkedOutlinerItem(SharedThis(this));

			TrackAreaChildren.AddChild(LayerBar);
		}
	}

	// Keep our existing children alive
	FScopedViewModelListHead RecycledOutliner(This, EViewModelListType::Recycled);
	OutlinerChildren.MoveChildrenTo<IRecyclableExtension>(RecycledOutliner.GetChildren(), IRecyclableExtension::CallOnRecycle);

	// Create/recycle models for any children
	for (UMovieSceneFolder* ChildFolder : Folder->GetChildFolders())
	{
		TSharedPtr<FFolderModel> ChildModel = FolderStorage->CreateModelForFolder(ChildFolder, This);
		OutlinerChildren.AddChild(ChildModel);
	}

	// Create/recycle models for any object bindings
	UMovieScene* MovieScene = OwnerModel->GetMovieScene();
	if (MovieScene)
	{
		for (const FGuid& ObjectBinding : Folder->GetChildObjectBindings())
		{
			const FMovieSceneBinding* Binding = MovieScene->FindBinding(ObjectBinding);
			if (ensureMsgf(Binding, TEXT("Folder '%s' contains an invalid binding reference ID: %s"), *Folder->GetFolderName().ToString(), *LexToString(ObjectBinding)))
			{
				TSharedPtr<FViewModel> ObjectModel = ObjectBindingStorage->GetOrCreateModelForBinding(*Binding);
				OutlinerChildren.AddChild(ObjectModel);
			}
		}
	}

	// Create/recycle models for any tracks
	for (UMovieSceneTrack* Track : Folder->GetChildTracks())
	{
		if (Track)
		{
			TSharedPtr<FTrackModel> TrackModel = TrackStorage->CreateModelForTrack(Track, This);
			OutlinerChildren.AddChild(TrackModel);
		}
	}
}

void FFolderModel::OnPostUndo()
{
	RepopulateChildren();
}

void FFolderModel::OnTrackAdded(UMovieSceneTrack* Track)
{
	FTrackModelStorageExtension* TrackStorage = OwnerModel->CastDynamic<FTrackModelStorageExtension>();
	TSharedPtr<FTrackModel>      TrackModel   = TrackStorage->CreateModelForTrack(Track, AsShared());

	GetChildrenForList(&OutlinerChildList).AddChild(TrackModel);
}

void FFolderModel::OnTrackRemoved(UMovieSceneTrack* Track)
{
	TSharedPtr<FTrackModel> TrackModel = GetChildrenOfType<FTrackModel>().FindBy(Track, &FTrackModel::GetTrack);
	if (TrackModel)
	{
		// When tracks are removed we put them back to the root.
		// If the operation moves the track into another folder, that model
		// will receieve the OnTrackAdded callback and reparent it again
		OwnerModel->GetChildList(EViewModelListType::Outliner)
			.AddChild(TrackModel);
	}
}

void FFolderModel::OnObjectBindingAdded(const FGuid& ObjectBinding)
{
	TSharedPtr<FFolderModel> This = SharedThis(this);

	UMovieScene*              MovieScene = OwnerModel->GetMovieScene();
	const FMovieSceneBinding* Binding    = MovieScene ? MovieScene->FindBinding(ObjectBinding) : nullptr;
	if (ensure(Binding))
	{
		FObjectBindingModelStorageExtension* ObjectBindingStorage = OwnerModel->CastDynamic<FObjectBindingModelStorageExtension>();

		TSharedPtr<FViewModel> ObjectModel = ObjectBindingStorage->GetOrCreateModelForBinding(*Binding);

		GetChildList(EViewModelListType::Outliner)
			.AddChild(ObjectModel);
	}
}

void FFolderModel::OnObjectBindingRemoved(const FGuid& ObjectBinding)
{
	TSharedPtr<FObjectBindingModel> ObjectModel = GetChildrenOfType<FObjectBindingModel>().FindBy(ObjectBinding, &FObjectBindingModel::GetObjectGuid);
	if (ObjectModel)
	{
		// When tracks are removed we put them back to the root.
		// If the operation moves the track into another folder, that model
		// will receieve the OnTrackAdded callback and reparent it again
		OwnerModel->GetChildList(EViewModelListType::Outliner)
			.AddChild(ObjectModel);
	}
}

void FFolderModel::OnChildFolderAdded(UMovieSceneFolder* Folder)
{
	TSharedPtr<FFolderModel> This = SharedThis(this);

	FFolderModelStorageExtension* FolderStorage = OwnerModel->CastDynamic<FFolderModelStorageExtension>();
	TSharedPtr<FFolderModel>      ChildModel    = FolderStorage->CreateModelForFolder(Folder, This);

	GetChildList(EViewModelListType::Outliner)
		.AddChild(ChildModel);
}

void FFolderModel::OnChildFolderRemoved(UMovieSceneFolder* Folder)
{
	TSharedPtr<FFolderModel> FolderModel = GetChildrenOfType<FFolderModel>().FindBy(Folder, &FFolderModel::GetFolder);
	if (FolderModel)
	{
		// When folders are removed we remove them entirely - 
		// If they get added to the root set, FFolderModelStorageExtension will pick up that operation
		FolderModel->RemoveFromParent();
	}
}

FOutlinerSizing FFolderModel::GetOutlinerSizing() const
{
	const float CompactHeight = 28.f;
	const TSharedPtr<FSequencerEditorViewModel> Editor = GetEditor();
	FViewDensityInfo Density = Editor ? Editor->GetViewDensity() : FViewDensityInfo();
	return FOutlinerSizing(Density.UniformHeight.Get(CompactHeight));
}

void FFolderModel::GetIdentifierForGrouping(TStringBuilder<128>& OutString) const
{
	FOutlinerItemModel::GetIdentifier().ToString(OutString);
}

FTrackAreaParameters FFolderModel::GetTrackAreaParameters() const
{
	FTrackAreaParameters Params;
	Params.LaneType = ETrackAreaLaneType::Inline;
	return Params;
}

FViewModelVariantIterator FFolderModel::GetTrackAreaModelList() const
{
	return &TrackAreaList;
}

bool FFolderModel::CanRename() const
{
	return true;
}

void FFolderModel::Rename(const FText& NewName)
{
	UMovieSceneFolder* FolderObject = WeakFolder.Get();
	if (!FolderObject)
	{
		return;
	}

	FName DesiredName(*NewName.ToString());
	if (FolderObject->GetFolderName() != FName(*NewName.ToString()))
	{
		TArray<FName> SiblingNames;

		TSharedPtr<FViewModel> Parent = GetParent();
		if (ensure(Parent))
		{
			for (TSharedPtr<FFolderModel> Sibling : Parent->GetChildrenOfType<FFolderModel>())
			{
				UMovieSceneFolder* SiblingFolder = Sibling->GetFolder();
				if (SiblingFolder != FolderObject)
				{
					SiblingNames.Add(SiblingFolder->GetFolderName());
				}
			}
		}

		FName UniqueName = FSequencerUtilities::GetUniqueName(DesiredName, SiblingNames);

		const FScopedTransaction Transaction( NSLOCTEXT( "SequencerFolderNode", "RenameFolder", "Rename folder." ) );
		FolderObject->SetFolderName( UniqueName );

		SetIdentifier(UniqueName);
	}
}

bool FFolderModel::CanDrag() const
{
	return true;
}

void FFolderModel::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<FSequencer> Sequencer = StaticCastSharedPtr<FSequencer>(GetEditor()->GetSequencer());
	const bool bIsReadOnly = Sequencer->GetRootMovieSceneSequence()->GetMovieScene()->IsReadOnly();
	
	MenuBuilder.BeginSection("ObjectBindings"); // Create the section to allow extension
	if (!bIsReadOnly)
	{
		Sequencer->BuildAddObjectBindingsMenu(MenuBuilder);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("AddTracks"); // Create the section to allow extension
	if (!bIsReadOnly)
	{
		Sequencer->BuildAddTrackMenu(MenuBuilder);
	}
	MenuBuilder.EndSection();

	FOutlinerItemModel::BuildContextMenu(MenuBuilder);

	MenuBuilder.BeginSection("Folder", LOCTEXT("FolderContextMenuSectionName", "Folder"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("SetColor", "Set Color"),
			LOCTEXT("SetColorTooltip", "Set the color for the selected folders"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FFolderModel::SetFolderColor))
		);
	}
	MenuBuilder.EndSection();
}

void FFolderModel::BuildSidebarMenu(FMenuBuilder& MenuBuilder)
{
	FOutlinerItemModel::BuildSidebarMenu(MenuBuilder);

	MenuBuilder.BeginSection(TEXT("Folder"), LOCTEXT("FolderContextMenuSectionName", "Folder"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("SetColor", "Set Color"),
			LOCTEXT("SetColorTooltip", "Set the color for the selected folders"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FFolderModel::SetFolderColor))
		);
	}
	MenuBuilder.EndSection();
}

TSharedPtr<SWidget> FFolderModel::CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName)
{
	class SOutlinerFolderView
		: public SOutlinerItemViewBase
	{
	public:
		void Construct(const FArguments& InArgs, TWeakPtr<FFolderModel> InWeakModel, TWeakPtr<FEditorViewModel> InWeakEditor, const TSharedRef<ISequencerTreeViewRow>& InTableRow)
		{
			IsExpandedAttribute = MakeAttributeSP(&InTableRow.Get(), &ISequencerTreeViewRow::IsItemExpanded);

			SOutlinerItemViewBase::Construct(InArgs, TWeakViewModelPtr<IOutlinerExtension>(InWeakModel), InWeakEditor, InTableRow);
		}

		const FSlateBrush* GetIconBrush() const
		{
			return IsExpandedAttribute.Get()
				? FAppStyle::GetBrush( "ContentBrowser.AssetTreeFolderOpen" )
				: FAppStyle::GetBrush( "ContentBrowser.AssetTreeFolderClosed" );
		}
		
		TAttribute<bool> IsExpandedAttribute;
	};

	if (InColumnName == FCommonOutlinerNames::Label)
	{
		return SNew(SOutlinerFolderView, SharedThis(this), InParams.Editor, InParams.TreeViewRow);
	}

	return nullptr;
}

void FFolderModel::SetFolderColor()
{
	bFolderPickerWasCancelled = false;
	InitialFolderColors.Empty();

	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();

	TArray<UMovieSceneFolder*> MovieSceneFolders;
	Sequencer->GetSelectedFolders(MovieSceneFolders);

	if (MovieSceneFolders.Num() == 0)
	{
		return;
	}

	for (UMovieSceneFolder* MovieSceneFolder : MovieSceneFolders)
	{
		InitialFolderColors.Add(MovieSceneFolder, MovieSceneFolder->GetFolderColor());
	}

	FColorPickerArgs PickerArgs;
	PickerArgs.bUseAlpha = false;
	PickerArgs.DisplayGamma = TAttribute<float>::Create( TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma) );
	PickerArgs.InitialColor = (*InitialFolderColors.CreateIterator()).Value.ReinterpretAsLinear();
	PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &FFolderModel::OnColorPickerPicked);
	PickerArgs.OnColorPickerWindowClosed = FOnWindowClosed::CreateSP(this, &FFolderModel::OnColorPickerClosed);
	PickerArgs.OnColorPickerCancelled  = FOnColorPickerCancelled::CreateSP(this, &FFolderModel::OnColorPickerCancelled );

	OpenColorPicker(PickerArgs);
}

void FFolderModel::OnColorPickerPicked(FLinearColor NewFolderColor)
{
	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();

	TArray<UMovieSceneFolder*> MovieSceneFolders;
	Sequencer->GetSelectedFolders(MovieSceneFolders);

	for (UMovieSceneFolder* MovieSceneFolder : MovieSceneFolders)
	{
		MovieSceneFolder->SetFolderColor(NewFolderColor.ToFColor(false));
	}
}

void FFolderModel::OnColorPickerClosed(const TSharedRef<SWindow>& Window)
{
	if (!bFolderPickerWasCancelled)
	{
		TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
		TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();

		TArray<UMovieSceneFolder*> MovieSceneFolders;
		Sequencer->GetSelectedFolders(MovieSceneFolders);

		const FScopedTransaction Transaction(NSLOCTEXT("SequencerFolderNode", "SetFolderColor", "Set Folder Color"));

		for (UMovieSceneFolder* MovieSceneFolder : MovieSceneFolders)
		{
			FColor CurrentColor = MovieSceneFolder->GetFolderColor();
			FColor InitialFolderColor = InitialFolderColors.Contains(MovieSceneFolder) ? InitialFolderColors[MovieSceneFolder] : FColor();
			MovieSceneFolder->SetFolderColor(InitialFolderColor);
			MovieSceneFolder->Modify();
			MovieSceneFolder->SetFolderColor(CurrentColor);
		}
	}
	InitialFolderColors.Empty();
}

void FFolderModel::OnColorPickerCancelled(FLinearColor NewFolderColor)
{
	bFolderPickerWasCancelled = true;

	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();

	TArray<UMovieSceneFolder*> MovieSceneFolders;
	Sequencer->GetSelectedFolders(MovieSceneFolders);

	for (UMovieSceneFolder* MovieSceneFolder : MovieSceneFolders)
	{
		if (InitialFolderColors.Contains(MovieSceneFolder))
		{
			MovieSceneFolder->SetFolderColor(InitialFolderColors[MovieSceneFolder]);
		}
	}

	InitialFolderColors.Empty();
}

FText FFolderModel::GetLabel() const
{
	UMovieSceneFolder* Folder = GetFolder();
	return Folder ? FText::FromName(Folder->GetFolderName()) : LOCTEXT("ExpiredText", "<<EXPIRED>>");
}

const FSlateBrush* FFolderModel::GetIconBrush() const
{
	return FAppStyle::GetBrush( "ContentBrowser.AssetTreeFolderClosed" );
}

FSlateColor FFolderModel::GetIconTint() const
{
	UMovieSceneFolder* Folder = GetFolder();

	if (!Folder)
	{
		return FLinearColor::White;
	}

	if (IsDimmed())
	{
		return Folder->GetFolderColor().ReinterpretAsLinear().Desaturate(0.6f);
	}

	return Folder->GetFolderColor();
}

FSlateColor FFolderModel::GetLabelColor() const
{
	if (IsDimmed())
	{
		return FSlateColor::UseSubduedForeground();
	}

	return FSlateColor::UseForeground();
}

void FFolderModel::SortChildren()
{
	ISortableExtension::SortChildren(SharedThis(this), ESortingMode::Default);
}

FSortingKey FFolderModel::GetSortingKey() const
{
	FSortingKey SortingKey;
	if (UMovieSceneFolder* Folder = GetFolder())
	{
		SortingKey.DisplayName = FText::FromName(Folder->GetFolderName());
		SortingKey.CustomOrder = Folder->GetSortingOrder();
	}
	return SortingKey.PrioritizeBy(3);
}

void FFolderModel::SetCustomOrder(int32 InCustomOrder)
{
	if (UMovieSceneFolder* Folder = GetFolder())
	{
		Folder->SetSortingOrder(InCustomOrder);
	}
}

TOptional<EItemDropZone> FFolderModel::CanAcceptDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone)
{
	TSharedPtr<FActorDragDropOp> ActorDragDropOp = DragDropEvent.GetOperationAs<FActorDragDropOp>();
	if (ActorDragDropOp)
	{
		return ItemDropZone;
	}

	TSharedPtr<FSequencerOutlinerDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSequencerOutlinerDragDropOp>();
	if (!DragDropOp)
	{
		return TOptional<EItemDropZone>();
	}

	DragDropOp->ResetToDefaultToolTip();

	// Prevent taking any parent that's part of the dragged node hierarchy from being put inside a child of itself
	// This is done first before the other checks so that the UI stays consistent as you move between them, otherwise
	// when you are above/below a node it reports this error, but if you were on top of a node it would do the standard
	// no-drag-drop due to OntoItem being blocked.
	if (!DragDropOp->ValidateParentChildDrop(*this))
	{
		return TOptional<EItemDropZone>();
	}

	for (const TWeakViewModelPtr<IOutlinerExtension>& WeakModel : DragDropOp->GetDraggedViewModels())
	{
		FViewModelPtr Model = WeakModel.Pin();
		if (!Model)
		{
			// Silently allow null models
			continue;
		}
		if (Model.Get() == this || Model == TargetModel)
		{
			// Cannot drop onto self
			return TOptional<EItemDropZone>();
		}

		if (Model->IsA<FFolderModel>())
		{
			// We can always drop folders into other folders
		}
		else if (TViewModelPtr<IObjectBindingExtension> Object = Model.ImplicitCast())
		{
			if (TViewModelPtr<IObjectBindingExtension> ParentObject = Model->CastParent<IObjectBindingExtension>())
			{
				// This operation should not have been allowed in the first place (child objects should never be draggable)
				return TOptional<EItemDropZone>();
			}
		}
		else if (TViewModelPtr<FTrackModel> Track = Model.ImplicitCast())
		{
			if (TViewModelPtr<IObjectBindingExtension> ParentObject = Model->CastParent<IObjectBindingExtension>())
			{
				// This operation should not have been allowed in the first place (tracks bound to an object should never be draggable)
				return TOptional<EItemDropZone>();
			}
		}
		else
		{
			// Unknown model type - do we silent not drag these or return an error?
			// For now we disallow the drag, but maybe that's not the right choice
			return TOptional<EItemDropZone>();
		}
	}

	// The dragged nodes were either all in folders, or all at the sequencer root.
	return ItemDropZone;
}

void FFolderModel::PerformDropActors(const FViewModelPtr& TargetModel, TSharedPtr<FActorDragDropOp> ActorDragDropEvent, TSharedPtr<FViewModel> AttachAfter)
{
	UMovieSceneFolder* Folder = WeakFolder.Get();
	if (!Folder)
	{
		return;
	}
	
	TSharedPtr<FSequencer> Sequencer = StaticCastSharedPtr<FSequencer>(GetEditor()->GetSequencer());
	if (!Sequencer)
	{
		return;
	}

	TArray<FGuid> PossessableGuids = FSequencerUtilities::AddActors(Sequencer.ToSharedRef(), ActorDragDropEvent->Actors);

	for (FGuid PossessableGuid : PossessableGuids)
	{
		Folder->AddChildObjectBinding(PossessableGuid);
	}
}

void FFolderModel::PerformDropOutliner(const FViewModelPtr& TargetModel, TSharedPtr<FSequencerOutlinerDragDropOp> OutlinerDragDropEvent, TSharedPtr<FViewModel> AttachAfter)
{
	UMovieSceneFolder* Folder = WeakFolder.Get();
	if (!Folder)
	{
		return;
	}

	// Drop handing for outliner drag/drop operations.
	// Warning: this handler may be dragging nodes from a *different* sequence
	FViewModelChildren OutlinerChildren = GetChildList(EViewModelListType::Outliner);

	UMovieScene* MovieScene = Folder->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		MovieScene->SetFlags(RF_Transactional);
		MovieScene->Modify();
	}

	for (const TWeakViewModelPtr<IOutlinerExtension>& WeakModel : OutlinerDragDropEvent->GetDraggedViewModels())
	{
		FViewModelPtr DraggedModel = WeakModel.Pin();
		if (!DraggedModel || DraggedModel == AttachAfter)
		{
			continue;
		}

		TViewModelPtr<FFolderModel> ExistingParentFolder = DraggedModel->CastParent<FFolderModel>();
		UMovieSceneFolder* OldFolder = ExistingParentFolder ? ExistingParentFolder->GetFolder() : nullptr;

		bool bSuccess = false;

		// Handle dropoping a folder into another folder
		// @todo: if we ever support folders within object bindings this will need to have better validation
		if (TSharedPtr<FFolderModel> DraggedFolderModel = DraggedModel.ImplicitCast())
		{
			if (UMovieSceneFolder* DraggedFolder = DraggedFolderModel->GetFolder())
			{
				if (OldFolder)
				{
					OldFolder->SetFlags(RF_Transactional);
					OldFolder->Modify();
					OldFolder->RemoveChildFolder(DraggedFolder);
				}
				else if (MovieScene)
				{
					MovieScene->RemoveRootFolder(DraggedFolder);
				}

				// Give this folder a unique name inside its new parent if necessary
				FName FolderName = Folder->MakeUniqueChildFolderName(DraggedFolder->GetFolderName());

				if (FolderName != DraggedFolder->GetFolderName())
				{
					DraggedFolder->SetFlags(RF_Transactional);
					DraggedFolder->Modify();
					DraggedFolder->SetFolderName(FolderName);
				}

				Folder->AddChildFolder(DraggedFolder);
				bSuccess = true;
			}
		}
		else if (TSharedPtr<IObjectBindingExtension> ObjectBinding = DraggedModel.ImplicitCast())
		{
			TViewModelPtr<IObjectBindingExtension> ParentObject = DraggedModel->CastParent<IObjectBindingExtension>();
			// Don't allow dropping an object binding if it has an object parent
			if (ParentObject == nullptr)
			{
				if (OldFolder)
				{
					OldFolder->SetFlags(RF_Transactional);
					OldFolder->Modify();
					OldFolder->RemoveChildObjectBinding(ObjectBinding->GetObjectGuid());
				}

				Folder->AddChildObjectBinding(ObjectBinding->GetObjectGuid());
				bSuccess = true;
			}
		}
		else if (TSharedPtr<FTrackModel> TrackModel = DraggedModel.ImplicitCast())
		{
			UMovieSceneTrack* Track = TrackModel->GetTrack();
			TViewModelPtr<IObjectBindingExtension> ParentObject = DraggedModel->CastParent<IObjectBindingExtension>();
			// Don't allow dropping a track if it has an object parent
			if (ParentObject == nullptr && Track != nullptr)
			{
				if (OldFolder)
				{
					OldFolder->SetFlags(RF_Transactional);
					OldFolder->Modify();
					OldFolder->RemoveChildTrack(Track);
				}

				Folder->AddChildTrack(Track);
				bSuccess = true;
			}
		}

		if (bSuccess)
		{
			// Attach it to the right node, possibly setting its parent too
			OutlinerChildren.InsertChild(DraggedModel, AttachAfter);
		}
	}
}

void FFolderModel::PerformDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone)
{
	FViewModelHierarchyOperation ScopedOperation(GetSharedData());

	UMovieSceneFolder* Folder = WeakFolder.Get();
	if (!Folder)
	{
		return;
	}

	const FScopedTransaction Transaction(FText::Format(LOCTEXT("MoveItems", "Move into {0}"), FText::FromName(Folder->GetFolderName())));

	Folder->SetFlags(RF_Transactional);
	Folder->Modify();

	TSharedPtr<FViewModel> AttachAfter;
	if (TargetModel)
	{
		if (InItemDropZone == EItemDropZone::AboveItem)
		{
			AttachAfter = TargetModel->GetPreviousSibling();
		}
		else if (InItemDropZone == EItemDropZone::BelowItem)
		{
			AttachAfter = TargetModel;
		}
	}

	// Open this folder if an item was dropped into the folder
	if (InItemDropZone == EItemDropZone::OntoItem && !TargetModel)
	{
		SetExpansion(true);
	}

	TSharedPtr<FActorDragDropOp> ActorDragDropOp = DragDropEvent.GetOperationAs<FActorDragDropOp>();
	if (ActorDragDropOp)
	{
		PerformDropActors(TargetModel, ActorDragDropOp, AttachAfter);
	}

	TSharedPtr<FSequencerOutlinerDragDropOp> OutlinerDragDropOp = DragDropEvent.GetOperationAs<FSequencerOutlinerDragDropOp>();
	if (OutlinerDragDropOp)
	{
		PerformDropOutliner(TargetModel, OutlinerDragDropOp, AttachAfter);
	}

	// Forcibly update sorting
	int32 CustomSort = 0;
	for (const TViewModelPtr<ISortableExtension>& Sortable : GetChildrenOfType<ISortableExtension>())
	{
		Sortable->SetCustomOrder(CustomSort);
		++CustomSort;
	}
}

bool FFolderModel::CanDelete(FText* OutErrorMessage) const
{
	for (const TViewModelPtr<IDeletableExtension>& Child : GetChildrenOfType<IDeletableExtension>(EViewModelListType::Outliner))
	{
		if (!Child->CanDelete(OutErrorMessage))
		{
			return false;
		}
	}
	return true;
}

void FFolderModel::Delete()
{
	check(OwnerModel);

	// When we delete a folder, we also delete all the items inside it. Some of
	// these items (like object bindings) remove themselves from any parent folder,
	// which is us in this case, and then remove themselves entirely from the
	// movie scene. Since these operations call events (such as IFolderEventHandler
	// events, which we implement), it messes up the list of children that we want
	// to iterate over here. We therefore have to cache that list before using it.
	TArray<TViewModelPtr<IDeletableExtension>> DeletableChildren(
			GetChildrenOfType<IDeletableExtension>(EViewModelListType::Outliner).ToArray());
	for (const TViewModelPtr<IDeletableExtension>& Child : DeletableChildren)
	{
		Child->Delete();
	}

	UMovieScene* MovieScene = OwnerModel->GetMovieScene();

	// Remove from a parent folder if necessary.
	if (TViewModelPtr<FFolderModel> ParentFolder = CastParent<FFolderModel>())
	{
		ParentFolder->GetFolder()->Modify();
		ParentFolder->GetFolder()->RemoveChildFolder(GetFolder());
	}
	else
	{
		MovieScene->Modify();
		MovieScene->RemoveRootFolder(GetFolder());
	}
}

bool FFolderModel::IsMuted() const
{
	UMovieSceneFolder* Folder = WeakFolder.Get();
	if (!Folder)
	{
		return false;
	}

	if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = Folder->FindDecoration<UMovieSceneMuteSoloDecoration>())
	{
		return MuteSoloDecoration->IsMuted();
	}

	return false;
}

void FFolderModel::SetIsMuted(bool bIsMuted)
{
	UMovieSceneFolder* Folder = WeakFolder.Get();
	if (!Folder)
	{
		return;
	}

	const bool bAlwaysMarkDirty = false;
	Folder->Modify(bAlwaysMarkDirty);

	if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = Folder->GetOrCreateDecoration<UMovieSceneMuteSoloDecoration>())
	{
		MuteSoloDecoration->SetMuted(bIsMuted);
	}
}

bool FFolderModel::IsSolo() const
{
	UMovieSceneFolder* Folder = WeakFolder.Get();
	if (!Folder)
	{
		return false;
	}

	if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = Folder->FindDecoration<UMovieSceneMuteSoloDecoration>())
	{
		return MuteSoloDecoration->IsSoloed();
	}

	return false; 
}

void FFolderModel::SetIsSoloed(bool bIsSoloed)
{
	UMovieSceneFolder* Folder = WeakFolder.Get();
	if (!Folder)
	{
		return;
	}

	const bool bAlwaysMarkDirty = false;
	Folder->Modify(bAlwaysMarkDirty);

	if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = Folder->GetOrCreateDecoration<UMovieSceneMuteSoloDecoration>())
	{
		MuteSoloDecoration->SetSoloed(bIsSoloed);
	}
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
