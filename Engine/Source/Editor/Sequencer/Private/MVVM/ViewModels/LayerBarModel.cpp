// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/LayerBarModel.h"

#include "Containers/ArrayView.h"
#include "HAL/Platform.h"
#include "MVVM/Extensions/ILayerBarExtension.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/Views/STrackLane.h"
#include "Math/RangeBound.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "MovieSceneTimeHelpers.h"
#include "Templates/TypeHash.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SSequencerLayerBar.h"

namespace UE
{
namespace Sequencer
{

FLayerBarModel::FLayerBarModel(TWeakPtr<FViewModel> LayerRoot)
	: TViewModelExtensionCollection<ILayerBarExtension>(LayerRoot)
{
}

FLayerBarModel::~FLayerBarModel()
{
}

void FLayerBarModel::OnConstruct()
{
	TViewModelExtensionCollection<ILayerBarExtension>::Initialize();
}

void FLayerBarModel::OnDestruct()
{
	TViewModelExtensionCollection<ILayerBarExtension>::Destroy();
}

TRange<FFrameNumber> FLayerBarModel::ComputeRange() const
{
	using namespace UE::MovieScene;

	TRange<FFrameNumber> Range = TRange<FFrameNumber>::Empty();

	for (ILayerBarExtension* Extension : GetExtensions())
	{
		TRange<FFrameNumber> ThisRange = Extension->GetLayerBarRange();
		Range = TRange<FFrameNumber>::Hull(ThisRange, Range);
	}

	if (!ensure(Range.GetLowerBound().IsClosed() && Range.GetUpperBound().IsClosed()))
	{
		Range = TRange<FFrameNumber>::Empty();
	}
	else if (DiscreteSize(Range) <= 1)
	{
		Range = TRange<FFrameNumber>::Empty();
	}


	if (!Range.IsEmpty() && !Range.GetUpperBound().IsOpen())
	{
		Range.SetUpperBound(TRangeBound<FFrameNumber>::Exclusive(Range.GetUpperBoundValue()));
	}

	return Range;
}

void FLayerBarModel::OnExtensionsDirtied()
{
}

void FLayerBarModel::SetOverlayText(TAttribute<FText> InText)
{
	OverlayText = MoveTemp(InText);
}

const TAttribute<FText>& FLayerBarModel::GetOverlayText() const
{
	return OverlayText;
}

TSharedPtr<ITrackLaneWidget> FLayerBarModel::CreateTrackLaneView(const FCreateTrackLaneViewParams& InParams)
{
	return SNew(SSequencerLayerBar, InParams.OwningTrackLane->GetTrackAreaView(), InParams.Editor->CastThisSharedChecked<FSequencerEditorViewModel>(), SharedThis(this))
		.OverlayText(OverlayText);
}

FTrackLaneVirtualAlignment FLayerBarModel::ArrangeVirtualTrackLaneView() const
{
	TRange<FFrameNumber> Range = ComputeRange();

	FTrackLaneVirtualAlignment Result = FTrackLaneVirtualAlignment::Fixed(Range, 10.f, VAlign_Top);

	TViewModelPtr<IOutlinerExtension>  OutlinerItem = GetLinkedOutlinerItem();
	TViewModelPtr<ITrackAreaExtension> TrackArea = OutlinerItem.ImplicitCast();

	if (TrackArea && OutlinerItem)
	{
		FTrackAreaParameters Parameters = TrackArea->GetTrackAreaParameters();

		Result.VerticalAlignment.VSizeParam = OutlinerItem->GetOutlinerSizing().GetTotalHeight();
		Result.VerticalAlignment.VPadding   = Parameters.TrackLanePadding.Top + Parameters.TrackLanePadding.Bottom;
	}

	return Result;
}

ESelectionIntent FLayerBarModel::IsSelectable() const
{
	// Layer bars do not support context menus at the moment
	return ESelectionIntent::PersistentSelection;
}

void FLayerBarModel::Offset(FFrameNumber Offset)
{
	for (ILayerBarExtension* Extension : GetExtensions())
	{
		Extension->OffsetLayerBar(Offset);
	}
}

void FLayerBarModel::AddToSnapField(const ISnapCandidate& Candidate, ISnapField& SnapField) const
{
	using namespace UE::MovieScene;

	TRange<FFrameNumber> Range = ComputeRange();
	SnapField.AddSnapPoint(FSnapPoint{ FSnapPoint::CustomSection, DiscreteInclusiveLower(Range), 1.f });
	SnapField.AddSnapPoint(FSnapPoint{ FSnapPoint::CustomSection, DiscreteExclusiveUpper(Range), 1.f });
}

bool FLayerBarModel::CanDrag() const
{
	return true;
}

void FLayerBarModel::OnBeginDrag(IDragOperation& DragOperation)
{
	using namespace UE::MovieScene;

	if (TSharedPtr<FViewModel> ObservedModel = GetObservedModel())
	{
		TParentFirstChildIterator<IDraggableTrackAreaExtension> It(ObservedModel);
		for (TSharedPtr<IDraggableTrackAreaExtension> Draggable : It)
		{
			if (Draggable.Get() != this)
			{
				Draggable->OnBeginDrag(DragOperation);
			}
		}

		TRange<FFrameNumber> Range = ComputeRange();
		DragOperation.AddSnapTime(DiscreteInclusiveLower(Range));
		DragOperation.AddSnapTime(DiscreteExclusiveUpper(Range));
	}
}

void FLayerBarModel::OnEndDrag(IDragOperation& DragOperation)
{
}

void FLayerBarModel::OnInitiateStretch(IStretchOperation& StretchOperation, EStretchConstraint Constraint, FStretchParameters* InOutGlobalParameters)
{
	TSharedPtr<FViewModel> ObservedModel = GetObservedModel();
	TRange<FFrameNumber> Range = ComputeRange();
	if (ObservedModel && !Range.IsEmpty())
	{
		FStretchParameters StrechParams;

		if (Constraint == EStretchConstraint::AnchorToStart)
		{
			StrechParams.Anchor = Range.GetLowerBoundValue().Value;
			StrechParams.Handle = Range.GetUpperBoundValue().Value;

			if (InOutGlobalParameters->Anchor > StrechParams.Anchor)
			{
				InOutGlobalParameters->Anchor = StrechParams.Anchor;
			}
		}
		else
		{
			StrechParams.Anchor = Range.GetUpperBoundValue().Value;
			StrechParams.Handle = Range.GetLowerBoundValue().Value;

			if (InOutGlobalParameters->Anchor < StrechParams.Anchor)
			{
				InOutGlobalParameters->Anchor = StrechParams.Anchor;
			}
		}

		TSharedPtr<FLayerBarModel> This = SharedThis(this);

		// The priority of this layer bar's stretch parameters is its depth within the tree.
		// This means that any child layer bars that are also explicitly part of the stretch
		// will override this layer bar's stretch
		const int32 Priority = GetHierarchicalDepth();

		TParentFirstChildIterator<IStretchableExtension> It(ObservedModel);
		for (TSharedPtr<IStretchableExtension> Target : It)
		{
			if (Target != This)
			{
				StretchOperation.InitiateStretch(This, Target, Priority, StrechParams);
			}
		}
	}
}

} // namespace Sequencer
} // namespace UE

