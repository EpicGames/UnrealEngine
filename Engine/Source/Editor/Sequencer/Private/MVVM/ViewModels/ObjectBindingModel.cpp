// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/FolderModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/LayerBarModel.h"
#include "MVVM/ViewModels/BindingLifetimeTrackModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/TrackModelStorageExtension.h"
#include "MVVM/ObjectBindingModelStorageExtension.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/OutlinerViewModelDragDropOp.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "MVVM/Views/SOutlinerObjectBindingView.h"
#include "MVVM/Views/STrackLane.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/Extensions/IRecyclableExtension.h"
#include "MVVM/Extensions/IBindingLifetimeExtension.h"
#include "ISequencerObjectSchema.h"
#include "Algo/Sort.h"
#include "AnimatedRange.h"
#include "ClassViewerModule.h"
#include "Containers/ArrayBuilder.h"
#include "Engine/LevelStreaming.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailCustomization.h"
#include "IDetailsView.h"
#include "ISequencerModule.h"
#include "ISequencerTrackEditor.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneBinding.h"
#include "MovieSceneDynamicBindingCustomization.h"
#include "UniversalObjectLocator.h"
#include "MovieSceneFolder.h"
#include "ObjectBindingTagCache.h"
#include "ObjectEditorUtils.h"
#include "PropertyEditorModule.h"
#include "PropertyPath.h"
#include "SObjectBindingTag.h"
#include "ScopedTransaction.h"
#include "Sequencer.h"
#include "SequencerCommands.h"
#include "SequencerNodeTree.h"
#include "SequencerSettings.h"
#include "StructUtils/PropertyBag.h"
#include "MVVM/Views/ViewUtilities.h"
#include "Misc/SequencerObjectBindingHelper.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/SSequencerBindingLifetimeOverlay.h"
#include "Decorations/MovieSceneMuteSoloDecoration.h"

#define LOCTEXT_NAMESPACE "ObjectBindingModel"

namespace UE::Sequencer
{

bool GSequencerObjectBindingShowNestedProperties = false;
FAutoConsoleVariableRef CVarSequencerObjectBindingShowNestedProperties(
	TEXT("Sequencer.ObjectBinding.ShowNestedProperties"),
	GSequencerObjectBindingShowNestedProperties,
	TEXT("(Default: false) When enabled, always show bound object properties as sub-menus reflecting the hierarchy of nested structures. When disabled, only do that for Level Sequences, make others use flat menus."),
	ECVF_Default
);

FObjectBindingModel::FObjectBindingModel(FSequenceModel* InOwnerModel, const FMovieSceneBinding& InBinding)
	: ObjectBindingID(InBinding.GetObjectGuid())
	, TrackAreaList(EViewModelListType::TrackArea)
	, TopLevelChildTrackAreaList(GetTopLevelChildTrackAreaGroupType())
	, OwnerModel(InOwnerModel)
{
	RegisterChildList(&TrackAreaList);
	RegisterChildList(&TopLevelChildTrackAreaList);

	SetIdentifier(*ObjectBindingID.ToString());
}

FObjectBindingModel::~FObjectBindingModel()
{
}

EViewModelListType FObjectBindingModel::GetTopLevelChildTrackAreaGroupType()
{
	static EViewModelListType TopLevelChildTrackAreaGroup = RegisterCustomModelListType();
	return TopLevelChildTrackAreaGroup;
}

void FObjectBindingModel::OnConstruct()
{
	if (!LayerBar)
	{
		TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
		TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();

		if (Sequencer->GetSequencerSettings()->GetShowLayerBars())
		{
			LayerBar = MakeShared<FLayerBarModel>(AsShared());
			LayerBar->SetLinkedOutlinerItem(SharedThis(this));

			GetChildrenForList(&TopLevelChildTrackAreaList).AddChild(LayerBar);
		}
	}

	UMovieScene* MovieScene = OwnerModel->GetMovieScene();
	check(MovieScene);

	FMovieSceneBinding* Binding = MovieScene->FindBinding(ObjectBindingID);
	check(Binding);

	FScopedViewModelListHead RecycledHead(AsShared(), EViewModelListType::Recycled);
	GetChildrenForList(&OutlinerChildList).MoveChildrenTo<IRecyclableExtension>(RecycledHead.GetChildren(), IRecyclableExtension::CallOnRecycle);

	for (UMovieSceneTrack* Track : Binding->GetTracks())
	{
		// Don't show child tracks here- they'll be shown underneath their proper outliner parent (e.g. anim mixer)
		if (Track && Track->ParentTrack != nullptr)
		{
			continue;
		}
		AddTrack(Track);
	}
}

void FObjectBindingModel::SetParentBindingID(const FGuid& InObjectBindingID)
{
	ParentObjectBindingID = InObjectBindingID;
}

FGuid FObjectBindingModel::GetDesiredParentBinding() const
{
	return ParentObjectBindingID;
}

EObjectBindingType FObjectBindingModel::GetType() const
{
	return EObjectBindingType::Unknown;
}

const UClass* FObjectBindingModel::FindObjectClass() const
{
	return UObject::StaticClass();
}

bool FObjectBindingModel::SupportsRebinding() const
{
	return true;
}

FTrackAreaParameters FObjectBindingModel::GetTrackAreaParameters() const
{
	FTrackAreaParameters Params;
	Params.LaneType = ETrackAreaLaneType::Nested;
	return Params;
}

FViewModelVariantIterator FObjectBindingModel::GetTrackAreaModelList() const
{
	return &TrackAreaList;
}

FViewModelVariantIterator FObjectBindingModel::GetTopLevelChildTrackAreaModels() const
{
	return &TopLevelChildTrackAreaList;
}

void FObjectBindingModel::AddTrack(UMovieSceneTrack* Track)
{
	FTrackModelStorageExtension* TrackStorage = OwnerModel->CastDynamic<FTrackModelStorageExtension>();

	TViewModelPtr<FTrackModel> TrackModel = TrackStorage->CreateModelForTrack(Track, AsShared());

	GetChildrenForList(&OutlinerChildList).AddChild(TrackModel);

	if (TrackModel->IsA<IBindingLifetimeExtension>())
	{
		if (!BindingLifetimeOverlayModel)
		{
			BindingLifetimeOverlayModel = MakeShared<FBindingLifetimeOverlayModel>(AsShared(), GetEditor(), TrackModel.ImplicitCast());
			BindingLifetimeOverlayModel->SetLinkedOutlinerItem(SharedThis(this));
			GetChildrenForList(&TrackAreaList).AddChild(BindingLifetimeOverlayModel);
		}
	}
}

void FObjectBindingModel::RemoveTrack(UMovieSceneTrack* Track)
{
	FTrackModelStorageExtension* TrackStorage = OwnerModel->CastDynamic<FTrackModelStorageExtension>();

	TSharedPtr<FTrackModel> TrackModel = GetChildrenOfType<FTrackModel>().FindBy(Track, &FTrackModel::GetTrack);
	if (TrackModel)
	{
		TrackModel->RemoveFromParent();
		if (TrackModel->IsA<IBindingLifetimeExtension>())
		{
			if (BindingLifetimeOverlayModel)
			{
				BindingLifetimeOverlayModel->RemoveFromParent();
				BindingLifetimeOverlayModel.Reset();
			}
		}
	}
}

FGuid FObjectBindingModel::GetObjectGuid() const
{
	return ObjectBindingID;
}

FOutlinerSizing FObjectBindingModel::GetOutlinerSizing() const
{
	const float CompactHeight = 28.f;
	const TSharedPtr<FSequencerEditorViewModel> Editor = GetEditor();
	FViewDensityInfo Density = Editor ? Editor->GetViewDensity() : FViewDensityInfo();
	return FOutlinerSizing(Density.UniformHeight.Get(CompactHeight));
}

void FObjectBindingModel::GetIdentifierForGrouping(TStringBuilder<128>& OutString) const
{
	FOutlinerItemModel::GetIdentifier().ToString(OutString);
}

TSharedPtr<SWidget> FObjectBindingModel::CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName)
{
	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	TSharedPtr<FSequencer>                Sequencer       = EditorViewModel->GetSequencerImpl();

	if (InColumnName == FCommonOutlinerNames::Label)
	{
		const FMovieSceneSequenceID SequenceID = OwnerModel->GetSequenceID();
		const MovieScene::FFixedObjectBindingID FixedObjectBindingID(ObjectBindingID, SequenceID);

		return SNew(SOutlinerItemViewBase, SharedThis(this), EditorViewModel, InParams.TreeViewRow)
			.AdditionalLabelContent()
			[
				SNew(SObjectBindingTags, FixedObjectBindingID, Sequencer->GetObjectBindingTagCache())
			];
	}

	if (InColumnName == FCommonOutlinerNames::Add)
	{
		return UE::Sequencer::MakeAddButton(
			LOCTEXT("TrackText", "Track"),
			FOnGetContent::CreateSP(this, &FObjectBindingModel::GetAddTrackMenuContent),
			SharedThis(this));
	}


	// Ask track editors to populate the column.
	// @todo: this is potentially very slow and will not scale as the number of track editors increases.
	const bool bIsEditColumn = InColumnName == FCommonOutlinerNames::Edit;
	TSharedPtr<SHorizontalBox> Box;

	auto GetEditBox = [&Box]
	{
		if (!Box)
		{
			Box = SNew(SHorizontalBox);

			auto CollapsedIfAllSlotsCollapsed = [Box]() -> EVisibility
			{
				for (int32 Index = 0; Index < Box->NumSlots(); ++Index)
				{
					EVisibility SlotVisibility = Box->GetSlot(Index).GetWidget()->GetVisibility();
					if (SlotVisibility != EVisibility::Collapsed)
					{
						return EVisibility::SelfHitTestInvisible;
					}
				}
				return EVisibility::Collapsed;
			};

			// Make the edit box collapsed if all of its slots are collapsed (or it has none)
			Box->SetVisibility(MakeAttributeLambda(CollapsedIfAllSlotsCollapsed));
		}
		return Box.ToSharedRef();
	};

	for (const TSharedPtr<ISequencerTrackEditor>& TrackEditor : Sequencer->GetTrackEditors())
	{
		TrackEditor->BuildObjectBindingColumnWidgets(GetEditBox, SharedThis(this), InParams, InColumnName);

		if (bIsEditColumn)
		{
			// Backwards compat
			GetEditBox();
			TrackEditor->BuildObjectBindingEditButtons(Box, ObjectBindingID, FindObjectClass());
		}
	}

	return Box && Box->NumSlots() != 0 ? Box : nullptr;
}

bool FObjectBindingModel::GetDefaultExpansionState() const
{
	// Object binding nodes are always expanded by default
	return true;
}

bool FObjectBindingModel::CanRename() const
{
	return true;
}

void FObjectBindingModel::Rename(const FText& NewName)
{
	TSharedPtr<ISequencer> Sequencer = OwnerModel->GetSequencer();
	UMovieSceneSequence* MovieSceneSequence = OwnerModel->GetSequence();

	if (MovieSceneSequence && Sequencer)
	{
		UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene();

		FScopedTransaction Transaction(LOCTEXT("SetTrackName", "Set Track Name"));

		// Modify the movie scene so that it gets marked dirty and renames are saved consistently.
		MovieScene->Modify();

		FMovieSceneSpawnable*   Spawnable   = MovieScene->FindSpawnable(ObjectBindingID);
		FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectBindingID);

		// If there is only one binding, set the name of the bound actor
		TArrayView<TWeakObjectPtr<>> Objects = Sequencer->FindObjectsInCurrentSequence(ObjectBindingID);
		if (Objects.Num() == 1)
		{
			if (AActor* Actor = Cast<AActor>(Objects[0].Get()))
			{
				Actor->SetActorLabel(NewName.ToString());
			}
		}

		if (Spawnable)
		{
			// Otherwise set our display name
			Spawnable->SetName(NewName.ToString());
		}
		else if (Possessable)
		{
			Possessable->SetName(NewName.ToString());
		}
		else
		{
			MovieScene->SetObjectDisplayName(ObjectBindingID, NewName);
		}
	}
}

FText FObjectBindingModel::GetLabel() const
{
	UMovieSceneSequence* MovieSceneSequence = OwnerModel->GetSequence();
	if (MovieSceneSequence != nullptr)
	{
		return MovieSceneSequence->GetMovieScene()->GetObjectDisplayName(ObjectBindingID);
	}

	return FText();
}

FSlateColor FObjectBindingModel::GetLabelColor() const
{
	TSharedPtr<ISequencer> Sequencer = OwnerModel->GetSequencer();

	if (!Sequencer)
	{
		return FLinearColor::Red;
	}

	TArrayView<TWeakObjectPtr<> > BoundObjects = Sequencer->FindBoundObjects(ObjectBindingID, OwnerModel->GetSequenceID());

	if (BoundObjects.Num() > 0)
	{
		int32 NumValidObjects = 0;
		for (const TWeakObjectPtr<>& BoundObject : BoundObjects)
		{
			if (BoundObject.IsValid())
			{
				++NumValidObjects;
			}
		}

		if (NumValidObjects == BoundObjects.Num())
		{
			return FOutlinerItemModel::GetLabelColor();
		}

		if (NumValidObjects > 0)
		{
			return FLinearColor::Yellow;
		}
	}

	// Find the last objecting binding ancestor and ask it for the invalid color to use.
	// e.g. Spawnables don't have valid object bindings when their track hasn't spawned them yet,
	// so we override the default behavior of red with a gray so that users don't think there is something wrong.
	FMovieSceneEvaluationState* EvaluationState = Sequencer->GetEvaluationState();
	TFunction<FSlateColor(const FObjectBindingModel&)> GetObjectBindingAncestorInvalidLabelColor = [&](const FObjectBindingModel& InObjectBindingModel) -> FSlateColor {
		if (!EvaluationState->GetBindingActivation(InObjectBindingModel.GetObjectGuid(), OwnerModel->GetSequenceID()))
		{
			return FSlateColor::UseSubduedForeground();
		}
		
		if (TSharedPtr<FObjectBindingModel> ParentBindingModel = InObjectBindingModel.FindAncestorOfType<FObjectBindingModel>())
		{
			return GetObjectBindingAncestorInvalidLabelColor(*ParentBindingModel.Get());
		}
		return InObjectBindingModel.GetInvalidBindingLabelColor();
	};

	return GetObjectBindingAncestorInvalidLabelColor(*this);
}

FText FObjectBindingModel::GetTooltipForSingleObjectBinding() const
{
	return FText::Format(LOCTEXT("PossessableBoundObjectToolTip", "(BindingID: {0}"), FText::FromString(LexToString(ObjectBindingID)));
}

FText FObjectBindingModel::GetLabelToolTipText() const
{
	TSharedPtr<ISequencer> Sequencer = OwnerModel->GetSequencer();
	if (!Sequencer)
	{
		return FText();
	}

	TArrayView<TWeakObjectPtr<>> BoundObjects = Sequencer->FindBoundObjects(ObjectBindingID, OwnerModel->GetSequenceID());

	if ( BoundObjects.Num() == 0 )
	{
		return FText::Format(LOCTEXT("InvalidBoundObjectToolTip", "The object bound to this track is missing (BindingID: {0})."), FText::FromString(LexToString(ObjectBindingID)));
	}
	else
	{
		TArray<FString> ValidBoundObjectLabels;
		FName BoundObjectClass;
		bool bAddEllipsis = false;
		int32 NumMissing = 0;
		for (const TWeakObjectPtr<>& Ptr : BoundObjects)
		{
			UObject* Obj = Ptr.Get();

			if (Obj == nullptr)
			{
				++NumMissing;
				continue;
			}

			if (Obj->GetClass())
			{
				BoundObjectClass = Obj->GetClass()->GetFName();
			}

			if (AActor* Actor = Cast<AActor>(Obj))
			{
				ValidBoundObjectLabels.Add(Actor->GetActorLabel());
			}
			else
			{
				ValidBoundObjectLabels.Add(Obj->GetName());
			}

			if (ValidBoundObjectLabels.Num() > 3)
			{
				bAddEllipsis = true;
				break;
			}
		}

		// If only 1 bound object, display a simpler tooltip.
		if (ValidBoundObjectLabels.Num() == 1 && NumMissing == 0)
		{
			return GetTooltipForSingleObjectBinding();
		}
		else if (ValidBoundObjectLabels.Num() == 0 && NumMissing == 1)
		{
			return FText::Format(LOCTEXT("InvalidBoundObjectToolTip", "The object bound to this track is missing (BindingID: {0})."), FText::FromString(LexToString(ObjectBindingID)));
		}

		FString MultipleBoundObjectLabel = FString::Join(ValidBoundObjectLabels, TEXT(", "));
		if (bAddEllipsis)
		{
			MultipleBoundObjectLabel += FString::Printf(TEXT("... %d more"), BoundObjects.Num()-3);
		}

		if (NumMissing != 0)
		{
			MultipleBoundObjectLabel += FString::Printf(TEXT(" (%d missing)"), NumMissing);
		}

		return FText::FromString(MultipleBoundObjectLabel + FString::Printf(TEXT(" Class: %s (BindingID: %s)"), *LexToString(BoundObjectClass), *LexToString(ObjectBindingID)));
	}
}

const FSlateBrush* FObjectBindingModel::GetIconBrush() const
{
	const UClass* ClassForObjectBinding = FindObjectClass();
	if (ClassForObjectBinding)
	{
		return FSlateIconFinder::FindIconBrushForClass(ClassForObjectBinding);
	}

	return FAppStyle::GetBrush("Sequencer.InvalidSpawnableIcon");
}

TSharedRef<SWidget> FObjectBindingModel::GetAddTrackMenuContent()
{
	TSharedPtr<FSequencer> Sequencer = OwnerModel->GetSequencerImpl();
	check(Sequencer);

	UObject* BoundObject = GetFirstBoundObject(ObjectBindingID);
	const UClass* MainSelectionObjectClass = FindObjectClass();

	TArray<FGuid> ObjectBindings;
	ObjectBindings.Add(ObjectBindingID);

	TArray<UClass*> ObjectClasses;
	ObjectClasses.Add(const_cast<UClass*>(MainSelectionObjectClass));

	// Only include other selected object bindings if this binding is selected. Otherwise, this will lead to 
	// confusion with multiple tracks being added to possibly unrelated objects
	if (OwnerModel->GetEditor()->GetSelection()->Outliner.IsSelected(SharedThis(this)))
	{
		for (TViewModelPtr<FObjectBindingModel> ObjectBindingNode : OwnerModel->GetEditor()->GetSelection()->Outliner.Filter<FObjectBindingModel>())
		{
			const FGuid Guid = ObjectBindingNode->GetObjectGuid();
			for (auto RuntimeObject : Sequencer->FindBoundObjects(Guid, OwnerModel->GetSequenceID()))
			{
				if (RuntimeObject.Get() != nullptr)
				{
					ObjectBindings.AddUnique(Guid);
					ObjectClasses.Add(RuntimeObject->GetClass());
					continue;
				}
			}
		}
	}

	ISequencerModule& SequencerModule = FModuleManager::GetModuleChecked<ISequencerModule>( "Sequencer" );
	TSharedRef<FUICommandList> CommandList = MakeShared<FUICommandList>();

	TSharedRef<FExtender> Extender = SequencerModule.GetAddTrackMenuExtensibilityManager()->GetAllExtenders(CommandList, TArrayBuilder<UObject*>().Add(BoundObject)).ToSharedRef();

	TArray<TSharedPtr<FExtender>> AllExtenders;
	AllExtenders.Add(Extender);

	TArrayView<UObject* const>                   ContextObjects = BoundObject ? MakeArrayView(&BoundObject, 1) : TArrayView<UObject* const>();
	TMap<const IObjectSchema*, TArray<UObject*>> Map            = IObjectSchema::ComputeRelevancy(ContextObjects);

	for (const TPair<const IObjectSchema*, TArray<UObject*>>& Pair : Map)
	{
		TSharedPtr<FExtender> NewExtension = Pair.Key->ExtendObjectBindingMenu(CommandList, Sequencer, Pair.Value);
		if (NewExtension)
		{
			AllExtenders.Add(NewExtension);
		}
	}
	if (AllExtenders.Num())
	{
		Extender = FExtender::Combine(AllExtenders);
	}

	const UClass* ObjectClass = UClass::FindCommonBase(ObjectClasses);

	for (const TSharedPtr<ISequencerTrackEditor>& CurTrackEditor : Sequencer->GetTrackEditors())
	{
		CurTrackEditor->ExtendObjectBindingTrackMenu(Extender, ObjectBindings, ObjectClass);
	}

	// The menu are generated through reflection and sometime the API exposes some recursivity (think about a Widget returning it parent which is also a Widget). Just by reflection
	// it is not possible to determine when the root object is reached. It needs a kind of simulation which is not implemented. Also, even if the recursivity was correctly handled, the possible
	// permutations tend to grow exponentially. Until a clever solution is found, the simple approach is to disable recursively searching those menus. User can still search the current one though.
	// See UE-131257
	const bool bInRecursivelySearchable = false;

	FMenuBuilder AddTrackMenuBuilder(true, nullptr, Extender, false, &FCoreStyle::Get(), true, NAME_None, bInRecursivelySearchable);

	const int32 NumStartingBlocks = AddTrackMenuBuilder.GetMultiBox()->GetBlocks().Num();

	AddTrackMenuBuilder.BeginSection("Tracks", LOCTEXT("TracksMenuHeader" , "Tracks"));
	Sequencer->BuildObjectBindingTrackMenu(AddTrackMenuBuilder, ObjectBindings, ObjectClass);
	AddTrackMenuBuilder.EndSection();

	TArray<FPropertyPath> KeyablePropertyPaths;

	if (BoundObject != nullptr)
	{
		TSharedRef<ISequencer> SequencerInterface = Sequencer.ToSharedRef();
		FSequencerObjectBindingHelper::GetKeyablePropertyPaths(BoundObject, SequencerInterface, KeyablePropertyPaths);
	}

	AddPropertyMenuItems(AddTrackMenuBuilder, NumStartingBlocks, KeyablePropertyPaths, 0);

	return AddTrackMenuBuilder.MakeWidget();
}

void FObjectBindingModel::AddPropertyMenuItems(FMenuBuilder& AddTrackMenuBuilder, int32 NumStartingBlocks, TArray<FPropertyPath> KeyablePropertyPaths, int32 PropertyNameIndexStart)
{
	// KeyablePropertyPaths contain a property path for each property, nested or not, that we can key. For instance:
	//
	// [MyFloat]                              (float, via the float property track)
	// [SomeStruct] [MyColor]                 (SomeStruct isn't keyable so key its color, via the color property track)
	// [SomeStruct] [OtherStruct] [MyInt]     (SomeStruct and OtherStruct aren't keyable so key the integer, via the int property track)
	// [SomeStruct] [KnownStruct]			  (KnownStruct has a custom track to key its properties)


	// If PropertyNameIndexStart is greater that zero, we are showing the sub-menu of a property path.
	// That is, if we have property paths like this:
	//
	// [SomeStruct] [OtherStruct] [MyInt]
	// [SomeStruct] [OtherStruct] [MyColor]
	//
	// ...and if we are showing the sub-menu for [OtherStruct]
	//
	// ...then PropertyNameIndexStart is 2, and we only need to show [MyInt] and [MyColor].

	static const FString DefaultPropertyCategory = TEXT("Default");

	// Properties with the category "Default" have no category and should be sorted to the top
	struct FCategorySortPredicate
	{
		bool operator()(const FString& A, const FString& B) const
		{
			if (A == DefaultPropertyCategory)
			{
				return true;
			}
			else if (B == DefaultPropertyCategory)
			{
				return false;
			}
			else
			{
				return A.Compare(B) < 0;
			}
		}
	};

	bool bDefaultCategoryFound = false;
	const bool bIsRootMenu = (PropertyNameIndexStart == 0);

	// Create property menu data based on keyable property paths
	TMap<FString, TArray<FPropertyMenuData>> KeyablePropertyMenuData;
	for (const FPropertyPath& KeyablePropertyPath : KeyablePropertyPaths)
	{
		if (!ensure(KeyablePropertyPath.GetNumProperties() > PropertyNameIndexStart))
		{
			continue;
		}

		const FPropertyInfo& PropertyInfo = KeyablePropertyPath.GetPropertyInfo(PropertyNameIndexStart);
		if (const FProperty* Property = PropertyInfo.Property.Get())
		{
			FPropertyMenuData KeyableMenuData;
			KeyableMenuData.PropertyPath = KeyablePropertyPath;
			if (PropertyInfo.ArrayIndex != INDEX_NONE)
			{
				KeyableMenuData.MenuName = FText::Format(LOCTEXT("PropertyMenuTextFormat", "{0} [{1}]"), Property->GetDisplayNameText(), FText::AsNumber(PropertyInfo.ArrayIndex)).ToString();
			}
			else
			{
				KeyableMenuData.MenuName = Property->GetDisplayNameText().ToString();
			}

			FString CategoryText = FObjectEditorUtils::GetCategory(Property);

			if (CategoryText == DefaultPropertyCategory)
			{
				bDefaultCategoryFound = true;
			}

			KeyablePropertyMenuData.FindOrAdd(CategoryText).Add(KeyableMenuData);
		}
	}

	KeyablePropertyMenuData.KeySort(FCategorySortPredicate());

	// Always add an extension point for Properties section even if none are found (Components rely on this) 
	if (!bDefaultCategoryFound && bIsRootMenu)
	{
		AddTrackMenuBuilder.BeginSection(SequencerMenuExtensionPoints::AddTrackMenu_PropertiesSection, LOCTEXT("PropertiesMenuHeader", "Properties"));
		AddTrackMenuBuilder.EndSection();
	}

	// Add menu items
	TSharedPtr<FSequencer> Sequencer = OwnerModel->GetSequencerImpl();
	check(Sequencer);
	const bool bUseSubMenus = GSequencerObjectBindingShowNestedProperties || Sequencer->IsLevelEditorSequencer();

	for (TPair<FString, TArray<FPropertyMenuData>>& Pair : KeyablePropertyMenuData)
	{
		const FString CategoryText = Pair.Key;
		TArray<FPropertyMenuData>& KeyablePropertySubMenuData = Pair.Value;
		
		// Sort on the property name
		KeyablePropertySubMenuData.Sort();
		
		if (CategoryText == DefaultPropertyCategory)
		{
			AddTrackMenuBuilder.BeginSection(SequencerMenuExtensionPoints::AddTrackMenu_PropertiesSection, LOCTEXT("PropertiesMenuHeader", "Properties"));
		}
		else
		{
			AddTrackMenuBuilder.BeginSection(NAME_None, FText::FromString(CategoryText));
		}
	
		for (int32 MenuDataIndex = 0; MenuDataIndex < KeyablePropertySubMenuData.Num(); )
		{
			// If this menu data only has one property name left in it, add the menu item
			if (KeyablePropertySubMenuData[MenuDataIndex].PropertyPath.GetNumProperties() == PropertyNameIndexStart + 1)
			{
				AddPropertyMenuItem(AddTrackMenuBuilder, KeyablePropertySubMenuData[MenuDataIndex]);
				++MenuDataIndex;
			}
			// If we don't want sub-menus, concatenate the property names left to handle, and add the menu item.
			else if (!bUseSubMenus)
			{
				TArray<FString> PropertyNames;
				const FPropertyPath& CurPropertyPath(KeyablePropertySubMenuData[MenuDataIndex].PropertyPath);
				for (int32 PropertyNameIndex = PropertyNameIndexStart; PropertyNameIndex < CurPropertyPath.GetNumProperties(); ++PropertyNameIndex)
				{
					const FPropertyInfo& CurPropertyInfo(CurPropertyPath.GetPropertyInfo(PropertyNameIndex));
					if (CurPropertyInfo.ArrayIndex != INDEX_NONE)
					{
						PropertyNames.Add(FText::Format(
									LOCTEXT("PropertyMenuTextFormat", "{0} [{1}]"), 
									CurPropertyInfo.Property.Get()->GetDisplayNameText(), 
									FText::AsNumber(CurPropertyInfo.ArrayIndex)).ToString());
					}
					else
					{
						PropertyNames.Add(CurPropertyInfo.Property.Get()->GetDisplayNameText().ToString());
					}
				}
				KeyablePropertySubMenuData[MenuDataIndex].MenuName = FString::Join(PropertyNames, TEXT("."));

				AddPropertyMenuItem(AddTrackMenuBuilder, KeyablePropertySubMenuData[MenuDataIndex]);
				++MenuDataIndex;
			}
			// Otherwise, look to the next menu data to gather up new data
			else
			{
				TArray<FPropertyPath> KeyableSubMenuPropertyPaths;
				KeyableSubMenuPropertyPaths.Add(KeyablePropertySubMenuData[MenuDataIndex].PropertyPath);

				for (; MenuDataIndex < KeyablePropertySubMenuData.Num() - 1; )
				{
					if (KeyablePropertySubMenuData[MenuDataIndex].MenuName == KeyablePropertySubMenuData[MenuDataIndex + 1].MenuName)
					{	
						++MenuDataIndex;
						KeyableSubMenuPropertyPaths.Add(KeyablePropertySubMenuData[MenuDataIndex].PropertyPath);
					}
					else
					{
						break;
					}
				}

				AddTrackMenuBuilder.AddSubMenu(
					FText::FromString(KeyablePropertySubMenuData[MenuDataIndex].MenuName),
					FText::GetEmpty(), 
					FNewMenuDelegate::CreateSP(this, &FObjectBindingModel::HandleAddTrackSubMenuNew, KeyableSubMenuPropertyPaths, PropertyNameIndexStart + 1));

				++MenuDataIndex;
			}
		}

		AddTrackMenuBuilder.EndSection();
	}

	if (AddTrackMenuBuilder.GetMultiBox()->GetBlocks().Num() == NumStartingBlocks)
	{
		TSharedRef<SWidget> EmptyTip = SNew(SBox)
			.Padding(FMargin(15.f, 7.5f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoKeyablePropertiesFound", "No keyable properties or tracks"))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			];

		AddTrackMenuBuilder.AddWidget(EmptyTip, FText(), true, false);
	}
}

void FObjectBindingModel::HandleAddTrackSubMenuNew(FMenuBuilder& AddTrackMenuBuilder, TArray<FPropertyPath> KeyablePropertyPaths, int32 PropertyNameIndexStart)
{
	AddPropertyMenuItems(AddTrackMenuBuilder, 0, KeyablePropertyPaths, PropertyNameIndexStart);
}

void FObjectBindingModel::HandlePropertyMenuItemExecute(FPropertyPath PropertyPath)
{
	TSharedPtr<FSequencer> Sequencer = OwnerModel->GetSequencerImpl();
	UObject *BoundObject = GetFirstBoundObject(ObjectBindingID);
	TArray<UObject*> KeyableBoundObjects;
	if (BoundObject != nullptr)
	{
		if (Sequencer->CanKeyProperty(FCanKeyPropertyParams(BoundObject->GetClass(), PropertyPath)))
		{
			KeyableBoundObjects.Add(BoundObject);
		}
	}

	// Only include other selected object bindings if this binding is selected. Otherwise, this will lead to 
	// confusion with multiple tracks being added to possibly unrelated objects
	if (OwnerModel->GetEditor()->GetSelection()->Outliner.IsSelected(SharedThis(this)))
	{
		for (TViewModelPtr<FObjectBindingModel> ObjectBindingNode : OwnerModel->GetEditor()->GetSelection()->Outliner.Filter<FObjectBindingModel>())
		{
			FGuid Guid = ObjectBindingNode->GetObjectGuid();
			for (auto RuntimeObject : Sequencer->FindBoundObjects(Guid, OwnerModel->GetSequenceID()))
			{
				if (Sequencer->CanKeyProperty(FCanKeyPropertyParams(RuntimeObject->GetClass(), PropertyPath)))
				{
					KeyableBoundObjects.AddUnique(RuntimeObject.Get());
				}
			}
		}
	}

	// When auto setting track defaults are disabled, force add a key so that the changed
	// value is saved and is propagated to the property.
	FKeyPropertyParams KeyPropertyParams(KeyableBoundObjects, PropertyPath, Sequencer->GetAutoSetTrackDefaults() == false ? ESequencerKeyMode::ManualKeyForced : ESequencerKeyMode::ManualKey);

	Sequencer->KeyProperty(KeyPropertyParams);
}

void FObjectBindingModel::AddPropertyMenuItem(FMenuBuilder& AddTrackMenuBuilder, const FPropertyMenuData& KeyablePropertyMenuData)
{
	FUIAction AddTrackMenuAction(FExecuteAction::CreateSP(this, &FObjectBindingModel::HandlePropertyMenuItemExecute, KeyablePropertyMenuData.PropertyPath));
	AddTrackMenuBuilder.AddMenuEntry(FText::FromString(KeyablePropertyMenuData.MenuName), FText(), FSlateIcon(), AddTrackMenuAction);
}

void FObjectBindingModel::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	FSequencer* Sequencer = EditorViewModel->GetSequencerImpl().Get();
	ISequencerModule& SequencerModule = FModuleManager::GetModuleChecked<ISequencerModule>("Sequencer");
	UObject* BoundObject = GetFirstBoundObject(ObjectBindingID);
	const UClass* ObjectClass = FindObjectClass();
	
	TSharedPtr<FExtender> Extender = EditorViewModel->GetSequencerMenuExtender(
		SequencerModule.GetObjectBindingContextMenuExtensibilityManager(), TArrayBuilder<UObject*>().Add(BoundObject),
		&FSequencerCustomizationInfo::OnBuildObjectBindingContextMenu, SharedThis(this));
	if (Extender.IsValid())
	{
		MenuBuilder.PushExtender(Extender.ToSharedRef());
	}
	
	// Extenders can go in there.
	MenuBuilder.BeginSection("ObjectBindingActions");
	MenuBuilder.EndSection();

	// External extension.
	Sequencer->BuildCustomContextMenuForGuid(MenuBuilder, ObjectBindingID);

	// Track editor extension.
	TArray<FGuid> ObjectBindings;
	ObjectBindings.Add(ObjectBindingID);

	// Add other selected objects with the same class as the first one
	for (TViewModelPtr<FObjectBindingModel> ObjectBindingNode : OwnerModel->GetEditor()->GetSelection()->Outliner.Filter<FObjectBindingModel>())
	{
		FGuid Guid = ObjectBindingNode->GetObjectGuid();
		if (Guid != ObjectBindingID)
		{
			if (UObject* OtherBoundObject = GetFirstBoundObject(Guid))
			{
				if (ObjectClass == ObjectBindingNode->FindObjectClass())
				{
					ObjectBindings.Add(Guid);
				}
			}
		}
	}


	for (const TSharedPtr<ISequencerTrackEditor>& TrackEditor : Sequencer->GetTrackEditors())
	{
		TrackEditor->BuildObjectBindingContextMenu(MenuBuilder, ObjectBindings, ObjectClass);
	}

	// Up-call.
	FOutlinerItemModel::BuildContextMenu(MenuBuilder);
}

void FObjectBindingModel::BuildOrganizeContextMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddSubMenu(
		LOCTEXT("TagsLabel", "Tags"),
		LOCTEXT("TagsTooltip", "Show this object binding's tags"),
		FNewMenuDelegate::CreateSP(this, &FObjectBindingModel::AddTagMenu)
	);

	FOutlinerItemModel::BuildOrganizeContextMenu(MenuBuilder);
}

void FObjectBindingModel::BuildSidebarMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	FSequencer* const Sequencer = EditorViewModel->GetSequencerImpl().Get();
	if (!Sequencer)
	{
		return;
	} 

	ISequencerModule& SequencerModule = FModuleManager::GetModuleChecked<ISequencerModule>(TEXT("Sequencer"));
	UObject* BoundObject = GetFirstBoundObject(ObjectBindingID);

	const TSharedPtr<FExtender> Extender = EditorViewModel->GetSequencerMenuExtender(SequencerModule.GetSidebarExtensibilityManager()
		, TArrayBuilder<UObject*>().Add(BoundObject), &FSequencerCustomizationInfo::OnBuildSidebarMenu, SharedThis(this));
	if (Extender.IsValid())
	{
		MenuBuilder.PushExtender(Extender.ToSharedRef());
	}

	MenuBuilder.BeginSection(TEXT("ObjectBindingActions"), LOCTEXT("ObjectBindingsMenuSection", "Object Bindings"));
	MenuBuilder.EndSection();

	// External extension.
	Sequencer->BuildCustomContextMenuForGuid(MenuBuilder, ObjectBindingID);

	// Track editor extension.
	TArray<FGuid> ObjectBindings;
	ObjectBindings.Add(ObjectBindingID);

	const UClass* const ObjectClass = FindObjectClass();
	for (const TSharedPtr<ISequencerTrackEditor>& TrackEditor : Sequencer->GetTrackEditors())
	{
		TrackEditor->BuildObjectBindingContextMenu(MenuBuilder, ObjectBindings, ObjectClass);
	}

	FOutlinerItemModel::BuildSidebarMenu(MenuBuilder);
}

void FObjectBindingModel::AddTagMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(FSequencerCommands::Get().OpenTaggedBindingManager);

	TSharedPtr<FSequencer> Sequencer = OwnerModel->GetSequencerImpl();

	UMovieSceneSequence* Sequence   = Sequencer->GetRootMovieSceneSequence();
	UMovieScene*         MovieScene = Sequence->GetMovieScene();

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ObjectTagsHeader", "Object Tags"));
	{
		TSet<FName> AllTags;

		// Gather all the tags on all currently selected object binding IDs
		FMovieSceneSequenceID SequenceID = OwnerModel->GetSequenceID();
		for (TViewModelPtr<FObjectBindingModel> ObjectBindingNode : OwnerModel->GetEditor()->GetSelection()->Outliner.Filter<FObjectBindingModel>())
		{
			const FGuid& ObjectID = ObjectBindingNode->GetObjectGuid();

			UE::MovieScene::FFixedObjectBindingID BindingID(ObjectID, SequenceID);
			for (auto It = Sequencer->GetObjectBindingTagCache()->IterateTags(BindingID); It; ++It)
			{
				AllTags.Add(It.Value());
			}
		}

		bool bIsReadOnly = MovieScene->IsReadOnly();
		for (const FName& TagName : AllTags)
		{
			MenuBuilder.AddMenuEntry(
				FText::FromName(TagName),
				FText(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &FObjectBindingModel::ToggleTag, TagName),
					FCanExecuteAction::CreateLambda([bIsReadOnly] { return bIsReadOnly == false; }),
					FGetActionCheckState::CreateSP(this, &FObjectBindingModel::GetTagCheckState, TagName)
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("AddNewHeader", "Add Tag"));
	{
		if (!MovieScene->IsReadOnly())
		{
			TSharedRef<SWidget> Widget =
				SNew(SObjectBindingTag)
				.OnCreateNew(this, &FObjectBindingModel::HandleAddTag);

			MenuBuilder.AddWidget(Widget, FText());
		}
	}
	MenuBuilder.EndSection();
}

ECheckBoxState FObjectBindingModel::GetTagCheckState(FName TagName)
{
	ECheckBoxState CheckBoxState = ECheckBoxState::Undetermined;

	TSharedPtr<FSequencer> Sequencer = OwnerModel->GetSequencerImpl();
	FMovieSceneSequenceID SequenceID = OwnerModel->GetSequenceID();

	for (TViewModelPtr<FObjectBindingModel> ObjectBindingNode : OwnerModel->GetEditor()->GetSelection()->Outliner.Filter<FObjectBindingModel>())
	{
		const FGuid& ObjectID = ObjectBindingNode->GetObjectGuid();

		UE::MovieScene::FFixedObjectBindingID BindingID(ObjectID, SequenceID);
		ECheckBoxState ThisCheckState = Sequencer->GetObjectBindingTagCache()->HasTag(BindingID, TagName)
			? ECheckBoxState::Checked
			: ECheckBoxState::Unchecked;

		if (CheckBoxState == ECheckBoxState::Undetermined)
		{
			CheckBoxState = ThisCheckState;
		}
		else if (CheckBoxState != ThisCheckState)
		{
			return ECheckBoxState::Undetermined;
		}
	}

	return CheckBoxState;
}

void FObjectBindingModel::ToggleTag(FName TagName)
{
	TSharedPtr<FSequencer> Sequencer = OwnerModel->GetSequencerImpl();
	FMovieSceneSequenceID SequenceID = OwnerModel->GetSequenceID();

	for (TViewModelPtr<FObjectBindingModel> ObjectBindingNode : OwnerModel->GetEditor()->GetSelection()->Outliner.Filter<FObjectBindingModel>())
	{
		const FGuid& ObjectID = ObjectBindingNode->GetObjectGuid();

		UE::MovieScene::FFixedObjectBindingID BindingID(ObjectID, SequenceID);
		if (!Sequencer->GetObjectBindingTagCache()->HasTag(BindingID, TagName))
		{
			HandleAddTag(TagName);
			return;
		}
	}

	HandleDeleteTag(TagName);
}

void FObjectBindingModel::HandleDeleteTag(FName TagName)
{
	FScopedTransaction Transaction(FText::Format(LOCTEXT("RemoveBindingTag", "Remove tag '{0}' from binding(s)"), FText::FromName(TagName)));

	TSharedPtr<ISequencer> Sequencer = OwnerModel->GetSequencer();
	UMovieScene* MovieScene = Sequencer->GetRootMovieSceneSequence()->GetMovieScene();
	MovieScene->Modify();

	FMovieSceneSequenceID SequenceID = OwnerModel->GetSequenceID();
	for (TViewModelPtr<FObjectBindingModel> ObjectBindingNode : OwnerModel->GetEditor()->GetSelection()->Outliner.Filter<FObjectBindingModel>())
	{
		const FGuid& ObjectID = ObjectBindingNode->GetObjectGuid();
		MovieScene->UntagBinding(TagName, UE::MovieScene::FFixedObjectBindingID(ObjectID, SequenceID));
	}
}

void FObjectBindingModel::HandleAddTag(FName TagName)
{
	FScopedTransaction Transaction(FText::Format(LOCTEXT("CreateBindingTag", "Add new tag {0} to binding(s)"), FText::FromName(TagName)));

	TSharedPtr<ISequencer> Sequencer = OwnerModel->GetSequencer();
	UMovieScene* MovieScene = Sequencer->GetRootMovieSceneSequence()->GetMovieScene();
	MovieScene->Modify();

	FMovieSceneSequenceID SequenceID = OwnerModel->GetSequenceID();
	for (TViewModelPtr<FObjectBindingModel> ObjectBindingNode : OwnerModel->GetEditor()->GetSelection()->Outliner.Filter<FObjectBindingModel>())
	{
		const FGuid& ObjectID = ObjectBindingNode->GetObjectGuid();
		MovieScene->TagBinding(TagName, UE::MovieScene::FFixedObjectBindingID(ObjectID, SequenceID));
	}
}

void FObjectBindingModel::SortChildren()
{
	ISortableExtension::SortChildren(SharedThis(this), ESortingMode::PriorityFirst);
}

FSortingKey FObjectBindingModel::GetSortingKey() const
{
	FSortingKey SortingKey;

	if (OwnerModel)
	{
		UMovieScene* MovieScene = OwnerModel->GetMovieScene();
		const FMovieSceneBinding* MovieSceneBinding = MovieScene->FindBinding(ObjectBindingID);

		if (MovieSceneBinding)
		{
			SortingKey.CustomOrder = MovieSceneBinding->GetSortingOrder();
		}

		SortingKey.DisplayName = MovieScene->GetObjectDisplayName(ObjectBindingID);
	}

	// When inside object bindings, we come before tracks. Elsewhere, we come after tracks.
	const bool bHasParentObjectBinding = (CastParent<IObjectBindingExtension>() != nullptr);
	SortingKey.PrioritizeBy(bHasParentObjectBinding ? 2 : 1);

	return SortingKey;
}

void FObjectBindingModel::SetCustomOrder(int32 InCustomOrder)
{
	if (OwnerModel)
	{
		UMovieScene* MovieScene = OwnerModel->GetMovieScene();
		FMovieSceneBinding* MovieSceneBinding = MovieScene->FindBinding(ObjectBindingID);
		if (MovieSceneBinding)
		{
			MovieSceneBinding->SetSortingOrder(InCustomOrder);
		}
	}
}

bool FObjectBindingModel::CanDrag() const
{
	// Can only drag top level object bindings
	TSharedPtr<IObjectBindingExtension> ObjectBindingExtension = FindAncestorOfType<IObjectBindingExtension>();
	return ObjectBindingExtension == nullptr;
}

bool FObjectBindingModel::CanDelete(FText* OutErrorMessage) const
{
	return true;
}

void FObjectBindingModel::Delete()
{
	if (OwnerModel)
	{
		TSharedPtr<ISequencer> Sequencer = OwnerModel->GetSequencer();
		UMovieScene* MovieScene = Sequencer->GetRootMovieSceneSequence()->GetMovieScene();

		MovieScene->Modify();

		// Untag this binding
		UE::MovieScene::FFixedObjectBindingID BindingID(ObjectBindingID, OwnerModel->GetSequenceID());
		for (auto It = OwnerModel->GetSequencerImpl()->GetObjectBindingTagCache()->IterateTags(BindingID); It; ++It)
		{
			MovieScene->UntagBinding(It.Value(), BindingID);
		}

		// Delete any child object bindings - this will remove their tracks implicitly
		// so no need to delete those manually
		for (const TViewModelPtr<FObjectBindingModel>& ChildObject : GetChildrenOfType<FObjectBindingModel>(EViewModelListType::Outliner).ToArray())
		{
			ChildObject->Delete();
		}

		// Remove from a parent folder if necessary.
		if (TViewModelPtr<FFolderModel> ParentFolder = CastParent<FFolderModel>())
		{
			ParentFolder->GetFolder()->RemoveChildObjectBinding(ObjectBindingID);
		}

		BindingLifetimeOverlayModel.Reset();
	}
}

bool FObjectBindingModel::IsMuted() const
{
	if (OwnerModel)
	{
		UMovieScene* MovieScene = OwnerModel->GetMovieScene();
		FMovieSceneBinding* MovieSceneBinding = MovieScene->FindBinding(ObjectBindingID);
		if (MovieSceneBinding)
		{
			if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = MovieSceneBinding->FindDecoration<UMovieSceneMuteSoloDecoration>())
			{
				return MuteSoloDecoration->IsMuted();
			}
		}
	}
	
	return false;
}

void FObjectBindingModel::SetIsMuted(bool bIsMuted)
{
	if (OwnerModel)
	{
		UMovieScene* MovieScene = OwnerModel->GetMovieScene();
		FMovieSceneBinding* MovieSceneBinding = MovieScene->FindBinding(ObjectBindingID);
		if (MovieSceneBinding)
		{	
			const bool bAlwaysMarkDirty = false;
			MovieScene->Modify(bAlwaysMarkDirty);

			if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = Cast<UMovieSceneMuteSoloDecoration>(MovieSceneBinding->GetOrCreateDecoration(UMovieSceneMuteSoloDecoration::StaticClass(), MovieScene, [this](UObject* Decoration) {})))
			{
				MuteSoloDecoration->SetMuted(bIsMuted);
			}
		}
	}
}

bool FObjectBindingModel::IsSolo() const
{
	if (OwnerModel)
	{
		UMovieScene* MovieScene = OwnerModel->GetMovieScene();
		FMovieSceneBinding* MovieSceneBinding = MovieScene->FindBinding(ObjectBindingID);
		if (MovieSceneBinding)
		{
			if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = MovieSceneBinding->FindDecoration<UMovieSceneMuteSoloDecoration>())
			{
				return MuteSoloDecoration->IsSoloed();
			}
		}
	}

	return false;
}

void FObjectBindingModel::SetIsSoloed(bool bIsSoloed)
{
	if (OwnerModel)
	{
		UMovieScene* MovieScene = OwnerModel->GetMovieScene();
		FMovieSceneBinding* MovieSceneBinding = MovieScene->FindBinding(ObjectBindingID);
		if (MovieSceneBinding)
		{
			const bool bAlwaysMarkDirty = false;
			MovieScene->Modify(bAlwaysMarkDirty);

			if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = Cast<UMovieSceneMuteSoloDecoration>(MovieSceneBinding->GetOrCreateDecoration(UMovieSceneMuteSoloDecoration::StaticClass(), MovieScene, [this](UObject* Decoration) {})))
			{
				MuteSoloDecoration->SetSoloed(bIsSoloed);
			}
		}
	}
}

UObject* FObjectBindingModel::GetFirstBoundObject(FGuid InGuid)
{
	TSharedPtr<ISequencer> Sequencer = OwnerModel->GetSequencer();
	if (!Sequencer)
	{
		return nullptr;
	}

	for (TWeakObjectPtr<UObject> RuntimeObject : Sequencer->FindObjectsInCurrentSequence(InGuid))
	{
		UObject* BoundObject = RuntimeObject.Get();
		if (BoundObject)
		{
			return BoundObject;
		}
	}

	return nullptr;
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

