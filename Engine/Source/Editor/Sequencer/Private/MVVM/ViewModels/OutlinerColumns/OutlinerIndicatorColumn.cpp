// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerColumns/OutlinerIndicatorColumn.h"

#include "Sequencer.h"
#include "MVVM/Extensions/IConditionableExtension.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/OutlinerIndicators/IOutlinerIndicatorBuilder.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/OutlinerIndicators/SConditionIndicatorWidget.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "FOutlinerIndicatorColumn"

namespace UE::Sequencer
{

FOutlinerIndicatorColumn::FOutlinerIndicatorColumn()
{
	Name = FCommonOutlinerNames::Indicator;
	Label = LOCTEXT("IndicatorColumnLabel", "Indicators");
	Position = FOutlinerColumnPosition{ 0, EOutlinerColumnGroup::LeftGutter };
	Layout = FOutlinerColumnLayout{ 14, FMargin(0.f, 0.f), HAlign_Fill, VAlign_Fill, EOutlinerColumnSizeMode::Fixed, EOutlinerColumnFlags::None };
}

bool FOutlinerIndicatorColumn::IsItemCompatibleWithColumn(const FCreateOutlinerColumnParams& InParams) const
{
	TViewModelPtr<FSequencerEditorViewModel> Editor = InParams.Editor->CastThisShared<FSequencerEditorViewModel>();
	if (!Editor)
	{
		return false;
	}

	TSharedPtr<FSequencer> Sequencer = Editor->GetSequencerImpl();
	if (!Sequencer)
	{
		return false;
	}

	for (const TTuple< FName, TSharedPtr<IOutlinerIndicatorBuilder> >& OutlinerIndicator : Sequencer->GetOutlinerIndicators())
	{
		if (OutlinerIndicator.Value.Get()->IsItemCompatibleWithIndicator(InParams))
		{
			return true;
		}
	}

	return false;
}

TSharedPtr<SWidget> FOutlinerIndicatorColumn::CreateColumnWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow)
{
	TViewModelPtr<FSequencerEditorViewModel> Editor = InParams.Editor->CastThisShared<FSequencerEditorViewModel>();
	if (!Editor)
	{
		return nullptr;
	}

	TSharedPtr<FSequencer> Sequencer = Editor->GetSequencerImpl();
	if (!Sequencer)
	{
		return nullptr;
	}

	ColumnWidget = SNew(SHorizontalBox);

	TArray<TSharedPtr<IOutlinerIndicatorBuilder> > CompatibleIndicators;

	for (const TTuple< FName, TSharedPtr<IOutlinerIndicatorBuilder> >& OutlinerIndicator : Sequencer->GetOutlinerIndicators())
	{
		TSharedPtr<IOutlinerIndicatorBuilder> Indicator = OutlinerIndicator.Value;

		if (Indicator->IsItemCompatibleWithIndicator(InParams))
		{
			CompatibleIndicators.Add(Indicator);
		}
	}

	const int32 NumCompatibleIndicators = CompatibleIndicators.Num();

	for (TSharedPtr<IOutlinerIndicatorBuilder>& Indicator : CompatibleIndicators)
	{
		ColumnWidget->AddSlot()
		[
			Indicator->CreateIndicatorWidget(InParams, TreeViewRow, SharedThis(this), NumCompatibleIndicators).ToSharedRef()
		];
	}

	return ColumnWidget;
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE