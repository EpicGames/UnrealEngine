// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnBase.h"

class SHorizontalBox;

namespace UE::Sequencer
{

/**
 * A column for showing various decorators on the presence of features (e.g. conditions, time warp) on that row.
 */
class FOutlinerIndicatorColumn
	: public FOutlinerColumnBase
{
public:

	FOutlinerIndicatorColumn();

	bool IsItemCompatibleWithColumn(const FCreateOutlinerColumnParams& InParams) const override;
	TSharedPtr<SWidget> CreateColumnWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow) override;

private:
	TSharedPtr<SHorizontalBox> ColumnWidget;
};

} // namespace UE::Sequencer