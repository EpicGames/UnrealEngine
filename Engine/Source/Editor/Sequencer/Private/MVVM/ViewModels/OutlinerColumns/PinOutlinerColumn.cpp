// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerColumns/PinOutlinerColumn.h"

#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/PinEditorExtension.h"
#include "MVVM/ViewModels/OutlinerItemModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "Widgets/OutlinerColumns/SPinColumnWidget.h"
#include "Widgets/Layout/SScaleBox.h"

#define LOCTEXT_NAMESPACE "FPinOutlinerColumn"

namespace UE::Sequencer
{

FPinOutlinerColumn::FPinOutlinerColumn()
{
	Name     = FCommonOutlinerNames::Pin;
	Label    = LOCTEXT("PinColumnLabel", "Pin");
	Position = FOutlinerColumnPosition{ 0, EOutlinerColumnGroup::AssetToggles };
	Layout   = FOutlinerColumnLayout{ 14, FMargin(4.f, 0.f), HAlign_Center, VAlign_Center, EOutlinerColumnSizeMode::Fixed, EOutlinerColumnFlags::None };
}

bool FPinOutlinerColumn::IsItemCompatibleWithColumn(const FCreateOutlinerColumnParams& InParams) const
{
	if (InParams.Editor)
	{
		FPinEditorExtension* PinEditorExtension = InParams.Editor->CastDynamic<FPinEditorExtension>();
		if (PinEditorExtension)
		{
			return PinEditorExtension->IsNodePinnable(InParams.OutlinerExtension);
		}
	}

	return false;
}

TSharedPtr<SWidget> FPinOutlinerColumn::CreateColumnWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow)
{
	return SNew(SPinColumnWidget, SharedThis(this), InParams);
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE