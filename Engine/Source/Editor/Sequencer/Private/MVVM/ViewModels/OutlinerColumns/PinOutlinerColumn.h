// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnBase.h"

namespace UE::Sequencer
{

/**
 * A column for pinning/unpinning tracks.
 */
class FPinOutlinerColumn
	: public FOutlinerColumnBase
{
public:

	FPinOutlinerColumn();

	bool IsItemCompatibleWithColumn(const FCreateOutlinerColumnParams& InParams) const override;
	TSharedPtr<SWidget> CreateColumnWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow) override;
};

} // namespace UE::Sequencer