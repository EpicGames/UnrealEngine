// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutlinerFilterAreaManager.h"

#include "Filters/SequencerFilterBar.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "Filters/Utils/LinkedFilteringViewUtils.h"
#include "Filters/Linking/Mode/ILinkedFilterViewModel.h"
#include "Filters/Linking/Mode/LinkedFilterViewModel.h"
#include "SequencerNodeTree.h"
#include "Filters/FilterConfigIdentifiers.h"

namespace UE::Sequencer
{
FOutlinerFilterAreaManager::FOutlinerFilterAreaManager(
	TWeakPtr<FSequencer> InSequencer, const TSharedRef<FLinkedFilterViewModel>& InSequencerFilterViewModel
	)
	: FFilterAreaManager(ConfigIds::FilterArea_Outliner, MoveTemp(InSequencer), InSequencerFilterViewModel)
{
	GetFilterModel()->OnFilterModeChanged().AddRaw(this, &FOutlinerFilterAreaManager::OnFilterModeChanged);
	
	const TSharedRef<FSequencerFilterBar> SharedFilterBar = GetFilterModel()->GetLinkedFilterBarImpl();
	SharedFilterBar->OnChangeOutlinerExtensionFilterState().AddRaw(this, &FOutlinerFilterAreaManager::OnFilterStateChanged_LinkedFilterBar);
	
	const TSharedRef<FSequencerFilterBar> InstancedFilterBar = GetFilterModel()->GetInstancedFilterBarImpl();
	InstancedFilterBar->OnChangeOutlinerExtensionFilterState().AddRaw(this, &FOutlinerFilterAreaManager::OnFilterStateChanged_InstancedFilterBar);
}

FOutlinerFilterAreaManager::~FOutlinerFilterAreaManager()
{
	GetFilterModel()->OnFilterModeChanged().RemoveAll(this);
	
	const TSharedRef<FSequencerFilterBar> SharedFilterBar = GetFilterModel()->GetLinkedFilterBarImpl();
	SharedFilterBar->OnChangeOutlinerExtensionFilterState().RemoveAll(this);
	
	const TSharedRef<FSequencerFilterBar> InstancedFilterBar = GetFilterModel()->GetInstancedFilterBarImpl();
	InstancedFilterBar->OnChangeOutlinerExtensionFilterState().RemoveAll(this);
}

void FOutlinerFilterAreaManager::BindCommands(const TSharedRef<FUICommandList>& InSequencerCommandList) const
{
	InSequencerCommandList->Append(GetFilterAreaCommandList());
}

ELinkedFilterMode FOutlinerFilterAreaManager::GetFilterMode() const
{
	return GetFilterModel()->GetFilterMode();
}

void FOutlinerFilterAreaManager::OnFilterModeChanged()
{
	// TODO: The goal of this is to call SetFilteredOut on all items again, but it does more work then it needs to. Instead of rerunning the filter,
	// we could iterate through the hierarchy, and use FSequencerFilterBar::GetFilterData to call SetFilteredOut with the correct filter state.
	GetFilterModel()->GetActiveFilterBar()->RequestFilterUpdate();
}

void FOutlinerFilterAreaManager::OnFilterStateChanged_LinkedFilterBar(
	const TViewModelPtr<IOutlinerExtension>& InItem, bool bInIsFilteredOut
	) const
{
	if (GetFilterMode() == ELinkedFilterMode::Linked)
	{
		InItem->SetFilteredOut(bInIsFilteredOut);
	}
}

void FOutlinerFilterAreaManager::OnFilterStateChanged_InstancedFilterBar(
	const TViewModelPtr<IOutlinerExtension>& InItem, bool bInIsFilteredOut
	) const
{
	if (GetFilterMode() == ELinkedFilterMode::Instanced)
	{
		InItem->SetFilteredOut(bInIsFilteredOut);
	}
}
} // namespace UE::Sequencer
