// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/Linking/FilterAreaManager.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModels/ViewModel.h"

class FUICommandList;

namespace UE::Sequencer
{
class FLinkedFilterViewModel;
enum class ELinkedFilterMode : uint8;

/** 
 * Hooks up Sequencer's own FLinkedFilterViewModel with the relevant systems.
 * 
 * Its responsibility is to handle updating any systems when the linked filtering mode changes.
 * For example:
 * - Calling IOutlinerExtension::SetFilteredOut when the item's filter state changes
 * - Binding commands that are bound by FSequencerFilterBar and making sure correct commands are executed depending on which one is active.
 */
class FOutlinerFilterAreaManager : public FFilterAreaManager
{
public:
	
	explicit FOutlinerFilterAreaManager(TWeakPtr<FSequencer> InSequencer, const TSharedRef<FLinkedFilterViewModel>& InSequencerFilterViewModel);
	~FOutlinerFilterAreaManager();
	
	/** Binds the filter bar commands. */
	void BindCommands(const TSharedRef<FUICommandList>& InSequencerCommandList) const;
	
private:
	
	/** @return The filter mode that Sequencer is currently using. */
	ELinkedFilterMode GetFilterMode() const;

	/** Schedules the IOutlinerExtensions to be updated with SetFilteredOut calls to reflect the state of the new filter bar. */
	void OnFilterModeChanged();
	
	/** Handles propagating the IOutlinerExtension's filter out state if the active filter bar is the linked one. */
	void OnFilterStateChanged_LinkedFilterBar(const TViewModelPtr<IOutlinerExtension>& InItem, bool bInIsFilteredOut) const;
	
	/** Handles propagating the IOutlinerExtension's filter out state if the active filter bar is the instanced one. */
	void OnFilterStateChanged_InstancedFilterBar(const TViewModelPtr<IOutlinerExtension>& InItem, bool bInIsFilteredOut) const;
};
} // namespace UE::Sequencer
