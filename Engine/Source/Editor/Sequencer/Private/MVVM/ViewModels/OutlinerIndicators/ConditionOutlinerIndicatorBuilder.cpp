// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerIndicators/ConditionOutlinerIndicatorBuilder.h"

#include "MVVM/ViewModels/OutlinerColumns/IOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "Widgets/OutlinerIndicators/SConditionIndicatorWidget.h"
#include "MVVM/Extensions/IConditionableExtension.h"
#include "MVVM/SharedViewModelData.h"
#include "Widgets/Layout/SBorder.h"

#define LOCTEXT_NAMESPACE "FConditionOutlinerIndicatorBuilder"

namespace UE::Sequencer
{

FConditionOutlinerIndicatorBuilder::FConditionOutlinerIndicatorBuilder()
{
}

FName FConditionOutlinerIndicatorBuilder::GetIndicatorName() const
{
	return FCommonOutlinerNames::Condition;
}

bool FConditionOutlinerIndicatorBuilder::IsItemCompatibleWithIndicator(const FCreateOutlinerColumnParams& InParams) const
{
	if (FConditionStateCacheExtension* ConditionStateCache = InParams.OutlinerExtension.AsModel()->GetSharedData()->CastThis<FConditionStateCacheExtension>())
	{
		return EnumHasAnyFlags(ConditionStateCache->GetCachedFlags(InParams.OutlinerExtension), ECachedConditionState::HasCondition | ECachedConditionState::ParentHasCondition | ECachedConditionState::ChildHasCondition | ECachedConditionState::SectionHasCondition);
	}

	return false;
}

TSharedPtr<SWidget> FConditionOutlinerIndicatorBuilder::CreateIndicatorWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow, const TSharedRef<IOutlinerColumn>& OutlinerColumn, const int32 NumCompatibleIndicators)
{
	static const FLinearColor ConditionColor = FLinearColor::FromSRGBColor(FColor(92, 220, 205));

	TSharedRef<SConditionIndicatorWidget> ConditionIndicatorWidget = SNew(SConditionIndicatorWidget, OutlinerColumn, InParams);
	if (NumCompatibleIndicators > 1)
	{
		return SNew(SBorder)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			.Padding(0.0f)
			.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
			.BorderBackgroundColor(ConditionColor);
	}

	return SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SBorder)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.Padding(0.0f)
				.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
				.BorderBackgroundColor_Lambda([WeakWidget = ConditionIndicatorWidget.ToWeakPtr()]() -> FLinearColor
					{
						if (WeakWidget.IsValid())
						{
							return WeakWidget.Pin()->GetIndicatorBackgroundColorAndOpacity().GetSpecifiedColor();
						}
						return FLinearColor(0, 0, 0, 0);
					})
		]

		+ SOverlay::Slot()
		[
			SNew(SBox)
				.WidthOverride(12.f)
				.HeightOverride(12.f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					ConditionIndicatorWidget
				]
		];
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE