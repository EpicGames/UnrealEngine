// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/OutlinerIndicators/OutlinerIndicatorBuilderBase.h"

namespace UE::Sequencer
{

struct FCreateOutlinerColumnParams;
class IOutlinerColumn;
class SConditionIndicatorWidget;

class FConditionOutlinerIndicatorBuilder
	: public FOutlinerIndicatorBuilderBase
{
public:
	FConditionOutlinerIndicatorBuilder();
	
	virtual FName GetIndicatorName() const override;
	virtual bool IsItemCompatibleWithIndicator(const FCreateOutlinerColumnParams& InParams) const override;
	virtual TSharedPtr<SWidget> CreateIndicatorWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow, const TSharedRef<IOutlinerColumn>& OutlinerColumn, const int32 NumCompatibleIndicators) override;
};

} // namespace UE::Sequencer

