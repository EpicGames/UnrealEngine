// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerIndicators/TimeWarpOutlinerIndicatorBuilder.h"

#include "MovieSceneSection.h"
#include "MVVM/Extensions/ISectionOwnerExtension.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/OutlinerColumns/IOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "Variants/MovieSceneTimeWarpVariant.h"
#include "Variants/MovieSceneTimeWarpGetter.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/OutlinerIndicators/STimeWarpIndicatorWidget.h"

#define LOCTEXT_NAMESPACE "FTimeWarpOutlinerIndicatorBuilder"

namespace UE::Sequencer
{

FTimeWarpOutlinerIndicatorBuilder::FTimeWarpOutlinerIndicatorBuilder()
{
}

FName FTimeWarpOutlinerIndicatorBuilder::GetIndicatorName() const
{
	return FCommonOutlinerNames::TimeWarp;
}

bool FTimeWarpOutlinerIndicatorBuilder::IsItemCompatibleWithIndicator(const FCreateOutlinerColumnParams& InParams) const
{
	if (TViewModelPtr<ISectionOwnerExtension> SectionOwner = InParams.OutlinerExtension.ImplicitCast())
	{
		for (UMovieSceneSection* Section : SectionOwner->GetSections())
		{
			if (!Section)
			{
				continue;
			}

			if (FMovieSceneTimeWarpVariant* Variant = Section->GetTimeWarp())
			{
				if (Variant->GetType() == EMovieSceneTimeWarpType::Custom && Variant->AsCustom() != nullptr)
				{
					return true;
				}
			}
		}
	}

	return false;
}

TSharedPtr<SWidget> FTimeWarpOutlinerIndicatorBuilder::CreateIndicatorWidget(const FCreateOutlinerColumnParams& InParams, const TSharedRef<ISequencerTreeViewRow>& TreeViewRow, const TSharedRef<IOutlinerColumn>& OutlinerColumn, const int32 NumCompatibleIndicators)
{
	static const FLinearColor ConditionColor = FLinearColor::FromSRGBColor(FColor(212, 147, 20));

	if (NumCompatibleIndicators > 1)
	{
		return SNew(SBorder)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			.Padding(0.0f)
			.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
			.BorderBackgroundColor_Lambda([this]() -> FLinearColor {  return ConditionColor; });
	}

	return SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SBorder)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.Padding(0.0f)
				.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
				.BorderBackgroundColor_Lambda([this]() -> FLinearColor { return ConditionColor; })
		]

		+ SOverlay::Slot()
		[
			SNew(SBox)
				.WidthOverride(12.f)
				.HeightOverride(12.f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STimeWarpIndicatorWidget, OutlinerColumn, InParams)
				]
		];
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
