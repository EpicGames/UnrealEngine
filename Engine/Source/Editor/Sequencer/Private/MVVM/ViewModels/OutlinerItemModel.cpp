// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/OutlinerItemModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/FolderModel.h"
#include "MVVM/Extensions/IDraggableOutlinerExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Extensions/ITrackRowExtension.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/Selection/Selection.h"
#include "CurveEditor.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISettingsModule.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "ScopedTransaction.h"
#include "Sequencer.h"
#include "SequencerSelectionCurveFilter.h"
#include "SequencerSettings.h"
#include "Styling/AppStyle.h"
#include "Tree/CurveEditorTreeFilter.h"
#include "Tree/SCurveEditorTreePin.h"
#include "Tree/SCurveEditorTreeSelect.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SSpacer.h"
#include "SequencerCommonHelpers.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "PropertyEditorModule.h"
#include "IStructureDetailsView.h"
#include "IStructureDataProvider.h"
#include "Conditions/MovieSceneConditionCustomization.h"
#include "Conditions/MovieSceneDirectorBlueprintConditionCustomization.h"

#define LOCTEXT_NAMESPACE "OutlinerItemModel"

namespace UE
{
namespace Sequencer
{

static bool NodeMatchesTextFilterTerm(TViewModelPtr<const UE::Sequencer::IOutlinerExtension> Node, const FCurveEditorTreeTextFilterTerm& Term)
{
	using namespace UE::Sequencer;

	FCurveEditorTreeTextFilterTerm::FMatchResult Match(Term.ChildToParentTokens);

	while (Node && Match.IsPartialMatch())
	{
		FCurveEditorTreeTextFilterTerm::FMatchResult NewMatch = Match.Match(Node->GetLabel().ToString());
		if (NewMatch.IsAnyMatch())
		{
			// If we matched, keep searching parents using the remaining match result
			Match = NewMatch;
		}
		Node = Node.AsModel()->FindAncestorOfType<const IOutlinerExtension>();
	}

	return Match.IsTotalMatch();
}

void FOutlinerItemModelMixin::AddEvalOptionsPropertyMenuItem(FMenuBuilder& InMenuBuilder, const FBoolProperty* InProperty, TFunction<bool(UMovieSceneTrack*)> InValidator)
{
	auto IsChecked = [InProperty, InValidator](const TArray<UMovieSceneTrack*>& InTracks) -> bool
	{
		return InTracks.ContainsByPredicate(
			[InValidator, InProperty](UMovieSceneTrack* InTrack)
			{
				return (!InValidator || InValidator(InTrack)) && InProperty->GetPropertyValue(InProperty->ContainerPtrToValuePtr<void>(&InTrack->EvalOptions));
			});
	};

	InMenuBuilder.AddMenuEntry(
		InProperty->GetDisplayNameText(),
		InProperty->GetToolTipText(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, InProperty, InValidator, IsChecked]
			{
				FScopedTransaction Transaction(FText::Format(NSLOCTEXT("Sequencer", "TrackNodeSetRoundEvaluation", "Set '{0}'"), InProperty->GetDisplayNameText()));
				const TArray<UMovieSceneTrack*> AllTracks = GetSelectedTracks();
				for (UMovieSceneTrack* Track : AllTracks)
				{
					if (InValidator && !InValidator(Track))
					{
						continue;
					}
					void* PropertyContainer = InProperty->ContainerPtrToValuePtr<void>(&Track->EvalOptions);
					Track->Modify();
					InProperty->SetPropertyValue(PropertyContainer, !IsChecked(AllTracks));
				}
			}),
			FCanExecuteAction::CreateLambda([this]
			{
				TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
				if (!EditorViewModel)
				{
					return false;
				}
				const TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();
				if (Sequencer)
				{
					return !Sequencer->IsReadOnly();
				}
				return false;
			}),
			FIsActionChecked::CreateLambda([this, IsChecked]
			{
				const TArray<UMovieSceneTrack*> AllTracks = GetSelectedTracks();
				return IsChecked(AllTracks);
			})
		),
		NAME_None,
		EUserInterfaceActionType::Check
	);
}

void FOutlinerItemModelMixin::AddDisplayOptionsPropertyMenuItem(FMenuBuilder& InMenuBuilder, const FBoolProperty* InProperty, TFunction<bool(UMovieSceneTrack*)> InValidator)
{
	auto IsChecked = [InProperty, InValidator](const TArray<UMovieSceneTrack*>& InTracks) -> bool
	{
		return InTracks.ContainsByPredicate(
			[InValidator, InProperty](UMovieSceneTrack* InTrack)
			{
				return (!InValidator || InValidator(InTrack)) && InProperty->GetPropertyValue(InProperty->ContainerPtrToValuePtr<void>(&InTrack->DisplayOptions));
			});
	};

	InMenuBuilder.AddMenuEntry(
		InProperty->GetDisplayNameText(),
		InProperty->GetToolTipText(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, InProperty, InValidator, IsChecked] {
				FScopedTransaction Transaction(FText::Format(NSLOCTEXT("Sequencer", "TrackNodeSetDisplayOption", "Set '{0}'"), InProperty->GetDisplayNameText()));
				const TArray<UMovieSceneTrack*> AllTracks = GetSelectedTracks();
				for (UMovieSceneTrack* Track : AllTracks)
				{
					if (InValidator && !InValidator(Track))
					{
						continue;
					}
					void* PropertyContainer = InProperty->ContainerPtrToValuePtr<void>(&Track->DisplayOptions);
					Track->Modify();
					InProperty->SetPropertyValue(PropertyContainer, !IsChecked(AllTracks));
				}
			}),
			FCanExecuteAction::CreateLambda([this]
			{
				TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
				if (!EditorViewModel)
				{
					return false;
				}
				const TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();
				if (Sequencer)
				{
					return !Sequencer->IsReadOnly();
				}
				return false;
			}),
			FIsActionChecked::CreateLambda([this, IsChecked]
			{
				const TArray<UMovieSceneTrack*> AllTracks = GetSelectedTracks();
				return IsChecked(AllTracks);
			})
		),
		NAME_None,
		EUserInterfaceActionType::Check
	);
}

FOutlinerItemModelMixin::FOnPinStateChanged FOutlinerItemModelMixin::PinStateChangedDelegate;

FOutlinerItemModelMixin::FOutlinerItemModelMixin()
	: OutlinerChildList(EViewModelListType::Outliner)
	, bInitializedExpansion(false)
	, bInitializedPinnedState(false)
{
}

TSharedPtr<FSequencerEditorViewModel> FOutlinerItemModelMixin::GetEditor() const
{
	TSharedPtr<FSequenceModel> SequenceModel = AsViewModel()->FindAncestorOfType<FSequenceModel>();
	return SequenceModel ? SequenceModel->GetEditor() : nullptr;
}

FName FOutlinerItemModelMixin::GetIdentifier() const
{
	return TreeItemIdentifier;
}

void FOutlinerItemModelMixin::SetIdentifier(FName InNewIdentifier)
{
	TreeItemIdentifier = InNewIdentifier;

	const FViewModel* ViewModel = AsViewModel();
	if (ViewModel && ViewModel->IsConstructed())
	{
		TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
		if (EditorViewModel)
		{
			EditorViewModel->HandleDataHierarchyChanged();
		}
	}
}

bool FOutlinerItemModelMixin::IsExpanded() const
{
	const FViewModel* ViewModel = AsViewModel();

	if (!bInitializedExpansion)
	{
		bInitializedExpansion = true;

		TStringBuilder<256> StringBuilder;
		IOutlinerExtension::GetPathName(*ViewModel, StringBuilder);

		TSharedPtr<FSequenceModel> SequenceModel = ViewModel->FindAncestorOfType<FSequenceModel>();
		UMovieSceneSequence*       Sequence      = SequenceModel ? SequenceModel->GetSequence() : nullptr;
		UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;

		if (MovieScene)
		{
			FStringView StringView = StringBuilder.ToView();
			FMovieSceneEditorData& EditorData = MovieScene->GetEditorData();
			if (const FMovieSceneExpansionState* Expansion = EditorData.ExpansionStates.FindByHash(GetTypeHash(StringView), StringView))
			{
				const_cast<FOutlinerItemModelMixin*>(this)->bIsExpanded = Expansion->bExpanded;
			}
			else
			{
				const_cast<FOutlinerItemModelMixin*>(this)->bIsExpanded = GetDefaultExpansionState();
			}
		}
	}

	if (bIsExpanded)
	{
		// If there are no children, no need to allow this to be expanded
		FViewModelListIterator OutlinerChildren = ViewModel->GetChildren(EViewModelListType::Outliner);
		if (OutlinerChildren)
		{
			return true;
		}
	}

	return false;
}

bool FOutlinerItemModelMixin::GetDefaultExpansionState() const
{
	return false;
}

void FOutlinerItemModelMixin::SetExpansion(bool bInIsExpanded)
{
	FViewModel* ViewModel = AsViewModel();

	// If no children, there's no need to set this expanded
	FViewModelListIterator OutlinerChildren = ViewModel->GetChildren(EViewModelListType::Outliner);
	if (!OutlinerChildren)
	{
		return;
	}

	SetExpansionWithoutSaving(bInIsExpanded);

	if (ViewModel->GetParent())
	{
		// Expansion state has changed, save it to the movie scene now
		TSharedPtr<FSequenceModel> SequenceModel = ViewModel->FindAncestorOfType<FSequenceModel>();
		if (SequenceModel)
		{
			TSharedPtr<FSequencer> Sequencer = SequenceModel->GetSequencerImpl();
			if (Sequencer)
			{
				Sequencer->GetNodeTree()->SaveExpansionState(*ViewModel, bInIsExpanded);
			}
		}
	}
}

void FOutlinerItemModelMixin::SetExpansionWithoutSaving(bool bInIsExpanded)
{
	FOutlinerExtensionShim::SetExpansion(bInIsExpanded);

	// Force this flag in case a sub-class wants a given expansion state before the
	// getter is called.
	bInitializedExpansion = true;
}

bool FOutlinerItemModelMixin::IsFilteredOut() const
{
	return bIsFilteredOut;
}

bool FOutlinerItemModelMixin::IsPinned() const
{
	if (bInitializedPinnedState)
	{
		return FPinnableExtensionShim::IsPinned();
	}

	bInitializedPinnedState = true;

	// Initialize expansion states for tree items
	// Assign the saved expansion state when this node is initialized for the first time
	const bool bIsRootModel = (AsViewModel()->GetHierarchicalDepth() == 1);
	if (bIsRootModel)
	{
		TSharedPtr<FSequenceModel> SequenceModel = AsViewModel()->FindAncestorOfType<FSequenceModel>();
		TSharedPtr<FSequencer> Sequencer = SequenceModel->GetSequencerImpl();
		if (Sequencer)
		{
			const bool bWasPinned = Sequencer->GetNodeTree()->GetSavedPinnedState(*AsViewModel());		
			const_cast<FOutlinerItemModelMixin*>(this)->FPinnableExtensionShim::SetPinned(bWasPinned);
		}
	}

	return FPinnableExtensionShim::IsPinned();
}

void FOutlinerItemModelMixin::SetPinned(const bool bInIsPinned)
{
	FPinnableExtensionShim::SetPinned(bInIsPinned);

	PinStateChangedDelegate.Broadcast(AsViewModel()->CastThisShared<IPinnableExtension>(), bInIsPinned);
}

bool FOutlinerItemModelMixin::IsDimmed() const
{
	const FViewModel* const ViewModel = AsViewModel();
	const TSharedPtr<FSharedViewModelData> SharedData = ViewModel->GetSharedData();
	if (!SharedData.IsValid())
	{
		return false;
	}

	const FDeactiveStateCacheExtension* const DeactiveState = SharedData->CastThis<FDeactiveStateCacheExtension>();
	const FMuteStateCacheExtension* const MuteState = SharedData->CastThis<FMuteStateCacheExtension>();
	const FSoloStateCacheExtension* const SoloState = SharedData->CastThis<FSoloStateCacheExtension>();

	check(DeactiveState && MuteState && SoloState);

	const uint32 ModelID = ViewModel->GetModelID();

	const ECachedDeactiveState DeactiveFlags = DeactiveState->GetCachedFlags(ModelID);
	const ECachedMuteState MuteFlags = MuteState->GetCachedFlags(ModelID);
	const ECachedSoloState SoloFlags = SoloState->GetCachedFlags(ModelID);

	const bool bIsDeactive   = EnumHasAnyFlags(DeactiveFlags, ECachedDeactiveState::Deactivated | ECachedDeactiveState::ImplicitlyDeactivatedByParent);
	const bool bAnySoloNodes = EnumHasAnyFlags(SoloState->GetRootFlags(), ECachedSoloState::Soloed | ECachedSoloState::PartiallySoloedChildren);
	const bool bIsMuted      = EnumHasAnyFlags(MuteFlags, ECachedMuteState::Muted  | ECachedMuteState::ImplicitlyMutedByParent);
	const bool bIsSoloed     = EnumHasAnyFlags(SoloFlags, ECachedSoloState::Soloed | ECachedSoloState::ImplicitlySoloedByParent);

	const bool bDisableEval = bIsDeactive || bIsMuted || (bAnySoloNodes && !bIsSoloed);
	return bDisableEval;
}

bool FOutlinerItemModelMixin::IsRootModelPinned() const
{
	TSharedPtr<IPinnableExtension> PinnableParent = AsViewModel()->FindAncestorOfType<IPinnableExtension>(true);
	return PinnableParent && PinnableParent->IsPinned();
}

void FOutlinerItemModelMixin::ToggleRootModelPinned()
{
	FSequenceModel* RootModel = AsViewModel()->GetRoot()->CastThis<FSequenceModel>();
	TSharedPtr<IPinnableExtension> PinnableParent = AsViewModel()->FindAncestorOfType<IPinnableExtension>(true);
	if (RootModel && PinnableParent)
	{
		TSharedPtr<FOutlinerViewModel> Outliner = RootModel->GetEditor()->GetOutliner();
		Outliner->UnpinAllNodes();

		const bool bShouldPin = !PinnableParent->IsPinned();
		PinnableParent->SetPinned(bShouldPin);

		TSharedPtr<FSequencer> Sequencer = RootModel->GetSequencerImpl();
		if (Sequencer)
		{
			Sequencer->GetNodeTree()->SavePinnedState(*AsViewModel(), bShouldPin);
			Sequencer->RefreshTree();
		}
	}
}

ECheckBoxState FOutlinerItemModelMixin::SelectedModelsSoloState() const
{
	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel)
	{
		return ECheckBoxState::Unchecked;
	}

	const TSharedPtr<FSequencerSelection> Selection = EditorViewModel->GetSelection();
	if (!Selection.IsValid())
	{
		return ECheckBoxState::Unchecked;
	}

	FSoloStateCacheExtension* SoloStateCache = AsViewModel()->GetSharedData()->CastThis<FSoloStateCacheExtension>();
	check(SoloStateCache);

	int32 NumSoloables = 0;
	int32 NumSoloed = 0;
	for (FViewModelPtr Soloable : Selection->Outliner.Filter<ISoloableExtension>())
	{
		++NumSoloables;
		if (EnumHasAnyFlags(SoloStateCache->GetCachedFlags(Soloable), ECachedSoloState::Soloed))
		{
			++NumSoloed;
		}
	}

	if (NumSoloed == 0)
	{
		return ECheckBoxState::Unchecked;
	}
	return NumSoloables == NumSoloed ? ECheckBoxState::Checked : ECheckBoxState::Undetermined;
}

void FOutlinerItemModelMixin::ToggleSelectedModelsSolo()
{
	ECheckBoxState CurrentState = SelectedModelsSoloState();
	const bool bNewSoloState = CurrentState != ECheckBoxState::Checked;

	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "ToggleSolo", "Toggle Solo"));

	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	for (TViewModelPtr<ISoloableExtension> Soloable : EditorViewModel->GetSelection()->Outliner.Filter<ISoloableExtension>())
	{
		Soloable->SetIsSoloed(bNewSoloState);
	}

	TSharedPtr<ISequencer> Sequencer = EditorViewModel->GetSequencer();
	if (Sequencer)
	{
		Sequencer->RefreshTree();
	}
}

ECheckBoxState FOutlinerItemModelMixin::SelectedModelsMuteState() const
{
	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel)
	{
		return ECheckBoxState::Unchecked;
	}

	const TSharedPtr<FSequencerSelection> Selection = EditorViewModel->GetSelection();
	if (!Selection.IsValid())
	{
		return ECheckBoxState::Unchecked;
	}

	FMuteStateCacheExtension* MuteStateCache = AsViewModel()->GetSharedData()->CastThis<FMuteStateCacheExtension>();
	check(MuteStateCache);

	int32 NumMutables = 0;
	int32 NumMuted = 0;
	for (FViewModelPtr Mutable : Selection->Outliner.Filter<IMutableExtension>())
	{
		++NumMutables;
		if (EnumHasAnyFlags(MuteStateCache->GetCachedFlags(Mutable), ECachedMuteState::Muted))
		{
			++NumMuted;
		}
	}

	if (NumMuted == 0)
	{
		return ECheckBoxState::Unchecked;
	}
	return NumMutables == NumMuted ? ECheckBoxState::Checked : ECheckBoxState::Undetermined;
}

void FOutlinerItemModelMixin::ToggleSelectedModelsMuted()
{
	ECheckBoxState CurrentState = SelectedModelsMuteState();
	const bool bNewMuteState = CurrentState != ECheckBoxState::Checked;

	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "ToggleMute", "Toggle Mute"));

	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	for (TViewModelPtr<IMutableExtension> Muteable : EditorViewModel->GetSelection()->Outliner.Filter<IMutableExtension>())
	{
		Muteable->SetIsMuted(bNewMuteState);
	}

	TSharedPtr<ISequencer> Sequencer = EditorViewModel->GetSequencer();
	if (Sequencer)
	{
		Sequencer->RefreshTree();
	}
}

TSharedPtr<SWidget> FOutlinerItemModelMixin::CreateContextMenuWidget(const FCreateOutlinerContextMenuWidgetParams& InParams)
{
	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel)
	{
		return nullptr;
	}

	TSharedPtr<ISequencer> Sequencer = EditorViewModel->GetSequencer();
	if (!Sequencer)
	{
		return nullptr;
	}

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, Sequencer->GetCommandBindings());

	BuildContextMenu(MenuBuilder);

	return MenuBuilder.MakeWidget();
}

FSlateColor FOutlinerItemModelMixin::GetLabelColor() const
{
	if (TViewModelPtr<FSequenceModel> SequenceModel = AsViewModel()->FindAncestorOfType<FSequenceModel>())
	{
		if (TSharedPtr<FSequencerEditorViewModel> SequencerModel = SequenceModel->GetEditor())
		{
			if (IMovieScenePlayer* Player = SequencerModel->GetSequencer().Get())
			{
				if (TViewModelPtr<FObjectBindingModel> ObjectBindingModel = AsViewModel()->FindAncestorOfType<FObjectBindingModel>())
				{
					// If the object binding model has an invalid binding, we want to use its label color, as it may be red or gray depending on situation
					// and we want the children of that to have the same color.
					// Otherwise, we can use the track's label color below
					TArrayView<TWeakObjectPtr<> > BoundObjects = Player->FindBoundObjects(ObjectBindingModel->GetObjectGuid(), SequenceModel->GetSequenceID());
					if (BoundObjects.Num() == 0)
					{
						return ObjectBindingModel->GetLabelColor();
					}
				}
			}
		}
	}
	return IOutlinerExtension::GetLabelColor();
}

void FOutlinerItemModelMixin::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel)
	{
		return;
	}

	TSharedPtr<FSequencer> Sequencer = StaticCastSharedPtr<FSequencer>(EditorViewModel->GetSequencer());
	if (!Sequencer)
	{
		return;
	}

	TSharedRef<FOutlinerItemModelMixin> SharedThis(AsViewModel()->AsShared(), this);

	const bool bIsReadOnly = Sequencer->IsReadOnly();
	FCanExecuteAction CanExecute = FCanExecuteAction::CreateLambda([bIsReadOnly]{ return !bIsReadOnly; });

	MenuBuilder.BeginSection("Edit", LOCTEXT("EditContextMenuSectionName", "Edit"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleNodeLock", "Locked"),
			LOCTEXT("ToggleNodeLockTooltip", "Lock or unlock this node or selected tracks"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(Sequencer.Get(), &FSequencer::ToggleNodeLocked),
				CanExecute,
				FIsActionChecked::CreateSP(Sequencer.Get(), &FSequencer::IsNodeLocked)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		// Only support pinning root nodes
		const bool bIsRootModel = (AsViewModel()->GetHierarchicalDepth() == 1);
		if (bIsRootModel)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ToggleNodePin", "Pinned"),
				LOCTEXT("ToggleNodePinTooltip", "Pin or unpin this node or selected tracks"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(SharedThis, &FOutlinerItemModelMixin::ToggleRootModelPinned),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(SharedThis, &FOutlinerItemModelMixin::IsRootModelPinned)
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}

		// We already know we are soloable and mutable
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleNodeSolo", "Solo"),
			LOCTEXT("ToggleNodeSoloTooltip", "Solo or unsolo this node or selected tracks"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(SharedThis, &FOutlinerItemModelMixin::ToggleSelectedModelsSolo),
				CanExecute,
				FGetActionCheckState::CreateSP(SharedThis, &FOutlinerItemModelMixin::SelectedModelsSoloState)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleNodeMute", "Mute"),
			LOCTEXT("ToggleNodeMuteTooltip", "Mute or unmute this node or selected tracks"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(SharedThis, &FOutlinerItemModelMixin::ToggleSelectedModelsMuted),
				CanExecute,
				FGetActionCheckState::CreateSP(SharedThis, &FOutlinerItemModelMixin::SelectedModelsMuteState)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		// Add cut, copy and paste functions to the tracks
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);

		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
		
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
		
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
		
		TSharedRef<FViewModel> ThisNode = AsViewModel()->AsShared();

		MenuBuilder.AddMenuEntry(
			LOCTEXT("DeleteNode", "Delete"),
			LOCTEXT("DeleteNodeTooltip", "Delete this or selected tracks"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.Delete"),
			FUIAction(FExecuteAction::CreateSP(Sequencer.Get(), &FSequencer::DeleteNode, ThisNode, false), CanExecute)
		);

		if (ThisNode->IsA<IObjectBindingExtension>())
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("DeleteNodeAndKeepState", "Delete and Keep State"),
				LOCTEXT("DeleteNodeAndKeepStateTooltip", "Delete this object's tracks and keep its current animated state"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.Delete"),
				FUIAction(FExecuteAction::CreateSP(Sequencer.Get(), &FSequencer::DeleteNode, ThisNode, true), CanExecute)
			);
		}

		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Organize", LOCTEXT("OrganizeContextMenuSectionName", "Organize"));
	BuildOrganizeContextMenu(MenuBuilder);
	MenuBuilder.EndSection();

	const TArray<UMovieSceneTrack*> AllTracks = GetSelectedTracks();
	if (AllTracks.Num())
	{
		BuildTrackOptionsMenu(MenuBuilder, AllTracks);
		BuildTrackRowOptionsMenu(MenuBuilder);
		BuildDisplayOptionsMenu(MenuBuilder);
	}
}

void FOutlinerItemModelMixin::BuildOrganizeContextMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();
	if (!Sequencer.IsValid())
	{
		return;
	}

	FSequencer* const SequencerRaw = Sequencer.Get();
	TSharedRef<FViewModel> ThisNode = AsViewModel()->AsShared();

	const bool bFilterableNode = (ThisNode->IsA<ITrackExtension>() || ThisNode->IsA<ITrackRowExtension>() || ThisNode->IsA<IObjectBindingExtension>() || ThisNode->IsA<FFolderModel>());
	const bool bIsReadOnly = Sequencer->IsReadOnly();
	
	TArray<UMovieSceneTrack*> AllTracks;
	TArray<TSharedPtr<FViewModel> > DraggableNodes;
	for (const FViewModelPtr Node : EditorViewModel->GetSelection()->Outliner)
	{
		if (ITrackExtension* TrackExtension = Node->CastThis<ITrackExtension>())
		{
			UMovieSceneTrack* Track = TrackExtension->GetTrack();
			if (Track)
			{
				AllTracks.AddUnique(Track);
			}
		}

		if (ITrackRowExtension* TrackRowExtension = Node->CastThis<ITrackRowExtension>())
		{
			UMovieSceneTrack* Track = TrackRowExtension->GetParentTrack();
			if (Track)
			{
				AllTracks.AddUnique(Track);
			}
		}

		if (IDraggableOutlinerExtension* DraggableExtension = Node->CastThis<IDraggableOutlinerExtension>())
		{
			if (DraggableExtension->CanDrag())
			{
				DraggableNodes.Add(Node);
			}
		}
	}

	if (bFilterableNode && !bIsReadOnly)
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("AddNodesToNodeGroup", "Add to Group"),
			LOCTEXT("AddNodesToNodeGroupTooltip", "Add selected nodes to a group"),
			FNewMenuDelegate::CreateSP(SequencerRaw, &FSequencer::BuildAddSelectedToNodeGroupMenu));
	}

	if (DraggableNodes.Num() && !bIsReadOnly)
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("MoveToFolder", "Move to Folder"),
			LOCTEXT("MoveToFolderTooltip", "Move the selected nodes to a folder"),
			FNewMenuDelegate::CreateSP(SequencerRaw, &FSequencer::BuildAddSelectedToFolderMenu));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("RemoveFromFolder", "Remove from Folder"),
			LOCTEXT("RemoveFromFolderTooltip", "Remove selected nodes from their folders"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(SequencerRaw, &FSequencer::RemoveSelectedNodesFromFolders),
				FCanExecuteAction::CreateLambda([SequencerRaw] { return SequencerRaw->GetSelectedNodesInFolders().Num() > 0; } )));
	}

	if (!bIsReadOnly)
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("SortBy", "Sort by"),
			LOCTEXT("SortByTooltip", "Sort the selected tracks by start time of the first layer bar"),
			FNewMenuDelegate::CreateSP(SequencerRaw, &FSequencer::BuildSortMenu));
	}
}

void FOutlinerItemModelMixin::BuildDisplayOptionsMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = EditorViewModel->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedRef<FOutlinerItemModelMixin> SharedThis(AsViewModel()->AsShared(), this);

	const bool bIsReadOnly = Sequencer->IsReadOnly();
	const FCanExecuteAction CanExecute = FCanExecuteAction::CreateLambda([bIsReadOnly]{ return !bIsReadOnly; });

	TArray<UMovieSceneTrack*> AllTracks = GetSelectedTracks();
	if (AllTracks.IsEmpty())
	{
		return;
	}

	MenuBuilder.BeginSection(TEXT("TrackDisplayOptions"), LOCTEXT("TrackNodeDisplayOptions", "Display Options"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("SetColorTint", "Set Color Tint"),
			LOCTEXT("SetColorTintTooltip", "Set color tint from the preferences for the selected sections or the track's sections"),
			FNewMenuDelegate::CreateSP(SharedThis, &FOutlinerItemModelMixin::BuildSectionColorTintsMenu));

		UStruct* const DisplayOptionsStruct = FMovieSceneTrackDisplayOptions::StaticStruct();

		const FBoolProperty* const ShowVerticalFramesProperty = CastField<FBoolProperty>(DisplayOptionsStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMovieSceneTrackDisplayOptions, bShowVerticalFrames)));
		if (ShowVerticalFramesProperty)
		{
			AddDisplayOptionsPropertyMenuItem(MenuBuilder, ShowVerticalFramesProperty);
		}
	}
	MenuBuilder.EndSection();
}

void FOutlinerItemModelMixin::BuildTrackOptionsMenu(FMenuBuilder& MenuBuilder, const TArray<UMovieSceneTrack*>& InTracks)
{
	if (InTracks.IsEmpty())
	{
		return;
	}
	
	MenuBuilder.BeginSection(TEXT("GeneralTrackOptions"), LOCTEXT("TrackNodeGeneralOptions", "Track Options"));
	{
		UStruct* const EvalOptionsStruct = FMovieSceneTrackEvalOptions::StaticStruct();

		const FBoolProperty* const NearestSectionProperty = CastField<FBoolProperty>(EvalOptionsStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMovieSceneTrackEvalOptions, bEvalNearestSection)));
		auto CanEvaluateNearest = [](const UMovieSceneTrack* const InTrack) { return InTrack->EvalOptions.bCanEvaluateNearestSection != 0; };
		if (NearestSectionProperty && InTracks.ContainsByPredicate(CanEvaluateNearest))
		{
			TFunction<bool(UMovieSceneTrack*)> Validator = CanEvaluateNearest;
			AddEvalOptionsPropertyMenuItem(MenuBuilder, NearestSectionProperty, Validator);
		}

		const FBoolProperty* const PrerollProperty = CastField<FBoolProperty>(EvalOptionsStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMovieSceneTrackEvalOptions, bEvaluateInPreroll)));
		if (PrerollProperty)
		{
			AddEvalOptionsPropertyMenuItem(MenuBuilder, PrerollProperty);
		}

		const FBoolProperty* const PostrollProperty = CastField<FBoolProperty>(EvalOptionsStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMovieSceneTrackEvalOptions, bEvaluateInPostroll)));
		if (PostrollProperty)
		{
			AddEvalOptionsPropertyMenuItem(MenuBuilder, PostrollProperty);
		}
	}
	MenuBuilder.EndSection();
}

void FOutlinerItemModelMixin::BuildTrackRowOptionsMenu(FMenuBuilder& MenuBuilder)
{
	// Don't show track row metadata if we don't allow conditions, as for now this is the only item in track row metadata
	FViewModel* ViewModel = AsViewModel();
	TSharedPtr<FSequenceModel> SequenceModel = ViewModel ? ViewModel->FindAncestorOfType<FSequenceModel>() : nullptr;

	if (SequenceModel)
	{
		if (UMovieScene* MovieScene = SequenceModel->GetMovieScene())
		{
			if (!MovieScene->IsConditionClassAllowed(UMovieSceneCondition::StaticClass()))
			{
				return;	
			}
		}
	}
	
	TArray<TPair<UMovieSceneTrack*, int32>> AllTrackRows = GetSelectedTrackRows();
	if (AllTrackRows.IsEmpty())
	{
		return;
	}

	// Only show track row options for tracks that allow multiple rows
	if (Algo::AnyOf(AllTrackRows, [](const TPair<UMovieSceneTrack*, int32> TrackRow) {
		return TrackRow.Key && !TrackRow.Key->SupportsMultipleRows();
		}))
	{
		return;
	}

	MenuBuilder.BeginSection(TEXT("TrackRowMetadata"));
	{
		// Empty here, will be implemented by extension.
	}
	MenuBuilder.EndSection();
}

void FOutlinerItemModelMixin::BuildSidebarMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection(TEXT("Organize"), LOCTEXT("OrganizeContextMenuSectionName", "Organize"));
	BuildOrganizeContextMenu(MenuBuilder);
	MenuBuilder.EndSection();

	BuildTrackOptionsMenu(MenuBuilder, GetSelectedTracks());
	BuildTrackRowOptionsMenu(MenuBuilder);
	BuildDisplayOptionsMenu(MenuBuilder);
}

TArray<UMovieSceneSection*> FOutlinerItemModelMixin::GetSelectedSections() const
{
	TArray<UMovieSceneSection*> Sections;

	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return Sections;
	}

	const TSharedPtr<ISequencer> Sequencer = EditorViewModel->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return Sections;
	}

	const TSharedPtr<FSequencerSelection> Selection = EditorViewModel->GetSelection();
	if (!Selection.IsValid())
	{
		return Sections;
	}

	for (const TViewModelPtr<FSectionModel> SectionModel : Selection->Outliner.Filter<FSectionModel>())
	{
		if (UMovieSceneSection* const Section = SectionModel->GetSection())
		{
			Sections.Add(Section);
		}
	}

	if (Sections.IsEmpty())
	{
		for (const TViewModelPtr<ISectionOwnerExtension> SectionOwner : Selection->Outliner.Filter<ISectionOwnerExtension>())
		{
			for (UMovieSceneSection* const Section : SectionOwner->GetSections())
			{
				Sections.Add(Section);
			}
		}
	}

	return Sections;
}

TArray<UMovieSceneTrack*> FOutlinerItemModelMixin::GetSelectedTracks() const
{
	TArray<UMovieSceneTrack*> AllTracks;

	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return AllTracks;
	}

	const TSharedPtr<ISequencer> Sequencer = EditorViewModel->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return AllTracks;
	}

	const TSharedPtr<FSequencerSelection> Selection = EditorViewModel->GetSelection();
	if (!Selection.IsValid())
	{
		return AllTracks;
	}

	return Selection->GetSelectedTracks().Array();
}

TArray<TPair<UMovieSceneTrack*, int32>> FOutlinerItemModelMixin::GetSelectedTrackRows() const
{
	TArray<TPair<UMovieSceneTrack*, int32>> AllTrackRows;

	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return AllTrackRows;
	}

	const TSharedPtr<FSequencerSelection> Selection = EditorViewModel->GetSelection();
	if (!Selection.IsValid())
	{
		return AllTrackRows;
	}

	for (const TViewModelPtr<ITrackRowExtension> TrackRowExtension : Selection->Outliner.Filter<ITrackRowExtension>())
	{
		UMovieSceneTrack* const Track = TrackRowExtension->GetParentTrack();
		if (IsValid(Track))
		{
			AllTrackRows.Add(TPair<UMovieSceneTrack*, int32>(Track, TrackRowExtension->GetRowIndex()));
		}
	}

	return AllTrackRows;
}

void FOutlinerItemModelMixin::BuildSectionColorTintsMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TArray<UMovieSceneSection*> Sections = GetSelectedSections();
	if (Sections.IsEmpty())
	{
		return;
	}

	const TWeakPtr<FSequencer> WeakSequencer = Sequencer;

	FCanExecuteAction CanExecuteAction = FCanExecuteAction::CreateLambda([WeakSequencer]
		{
			return WeakSequencer.IsValid() ? !WeakSequencer.Pin()->IsReadOnly() : false;
		});

	const TArray<FColor> SectionColorTints = Sequencer->GetSequencerSettings()->GetSectionColorTints();

	for (const FColor& SectionColorTint : SectionColorTints)
	{
		TSharedPtr<SBox> ColorWidget = SNew(SBox)
			.WidthOverride(70.f)
			.HeightOverride(20.f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush(TEXT("WhiteBrush")))
				.BorderBackgroundColor(FLinearColor::FromSRGBColor(SectionColorTint))
			];

		MenuBuilder.AddMenuEntry(
			FUIAction(
				FExecuteAction::CreateLambda([this, WeakSequencer, SectionColorTint]
					{
						const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
						if (!Sequencer)
						{
							return;
						}

						const TArray<UMovieSceneSection*> Sections = GetSelectedSections();
						if (Sections.IsEmpty())
						{
							return;
						}

						Sequencer->SetSectionColorTint(Sections, SectionColorTint);
					}),
				CanExecuteAction),
			ColorWidget.ToSharedRef());
	}

	MenuBuilder.AddSeparator();

	// Clear any assigned color tints
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ClearColorTintLabel", "Clear"),
		LOCTEXT("ClearColorTintTooltip", "Clear any assigned color tints"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this, WeakSequencer]
				{
					const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
					if (!Sequencer)
					{
						return;
					}

					const TArray<UMovieSceneSection*> Sections = GetSelectedSections();
					if (Sections.IsEmpty())
					{
						return;
					}

					Sequencer->SetSectionColorTint(Sections, FColor(0, 0, 0, 0));
				}),
			CanExecuteAction));

	// Pop up preferences to edit custom color tints
	MenuBuilder.AddMenuEntry(
		LOCTEXT("EditColorTintLabel", "Edit Color Tints..."),
		LOCTEXT("EditColorTintTooltip", "Edit the custom color tints"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([WeakSequencer]
			{
				const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
				if (!Sequencer)
				{
					return;
				}

				const USequencerSettings* SequencerSettings = Sequencer->GetSequencerSettings();
				if (!IsValid(SequencerSettings))
				{
					return;
				}

				ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>(TEXT("Settings"));
				SettingsModule.ShowViewer("Editor", "ContentEditors", *SequencerSettings->GetName()); 
			})
		));
}

bool FOutlinerItemModelMixin::HasCurves() const
{
	return false;
}

TOptional<FString> FOutlinerItemModelMixin::GetUniquePathName() const
{
	TStringBuilder<256> StringBuilder;
	IOutlinerExtension::GetPathName(*AsViewModel(), StringBuilder);
	FString PathName(StringBuilder.ToString());
	TOptional<FString> FullPathName = PathName;
	return FullPathName;
}

TSharedPtr<ICurveEditorTreeItem> FOutlinerItemModelMixin::GetCurveEditorTreeItem() const
{
	TSharedRef<FViewModel> ThisShared(const_cast<FViewModel*>(AsViewModel())->AsShared());
	return TSharedPtr<ICurveEditorTreeItem>(ThisShared, const_cast<FOutlinerItemModelMixin*>(this));
}

TSharedPtr<SWidget> FOutlinerItemModelMixin::GenerateCurveEditorTreeWidget(const FName& InColumnName, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID, const TSharedRef<ITableRow>& TableRow)
{
	using namespace UE::Sequencer;

	TSharedRef<FOutlinerItemModelMixin> SharedThis(AsViewModel()->AsShared(), this);

	auto GetCurveEditorHighlightText = [](TWeakPtr<FCurveEditor> InCurveEditor) -> FText 
	{
		TSharedPtr<FCurveEditor> PinnedCurveEditor = InCurveEditor.Pin();
		if (!PinnedCurveEditor)
		{
			return FText::GetEmpty();
		}

		const FCurveEditorTreeFilter* Filter = PinnedCurveEditor->GetTree()->FindFilterByType(ECurveEditorTreeFilterType::Text);
		if (Filter)
		{
			return static_cast<const FCurveEditorTreeTextFilter*>(Filter)->InputText;
		}

		return FText::GetEmpty();
	};

	if (InColumnName == ColumnNames.Label)
	{
		return SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image(SharedThis, &FOutlinerItemModelMixin::GetIconBrush)
					.ColorAndOpacity(SharedThis, &FOutlinerItemModelMixin::GetIconTint)
				]

				+ SOverlay::Slot()
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Right)
				[
					SNew(SImage)
					.Image(SharedThis, &FOutlinerItemModelMixin::GetIconOverlayBrush)
				]

				+ SOverlay::Slot()
				[
					SNew(SSpacer)
					.Visibility(EVisibility::Visible)
					.ToolTipText(SharedThis, &FOutlinerItemModelMixin::GetIconToolTipText)
				]
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 4.f, 0.f, 4.f))
			[
				SNew(STextBlock)
				.Text(SharedThis, &FOutlinerItemModelMixin::GetLabel)
				.Font(SharedThis, &FOutlinerItemModelMixin::GetLabelFont)
				.HighlightText_Static(GetCurveEditorHighlightText, InCurveEditor)
				.ToolTipText(SharedThis, &FOutlinerItemModelMixin::GetLabelToolTipText)
			];
	}
	else if (InColumnName == ColumnNames.SelectHeader)
	{
		return SNew(SCurveEditorTreeSelect, InCurveEditor, InTreeItemID, TableRow);
	}
	else if (InColumnName == ColumnNames.PinHeader)
	{
		return SNew(SCurveEditorTreePin, InCurveEditor, InTreeItemID, TableRow);
	}

	return nullptr;
}

void FOutlinerItemModelMixin::CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels)
{
}

bool FOutlinerItemModelMixin::PassesFilter(const FCurveEditorTreeFilter* InFilter) const
{
	if (InFilter->GetType() == ECurveEditorTreeFilterType::Text)
	{
		const FCurveEditorTreeTextFilter* Filter = static_cast<const FCurveEditorTreeTextFilter*>(InFilter);

		TViewModelPtr<const IOutlinerExtension> This = AsViewModel()->CastThisShared<IOutlinerExtension>();

		// Must match all text tokens
		for (const FCurveEditorTreeTextFilterTerm& Term : Filter->GetTerms())
		{
			if (!NodeMatchesTextFilterTerm(This, Term))
			{
				return false;
			}
		}

		return true;
	}
	else if (InFilter->GetType() == ISequencerModule::GetSequencerSelectionFilterType())
	{
		const FSequencerSelectionCurveFilter* Filter = static_cast<const FSequencerSelectionCurveFilter*>(InFilter);
		return Filter->Match(AsViewModel()->AsShared());
	}
	return false;
}

bool FEvaluableOutlinerItemModel::IsDeactivated() const
{
	const TParentFirstChildIterator<ITrackExtension> TrackDescendants = GetDescendantsOfType<ITrackExtension>(true);
	const TParentFirstChildIterator<ITrackRowExtension> TrackRowDescendants = GetDescendantsOfType<ITrackRowExtension>(true);
	
	if (!TrackDescendants && !TrackRowDescendants)
	{
		return false;
	}



	bool bNoTrackAreaModels = true;

	for (const TViewModelPtr<ITrackExtension>& TrackNode : TrackDescendants)
	{
		const UMovieSceneTrack* const Track = TrackNode->GetTrack();
		if (!IsValid(Track))
		{
			continue;
		}

		const TViewModelPtr<ITrackAreaExtension> TrackAreaModel = TrackNode.ImplicitCast();
		if (!TrackAreaModel->GetTrackAreaModelList())
		{
			continue;
		}

		bNoTrackAreaModels = false;

		if (!Track->IsEvalDisabled(/*bInCheckLocal=*/false))
		{
			return false;
		}
	}

	for (const TViewModelPtr<ITrackRowExtension>& TrackRowNode : TrackRowDescendants)
	{
		const UMovieSceneTrack* const Track = TrackRowNode->GetParentTrack();
		if (!IsValid(Track))
		{
			continue;
		}

		const TViewModelPtr<ITrackAreaExtension> TrackAreaModel = TrackRowNode.ImplicitCast();
		if (!TrackAreaModel->GetTrackAreaModelList())
		{
			continue;
		}

		bNoTrackAreaModels = false;

		if (!Track->IsRowEvalDisabled(TrackRowNode->GetRowIndex(), /*bInCheckLocal=*/false))
		{
			return false;
		}
	}

	return !bNoTrackAreaModels;
}

void FEvaluableOutlinerItemModel::SetIsDeactivated(const bool bInIsDeactivated)
{
	bool bAnyChanged = false;

	for (const TViewModelPtr<ITrackExtension>& TrackNode : GetDescendantsOfType<ITrackExtension>(true))
	{
		UMovieSceneTrack* const Track = TrackNode->GetTrack();
		if (!IsValid(Track))
		{
			continue;
		}

		// Deactive state (dirtying, saved with asset, evaluation)
		if (bInIsDeactivated != Track->IsEvalDisabled(/*bInCheckLocal=*/false))
		{
			Track->Modify();
			Track->SetEvalDisabled(bInIsDeactivated);
			bAnyChanged = true;
		}
	}

	for (const TViewModelPtr<ITrackRowExtension>& TrackRowNode : GetDescendantsOfType<ITrackRowExtension>(true))
	{
		UMovieSceneTrack* const Track = TrackRowNode->GetParentTrack();
		if (!IsValid(Track))
		{
			continue;
		}

		// Deactive state (dirtying, saved with asset, evaluation)
		if (bInIsDeactivated != Track->IsRowEvalDisabled(TrackRowNode->GetRowIndex(), /*bInCheckLocal=*/false))
		{
			Track->Modify();
			Track->SetRowEvalDisabled(bInIsDeactivated, TrackRowNode->GetRowIndex());
			bAnyChanged = true;
		}
	}

	if (bAnyChanged)
	{
		TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
		if (EditorViewModel)
		{
			TSharedPtr<ISequencer> Sequencer = EditorViewModel->GetSequencer();
			if (Sequencer)
			{
				Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
			}
		}
	}
}

} // namespace Sequencer
} // namespace UE

#undef LOCTEXT_NAMESPACE

