// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/PossessableModel.h"

#include "Containers/ArrayView.h"
#include "HAL/Platform.h"
#include "ISequencer.h"
#include "Internationalization/Internationalization.h"
#include "MVVM/Extensions/IRecyclableExtension.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "MovieScene.h"
#include "MovieSceneFwd.h"
#include "MovieScenePossessable.h"
#include "MovieSceneSequence.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "MovieSceneBindingReferences.h"

class UClass;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "FPossessableModel"

namespace UE
{
namespace Sequencer
{

FPossessableModel::FPossessableModel(FSequenceModel* InOwnerModel, const FMovieSceneBinding& InBinding, const FMovieScenePossessable& InPossessable)
	: FObjectBindingModel(InOwnerModel, InBinding)
{
	ParentObjectBindingID = InPossessable.GetParent();
}

FPossessableModel::~FPossessableModel()
{
}

EObjectBindingType FPossessableModel::GetType() const
{
	return EObjectBindingType::Possessable;
}

bool FPossessableModel::SupportsRebinding() const
{
	UMovieScene* MovieScene = OwnerModel->GetMovieScene();
	UMovieSceneSequence* Sequence = OwnerModel->GetSequence();
	check(MovieScene && Sequence);

	const FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectBindingID);
	return Possessable && Sequence->CanRebindPossessable(*Possessable);
}

void FPossessableModel::OnConstruct()
{
	using namespace UE::MovieScene;

	UMovieScene* MovieScene = OwnerModel->GetMovieScene();
	check(MovieScene);

	const FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectBindingID);
	check(Possessable);

	FScopedViewModelListHead RecycledHead(AsShared(), EViewModelListType::Recycled);
	GetChildrenForList(&OutlinerChildList).MoveChildrenTo<IRecyclableExtension>(RecycledHead.GetChildren(), IRecyclableExtension::CallOnRecycle);

	FObjectBindingModel::OnConstruct();

	// Preserve objectbindings since those are always added by the object model storage
	// on reinitialize or in response to an object event.
	FViewModelChildren OutlinerChildren = GetChildrenForList(&OutlinerChildList);
	for (TViewModelPtr<FObjectBindingModel> Child : RecycledHead.GetChildren().IterateSubList<FObjectBindingModel>().ToArray())
	{
		if (Child)
		{
			OutlinerChildren.AddChild(Child);
		}
	}
}

FText FPossessableModel::GetIconToolTipText() const
{
	TSharedPtr<ISequencer> Sequencer = GetEditor()->GetSequencer();
	if (Sequencer)
	{
		const int32 NumBoundObjects = Sequencer->FindObjectsInCurrentSequence(ObjectBindingID).Num();
		if (NumBoundObjects > 1)
		{
			return LOCTEXT("MultiplePossessableToolTip", "This item is bound to multiple objects.");
		}
	}

	UMovieSceneSequence* Sequence = OwnerModel ? OwnerModel->GetSequence() : nullptr;
	if (Sequence)
	{
		if (const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
		{
			for (const FMovieSceneBindingReference& BindingReference : BindingReferences->GetReferences(ObjectBindingID))
			{
				if (BindingReference.CustomBinding)
				{
					FText CustomTooltipText = BindingReference.CustomBinding->GetBindingTrackIconTooltip();
					if (CustomTooltipText.IsEmpty())
					{
						CustomTooltipText = FText::Format(LOCTEXT("DefaultCustomBindingTooltipText", "This is a custom binding of type {0}"), BindingReference.CustomBinding->GetBindingTypePrettyName());
					}
					if (!CustomTooltipText.IsEmpty())
					{
						return CustomTooltipText;
					}
				}
			}
		}
	}

	return LOCTEXT("PossessableToolTip", "This item is a possessable reference to an existing object.");
}

const FSlateBrush* FPossessableModel::GetIconOverlayBrush() const
{
	TSharedPtr<ISequencer> Sequencer = GetEditor()->GetSequencer();
	if (Sequencer)
	{
		const int32 NumBoundObjects = Sequencer->FindObjectsInCurrentSequence(ObjectBindingID).Num();
		if (NumBoundObjects > 1)
		{
			return FAppStyle::GetBrush("Sequencer.MultipleIconOverlay");
		}
	}

	UMovieSceneSequence* Sequence = OwnerModel ? OwnerModel->GetSequence() : nullptr;
	if (Sequence)
	{
		const FMovieScenePossessable* Possessable = Sequence->GetMovieScene()->FindPossessable(ObjectBindingID);
		if (Possessable && Possessable->GetSpawnableObjectBindingID().IsValid())
		{
			return FAppStyle::GetBrush("Sequencer.ProxyIconOverlay");
		}

		if (const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
		{
			for (const FMovieSceneBindingReference& BindingReference : BindingReferences->GetReferences(ObjectBindingID))
			{
				if (const FSlateBrush* CustomBrush = BindingReference.CustomBinding ? BindingReference.CustomBinding->GetBindingTrackCustomIconOverlay().GetIcon() : nullptr)
				{
					return CustomBrush;
				}
			}
		}
		else if (const FSlateBrush* CustomBrush = Sequence->GetCustomBrushForBinding(ObjectBindingID))
		{
			return CustomBrush;
		}
	}

	return nullptr;
}

const UClass* FPossessableModel::FindObjectClass() const
{
	const UClass* Class = UObject::StaticClass();
	if (OwnerModel)
	{
		if (UMovieScene* MovieScene = OwnerModel->GetMovieScene())
		{
			FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectBindingID);
			if (TSharedPtr<ISequencer> Sequencer = OwnerModel->GetSequencer())
			{
				UMovieSceneSequence* Sequence = OwnerModel->GetSequence();
				FGuid BindingID = ObjectBindingID;
				FMovieSceneSequenceID SequenceID = Sequencer->GetFocusedTemplateID();

				using VisitedKeyType = TTuple<UMovieSceneSequence*, FGuid>;
				TSet<VisitedKeyType, DefaultKeyFuncs<VisitedKeyType>, TInlineSetAllocator<8>> VisitedBindings;
				VisitedBindings.Add(MakeTuple(Sequence, BindingID));

				// Find the tar\get for the proxy by looping through proxies
				while (Possessable && Possessable->GetSpawnableObjectBindingID().IsValid())
				{
					SequenceID = Possessable->GetSpawnableObjectBindingID().ResolveSequenceID(SequenceID, *Sequencer);
					BindingID  = Possessable->GetSpawnableObjectBindingID().GetGuid();
					Sequence   = Sequencer->GetSharedPlaybackState()->GetSequence(SequenceID);

					if (!Sequence)
					{
						// Broken link!
						return Class;
					}
					if (VisitedBindings.Contains(MakeTuple(Sequence, BindingID)))
					{
						// Cyclic dependency!
						return nullptr;
					}

					VisitedBindings.Add(MakeTuple(Sequence, BindingID));
					Possessable = Sequence->GetMovieScene()->FindPossessable(BindingID);
				}

				if (UObject* Template = MovieSceneHelpers::GetObjectTemplate(Sequence, BindingID, Sequencer->GetSharedPlaybackState()))
				{
					Class = Template->GetClass();
				}
				else if (Possessable && Possessable->GetPossessedObjectClass() != nullptr)
				{
					Class = Possessable->GetPossessedObjectClass();
				}
			}
		}
	}
	return Class;
}

void FPossessableModel::Delete()
{
	FObjectBindingModel::Delete();

	UMovieSceneSequence* Sequence   = OwnerModel ? OwnerModel->GetSequence() : nullptr;
	UMovieScene*         MovieScene = Sequence   ? Sequence->GetMovieScene() : nullptr;

	if (MovieScene)
	{
		MovieScene->Modify();

		if (MovieScene->RemovePossessable(ObjectBindingID))
		{
			Sequence->Modify();
			if (OwnerModel)
			{
				TSharedPtr<ISequencer> Sequencer = OwnerModel->GetSequencer();
				// If we have a custom spawnable, destroy it
				if (const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
				{
					int32 BindingIndex = 0;
					for (const FMovieSceneBindingReference& BindingReference : BindingReferences->GetReferences(ObjectBindingID))
					{
						if (BindingReference.CustomBinding && BindingReference.CustomBinding->WillSpawnObject(Sequencer->GetSharedPlaybackState()))
						{
							Sequencer->GetSpawnRegister().DestroySpawnedObject(ObjectBindingID, OwnerModel->GetSequenceID(), Sequencer->GetSharedPlaybackState(), BindingIndex);
						}
						BindingIndex++;
					}
				}
			}
			Sequence->UnbindPossessableObjects(ObjectBindingID);
		}
	}
}

FSlateColor FPossessableModel::GetInvalidBindingLabelColor() const
{
	UMovieSceneSequence* Sequence = OwnerModel ? OwnerModel->GetSequence() : nullptr;
	UMovieScene* MovieScene = (OwnerModel && Sequence) ? OwnerModel->GetMovieScene() : nullptr;
	FMovieScenePossessable* Possessable = MovieScene ? MovieScene->FindPossessable(ObjectBindingID) : nullptr;
	if (Possessable)
	{
		if (Possessable->GetSpawnableObjectBindingID().IsValid())
		{
			return FSlateColor::UseSubduedForeground();
		}
		if (const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
		{
			for (const FMovieSceneBindingReference& BindingReference : BindingReferences->GetReferences(ObjectBindingID))
			{
				if (BindingReference.Locator.IsEmpty())
				{
					TSharedPtr<ISequencer> Sequencer = OwnerModel->GetSequencer();
					if (BindingReference.CustomBinding == nullptr)
					{
						// Show empty bindings as yellow rather than red
						return FLinearColor::Yellow;
					}
					else if (BindingReference.CustomBinding->WillSpawnObject(Sequencer->GetSharedPlaybackState()))
					{
						return FSlateColor::UseSubduedForeground();
					}
				}
			}

		}
	}
	return FLinearColor::Red;
}

} // namespace Sequencer
} // namespace UE

#undef LOCTEXT_NAMESPACE
