// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Extensions/IViewSpaceClientExtension.h"
#include "IKeyArea.h"
#include "Sequencer.h"
#include "SSequencerSection.h"

#include "MovieSceneSection.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "ScopedTransaction.h"

namespace UE
{
namespace Sequencer
{

struct FSectionModel_StretchParams : IDynamicExtension
{
	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID(FSectionModel_StretchParams);

	TRange<FFrameNumber> PreStretchSectionRange;

	struct FChannelKeys
	{
		FMovieSceneChannelHandle Channel;
		TArray<FKeyHandle> Handles;
		TArray<double> StretchFactors;
	};

	TArray<FChannelKeys> ChannelKeys;
};

UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FSectionModel_StretchParams);

FSectionModel::FSectionModel()
	: ChannelList(FTrackModel::GetTopLevelChannelType() | EViewModelListType::Generic)
{
	RegisterChildList(&ChannelList);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FSectionModel::~FSectionModel()
{
	SectionInterface.Reset();
	WeakSection.Reset();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FSectionModel::InitializeSection(TSharedPtr<ISequencerSection> InSectionInterface)
{
	SectionInterface = InSectionInterface;
}

void FSectionModel::InitializeObject(TWeakObjectPtr<> InWeakObject)
{
	UMovieSceneSection* Section = CastChecked<UMovieSceneSection>(InWeakObject.Get());

	WeakSection = Section;

	Section->EventHandlers.Link(this);
	Section->UMovieSceneSignedObject::EventHandlers.Link(this);

	UpdateCachedData();
}

UObject* FSectionModel::GetObject() const
{
	return WeakSection.Get();
}

TRange<FFrameNumber> FSectionModel::GetRange() const
{
	UMovieSceneSection* Section = WeakSection.Get();
	return Section ? Section->GetRange() : TRange<FFrameNumber>::Empty();
}

UMovieSceneSection* FSectionModel::GetSection() const
{
	return WeakSection.Get();
}

TSharedPtr<ISequencerSection> FSectionModel::GetSectionInterface() const
{
	return SectionInterface;
}

TViewModelPtr<ISectionOwnerExtension> FSectionModel::GetParentSectionOwnerModel() const
{
	return FindAncestorOfType<ISectionOwnerExtension>();
}

TViewModelPtr<ITrackExtension> FSectionModel::GetParentTrackModel() const
{
	return FindAncestorOfType<ITrackExtension>();
}

TViewModelPtr<ITrackExtension> FSectionModel::GetParentTrackExtension() const
{
	return FindAncestorOfType<ITrackExtension>();
}

int32 FSectionModel::GetPreRollFrames() const
{
	UMovieSceneSection* Section = GetSection();
	return Section ? Section->GetPreRollFrames() : 0;
}

int32 FSectionModel::GetPostRollFrames() const
{
	UMovieSceneSection* Section = GetSection();
	return Section ? Section->GetPostRollFrames() : 0;
}

TSharedPtr<ITrackLaneWidget> FSectionModel::CreateTrackLaneView(const FCreateTrackLaneViewParams& InParams)
{
	UMovieSceneSection* Section = WeakSection.Get();
	if (!Section)
	{
		return nullptr;
	}

	TSharedPtr<FSequencer> Sequencer = InParams.Editor->CastThisChecked<FSequencerEditorViewModel>()->GetSequencerImpl();
	return SNew(SSequencerSection, Sequencer, SharedThis(this), InParams.OwningTrackLane);
}

FTrackLaneVirtualAlignment FSectionModel::ArrangeVirtualTrackLaneView() const
{
	using namespace UE::MovieScene;

	// Use CastThisShared to ensure that we cast both native and dynamic extensions
	TSharedPtr<const IViewSpaceClientExtension> ViewSpaceClient = CastThisShared<const IViewSpaceClientExtension>();

	FGuid ViewSpaceID = ViewSpaceClient ? ViewSpaceClient->GetViewSpaceID() : FGuid();
	return FTrackLaneVirtualAlignment::Proportional(SectionRange, 1.f, VAlign_Center, ViewSpaceID);
}

TRange<FFrameNumber> FSectionModel::GetLayerBarRange() const
{
	return LayerBarRange;
}

void FSectionModel::OffsetLayerBar(FFrameNumber Amount)
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		Section->Modify();
		Section->MoveSection(Amount);
		Section->MarkAsChanged();
	}
}

void FSectionModel::OnModifiedDirectly(UMovieSceneSignedObject*)
{
	UpdateCachedData();
}

void FSectionModel::UpdateCachedData()
{
	SectionRange = TRange<FFrameNumber>::Empty();
	LayerBarRange = TRange<FFrameNumber>::Empty();

	UMovieSceneSection* Section = WeakSection.Get();
	if (!Section)
	{
		return;
	}

	SectionRange = Section->GetRange();

	// Compute the layer bar range from this section's effective key range
	TRange<FFrameNumber> KeyRangeHull = TRange<FFrameNumber>::Empty();

	if (SectionRange.GetLowerBound().IsClosed() && SectionRange.GetUpperBound().IsClosed())
	{
		KeyRangeHull = SectionRange;
	}
	else if (SectionRange.GetLowerBound().IsClosed())
	{
		KeyRangeHull = TRange<FFrameNumber>::Inclusive(SectionRange.GetLowerBoundValue(), SectionRange.GetLowerBoundValue());
	}
	else if (SectionRange.GetUpperBound().IsClosed())
	{
		KeyRangeHull = TRange<FFrameNumber>::Inclusive(SectionRange.GetUpperBoundValue(), SectionRange.GetUpperBoundValue());
	}

	// Find the first key time and use that
	const FMovieSceneChannelProxy& Proxy = Section->GetChannelProxy();
	for (const FMovieSceneChannelEntry& Entry : Proxy.GetAllEntries())
	{
		for (const FMovieSceneChannel* Channel : Entry.GetChannels())
		{
			TRange<FFrameNumber> ChannelRange = Channel->ComputeEffectiveRange();
			if (!ChannelRange.IsEmpty() && !ChannelRange.GetLowerBound().IsOpen() && !ChannelRange.GetUpperBound().IsOpen())
			{
				KeyRangeHull = TRange<FFrameNumber>::Hull(ChannelRange, KeyRangeHull);
			}
		}
	}

	if (!KeyRangeHull.IsEmpty() && KeyRangeHull.GetLowerBound().IsClosed() && KeyRangeHull.GetUpperBound().IsClosed())
	{
		LayerBarRange = TRange<FFrameNumber>::Intersection(KeyRangeHull, SectionRange);
	}
}

void FSectionModel::OnRowChanged(UMovieSceneSection*)
{
}

ESelectionIntent FSectionModel::IsSelectable() const
{
	if (GetRange() == TRange<FFrameNumber>::All())
	{
		// Infinite sections are only selectable through the context menu
		return ESelectionIntent::ContextMenu;
	}
	return ESelectionIntent::Any;
}

void FSectionModel::AddToSnapField(const ISnapCandidate& Candidate, ISnapField& SnapField) const
{
	UMovieSceneSection* Section = GetSection();
	if (!Section)
	{
		return;
	}

	if (Candidate.AreSectionBoundsApplicable(Section))
	{
		if (Section->HasStartFrame())
		{
			SnapField.AddSnapPoint(FSnapPoint{ FSnapPoint::SectionBounds, Section->GetInclusiveStartFrame() });
		}

		if (Section->HasEndFrame())
		{
			SnapField.AddSnapPoint(FSnapPoint{ FSnapPoint::SectionBounds, Section->GetExclusiveEndFrame() });
		}
	}

	if (Candidate.AreSectionCustomSnapsApplicable(Section))
	{
		TArray<FFrameNumber> CustomSnaps;
		Section->GetSnapTimes(CustomSnaps, false);
		for (FFrameNumber Time : CustomSnaps)
		{
			SnapField.AddSnapPoint(FSnapPoint{ FSnapPoint::CustomSection, Time });
		}
	}
}

bool FSectionModel::CanDrag() const
{
	return true;
}

void FSectionModel::OnBeginDrag(IDragOperation& DragOperation)
{
	if (UMovieSceneSection* Section = GetSection())
	{
		DragOperation.AddModel(SharedThis(this));

		if (Section->HasStartFrame())
		{
			DragOperation.AddSnapTime(Section->GetInclusiveStartFrame());
		}
		if (Section->HasEndFrame())
		{
			DragOperation.AddSnapTime(Section->GetExclusiveEndFrame());
		}
	}
}

void FSectionModel::OnEndDrag(IDragOperation& DragOperation)
{
}

void FSectionModel::OnInitiateStretch(IStretchOperation& StretchOperation, EStretchConstraint Constraint, FStretchParameters* InOutGlobalParameters)
{
	UMovieSceneSection* Section = GetSection();
	const bool bIsValid = Section && !Section->IsLocked() && !LayerBarRange.IsEmpty();
	if (bIsValid)
	{
		StretchOperation.DoNotSnapTo(SharedThis(this));

		FStretchParameters StrechParams;

		if (Constraint == EStretchConstraint::AnchorToStart)
		{
			StrechParams.Anchor = LayerBarRange.GetLowerBoundValue().Value;
			StrechParams.Handle = LayerBarRange.GetUpperBoundValue().Value;

			if (InOutGlobalParameters->Anchor > StrechParams.Anchor)
			{
				InOutGlobalParameters->Anchor = StrechParams.Anchor;
			}
		}
		else
		{
			StrechParams.Anchor = LayerBarRange.GetUpperBoundValue().Value;
			StrechParams.Handle = LayerBarRange.GetLowerBoundValue().Value;

			if (InOutGlobalParameters->Anchor < StrechParams.Anchor)
			{
				InOutGlobalParameters->Anchor = StrechParams.Anchor;
			}
		}

		TSharedPtr<FSectionModel> This = SharedThis(this);

		const int32 Priority = GetHierarchicalDepth();
		StretchOperation.InitiateStretch(This, This, Priority, StrechParams);
	}
}

EStretchResult FSectionModel::OnBeginStretch(const IStretchOperation& StretchOperation, const FStretchScreenParameters& ScreenParameters, FStretchParameters* InOutParameters)
{
	UMovieSceneSection* Section = GetSection();
	const bool bIsValid = Section && !Section->IsLocked();
	if (!bIsValid)
	{
		return EStretchResult::Failure;
	}

	Section->Modify();

	FSectionModel_StretchParams& StretchData = AddDynamicExtension<FSectionModel_StretchParams>();
	StretchData.PreStretchSectionRange  = Section->GetRange();

	const double Anchor    = InOutParameters->Anchor;
	const double StartUnit = InOutParameters->Handle - InOutParameters->Anchor;

	TArray<FFrameNumber> KeyTimesScratch;

	FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
	for (const FMovieSceneChannelEntry& Entry : ChannelProxy.GetAllEntries())
	{
		TArrayView<FMovieSceneChannel* const> Channels = Entry.GetChannels();
		for (int32 ChannelIndex = 0; ChannelIndex < Channels.Num(); ++ChannelIndex)
		{
			KeyTimesScratch.Reset();

			FSectionModel_StretchParams::FChannelKeys& Keys = StretchData.ChannelKeys.Emplace_GetRef();
			Keys.Channel = ChannelProxy.MakeHandle(Entry.GetChannelTypeName(), ChannelIndex);

			Channels[ChannelIndex]->GetKeys(TRange<FFrameNumber>::All(), &KeyTimesScratch, &Keys.Handles);

			const int32 NumKeys = KeyTimesScratch.Num();
			Keys.StretchFactors.SetNumUninitialized(NumKeys);
			for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
			{
				const double KeyTimeAsDecimal = KeyTimesScratch[KeyIndex].Value;
				Keys.StretchFactors[KeyIndex] = (KeyTimeAsDecimal - Anchor) / StartUnit;
			}
		}
	}

	return EStretchResult::Success;
}

void FSectionModel::OnStretch(const IStretchOperation& StretchOperation, const FStretchScreenParameters& ScreenParameters, FStretchParameters* InOutParameters)
{
	UMovieSceneSection* Section = GetSection();
	const bool bIsValid = Section && !Section->IsLocked();
	if (!ensure(bIsValid))
	{
	return;
	}

	// If we never initiated a stretch for this section, don't do anything.
	// This could happen if stretching happened to create a new section
	FSectionModel_StretchParams* StretchData = CastThis<FSectionModel_StretchParams>();
	if (!StretchData)
	{
		return;
	}

	const double StretchDelta = ScreenParameters.CurrentDragPosition.AsDecimal() - ScreenParameters.DragStartPosition.AsDecimal();

	const double Anchor    = InOutParameters->Anchor;
	const double StartUnit = InOutParameters->Handle - InOutParameters->Anchor;
	const double Unit      = StartUnit + StretchDelta;

	TArray<FFrameNumber> NewTimesScratch;
	for (FSectionModel_StretchParams::FChannelKeys& ChannelData : StretchData->ChannelKeys)
	{
		FMovieSceneChannel* Channel = ChannelData.Channel.Get();
		if (!Channel)
		{
			continue;
		}

		// Copy the key times
		const int32 NumKeys = ChannelData.StretchFactors.Num();

		// Stretch the key times
		NewTimesScratch.SetNumUninitialized(NumKeys);
		for (int32 Index = 0; Index < NumKeys; ++Index)
		{
			const double NewTime = Anchor + Unit * ChannelData.StretchFactors[Index];
			NewTimesScratch[Index] = FFrameNumber(static_cast<int32>(FMath::RoundToDouble(NewTime)));
		}

		Channel->SetKeyTimes(ChannelData.Handles, NewTimesScratch);

		ChannelData.Channel.Get();
	}

	// Also stretch the section range if necessary
	TRange<FFrameNumber> NewSectionRange = StretchData->PreStretchSectionRange;
	if (NewSectionRange.GetLowerBound().IsClosed())
	{
		const double BoundAsDecimal = NewSectionRange.GetLowerBoundValue().Value;
		const double StretchFactor  = (BoundAsDecimal - Anchor) / StartUnit;

		const double NewTime = Anchor + Unit * StretchFactor;

		FFrameNumber NewBound = FFrameNumber(static_cast<int32>(FMath::RoundToDouble(NewTime)));
		NewSectionRange.SetLowerBoundValue(NewBound);
	}
	if (NewSectionRange.GetUpperBound().IsClosed())
	{
		const double BoundAsDecimal = NewSectionRange.GetUpperBoundValue().Value;
		const double StretchFactor  = (BoundAsDecimal - Anchor) / StartUnit;

		const double NewTime = Anchor + Unit * StretchFactor;

		FFrameNumber NewBound = FFrameNumber(static_cast<int32>(FMath::RoundToDouble(NewTime)));
		NewSectionRange.SetUpperBoundValue(NewBound);
	}

	if (!NewSectionRange.IsEmpty() && !NewSectionRange.IsDegenerate())
	{
		Section->SetRange(NewSectionRange);
	}

	Section->MarkAsChanged();
}

void FSectionModel::OnEndStretch(const IStretchOperation& StretchOperation, const FStretchScreenParameters& ScreenParameters, FStretchParameters* InOutParameters)
{

}

TArray<FOverlappingSections> FSectionModel::GetUnderlappingSections()
{
	UMovieSceneSection* ThisSection = GetSection();
	TViewModelPtr<ITrackAreaExtension> Parent = CastParent<ITrackAreaExtension>();
	if (!Parent || !ThisSection)
	{
		return TArray<FOverlappingSections>();
	}

	TRange<FFrameNumber> ThisSectionRange = ThisSection->GetRange();

	TMovieSceneEvaluationTree<TSharedPtr<FSectionModel>> OverlapTree;

	// Iterate all siblings with <= overlap priority
	for (TSharedPtr<FSectionModel> Sibling : Parent->GetTrackAreaModelListAs<FSectionModel>())
	{
		UMovieSceneSection* SiblingSection = Sibling->GetSection();

		// Parent is either the track (for single lane tracks) or the track row (for multi-row tracks), so we don't need
		// to filter sections that are on different rows, we only get sections on the same row already.
		if (!SiblingSection || SiblingSection == ThisSection || SiblingSection->GetOverlapPriority() > ThisSection->GetOverlapPriority())
		{
			continue;
		}

		TRange<FFrameNumber> OtherSectionRange = SiblingSection->GetRange();
		TRange<FFrameNumber> Intersection = TRange<FFrameNumber>::Intersection(OtherSectionRange, ThisSectionRange);
		if (!Intersection.IsEmpty())
		{
			OverlapTree.Add(Intersection, Sibling);
		}
	}

	TArray<FOverlappingSections> Result;
	for (FMovieSceneEvaluationTreeRangeIterator It(OverlapTree); It; ++It)
	{
		FOverlappingSections NewRange;

		NewRange.Range = It.Range();
		for (TSharedPtr<FSectionModel> Section : OverlapTree.GetAllData(It.Node()))
		{
			NewRange.Sections.Add(Section);
		}

		if (NewRange.Sections.Num())
		{
			// Sort lowest to highest
			Algo::Sort(NewRange.Sections, [](const TWeakPtr<FSectionModel>& A, const TWeakPtr<FSectionModel>& B){
				return A.Pin()->GetSection()->GetOverlapPriority() < B.Pin()->GetSection()->GetOverlapPriority();
			});

			Result.Add(MoveTemp(NewRange));
		}
	}

	return Result;
}

TArray<FOverlappingSections> FSectionModel::GetEasingSegments()
{
	UMovieSceneSection* ThisSection = GetSection();
	TViewModelPtr<ITrackAreaExtension> Parent = CastParent<ITrackAreaExtension>();
	if (!Parent || !ThisSection)
	{
		return TArray<FOverlappingSections>();
	}

	TRange<FFrameNumber> ThisSectionRange = ThisSection->GetRange();

	TMovieSceneEvaluationTree<TSharedPtr<FSectionModel>> OverlapTree;

	// Iterate all siblings with <= overlap priority
	for (TSharedPtr<FSectionModel> Sibling : Parent->GetTrackAreaModelListAs<FSectionModel>())
	{
		UMovieSceneSection* SiblingSection = Sibling->GetSection();

		// Parent is either the track (for single lane tracks) or the track row (for multi-row tracks), so we don't need
		// to filter sections that are on different rows, we only get sections on the same row already.
		if (!SiblingSection || !SiblingSection->IsActive() || SiblingSection->GetOverlapPriority() > ThisSection->GetOverlapPriority())
		{
			continue;
		}

		TRange<FFrameNumber> Intersection = TRange<FFrameNumber>::Intersection(SiblingSection->GetEaseInRange(), ThisSectionRange);
		if (!Intersection.IsEmpty())
		{
			OverlapTree.Add(Intersection, Sibling);
		}

		Intersection = TRange<FFrameNumber>::Intersection(SiblingSection->GetEaseOutRange(), ThisSectionRange);
		if (!Intersection.IsEmpty())
		{
			OverlapTree.Add(Intersection, Sibling);
		}
	}

	TArray<FOverlappingSections> Result;
	for (FMovieSceneEvaluationTreeRangeIterator It(OverlapTree); It; ++It)
	{
		FOverlappingSections NewRange;

		NewRange.Range = It.Range();
		for (TSharedPtr<FSectionModel> Section : OverlapTree.GetAllData(It.Node()))
		{
			NewRange.Sections.Add(Section);
		}

		if (NewRange.Sections.Num())
		{
			// Sort lowest to highest
			Algo::Sort(NewRange.Sections, [](const TWeakPtr<FSectionModel>& A, const TWeakPtr<FSectionModel>& B){
				return A.Pin()->GetSection()->GetOverlapPriority() < B.Pin()->GetSection()->GetOverlapPriority();
			});

			Result.Add(MoveTemp(NewRange));
		}
	}

	return Result;
}

const UMovieSceneCondition* FSectionModel::GetCondition() const
{
	UMovieSceneSection* Section = GetSection();
	if (IsValid(Section))
	{
		return Section->ConditionContainer.Condition;
	}
	return nullptr;
}

EConditionableConditionState FSectionModel::GetConditionState() const
{
	UMovieSceneSection* Section = GetSection();
	if (IsValid(Section))
	{
		if (Section->ConditionContainer.Condition)
		{
			if (Section->ConditionContainer.Condition->bEditorForceTrue)
			{
				return EConditionableConditionState::HasConditionEditorForceTrue;
			}

			TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
			TSharedPtr<ISequencer> Sequencer = SequenceModel ? SequenceModel->GetSequencer() : nullptr;
			if (Sequencer)
			{
				FGuid BindingID;

				if (TSharedPtr<IObjectBindingExtension> ParentBinding = FindAncestorOfType<IObjectBindingExtension>())
				{
					BindingID = ParentBinding->GetObjectGuid();
				}

				if (MovieSceneHelpers::EvaluateSequenceCondition(BindingID, Sequencer->GetFocusedTemplateID(), Section->ConditionContainer.Condition, Section, Sequencer->GetSharedPlaybackState()))
				{
					return EConditionableConditionState::HasConditionEvaluatingTrue;
				}
				else
				{
					return EConditionableConditionState::HasConditionEvaluatingFalse;
				}
			}
		}
	}
	return EConditionableConditionState::None;
}

void FSectionModel::SetConditionEditorForceTrue(bool bEditorForceTrue)
{
	UMovieSceneSection* Section = GetSection();
	if (IsValid(Section))
	{
		if (Section->ConditionContainer.Condition)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("SequencerTrackNode", "ConditionEditorForceTrue", "Set Condition Editor Force True"));
			Section->ConditionContainer.Condition->Modify();
			Section->ConditionContainer.Condition->bEditorForceTrue = bEditorForceTrue;
		}
	}
}

} // namespace Sequencer
} // namespace UE

