// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/SectionOutlinerModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "MVVM/Views/SOutlinerItemViewBase.h"
#include "MVVM/Views/SSequencerKeyNavigationButtons.h"
#include "MVVM/Views/ViewUtilities.h"
#include "SKeyAreaEditorSwitcher.h"
#include "ISequencerModule.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "SequencerUtilities.h"
#include "Decorations/IMovieSceneSectionProviderDecoration.h"
#include "Decorations/MovieSceneDecorationContainer.h"
#include "Modules/ModuleManager.h"
#include "MVVM/DecorationModelStorageExtension.h"
#include "Widgets/SNullWidget.h"
#include "Decorations/MovieSceneMuteSoloDecoration.h"

namespace UE
{
namespace Sequencer
{

namespace
{
	// Helper function to check if a view model has any keyable areas
	bool ViewModelHasKeyableAreas(const FViewModel& ViewModel)
	{
		for (TSharedPtr<FChannelGroupModel> ChannelGroup : ViewModel.GetDescendantsOfType<FChannelGroupModel>())
		{
			for (const TWeakViewModelPtr<FChannelModel>& WeakChannel : ChannelGroup->GetChannels())
			{
				if (TViewModelPtr<FChannelModel> Channel = WeakChannel.Pin())
				{
					if (Channel->GetKeyArea())
					{
						return true;
					}
				}
			}
		}
		return false;
	}
}

UE_SEQUENCER_DEFINE_CASTABLE(FSectionOutlinerModel);

FSectionOutlinerModel::FSectionOutlinerModel(UMovieSceneSection* InSection, TSharedPtr<FSectionModel> InSectionModel)
	: TopLevelChannelList(FTrackModel::GetTopLevelChannelGroupType())
	, WeakSection(InSection)
	, WeakSectionModel(InSectionModel)
{
	RegisterChildList(&TopLevelChannelList);
	RegisterChildList(&SectionModelList);
}

FSectionOutlinerModel::~FSectionOutlinerModel()
{
	SectionEventHandler.Unlink();
	UE::MovieScene::TIntrusiveEventHandler<UE::MovieScene::ISignedObjectEventHandler>::Unlink();
}

void FSectionOutlinerModel::OnConstruct()
{
	UMovieSceneSection* Section = WeakSection.Get();
	if (Section && Section->GetTypedOuter<UMovieSceneTrack>())
	{
		TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
		if (SequenceModel)
		{
			TSharedPtr<ISequencer> Sequencer = SequenceModel->GetSequencer();
			if (Sequencer)
			{
				TrackEditor = Sequencer->GetTrackEditor(Section->GetTypedOuter<UMovieSceneTrack>());
			}
		}

		// Subscribe to section decoration events (unlink first in case OnConstruct is called again)
		SectionEventHandler.Unlink();
		Section->EventHandlers.Link(SectionEventHandler, this);

		// Listen for OnPostUndo so we can refresh decoration children when a
		// transaction revert removes entries that OnDecorationAdded/Removed
		// won't re-fire for.
		UE::MovieScene::TIntrusiveEventHandler<UE::MovieScene::ISignedObjectEventHandler>::Unlink();
		Section->UMovieSceneSignedObject::EventHandlers.Link(this);
	}

}

FViewModelChildren FSectionOutlinerModel::GetTrackAreaChildren()
{
	return GetChildrenForList(&SectionModelList);
}

TSharedPtr<FSectionModel> FSectionOutlinerModel::GetSectionModel() const
{
	return WeakSectionModel.Pin();
}

UMovieSceneTrack* FSectionOutlinerModel::GetTrack() const
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		return Section->GetTypedOuter<UMovieSceneTrack>();
	}
	return nullptr;
}

/*-----------------------------------------------------------------------------
	FEvaluableOutlinerItemModel
-----------------------------------------------------------------------------*/

bool FSectionOutlinerModel::IsDeactivated() const
{
	UMovieSceneSection* Section = WeakSection.Get();
	return Section && !Section->IsActive();
}

void FSectionOutlinerModel::SetIsDeactivated(bool bInIsDeactivated)
{
	UMovieSceneSection* Section = WeakSection.Get();
	if (!Section || Section->IsActive() == !bInIsDeactivated)
	{
		return;
	}

	Section->Modify();
	Section->SetIsActive(!bInIsDeactivated);

	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (EditorViewModel)
	{
		if (TSharedPtr<ISequencer> Sequencer = EditorViewModel->GetSequencer())
		{
			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		}
	}
}

/*-----------------------------------------------------------------------------
	FOutlinerItemModel
-----------------------------------------------------------------------------*/

void FSectionOutlinerModel::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UMovieSceneSection* Section = WeakSection.Get();
	if (!IsValid(Section))
	{
		return;
	}

	TArray<TWeakObjectPtr<>> WeakSections;
	WeakSections.Add(Section);
	SequencerHelpers::BuildEditSectionMenu(Sequencer, WeakSections, MenuBuilder, true);

	if (const TViewModelPtr<FChannelGroupModel> ChannelGroup = TopLevelChannelList.GetHead().ImplicitCast())
	{
		ChannelGroup->BuildChannelOverrideMenu(MenuBuilder);
	}

	FOutlinerItemModel::BuildContextMenu(MenuBuilder);
}

void FSectionOutlinerModel::BuildSidebarMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UMovieSceneSection* Section = WeakSection.Get();
	if (!IsValid(Section))
	{
		return;
	}

	TArray<TWeakObjectPtr<>> WeakSections;
	WeakSections.Add(Section);
	SequencerHelpers::BuildEditSectionMenu(Sequencer, WeakSections, MenuBuilder, false);

	if (const TViewModelPtr<FChannelGroupModel> ChannelGroup = TopLevelChannelList.GetHead().ImplicitCast())
	{
		ChannelGroup->BuildChannelOverrideMenu(MenuBuilder);
	}

	FOutlinerItemModel::BuildSidebarMenu(MenuBuilder);
}

void FSectionOutlinerModel::CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels)
{
	TViewModelPtr<FChannelGroupModel> ChannelGroup = TopLevelChannelList.GetHead().ImplicitCast();
	if (ChannelGroup)
	{
		ChannelGroup->CreateCurveModels(OutCurveModels);
	}
}

/*-----------------------------------------------------------------------------
	IOutlinerExtension
-----------------------------------------------------------------------------*/

FOutlinerSizing FSectionOutlinerModel::GetOutlinerSizing() const
{
	// Get height from the associated section model
	const TSharedPtr<FSequencerEditorViewModel> Editor = GetEditor();
	FViewDensityInfo Density = Editor ? Editor->GetViewDensity() : FViewDensityInfo();
	const float SectionAreaDefaultHeight = 27.0f;
	float Height = Density.UniformHeight.Get(SectionAreaDefaultHeight);

	if (TSharedPtr<FSectionModel> SectionModel = GetSectionModel())
	{
		if (TSharedPtr<ISequencerSection> SectionInterface = SectionModel->GetSectionInterface())
		{
			Height = SectionInterface->GetSectionHeight(Density);
		}
	}

	return FOutlinerSizing(Height);
}

TSharedPtr<SWidget> FSectionOutlinerModel::CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName)
{
	using namespace UE::Sequencer;

	TViewModelPtr<FSequencerEditorViewModel> Editor = InParams.Editor->CastThisShared<FSequencerEditorViewModel>();
	if (!Editor)
	{
		return SNullWidget::NullWidget;
	}

	// Label column - display the section's title
	if (InColumnName == FCommonOutlinerNames::Label)
	{
		return SNew(SOutlinerItemViewBase, SharedThis(this), InParams.Editor, InParams.TreeViewRow);
	}

	// KeyFrame column - show add key button if there are keyable channels
	if (InColumnName == FCommonOutlinerNames::KeyFrame)
	{
		if (ViewModelHasKeyableAreas(*this))
		{
			EKeyNavigationButtons Buttons = EKeyNavigationButtons::AddKey;
			return SNew(SSequencerKeyNavigationButtons, SharedThis(this), Editor->GetSequencer())
				.Buttons(Buttons);
		}
	}

	// Nav column - show navigation buttons if there are keys
	if (InColumnName == FCommonOutlinerNames::Nav)
	{
		if (ViewModelHasKeyableAreas(*this))
		{
			EKeyNavigationButtons Buttons = InParams.TreeViewRow->IsColumnVisible(FCommonOutlinerNames::KeyFrame)
				? EKeyNavigationButtons::NavOnly
				: EKeyNavigationButtons::All;

			return SNew(SSequencerKeyNavigationButtons, SharedThis(this), Editor->GetSequencer())
				.Buttons(Buttons);
		}
	}

	// Edit column - show key area editor for top-level channels
	if (InColumnName == FCommonOutlinerNames::Edit)
	{
		TOptional<FViewModelChildren> TopLevelChannels = FindChildList(FTrackModel::GetTopLevelChannelGroupType());
		TSharedPtr<FChannelGroupModel> TopLevelChannel = TopLevelChannels ? TopLevelChannels->FindFirstChildOfType<FChannelGroupModel>() : nullptr;
		if (TopLevelChannel)
		{
			return SNew(SKeyAreaEditorSwitcher, TopLevelChannel, Editor);
		}
	}

	// Add column - shows '+' button for decorations and, for sections that expose a time warp
	// variant via GetTimeWarp(), a Time Warp entry alongside.
	if (InColumnName == FCommonOutlinerNames::Add)
	{
		UMovieSceneSection* Section = GetSection();
		const bool bHasDecorations  = Section && SequencerHelpers::HasCompatibleDecorations(Section);
		const bool bSupportsTimeWarp = Section && Section->GetTimeWarp() != nullptr;
		if (bHasDecorations || bSupportsTimeWarp)
		{
			TViewModelPtr<IObjectBindingExtension> ObjectBinding = FindAncestorOfType<IObjectBindingExtension>();
			FGuid ObjectBindingID = ObjectBinding ? ObjectBinding->GetObjectGuid() : FGuid();

			TWeakObjectPtr<UMovieSceneSection> WeakSectionForMenu = Section;
			TWeakViewModelPtr<FViewModel>      WeakSelfForMenu    = SharedThis(this);
			auto GetMenuContent = [WeakSectionForMenu, Editor, ObjectBindingID, WeakSelfForMenu]() -> TSharedRef<SWidget>
			{
				FMenuBuilder MenuBuilder(true, nullptr);
				if (UMovieSceneSection* PinnedSection = WeakSectionForMenu.Get())
				{
					SequencerHelpers::BuildDecorationMenu(MenuBuilder, PinnedSection, ObjectBindingID, Editor->GetSequencer());

					if (PinnedSection->GetTimeWarp())
					{
						MenuBuilder.BeginSection(NAME_None, NSLOCTEXT("SectionOutlinerModel", "TimeWarp_Label", "Time Warp"));
						FSequencerUtilities::MakeTimeWarpMenuEntry(MenuBuilder, WeakSelfForMenu);
						MenuBuilder.EndSection();
					}
				}
				return MenuBuilder.MakeWidget();
			};

			return MakeAddButton(
				NSLOCTEXT("SectionOutlinerModel", "AddModifier", "Modifier"),
				FOnGetContent::CreateLambda(GetMenuContent),
				SharedThis(this));
		}
	}

	return nullptr;
}

bool FSectionOutlinerModel::SupportsOutlinerColumn(const FName& InColumnName) const
{
	if (TSharedPtr<FSectionModel> SectionModel = GetSectionModel())
	{
		if (TSharedPtr<ISequencerSection> SectionInterface = SectionModel->GetSectionInterface())
		{
			return SectionInterface->SupportsOutlinerColumnToggle(InColumnName);
		}
	}
	return true;
}

FText FSectionOutlinerModel::GetLabel() const
{
	// Try to get the title from the associated FSectionModel's ISequencerSection interface
	if (TSharedPtr<FSectionModel> SectionModel = GetSectionModel())
	{
		if (TSharedPtr<ISequencerSection> SectionInterface = SectionModel->GetSectionInterface())
		{
			FText SectionTitle = SectionInterface->GetSectionTitle();
			if (!SectionTitle.IsEmpty())
			{
				return SectionTitle;
			}
		}
	}

	// Fall back to class display name
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		return Section->GetClass()->GetDisplayNameText();
	}

	return FText::FromString(TEXT("Section"));
}

FSlateColor FSectionOutlinerModel::GetLabelColor() const
{
	UMovieSceneTrack* Track = GetTrack();
	if (!Track)
	{
		return FSlateColor::UseForeground();
	}

	FMovieSceneLabelParams LabelParams;
	LabelParams.bIsDimmed = IsDimmed();
	if (TViewModelPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>())
	{
		if (TSharedPtr<FSequencerEditorViewModel> SequencerModel = SequenceModel->GetEditor())
		{
			LabelParams.SequenceID = SequenceModel->GetSequenceID();
			LabelParams.Player = SequencerModel->GetSequencer().Get();
			if (LabelParams.Player)
			{
				if (TViewModelPtr<FObjectBindingModel> ObjectBindingModel = FindAncestorOfType<FObjectBindingModel>())
				{
					LabelParams.BindingID = ObjectBindingModel->GetObjectGuid();

					// If the object binding model has an invalid binding, we want to use its label color, as it may be red or gray depending on situation
					// and we want the children of that to have the same color.
					// Otherwise, we can use the track's label color below
					TArrayView<TWeakObjectPtr<> > BoundObjects = LabelParams.Player->FindBoundObjects(LabelParams.BindingID, LabelParams.SequenceID);
					if (BoundObjects.Num() == 0)
					{
						return ObjectBindingModel->GetLabelColor();
					}
				}
			}
		}
	}

	return Track->GetLabelColor(LabelParams);
}

FSlateFontInfo FSectionOutlinerModel::GetLabelFont() const
{
	bool bAllAnimated = false;
	TViewModelPtr<FChannelGroupModel> TopLevelChannel = TopLevelChannelList.GetHead().ImplicitCast();
	if (TopLevelChannel)
	{
		for (const TViewModelPtr<FChannelModel>& ChannelModel : TopLevelChannel->GetTrackAreaModelListAs<FChannelModel>())
		{
			FMovieSceneChannel* Channel = ChannelModel->GetChannel();
			if (!Channel || Channel->GetNumKeys() == 0)
			{
				return FOutlinerItemModel::GetLabelFont();
			}
			else
			{
				bAllAnimated = true;
			}
		}
		if (bAllAnimated == true)
		{
			return FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.ItalicFont");
		}
	}
	return FOutlinerItemModel::GetLabelFont();
}

const FSlateBrush* FSectionOutlinerModel::GetIconBrush() const
{
	if (TSharedPtr<FSectionModel> SectionModel = WeakSectionModel.Pin())
	{
		if (TSharedPtr<ISequencerSection> SectionInterface = SectionModel->GetSectionInterface())
		{
			if (const FSlateBrush* Brush = SectionInterface->GetIconBrush())
			{
				return Brush;
			}
		}
	}
	// Fall back to the track editor's icon when the section doesn't provide its own
	return TrackEditor ? TrackEditor->GetIconBrush() : nullptr;
}

/*-----------------------------------------------------------------------------
	ISectionOwnerExtension
-----------------------------------------------------------------------------*/
FViewModelChildren FSectionOutlinerModel::GetSectionModels()
{
	return GetChildrenForList(&SectionModelList);
}

/*-----------------------------------------------------------------------------
	ITopLevelChannelHolderExtension
-----------------------------------------------------------------------------*/

FViewModelChildren FSectionOutlinerModel::GetTopLevelChannels()
{
	return GetChildrenForList(&TopLevelChannelList);
}

/*-----------------------------------------------------------------------------
	IResizableExtension
-----------------------------------------------------------------------------*/

bool FSectionOutlinerModel::IsResizable() const
{
	UMovieSceneTrack* Track = GetTrack();
	return Track && TrackEditor && TrackEditor->IsResizable(Track);
}

void FSectionOutlinerModel::Resize(float NewSize)
{
	UMovieSceneTrack* Track = GetTrack();

	if (Track && TrackEditor && TrackEditor->IsResizable(Track))
	{
		TrackEditor->Resize(NewSize, Track);
	}
}

/*-----------------------------------------------------------------------------
	ITrackAreaExtension
-----------------------------------------------------------------------------*/

FTrackAreaParameters FSectionOutlinerModel::GetTrackAreaParameters() const
{
	FTrackAreaParameters Params;
	Params.LaneType = ETrackAreaLaneType::Nested;
	Params.TrackLanePadding.Bottom = 1.f;
	return Params;
}

FViewModelVariantIterator FSectionOutlinerModel::GetTrackAreaModelList() const
{
	// Return the section model list so the section appears in the track area for this outliner row
	return &SectionModelList;
}

FViewModelVariantIterator FSectionOutlinerModel::GetTopLevelChildTrackAreaModels() const
{
	// Return the top level channels so they render in the track area beneath this section
	return &TopLevelChannelList;
}

/*-----------------------------------------------------------------------------
	IDimmableExtension
-----------------------------------------------------------------------------*/

bool FSectionOutlinerModel::IsDimmed() const
{
	UMovieSceneTrack* Track = GetTrack();
	UMovieSceneSection* Section = GetSection();

	if (Track && Section)
	{
		if (Track->IsRowEvalDisabled(Section->GetRowIndex()))
		{
			return true;
		}

		if (!Section->IsActive())
		{
			return true;
		}

		FGuid BindingID;
		FMovieSceneSequenceID SequenceID = MovieSceneSequenceID::Root;
		if (TViewModelPtr<FObjectBindingModel> ObjectBindingModel = FindAncestorOfType<FObjectBindingModel>())
		{
			BindingID = ObjectBindingModel->GetObjectGuid();
		}

		if (TViewModelPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>())
		{
			SequenceID = SequenceModel->GetSequenceID();

			if (TSharedPtr<FSequencerEditorViewModel> SequencerModel = SequenceModel->GetEditor())
			{
				if (Section->ConditionContainer.Condition)
				{
					if (!MovieSceneHelpers::EvaluateSequenceCondition(BindingID, SequenceID, Section->ConditionContainer.Condition, Section, SequencerModel->GetSequencer()->GetSharedPlaybackState()))
					{
						return true;
					}
				}

				if (const FMovieSceneTrackRowMetadata* TrackRowMetadata = Track->FindTrackRowMetadata(Section->GetRowIndex()))
				{
					if (TrackRowMetadata->ConditionContainer.Condition)
					{
						if (!MovieSceneHelpers::EvaluateSequenceCondition(BindingID, SequenceID, TrackRowMetadata->ConditionContainer.Condition, Track, SequencerModel->GetSequencer()->GetSharedPlaybackState()))
						{
							return true;
						}
					}
				}

				if (Track->ConditionContainer.Condition)
				{
					if (!MovieSceneHelpers::EvaluateSequenceCondition(BindingID, SequenceID, Track->ConditionContainer.Condition, Track, SequencerModel->GetSequencer()->GetSharedPlaybackState()))
					{
						return true;
					}
				}
			}
		}
	}

	return FOutlinerItemModel::IsDimmed();
}

/*-----------------------------------------------------------------------------
	IDeletableExtension
-----------------------------------------------------------------------------*/

bool FSectionOutlinerModel::CanDelete(FText* OutErrorMessage) const
{
	if (!WeakSection.IsValid())
	{
		return false;
	}

	// Check if the section interface allows deletion
	if (TSharedPtr<FSectionModel> SectionModel = WeakSectionModel.Pin())
	{
		if (TSharedPtr<ISequencerSection> SectionInterface = SectionModel->GetSectionInterface())
		{
			if (!SectionInterface->IsDeletable())
			{
				return false;
			}
		}
	}

	return true;
}

void FSectionOutlinerModel::Delete()
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		// Check if this section is owned by a decoration that provides it
		// (decoration-provided sections have the decoration as their outer, not the track)
		UObject* Outer = Section->GetOuter();
		if (IMovieSceneSectionProviderDecoration* SectionProvider = Cast<IMovieSceneSectionProviderDecoration>(Outer))
		{
			TArrayView<TObjectPtr<UMovieSceneSection>> ProviderSections = SectionProvider->GetSections();
			if (ProviderSections.Num() <= 1)
			{
				// Last/only section - remove the entire decoration
				if (UMovieSceneDecorationContainerObject* Container = Cast<UMovieSceneDecorationContainerObject>(Outer->GetOuter()))
				{
					Container->Modify();
					Container->RemoveDecoration(Outer->GetClass());
					return;
				}
			}
			// Multi-section case: don't allow direct deletion from outliner.
			// User must use the decoration editor UI to manage sections.
			return;
		}

		// Standard case: section is owned directly by the track
		if (UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>())
		{
			Track->Modify();
			Track->RemoveSection(*Section);
		}
	}
}

/*-----------------------------------------------------------------------------
	ILockableExtension
-----------------------------------------------------------------------------*/

ELockableLockState FSectionOutlinerModel::GetLockState() const
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		return Section->IsLocked() ? ELockableLockState::Locked : ELockableLockState::None;
	}

	return ELockableLockState::None;
}

void FSectionOutlinerModel::SetIsLocked(bool bInIsLocked)
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		Section->SetIsLocked(bInIsLocked);
	}
}

/*-----------------------------------------------------------------------------
	IConditionableExtension
-----------------------------------------------------------------------------*/

const UMovieSceneCondition* FSectionOutlinerModel::GetCondition() const
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		return Section->ConditionContainer.Condition;
	}
	return nullptr;
}

EConditionableConditionState FSectionOutlinerModel::GetConditionState() const
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
		TSharedPtr<ISequencer> Sequencer = SequenceModel ? SequenceModel->GetSequencer() : nullptr;
		if (Sequencer)
		{
			FGuid BindingID;

			if (TSharedPtr<IObjectBindingExtension> ParentBinding = FindAncestorOfType<IObjectBindingExtension>())
			{
				BindingID = ParentBinding->GetObjectGuid();
			}
			if (Section->ConditionContainer.Condition)
			{
				if (Section->ConditionContainer.Condition->bEditorForceTrue)
				{
					return EConditionableConditionState::HasConditionEditorForceTrue;
				}

				if (MovieSceneHelpers::EvaluateSequenceCondition(BindingID, Sequencer->GetFocusedTemplateID(), Section->ConditionContainer.Condition, Section, Sequencer->GetSharedPlaybackState()))
				{
					return EConditionableConditionState::HasConditionEvaluatingTrue;
				}
				else
				{
					return EConditionableConditionState::HasConditionEvaluatingFalse;
				}
			}
		}
	}
	return EConditionableConditionState::None;
}

void FSectionOutlinerModel::SetConditionEditorForceTrue(bool bEditorForceTrue)
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		if (Section->ConditionContainer.Condition)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("SequencerTrackNode", "ConditionEditorForceTrue", "Set Condition Editor Force True"));
			Section->ConditionContainer.Condition->Modify();
			Section->ConditionContainer.Condition->bEditorForceTrue = bEditorForceTrue;
		}
	}
}

/*-----------------------------------------------------------------------------
	IMutableExtension / ISoloableExtension
-----------------------------------------------------------------------------*/

bool FSectionOutlinerModel::IsMuted() const
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = Section->FindDecoration<UMovieSceneMuteSoloDecoration>())
		{
			return MuteSoloDecoration->IsMuted();
		}
	}
	return false;
}

void FSectionOutlinerModel::SetIsMuted(bool bIsMuted)
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		const bool bAlwaysMarkDirty = false;
		Section->Modify(bAlwaysMarkDirty);

		if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = Section->GetOrCreateDecoration<UMovieSceneMuteSoloDecoration>())
		{
			MuteSoloDecoration->SetMuted(bIsMuted);
		}
	}
}

bool FSectionOutlinerModel::IsSolo() const
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = Section->FindDecoration<UMovieSceneMuteSoloDecoration>())
		{
			return MuteSoloDecoration->IsSoloed();
		}
	}
	return false;
}

void FSectionOutlinerModel::SetIsSoloed(bool bIsSoloed)
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		const bool bAlwaysMarkDirty = false;
		Section->Modify(bAlwaysMarkDirty);

		if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = Section->GetOrCreateDecoration<UMovieSceneMuteSoloDecoration>())
		{
			MuteSoloDecoration->SetSoloed(bIsSoloed);
		}
	}
}

/*-----------------------------------------------------------------------------
	ISectionEventHandler
-----------------------------------------------------------------------------*/

void FSectionOutlinerModel::OnDecorationAdded(UObject*)
{
	RefreshDecorations();
}

void FSectionOutlinerModel::OnDecorationRemoved(UObject*)
{
	RefreshDecorations();
}

#if WITH_EDITOR
void FSectionOutlinerModel::OnPostUndo()
{
	// During Apply, Section::Serialize nulls ChannelProxy before Super::Serialize
	// reverts Decorations. The dropped shared ptr triggers FChannelCurveModel
	// listeners that immediately rebuild the proxy against the still-pre-undo
	// Decorations array, leaving stale decoration channels in the cached proxy.
	// Invalidate here, after properties are reverted but before the deferred
	// broadcast, so view-tree rebuild reads a fresh proxy.
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		Section->InvalidateChannelProxy();
	}

	RefreshDecorations();
}
#endif

void FSectionOutlinerModel::RefreshDecorations()
{
	UMovieSceneSection* Section = WeakSection.Get();
	if (!Section)
	{
		return;
	}

	UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>();
	if (!Track)
	{
		return;
	}

	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	if (!SequenceModel)
	{
		return;
	}

	FDecorationModelStorageExtension* DecorationModelStorage = SequenceModel->CastDynamic<FDecorationModelStorageExtension>();
	if (!DecorationModelStorage)
	{
		return;
	}

	FViewModelChildren OutlinerChildren = GetChildList(EViewModelListType::Outliner);
	TSharedPtr<ISequencer> SequencerPtr = SequenceModel->GetSequencer();
	DecorationModelStorage->SyncDecorationModels(Section, Track, OutlinerChildren, SequencerPtr.Get());
}

} // namespace Sequencer
} // namespace UE
