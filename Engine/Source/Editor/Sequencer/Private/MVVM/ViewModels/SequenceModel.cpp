// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/IObjectModelExtension.h"
#include "MVVM/Extensions/ILockableExtension.h"
#include "MVVM/Extensions/IMutableExtension.h"
#include "MVVM/Extensions/ISoloableExtension.h"
#include "MVVM/ViewModels/FolderModel.h"
#include "MVVM/ViewModels/OutlinerItemModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/OutlinerSpacer.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/EditorSharedViewModelData.h"

#include "MovieScene.h"
#include "MovieSceneFolder.h"
#include "MovieSceneSequence.h"

#include "Sequencer.h"
#include "SequencerOutlinerItemDragDropOp.h"
#include "ISequencerCoreModule.h"

#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "FSequenceModel"

namespace UE
{
namespace Sequencer
{

FOnInitializeSequenceModel FSequenceModel::CreateExtensionsEvent;

FSequenceModel::FSequenceModel(TWeakPtr<FSequencerEditorViewModel> InEditorViewModel)
	: RootOutlinerItems(EViewModelListType::Outliner)
	, DecorationModelList(GetDecorationModelListType())
{
	WeakEditor = InEditorViewModel;

	RegisterChildList(&RootOutlinerItems);
	RegisterChildList(&DecorationModelList);

	TSharedPtr<FSharedViewModelData> NewSharedData = MakeShared<FEditorSharedViewModelData>(WeakEditor.Pin().ToSharedRef());
	SetSharedDataFromConstructor(NewSharedData);
}

EViewModelListType FSequenceModel::GetDecorationModelListType()
{
	static EViewModelListType DecorationModelList = RegisterCustomModelListType();
	return DecorationModelList;
}

void FSequenceModel::InitializeExtensions()
{
	TSharedPtr<FSharedViewModelData> ThisSharedData = GetSharedData();

	// Re-generate hierarchical caches when the sequence changes
	ThisSharedData->AddDynamicExtension(FOutlinerCacheExtension::ID);
	ThisSharedData->AddDynamicExtension(FDeactiveStateCacheExtension::ID);
	ThisSharedData->AddDynamicExtension(FMuteStateCacheExtension::ID);
	ThisSharedData->AddDynamicExtension(FSoloStateCacheExtension::ID);
	ThisSharedData->AddDynamicExtension(FLockStateCacheExtension::ID);
	ThisSharedData->AddDynamicExtension(FConditionStateCacheExtension::ID);

	// Add our hierarchical cache processor
	TSharedPtr<FOutlinerCacheExtension> OutlinerCache = ThisSharedData->CastThisSharedChecked<FOutlinerCacheExtension>();
	OutlinerCache->Initialize(AsShared());

	// Make sure the hierarchical cache is updated when our hierarchy changes
	FSimpleMulticastDelegate& HierarchyChanged = ThisSharedData->SubscribeToHierarchyChanged(AsShared());
	HierarchyChanged.AddSP(OutlinerCache.ToSharedRef(), &FOutlinerCacheExtension::OnHierarchyUpdated);

	// This needs to be run outside of the constructor because a shared pointer to 'this' can't
	// be created until after the object is fully built.

	// Create the bottom spacer item
	BottomSpacer = MakeShared<FOutlinerSpacer>(30.f);
	GetChildrenForList(&RootOutlinerItems).AddChild(BottomSpacer);

	// Create extensions
	TSharedPtr<FSequencerEditorViewModel> Editor = WeakEditor.Pin();
	check(Editor);
	CreateExtensionsEvent.Broadcast(Editor, SharedThis(this));

	// Post-initialize extensions - this does the bulk of the population work for the sequence model
	PostInitializeExtensions();
}

void FSequenceModel::OnDecorationAdded(UObject* AddedDecoration)
{
	static ISequencerCoreModule& SequencerCore = FModuleManager::Get().LoadModuleChecked<ISequencerCoreModule>("SequencerCore");

	TSharedPtr<FViewModel> NewModel = SequencerCore.FactoryNewModel(AddedDecoration);
	if (NewModel && NewModel->IsA<IObjectModelExtension>())
	{
		GetChildrenForList(&DecorationModelList).AddChild(NewModel);
	}
}

void FSequenceModel::OnDecorationRemoved(UObject* RemovedDecoration)
{
	TViewModelPtr<IObjectModelExtension> ObjectToDetach;
	for (TViewModelPtr<IObjectModelExtension> Object : GetChildrenForList(&DecorationModelList).IterateSubList<IObjectModelExtension>())
	{
		if (Object->GetObject() == RemovedDecoration)
		{
			ObjectToDetach = Object;
			break;
		}
	}
	
	if (ObjectToDetach)
	{
		ObjectToDetach.AsModel()->RemoveFromParent();
	}
}

FMovieSceneSequenceID FSequenceModel::GetSequenceID() const
{
	return SequenceID;
}

UMovieSceneSequence* FSequenceModel::GetSequence() const
{
	return WeakSequence.Get();
}

UMovieScene* FSequenceModel::GetMovieScene() const
{
	UMovieSceneSequence* Sequence = WeakSequence.Get();
	return Sequence ? Sequence->GetMovieScene() : nullptr;
}

void FSequenceModel::SetSequence(UMovieSceneSequence* InSequence, FMovieSceneSequenceID InSequenceID)
{
	if (WeakSequence.Get() != InSequence || SequenceID != InSequenceID)
	{
		FViewModelHierarchyOperation HierarchyOperation(GetSharedData());

		// Remove everything and re-add the bottom spacer node
		DiscardAllChildren();
		if (BottomSpacer)
		{
			GetChildrenForList(&RootOutlinerItems).AddChild(BottomSpacer);
		}

		WeakSequence = InSequence;
		SequenceID = InSequenceID;

		SequenceEventHandler.Unlink();
		MovieSceneEventHandler.Unlink();
		MovieSceneDataEventHandler.Unlink();

		if (InSequence)
		{
			InSequence->EventHandlers.Link(SequenceEventHandler, this);
			InSequence->GetMovieScene()->UMovieSceneSignedObject::EventHandlers.Link(MovieSceneEventHandler, this);
			InSequence->GetMovieScene()->EventHandlers.Link(MovieSceneDataEventHandler, this);
		}

		ReinitializeExtensions();

		GetChildrenForList(&DecorationModelList).Empty();
		if (InSequence)
		{
			for (UObject* Decoration : InSequence->GetMovieScene()->GetDecorations())
			{
				OnDecorationAdded(Decoration);
			}
		}
	}
}

void FSequenceModel::OnPostUndo()
{
	{
		FViewModelHierarchyOperation HierarchyOp(GetSharedData());

		for (TSharedPtr<FViewModel> Child : GetChildren().ToArray())
		{
			Child->RemoveFromParent();
		}
	}

	ReinitializeExtensions();
}

TSharedPtr<FSequencerEditorViewModel> FSequenceModel::GetEditor() const
{
	return WeakEditor.Pin();
}

TSharedPtr<ISequencer> FSequenceModel::GetSequencer() const
{
	TSharedPtr<FSequencerEditorViewModel> SequencerEditorViewModel = WeakEditor.Pin();
	if (SequencerEditorViewModel)
	{	
		return SequencerEditorViewModel->GetSequencer();
	}

	return nullptr;
}

TSharedPtr<FSequencer> FSequenceModel::GetSequencerImpl() const
{
	TSharedPtr<FSequencerEditorViewModel> SequencerEditorViewModel = WeakEditor.Pin();
	if (SequencerEditorViewModel)
	{
		return SequencerEditorViewModel->GetSequencerImpl();
	}

	return nullptr;
}

void FSequenceModel::SortChildren()
{
	ISortableExtension::SortChildren(SharedThis(this), ESortingMode::Default);
}

FSortingKey FSequenceModel::GetSortingKey() const
{
	return FSortingKey();
}

void FSequenceModel::SetCustomOrder(int32 InCustomOrder)
{
}

TOptional<EItemDropZone> FSequenceModel::CanAcceptDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone)
{
	TSharedPtr<FSequencerOutlinerDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSequencerOutlinerDragDropOp>();
	if (!DragDropOp)
	{
		return TOptional<EItemDropZone>();
	}

	DragDropOp->ResetToDefaultToolTip();

	if (TargetModel == BottomSpacer)
	{
		// Dropping on the bottom spacer
		return EItemDropZone::AboveItem;
	}

	for (const TWeakViewModelPtr<IOutlinerExtension>& WeakModel : DragDropOp->GetDraggedViewModels())
	{
		FViewModelPtr Model = WeakModel.Pin();
		if (!Model)
		{
			// Silently allow null models
			continue;
		}
		if (Model.Get() == this || Model == TargetModel)
		{
			// Cannot drop onto self
			return TOptional<EItemDropZone>();
		}

		if (Model->IsA<FFolderModel>())
		{
			// We can always drop folders at the root
		}
		else if (TViewModelPtr<IObjectBindingExtension> Object = Model.ImplicitCast())
		{
			if (TViewModelPtr<IObjectBindingExtension> ParentObject = Model->CastParent<IObjectBindingExtension>())
			{
				// This operation should not have been allowed in the first place (child objects should never be draggable)
				return TOptional<EItemDropZone>();
			}
		}
		else if (TViewModelPtr<FTrackModel> Track = Model.ImplicitCast())
		{
			if (TViewModelPtr<IObjectBindingExtension> ParentObject = Model->CastParent<IObjectBindingExtension>())
			{
				// This operation should not have been allowed in the first place (tracks bound to an object should never be draggable)
				return TOptional<EItemDropZone>();
			}
		}
		else
		{
			// Unknown model type - do we silent not drag these or return an error?
			// For now we disallow the drag, but maybe that's not the right choice
			return TOptional<EItemDropZone>();
		}
	}

	// The dragged nodes were either all in folders, or all at the sequencer root.
	return ItemDropZone;
}

void FSequenceModel::PerformDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone)
{
	FViewModelHierarchyOperation ScopedOperation(GetSharedData());

	UMovieSceneSequence* Sequence   = WeakSequence.Get();
	UMovieScene*         MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;;
	if (!MovieScene)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("MoveItems", "Move to Root"));

	MovieScene->SetFlags(RF_Transactional);
	MovieScene->Modify();

	TSharedPtr<FViewModel> AttachAfter;
	if (TargetModel)
	{
		if (InItemDropZone == EItemDropZone::AboveItem)
		{
			AttachAfter = TargetModel->GetPreviousSibling();
		}
		else if (InItemDropZone == EItemDropZone::BelowItem)
		{
			AttachAfter = TargetModel;
		}
	}

	TSharedPtr<FSequencerOutlinerDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSequencerOutlinerDragDropOp>();

	// Drop handing for outliner drag/drop operations.
	// Warning: this handler may be dragging nodes from a *different* sequence
	if (DragDropOp)
	{
		FViewModelChildren OutlinerChildren = GetChildList(EViewModelListType::Outliner);

		for (const TWeakViewModelPtr<IOutlinerExtension>& WeakModel : DragDropOp->GetDraggedViewModels())
		{
			FViewModelPtr DraggedModel = WeakModel.Pin();
			if (!DraggedModel || DraggedModel == AttachAfter)
			{
				continue;
			}

			TViewModelPtr<FFolderModel> ExistingParentFolder = DraggedModel->CastParent<FFolderModel>();
			UMovieSceneFolder* OldFolder = ExistingParentFolder ? ExistingParentFolder->GetFolder() : nullptr;

			bool bSuccess = false;

			// Handle dropping a folder into another folder
			// @todo: if we ever support folders within object bindings this will need to have better validation
			if (TSharedPtr<FFolderModel> DraggedFolderModel = DraggedModel.ImplicitCast())
			{
				if (UMovieSceneFolder* DraggedFolder = DraggedFolderModel->GetFolder())
				{
					if (OldFolder)
					{
						OldFolder->SetFlags(RF_Transactional);
						OldFolder->Modify();
						OldFolder->RemoveChildFolder(DraggedFolder);
					}

					// Give this folder a unique name inside its new parent if necessary
					TArray<UMovieSceneFolder*> ExistingFolders;
					MovieScene->GetRootFolders(ExistingFolders);
					ExistingFolders.Remove(DraggedFolder);

					FName FolderName = UMovieSceneFolder::MakeUniqueChildFolderName(DraggedFolder->GetFolderName(), ExistingFolders);
					if (FolderName != DraggedFolder->GetFolderName())
					{
						DraggedFolder->SetFlags(RF_Transactional);
						DraggedFolder->Modify();
						DraggedFolder->SetFolderName(FolderName);
					}

					// MovieScene has already had Modify called above
					MovieScene->AddRootFolder(DraggedFolder);
					bSuccess = true;
				}
			}
			else if (TSharedPtr<IObjectBindingExtension> ObjectBinding = DraggedModel.ImplicitCast())
			{
				TViewModelPtr<IObjectBindingExtension> ParentObject = DraggedModel->CastParent<IObjectBindingExtension>();
				// Don't allow dropping an object binding if it has an object parent
				if (ParentObject == nullptr)
				{
					if (OldFolder)
					{
						OldFolder->SetFlags(RF_Transactional);
						OldFolder->Modify();
						OldFolder->RemoveChildObjectBinding(ObjectBinding->GetObjectGuid());
					}

					bSuccess = true;
				}
			}
			else if (TSharedPtr<FTrackModel> TrackModel = DraggedModel.ImplicitCast())
			{
				UMovieSceneTrack* Track = TrackModel->GetTrack();
				TViewModelPtr<IObjectBindingExtension> ParentObject = DraggedModel->CastParent<IObjectBindingExtension>();
				// Don't allow dropping a track if it has an object parent
				if (ParentObject == nullptr && Track != nullptr)
				{
					if (OldFolder)
					{
						OldFolder->SetFlags(RF_Transactional);
						OldFolder->Modify();
						OldFolder->RemoveChildTrack(Track);
					}

					bSuccess = true;
				}
			}

			if (bSuccess)
			{
				// Attach it to the right node, possibly setting its parent too
				OutlinerChildren.InsertChild(DraggedModel, AttachAfter);
			}
		}
	}

	// Forcibly update sorting
	int32 CustomSort = 0;
	for (const TViewModelPtr<ISortableExtension>& Sortable : GetChildrenOfType<ISortableExtension>())
	{
		Sortable->SetCustomOrder(CustomSort);
		++CustomSort;
	}
}

void FSequenceModel::OnModifiedIndirectly(UMovieSceneSignedObject*)
{
	TSharedPtr<FSequencerEditorViewModel> SequencerEditorViewModel = WeakEditor.Pin();
	if (!SequencerEditorViewModel)
	{
		return;
	}

	TSharedPtr<ISequencer> Sequencer = SequencerEditorViewModel->GetSequencer();
	const bool bRecording = Sequencer && Sequencer->OnGetIsRecording().IsBound() && Sequencer->OnGetIsRecording().Execute();

	// Disregard updating cached flags during recording
	if (!bRecording)
	{
		if (FOutlinerCacheExtension* OutlinerCache = GetSharedData()->CastThis<FOutlinerCacheExtension>())
		{
			OutlinerCache->UpdateCachedFlags();
		}
	}
}

void FSequenceModel::OnModifiedDirectly(UMovieSceneSignedObject*)
{
	TSharedPtr<FSequencerEditorViewModel> SequencerEditorViewModel = WeakEditor.Pin();
	if (!SequencerEditorViewModel)
	{
		return;
	}

	TSharedPtr<ISequencer> Sequencer = SequencerEditorViewModel->GetSequencer();
	const bool bRecording = Sequencer && Sequencer->OnGetIsRecording().IsBound() && Sequencer->OnGetIsRecording().Execute();

	// Disregard updating cached flags during recording
	if (!bRecording)
	{
		if (FOutlinerCacheExtension* OutlinerCache = GetSharedData()->CastThis<FOutlinerCacheExtension>())
		{
			OutlinerCache->UpdateCachedFlags();
		}
	}
}

} // namespace Sequencer
} // namespace UE

#undef LOCTEXT_NAMESPACE

