// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/SequencerOutlinerViewModel.h"
#include "MVVM/ViewModels/SequencerTrackAreaViewModel.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/CurveEditorExtension.h"
#include "MVVM/PinEditorExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/SharedViewModelData.h"
#include "Scripting/SequencerModuleScriptingLayer.h"
#include "ISequencerModule.h"
#include "Sequencer.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneSequenceID.h"

namespace UE::Sequencer
{
	
FSequencerEditorViewModel::FSequencerEditorViewModel(TSharedRef<ISequencer> InSequencer, const FSequencerHostCapabilities& InHostCapabilities)
	: WeakSequencer(InSequencer)
	, bSupportsCurveEditor(InHostCapabilities.bSupportsCurveEditor)
{
}

void FSequencerEditorViewModel::PreInitializeEditorImpl()
{
	if (bSupportsCurveEditor)
	{
		AddDynamicExtension(FCurveEditorExtension::ID);
	}

	AddDynamicExtension(FPinEditorExtension::ID);
}

TSharedPtr<FViewModel> FSequencerEditorViewModel::CreateRootModelImpl()
{
	return MakeShared<FSequenceModel>(SharedThis(this));
}

TSharedPtr<FOutlinerViewModel> FSequencerEditorViewModel::CreateOutlinerImpl()
{
	return MakeShared<FSequencerOutlinerViewModel>();
}

TSharedPtr<FTrackAreaViewModel> FSequencerEditorViewModel::CreateTrackAreaImpl()
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	check(Sequencer.IsValid());

	TSharedRef<FSequencerTrackAreaViewModel> NewTrackArea = MakeShared<FSequencerTrackAreaViewModel>(Sequencer.ToSharedRef());
	NewTrackArea->GetOnHotspotChangedDelegate().AddSP(SharedThis(this), &FSequencerEditorViewModel::OnTrackAreaHotspotChanged);
	return NewTrackArea;
}

USequencerScriptingLayer* FSequencerEditorViewModel::CreateScriptingLayerImpl()
{
	return NewObject<USequencerModuleScriptingLayer>();
}

TViewModelPtr<FSequenceModel> FSequencerEditorViewModel::GetRootSequenceModel() const
{
	return GetRootModel().ImplicitCast();
}

void FSequencerEditorViewModel::InitializeEditorImpl()
{
}

TSharedPtr<FSequencerCoreSelection> FSequencerEditorViewModel::CreateSelectionImpl()
{
	TSharedPtr<FSequencerSelection> SelectionInst = MakeShared<FSequencerSelection>();
	SelectionInst->Initialize(SharedThis(this));
	return SelectionInst;
}

TSharedPtr<FTrackAreaViewModel> FSequencerEditorViewModel::GetPinnedTrackArea() const
{
	return GetTrackArea();
}

TSharedPtr<ISequencer> FSequencerEditorViewModel::GetSequencer() const
{
	return WeakSequencer.Pin();
}

TSharedPtr<FSequencer> FSequencerEditorViewModel::GetSequencerImpl() const
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	return StaticCastSharedPtr<FSequencer>(Sequencer);
}

void FSequencerEditorViewModel::SetSequence(UMovieSceneSequence* InRootSequence)
{
	TSharedPtr<FSequenceModel> SequenceModel = GetRootModel().ImplicitCast();
	SequenceModel->SetSequence(InRootSequence, MovieSceneSequenceID::Root);
}

bool FSequencerEditorViewModel::IsReadOnly() const
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	return !Sequencer || Sequencer->IsReadOnly();
}

TSharedPtr<FSequencerSelection> FSequencerEditorViewModel::GetSelection() const
{
	return StaticCastSharedPtr<FSequencerSelection>(FEditorViewModel::GetSelection());
}

TSharedPtr<ITrackAreaHotspot> FSequencerEditorViewModel::GetHotspot() const
{
	return CurrentHotspot;
}

void FSequencerEditorViewModel::OnTrackAreaHotspotChanged(TSharedPtr<ITrackAreaHotspot> NewHotspot)
{
	CurrentHotspot = NewHotspot;
}

FSequencerEditorViewModel::~FSequencerEditorViewModel()
{
	// Get rid of previously active customizations.
	for (const TUniquePtr<ISequencerCustomization>& Customization : ActiveCustomizations)
	{
		Customization->UnregisterSequencerCustomization();
	}
	ActiveCustomizations.Reset();
}

bool FSequencerEditorViewModel::UpdateSequencerCustomizations(const UMovieSceneSequence* PreviousFocusedSequence)
{
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	TSharedPtr<FSequencerCustomizationManager> Manager = SequencerModule.GetSequencerCustomizationManager();

	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	UMovieSceneSequence* FocusedSequence = Sequencer->GetFocusedMovieSceneSequence();
	check(FocusedSequence != nullptr);

	// See if we need to change any of our customizations.
	if (PreviousFocusedSequence && !Manager->NeedsCustomizationChange(PreviousFocusedSequence, FocusedSequence))
	{
		return false;
	}

	// Get rid of previously active customizations.
	for (const TUniquePtr<ISequencerCustomization>& Customization : ActiveCustomizations)
	{
		Customization->UnregisterSequencerCustomization();
	}
	ActiveCustomizations.Reset();

	// Get the customizations for the current sequence.
	Manager->GetSequencerCustomizations(FocusedSequence, ActiveCustomizations);

	// Get the customization info.
	FSequencerCustomizationBuilder Builder(*Sequencer, *FocusedSequence);
	for (const TUniquePtr<ISequencerCustomization>& Customization : ActiveCustomizations)
	{
		Customization->RegisterSequencerCustomization(Builder);
	}
	ActiveCustomizationInfos = Builder.GetCustomizations();

	return true;
}

TArrayView<const FSequencerCustomizationInfo> FSequencerEditorViewModel::GetActiveCustomizationInfos() const
{
	return ActiveCustomizationInfos;
}

TSharedPtr<FExtender> FSequencerEditorViewModel::GetSequencerMenuExtender(
		TSharedPtr<FExtensibilityManager> ExtensibilityManager, const TArray<UObject*>& ContextObjects,
		TFunctionRef<const FOnGetSequencerMenuExtender&(const FSequencerCustomizationInfo&)> Endpoint, FViewModelPtr InViewModel) const
{
	TArray<TSharedPtr<FExtender>> Extenders;
	// Get the extenders from the extensibility manager.
	{
		TSharedRef<FUICommandList> CommandList(new FUICommandList);
		TSharedPtr<FExtender> ManagerExtender = ExtensibilityManager->GetAllExtenders(CommandList, ContextObjects);
		if (ManagerExtender)
		{
			Extenders.Add(ManagerExtender);
		}
	}
	// Get the extenders from any active sequencer customizations.
	for (const FSequencerCustomizationInfo& CustomizationInfo : ActiveCustomizationInfos)
	{
		const FOnGetSequencerMenuExtender& Delegate = ::Invoke(Endpoint, CustomizationInfo);
		if (Delegate.IsBound())
		{
			TSharedPtr<FExtender> CustomizationExtender = Delegate.Execute(InViewModel);
			if (CustomizationExtender)
			{
				Extenders.Add(CustomizationExtender);
			}
		}
	}
	return FExtender::Combine(Extenders);
}

} // namespace UE::Sequencer

