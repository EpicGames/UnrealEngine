// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerEditorViewModelImpl.h"

#include "Filters/FilterConfigIdentifiers.h"
#include "Filters/Linking/Mode/LinkedFilterFactoryViewModel.h"
#include "Filters/Linking/Mode/LinkedFilterViewModel.h"
#include "Filters/Linking/Mode/MakeLinkedFilterViewModelArgs.h"
#include "OutlinerFilterAreaManager.h"
#include "Sequencer.h"

namespace UE::Sequencer
{
FSequencerEditorViewModelImpl::FSequencerEditorViewModelImpl(
	TSharedRef<ISequencer> InSequencer, const FSequencerHostCapabilities& InHostCapabilities
	)
	: FSequencerEditorViewModel(InSequencer, InHostCapabilities)
{}

void FSequencerEditorViewModelImpl::InitializeEditorImpl()
{
	Super::InitializeEditorImpl();
	
	InitLinkedFiltering();
}

void FSequencerEditorViewModelImpl::InitLinkedFiltering()
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencerImpl();
	
	// TODO UE-362151: Implement data migration.
	// Before linked filtering, the filter state was saved into the section string "*SequencerSettings->GetName()", e.g. "LevelSequencerEditor".
	// Going forward, we can separate linked and instanced, e.g. ""LevelSequencerEditor.Linked", "LevelSequencerEditor.MainInstance", respectively.
	// For now, filtering will default to linked mode. For linked mode, we specify no sub-identifier: that causes FSequencerFilterBar to continue 
	// using the old config entry, which would be SequencerSettings->GetName(), e.g. "LevelSequencerEditor".
	const TCHAR* ConfigId = ConfigIds::LinkedConfigSubId;
	// Linked filter state should be affected by the USequencerSettings::bIncludePinnedInFilter setting.
	constexpr EFilterFlags LinkedFilterFlags = EFilterFlags::PinnedItemsCanSkipFiltering;
	FFilterBarInitArgs LinkedFilterBarArgs(*Sequencer, ConfigId, LinkedFilterFlags);
	LinkedFilterBarArgs.FilterAreaConfigId = ConfigIds::FilterArea_Outliner;
	const TSharedRef<FSequencerFilterBar> LinkedFilterBar = FSequencerFilterBar::Make(LinkedFilterBarArgs);
	
	LinkedFilterFactory = MakeShared<FLinkedFilterFactoryViewModel>(
		*Sequencer, LinkedFilterBar, Sequencer->GetNodeTree()->GetFilterEvaluator()
		);
	
	// The unlinked filter state for the Outliner should also be affected by the USequencerSettings::bIncludePinnedInFilter setting.
	constexpr EFilterFlags UnlinkedFilterFlags = EFilterFlags::PinnedItemsCanSkipFiltering;
	FMakeLinkedFilterViewModelArgs OutlinerInstanceArgs(ConfigIds::Sequencer_InstancedConfigSubId, UnlinkedFilterFlags);
	OutlinerInstanceArgs.FilterAreaConfigId = ConfigIds::FilterArea_Outliner;
	SequenceFilterInstanceViewModel = LinkedFilterFactory->MakeFilteringModelImpl(OutlinerInstanceArgs);
	OutlinerFilterArea = MakeShared<FOutlinerFilterAreaManager>(Sequencer, SequenceFilterInstanceViewModel.ToSharedRef());
}
}
