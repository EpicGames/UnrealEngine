// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/SequencerEditorViewModel.h"

namespace UE::Sequencer
{
class FOutlinerFilterAreaManager;
class FLinkedFilterViewModel;
class FLinkedFilterFactoryViewModel;

/** Exposes private API for Sequencer. */
class FSequencerEditorViewModelImpl : public FSequencerEditorViewModel
{
	using Super = FSequencerEditorViewModel;
public:
	
	explicit FSequencerEditorViewModelImpl(TSharedRef<ISequencer> InSequencer, const FSequencerHostCapabilities& InHostCapabilities);
	
	TSharedPtr<FLinkedFilterViewModel> GetLinkedFilterViewModelImpl() const { return SequenceFilterInstanceViewModel; }
	TSharedPtr<FLinkedFilterFactoryViewModel> GetLinkedFilterViewModelFactoryImpl() const { return LinkedFilterFactory; }
	
	/** @return The object responsible for syncing the linked filters to Sequencer's systems. */
	TSharedPtr<FOutlinerFilterAreaManager> GetOutlinerFilterArea() const { return OutlinerFilterArea; }
	
protected:
	
	//~ Begin FEditorViewModel Interface
	virtual void InitializeEditorImpl() override;
	//~ End FEditorViewModel Interface
	
private:
	
	/** Creates view models so (external) UI instances can sync to the filter state of the Sequencer UI. */
	TSharedPtr<FLinkedFilterFactoryViewModel> LinkedFilterFactory;
	/** Filter state view model that Sequencer UI uses when in ELinkedFilterMode::Instanced. */
	TSharedPtr<FLinkedFilterViewModel> SequenceFilterInstanceViewModel;
	
	/** Hooks up Sequencer's own FLinkedFilterViewModel with the relevant systems.*/
	TSharedPtr<FOutlinerFilterAreaManager> OutlinerFilterArea;
	
	/** Inits the systems relevant for linked filtering. */
	void InitLinkedFiltering();
};
}

