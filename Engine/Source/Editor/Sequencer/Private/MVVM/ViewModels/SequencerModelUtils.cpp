// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/SequencerModelUtils.h"

#include "Algo/Reverse.h"
#include "HAL/PlatformCrt.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "SequencerCoreFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

namespace UE
{
namespace Sequencer
{

TViewModelPtr<ITrackExtension> GetParentTrackNodeAndNamePath(const TViewModelPtr<IOutlinerExtension>& Node, TArray<FName>& OutNamePath)
{
	using namespace UE::Sequencer;

	OutNamePath.Add(Node->GetIdentifier());

	for (const TViewModelPtr<IOutlinerExtension>& Parent : Node.AsModel()->GetAncestorsOfType<IOutlinerExtension>())
	{
		if (TViewModelPtr<ITrackExtension> Track = Parent.ImplicitCast())
		{
			Algo::Reverse(OutNamePath);
			return Track;
		}

		OutNamePath.Add(Parent->GetIdentifier());
	}

	OutNamePath.Empty();
	return nullptr;
}

namespace SharedParentDetail
{
static TArray<TViewModelPtr<FViewModel>> GetAncestorsIncludingThis(const FViewModel& InViewModel)
{
	TArray<TViewModelPtr<FViewModel>> Ancestors;
	for (const TViewModelPtr<FViewModel>& Item : InViewModel.GetAncestors())
	{
		Ancestors.Add(Item);
	}
	return Ancestors;
}
}

TOptional<FSharedParentInfo> FindSharedParent(FViewModel& ViewModelA, FViewModel& ViewModelB)
{
	// Most common fast-path. Avoids TArray allocations.
	if (ViewModelA.GetParent() == ViewModelB.GetParent())
	{
		return FSharedParentInfo{ ViewModelA.GetParent(), FViewModelPtr(&ViewModelA), FViewModelPtr(&ViewModelB) };
	}
	
	const TArray<TViewModelPtr<FViewModel>> AncestorsA = SharedParentDetail::GetAncestorsIncludingThis(ViewModelA);
	const TArray<TViewModelPtr<FViewModel>> AncestorsB = SharedParentDetail::GetAncestorsIncludingThis(ViewModelB);
	const int32 MinNum = FMath::Min(AncestorsA.Num(), AncestorsB.Num());
	for (int32 Index = 0; Index < MinNum; ++Index)
	{
		const bool bModelsShareParent = AncestorsA[Index] == AncestorsB[Index];
		if (bModelsShareParent)
		{
			continue;
		}
		
		if (Index == 0)
		{
			// The view models don't share any parent. 
			return {};
		}
		
		const TViewModelPtr<FViewModel> LastSharedParent = AncestorsA[Index - 1];
		return FSharedParentInfo(LastSharedParent, AncestorsA[Index], AncestorsB[Index]);
	}
	
	return {};
}

bool ComesFirstInHierarchy(const FViewModelPtr& ViewModelA, const FViewModelPtr& ViewModelB)
{
	const TOptional<FSharedParentInfo> ParentInfo = ViewModelA && ViewModelB 
		? FindSharedParent(*ViewModelA, *ViewModelB) 
		: TOptional<FSharedParentInfo>{};
	
	// Array sorting algorithms require consistent <. We default to this in error cases.
	const bool bIsLessByPtr = ViewModelA.Get() < ViewModelB.Get(); 
	if (!ParentInfo)
	{
		return bIsLessByPtr;
	}
	
	const FViewModelPtr& Parent = ParentInfo->Parent;
	for (const FViewModelPtr& Child : Parent->GetChildren())
	{
		if (Child == ViewModelA)
		{
			// ViewModelA appeared before ViewModelB, so ViewmodelA < ViewModelB
			return true;
		}
		
		if (Child == ViewModelB)
		{
			// ViewModelB appeared before ViewModelA, so ViewModelA > ViewModelB.
			return false;
		}
	}
				
	return bIsLessByPtr;
}
} // namespace Sequencer
} // namespace UE
