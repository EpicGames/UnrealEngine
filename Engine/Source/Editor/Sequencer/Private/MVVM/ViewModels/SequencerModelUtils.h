// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "MVVM/ViewModelPtr.h"

class FName;

namespace UE
{
namespace Sequencer
{

class IOutlinerExtension;
class ITrackExtension;

/**
 * Takes a display node and traverses it's parents to find the nearest track node if any.  Also collects the names of the nodes which make
 * up the path from the track node to the display node being checked.  The name path includes the name of the node being checked, but not
 * the name of the track node.
 */
TViewModelPtr<ITrackExtension> GetParentTrackNodeAndNamePath(const TViewModelPtr<IOutlinerExtension>& Node, TArray<FName>& OutNamePath);

/** 
 * Information about two FViewModels ItemA and ItemB that share some parent.
 * Examples: 
 *	- ItemA = Location.X, ItemB = Location.Y		->		 Parent = Location, DirectChildA = Location.X, DirectChildB = Location.Y
 *	- ItemA = Location.X, ItemB = Rotation.Y		->		 Parent = Transform, DirectChildA = Location, DirectChildB = Rotation
 */
struct FSharedParentInfo
{
	/** The parent that both ItemA and ItemB share. */
	FViewModelPtr Parent;
	/** Immediate child of Parent that leads to ItemA. If ItemA->GetParent() == Parent, then DirectChildA == ItemA. */
	FViewModelPtr DirectChildA;
	/** Immediate child of Parent that leads to ItemB. If ItemB->GetParent() == Parent, then DirectChildB == ItemB. */
	FViewModelPtr DirectChildB;
};

/** Walks up the hierarchy of both view models and finds the first shared parent. Returns unset if there is no shared parent. */
TOptional<FSharedParentInfo> FindSharedParent(const FViewModel& ViewModelA, const FViewModel& ViewModelB);

/**
 * Utility for implementing ViewModelA < ViewModelB based on whether ViewModelA appears first in the tree hierarchy.
 * @return Whether ViewModelA comes first in the tree hierarchy.
 */
bool ComesFirstInHierarchy(const FViewModelPtr& ViewModelA, const FViewModelPtr& ViewModelB);
} // namespace Sequencer
} // namespace UE