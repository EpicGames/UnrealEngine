// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/SequencerOutlinerViewModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/SequenceModel.h"

#include "Sequencer.h"
#include "SequencerKeyCollection.h"
#include "SequencerOutlinerItemDragDropOp.h"
#include "MVVM/ViewModels/TrackRowReorderDragDropOp.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MVVM/ViewModels/TrackModel.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/GenericCommands.h"

namespace UE
{
namespace Sequencer
{

FSequencerOutlinerViewModel::FSequencerOutlinerViewModel()
{
}

void FSequencerOutlinerViewModel::RequestUpdate()
{
	FSequencerEditorViewModel* EditorViewModel = GetEditor()->CastThisChecked<FSequencerEditorViewModel>();
	TSharedPtr<ISequencer> Sequencer = EditorViewModel->GetSequencer();

	if (Sequencer)
	{
		Sequencer->RefreshTree();
	}
}

TSharedPtr<SWidget> FSequencerOutlinerViewModel::CreateContextMenuWidget()
{
	FSequencerEditorViewModel* EditorViewModel = GetEditor()->CastThisChecked<FSequencerEditorViewModel>();
	TSharedPtr<ISequencer> Sequencer = EditorViewModel->GetSequencer();

	if (Sequencer)
	{
		const bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, Sequencer->GetCommandBindings());

		BuildContextMenu(MenuBuilder);

		MenuBuilder.BeginSection("Edit");
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	return nullptr;
}

void FSequencerOutlinerViewModel::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	FSequencerEditorViewModel* EditorViewModel = GetEditor()->CastThisChecked<FSequencerEditorViewModel>();
	TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();

	// let toolkits populate the menu
	MenuBuilder.BeginSection("MainMenu");
	OnGetAddMenuContent.ExecuteIfBound(MenuBuilder, Sequencer.ToSharedRef());
	MenuBuilder.EndSection();

	// let track editors & object bindings populate the menu

	// Always create the section so that we afford extension
	MenuBuilder.BeginSection("AAObjectBindings");
	if (Sequencer.IsValid())
	{
		Sequencer->BuildAddObjectBindingsMenu(MenuBuilder);
	}
	MenuBuilder.EndSection();

	// Always create the section so that we afford extension
	MenuBuilder.BeginSection("AddTracks");
	if (Sequencer.IsValid())
	{
		Sequencer->BuildAddTrackMenu(MenuBuilder);
	}
	MenuBuilder.EndSection();
}

void FSequencerOutlinerViewModel::BuildCustomContextMenuForGuid(FMenuBuilder& MenuBuilder, FGuid ObjectBinding)
{
	OnBuildCustomContextMenuForGuid.ExecuteIfBound(MenuBuilder, ObjectBinding);
}

TSharedRef<FDragDropOperation> FSequencerOutlinerViewModel::InitiateDrag(TArray<TWeakViewModelPtr<IOutlinerExtension>>&& InDraggedModels)
{
	// Check if all dragged items are ITrackRowExtensions from the same parent track
	bool bAllTrackRows = true;
	TSharedPtr<ITrackExtension> CommonParentTrack;

	for (const TWeakViewModelPtr<IOutlinerExtension>& WeakModel : InDraggedModels)
	{
		TViewModelPtr<ITrackRowExtension> RowModel = WeakModel.Pin().ImplicitCast();
		if (!RowModel)
		{
			bAllTrackRows = false;
			break;
		}

		TViewModelPtr<ITrackExtension> ParentTrack = RowModel.AsModel()->FindAncestorOfType<ITrackExtension>();
		if (!ParentTrack)
		{
			bAllTrackRows = false;
			break;
		}

		// Ensure all rows belong to same track
		if (!CommonParentTrack.IsValid())
		{
			CommonParentTrack = ParentTrack.ImplicitCast();
		}
		else if (TSharedPtr<ITrackExtension> OtherParent = ParentTrack.ImplicitCast())
		{
			if (CommonParentTrack != OtherParent)
			{
				bAllTrackRows = false;
				break;
			}
		}
		else
		{
			bAllTrackRows = false;
			break;
		}
	}

	// If all items are track rows from the same parent, use track row reorder operation
	if (bAllTrackRows && CommonParentTrack.IsValid())
	{
		FText DefaultText = FText::Format(NSLOCTEXT("SequencerOutlinerViewModel", "ReorderTrackRowsFormat", "Reorder {0} track row(s)"), FText::AsNumber(InDraggedModels.Num()));
		return FTrackRowReorderDragDropOp::New(MoveTemp(InDraggedModels), DefaultText, nullptr);
	}

	// Otherwise, use generic outliner drag/drop
	FText DefaultText = FText::Format( NSLOCTEXT( "SequencerOutlinerViewModel", "DefaultDragDropFormat", "Move {0} item(s)" ), FText::AsNumber( InDraggedModels.Num() ) );
	return FSequencerOutlinerDragDropOp::New( MoveTemp(InDraggedModels), DefaultText, nullptr );
}

FFrameNumber FSequencerOutlinerViewModel::GetNextKey(const TArray<TSharedRef<UE::Sequencer::FViewModel>>& InNodes, FFrameNumber FrameNumber, EMovieSceneTimeUnit TimeUnit, const TRange<FFrameNumber>& Range, const bool bAllowLoop)
{
	return GetNextKeyInternal(InNodes, FrameNumber, TimeUnit, Range, EFindKeyDirection::Forwards, bAllowLoop);
}

FFrameNumber FSequencerOutlinerViewModel::GetPreviousKey(const TArray<TSharedRef<UE::Sequencer::FViewModel>>& InNodes, FFrameNumber FrameNumber, EMovieSceneTimeUnit TimeUnit, const TRange<FFrameNumber>& Range, const bool bAllowLoop)
{
	return GetNextKeyInternal(InNodes, FrameNumber, TimeUnit, Range, EFindKeyDirection::Backwards, bAllowLoop);
}

FFrameNumber FSequencerOutlinerViewModel::GetNextKeyInternal(const TArray<TSharedRef<UE::Sequencer::FViewModel>>& InNodes, FFrameNumber FrameNumber, EMovieSceneTimeUnit TimeUnit, const TRange<FFrameNumber>& Range, EFindKeyDirection Direction, const bool bAllowLoop)
{
	FSequencerEditorViewModel* EditorViewModel = GetEditor()->CastThisChecked<FSequencerEditorViewModel>();
	TSharedPtr<ISequencer> Sequencer = EditorViewModel->GetSequencer();
	if (Sequencer.IsValid())
	{
		FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
		FFrameRate DisplayRate = Sequencer->GetFocusedDisplayRate();

		FSequencerKeyCollection* KeyCollection = Sequencer->GetKeyCollection();

		const float DuplicateThresholdSeconds = SMALL_NUMBER;
		const int64 TotalMaxSeconds = static_cast<int64>(TNumericLimits<int32>::Max() / TickResolution.AsDecimal());

		FFrameNumber ThresholdFrames = (DuplicateThresholdSeconds * TickResolution).FloorToFrame();
		if (ThresholdFrames.Value < -TotalMaxSeconds)
		{
			ThresholdFrames.Value = TotalMaxSeconds;
		}
		else if (ThresholdFrames.Value > TotalMaxSeconds)
		{
			ThresholdFrames.Value = TotalMaxSeconds;
		}

		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			FrameNumber = ConvertFrameTime(FrameNumber, DisplayRate, TickResolution).FloorToFrame();
		}

		KeyCollection->Update(FSequencerKeyCollectionSignature::FromNodesRecursive(InNodes, ThresholdFrames));

		TOptional<FFrameNumber> NextKey = KeyCollection->GetNextKey(FrameNumber, Direction, Range, EFindKeyType::FKT_All, bAllowLoop);
		if (NextKey.IsSet())
		{
			if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
			{
				return ConvertFrameTime(NextKey.GetValue(), TickResolution, DisplayRate).FloorToFrame();
			}

			return NextKey.GetValue();
		}
	}

	return FrameNumber;
}

} // namespace Sequencer
} // namespace UE

