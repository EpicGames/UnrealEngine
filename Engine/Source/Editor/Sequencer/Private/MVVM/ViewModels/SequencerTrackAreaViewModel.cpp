// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/SequencerTrackAreaViewModel.h"

#include "AnimatedRange.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "Sequencer.h"
#include "Tools/SequencerEditTool_Movement.h"
#include "Tools/SequencerEditTool_Selection.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/GenericCommands.h"

namespace UE
{
namespace Sequencer
{

FSequencerTrackAreaViewModel::FSequencerTrackAreaViewModel(TSharedRef<ISequencer> InSequencer)
	: WeakSequencer(InSequencer)
{
}

void FSequencerTrackAreaViewModel::InitializeDefaultEditTools(UE::Sequencer::STrackAreaView& InTrackArea)
{
	TSharedPtr<FSequencer> Sequencer = StaticCastSharedPtr<FSequencer>(WeakSequencer.Pin());
	if (ensure(Sequencer))
	{
		AddEditTool(MakeShared<FSequencerEditTool_Selection>(*Sequencer, InTrackArea));
		AddEditTool(MakeShared<FSequencerEditTool_Movement>(*Sequencer, InTrackArea));
	}
}

FFrameRate FSequencerTrackAreaViewModel::GetTickResolution() const
{
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		return Sequencer->GetFocusedTickResolution();
	}
	return FFrameRate();
}

TRange<double> FSequencerTrackAreaViewModel::GetViewRange() const
{
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		return Sequencer->GetViewRange();
	}
	return TRange<double>::Empty();
}

} // namespace Sequencer
} // namespace UE

