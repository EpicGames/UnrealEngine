// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/SpawnableModel.h"

#include "Containers/Array.h"
#include "ISequencer.h"
#include "Internationalization/Internationalization.h"
#include "MVVM/ObjectBindingModelStorageExtension.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "Misc/Guid.h"
#include "MovieScene.h"
#include "MovieSceneSpawnRegister.h"
#include "MovieSceneSpawnable.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/Object.h"

namespace UE::Sequencer { class FViewModel; }
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "FSpawnableModel"

namespace UE
{
namespace Sequencer
{

FSpawnableModel::FSpawnableModel(FSequenceModel* InOwnerModel, const FMovieSceneBinding& InBinding, const FMovieSceneSpawnable& InSpawnable)
	: FObjectBindingModel(InOwnerModel, InBinding)
{
}

FSpawnableModel::~FSpawnableModel()
{
}

EObjectBindingType FSpawnableModel::GetType() const
{
	return EObjectBindingType::Spawnable;
}

void FSpawnableModel::OnConstruct()
{
	FObjectBindingModel::OnConstruct();

	UMovieScene* MovieScene = OwnerModel ? OwnerModel->GetMovieScene() : nullptr;
	FMovieSceneSpawnable* Spawnable = MovieScene ? MovieScene->FindSpawnable(ObjectBindingID) : nullptr;

	if (Spawnable)
	{
		FObjectBindingModelStorageExtension* ObjectBindingStorage = OwnerModel->CastDynamic<FObjectBindingModelStorageExtension>();

		// Also create possessable models for the spawnable's child possessables
		for (FGuid ChildPossessableGuid : Spawnable->GetChildPossessables())
		{
			if (const FMovieSceneBinding* ChildBinding = MovieScene->FindBinding(ChildPossessableGuid))
			{
				TSharedPtr<FViewModel> ObjectModel = ObjectBindingStorage->GetOrCreateModelForBinding(*ChildBinding);
					
				GetChildList(EViewModelListType::Outliner)
					.AddChild(ObjectModel);
			}
		}
	}
}

const UClass* FSpawnableModel::FindObjectClass() const
{
	UMovieScene*          MovieScene = OwnerModel ? OwnerModel->GetMovieScene() : nullptr;
	FMovieSceneSpawnable* Spawnable  = MovieScene ? MovieScene->FindSpawnable(ObjectBindingID) : nullptr;

	if (!Spawnable || Spawnable->GetObjectTemplate() == nullptr)
	{
		return UObject::StaticClass();
	}

	return Spawnable->GetObjectTemplate()->GetClass();
}

FText FSpawnableModel::GetIconToolTipText() const
{
	return LOCTEXT("SpawnableToolTip", "This item is spawned by sequencer according to this object's spawn track.");
}

const FSlateBrush* FSpawnableModel::GetIconOverlayBrush() const
{
	return FAppStyle::GetBrush("Sequencer.SpawnableIconOverlay");
}

FText FSpawnableModel::GetTooltipForSingleObjectBinding() const
{
	const UClass* ClassForObjectBinding = FindObjectClass();
	if (ClassForObjectBinding)
	{
		return FText::Format(LOCTEXT("SpawnableBoundObjectToolTip", "Spawnable Class: {0} (BindingID: {1})"), FText::FromName(ClassForObjectBinding->GetFName()), FText::FromString(LexToString(ObjectBindingID)));
	}
	return FObjectBindingModel::GetTooltipForSingleObjectBinding();
}

void FSpawnableModel::Delete()
{
	FObjectBindingModel::Delete();

	if (OwnerModel)
	{
		TSharedPtr<ISequencer> Sequencer  = OwnerModel->GetSequencer();
		UMovieScene*           MovieScene = OwnerModel->GetMovieScene();

		if (MovieScene && MovieScene->RemoveSpawnable(ObjectBindingID))
		{
			Sequencer->GetSpawnRegister().DestroySpawnedObject(ObjectBindingID, OwnerModel->GetSequenceID(), *Sequencer);
		}
	}
}

} // namespace Sequencer
} // namespace UE

#undef LOCTEXT_NAMESPACE
