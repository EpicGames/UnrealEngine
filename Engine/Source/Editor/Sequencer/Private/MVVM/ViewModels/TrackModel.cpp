// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/TrackModel.h"

#include "MVVM/SharedViewModelData.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/IRecyclableExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/FolderModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/TrackModelLayoutBuilder.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MVVM/ViewModels/TrackRowReorderDragDropOp.h"
#include "MVVM/DecorationModelStorageExtension.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/TrackRowModelStorageExtension.h"
#include "MVVM/Views/SOutlinerTrackView.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/Selection/Selection.h"

#include "MovieScene.h"
#include "MovieSceneFolder.h"
#include "MovieSceneTrack.h"
#include "MovieSceneSection.h"
#include "MovieSceneNameableTrack.h"
#include "Decorations/MovieSceneMuteSoloDecoration.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieScenePrimitiveMaterialTrack.h"
#include "EntitySystem/IMovieSceneBlenderSystemSupport.h"

#include "ISequencer.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "SSequencer.h"
#include "SequencerNodeTree.h"
#include "SequencerUtilities.h"
#include "SequencerCommonHelpers.h"

#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "TrackModel"

namespace UE::Sequencer
{

FTrackModel::FTrackModel(UMovieSceneTrack* Track)
	: SectionList(EViewModelListType::TrackArea)
	, TopLevelChannelList(GetTopLevelChannelGroupType())
	, WeakTrack(Track)
	, bNeedsUpdate(false)
{
	RegisterChildList(&SectionList);
	RegisterChildList(&TopLevelChannelList);

	SetIdentifier(Track->GetFName());
}

FTrackModel::~FTrackModel()
{
}

EViewModelListType FTrackModel::GetTopLevelChannelType()
{
	static EViewModelListType TopLevelChannel = RegisterCustomModelListType();
	return TopLevelChannel;
}

EViewModelListType FTrackModel::GetTopLevelChannelGroupType()
{
	static EViewModelListType TopLevelChannelGroup = RegisterCustomModelListType();
	return TopLevelChannelGroup;
}

FViewModelChildren FTrackModel::GetSectionModels()
{
	return GetChildrenForList(&SectionList);
}

FViewModelChildren FTrackModel::GetTopLevelChannels()
{
	return GetChildrenForList(&TopLevelChannelList);
}

UMovieSceneTrack* FTrackModel::GetTrack() const
{
	return WeakTrack.Get();
}

TSharedPtr<ISequencerTrackEditor> FTrackModel::GetTrackEditor() const
{
	return TrackEditor;
}

void FTrackModel::OnConstruct()
{
	UMovieSceneTrack* Track = GetTrack();
	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	check(SequenceModel && Track);

	if (!IsLinked())
	{
		Track->EventHandlers.Link(this);
	}

	TrackEditor = SequenceModel->GetSequencer()->GetTrackEditor(Track);

	ForceUpdate();
}

void FTrackModel::OnModifiedDirectly(UMovieSceneSignedObject*)
{
	if (!bNeedsUpdate)
	{
		bNeedsUpdate = true;
		UMovieSceneSignedObject::AddFlushSignal(SharedThis(this));
	}
}

void FTrackModel::OnModifiedIndirectly(UMovieSceneSignedObject* InSource)
{
	if (!bNeedsUpdate)
	{
		bNeedsUpdate = true;
		UMovieSceneSignedObject::AddFlushSignal(SharedThis(this));
	}
}

void FTrackModel::OnDeferredModifyFlush()
{
	if (bNeedsUpdate)
	{
		ForceUpdate();
		bNeedsUpdate = false;
	}
}

void FTrackModel::ForceUpdate()
{
	FViewModelHierarchyOperation HierarchyOperation(GetSharedData());

	FViewModelChildren OutlinerChildren = GetChildList(EViewModelListType::Outliner);
	FViewModelChildren SectionChildren  = GetChildList(EViewModelListType::TrackArea);
	FViewModelChildren TopLevelChannelChildren = GetChildList(GetTopLevelChannelGroupType());

	UMovieSceneTrack* Track = WeakTrack.Get();
	if (!Track)
	{
		// Free outliner and section children, this track is gone.
		OutlinerChildren.Empty();
		SectionChildren.Empty();
		TopLevelChannelChildren.Empty();
		return;
	}

	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	if (!SequenceModel)
	{
		// Not part of a full sequence hierarchy yet - wait for OnSetSharedData()
		return;
	}

	FSectionModelStorageExtension* SectionModelStorage = SequenceModel->CastDynamic<FSectionModelStorageExtension>();

	FGuid ObjectBinding;
	if (TSharedPtr<IObjectBindingExtension> ObjectBindingExtension = FindAncestorOfType<IObjectBindingExtension>())
	{
		ObjectBinding = ObjectBindingExtension->GetObjectGuid();
	}

	TBitArray<> PopulatedRows;

	for (UMovieSceneSection* Section : Track->GetAllSections())
	{
		const int32 RowIndex = Section->GetRowIndex();
		PopulatedRows.PadToNum(RowIndex + 1, false);
		PopulatedRows[RowIndex] = true;
	}

	const int32 NumRows = PopulatedRows.CountSetBits();

	FDecorationModelStorageExtension* DecorationModelStorage = SequenceModel->CastDynamic<FDecorationModelStorageExtension>();

	if (NumRows == 0)
	{
		// Reset expansion state if this track can no longer be expanded
		SetExpansion(false);

		// Clear any left-over row models, layout models, or section models.
		OutlinerChildren.Empty();
		SectionChildren.Empty();
		TopLevelChannelChildren.Empty();
	}
	else if (NumRows == 1)
	{
		// Keep sections alive by retaining the previous list temporarily
		TSharedPtr<FViewModel> SectionsTail;

		FScopedViewModelListHead RecycledModels(AsShared(), EViewModelListType::Recycled);
		GetChildrenForList(&SectionList).MoveChildrenTo<IRecyclableExtension>(RecycledModels.GetChildren(), IRecyclableExtension::CallOnRecycle);

		bool bNeedsLayout = NumRows != PreviousLayoutNumRows;

		//Sections that weren't expanded
		TSet<TSharedPtr<FSectionModel>> CollapsedSections;
		// Add all sections directly to this track row
		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			TSharedPtr<FSectionModel> SectionModel = SectionModelStorage->FindModelForSection(Section);
			if (!SectionModel && TrackEditor)
			{
				TSharedRef<ISequencerSection> SectionInterface = TrackEditor->MakeSectionInterface(*Section, *Track, ObjectBinding);
				SectionModel = SectionModelStorage->CreateModelForSection(Section, SectionInterface);
				bNeedsLayout = true;
			}
			else
			{
				TViewModelPtr<IOutlinerExtension> Outliner = SectionModel->FindAncestorOfType<IOutlinerExtension>();
				if (Outliner && (Outliner->IsExpanded() == false))
				{
					CollapsedSections.Add(SectionModel);
				}
			}
			if (ensure(SectionModel))
			{
				bNeedsLayout |= SectionModel->NeedsLayout();

				// Move the child back into the real section list
				SectionChildren.InsertChild(SectionModel, SectionsTail);
				SectionsTail = SectionModel;
			}
		}

		// If we are discarding any sections (because they still remain in the recycled list) we must run the layout
		bNeedsLayout |= !RecycledModels.GetChildren().IsEmpty();

		if (bNeedsLayout)
		{

			OutlinerChildren.MoveChildrenTo<IRecyclableExtension>(RecycledModels.GetChildren(), IRecyclableExtension::CallOnRecycle);
			TopLevelChannelChildren.MoveChildrenTo<IRecyclableExtension>(RecycledModels.GetChildren(), IRecyclableExtension::CallOnRecycle);

			// Rebuild the outliner layout for this track. This will clear our children and rebuild them if needed
			// (with potentially recycled children), so if we went from, say, 2 rows to 1 row, it should correctly
			// discard any children we don't need anymore.
			FTrackModelLayoutBuilder LayoutBuilder(AsShared());

			for (TSharedPtr<FSectionModel> Section : TViewModelListIterator<FSectionModel>(&SectionList))
			{
				LayoutBuilder.RefreshLayout(Section);
				if (CollapsedSections.Contains(Section))
				{
					TViewModelPtr<IOutlinerExtension> Outliner = Section->FindAncestorOfType<IOutlinerExtension>();
					if (Outliner)
					{
						Outliner->SetExpansion(false);
					}
				}
			}

			if (OutlinerChildren.IsEmpty())
			{
				// Reset expansion state if this track can no longer be expanded
				SetExpansion(false);
			}
		}
	}
	else
	{
		// Always expand parent tracks
		SetExpansion(true);

		// Keep sections alive by retaining the previous list temporarily
		// This should only be required if this track previously represented
		// a single row, but now there are multiple rows
		FScopedViewModelListHead RecycledModels(AsShared(), EViewModelListType::Recycled);
		GetChildrenForList(&SectionList).MoveChildrenTo<IRecyclableExtension>(RecycledModels.GetChildren(), IRecyclableExtension::CallOnRecycle);
		OutlinerChildren.MoveChildrenTo<IRecyclableExtension>(RecycledModels.GetChildren(), IRecyclableExtension::CallOnRecycle);
		TopLevelChannelChildren.MoveChildrenTo<IRecyclableExtension>(RecycledModels.GetChildren(), IRecyclableExtension::CallOnRecycle);

		// We need to build row models so let's grab the storage for that
		FTrackRowModelStorageExtension* TrackRowModelStorage = SequenceModel->CastDynamic<FTrackRowModelStorageExtension>();
		check(TrackRowModelStorage);

		// Create track row models for all populated rows
		TSharedPtr<FTrackRowModel> LastTrackRowModel;
		for (TConstSetBitIterator<> It(PopulatedRows); It; ++It)
		{
			const int32 RowIndex = It.GetIndex();

			bool bNeedsLayout = NumRows != PreviousLayoutNumRows;

			TSharedPtr<FTrackRowModel> TrackRowModel = TrackRowModelStorage->FindModelForTrackRow(Track, RowIndex);
			if (!TrackRowModel)
			{
				TrackRowModel = TrackRowModelStorage->CreateModelForTrackRow(Track, RowIndex);
				bNeedsLayout = true;
			}

			if (ensure(TrackRowModel))
			{
				OutlinerChildren.InsertChild(TrackRowModel, LastTrackRowModel);
				LastTrackRowModel = TrackRowModel;

				TrackRowModel->RefreshLayout(bNeedsLayout);
			}
		}
	}

	// Handle track-level decorations
	if (DecorationModelStorage)
	{
		TSharedPtr<ISequencer> SequencerPtr = SequenceModel->GetSequencer();
		DecorationModelStorage->SyncDecorationModels(Track, Track, OutlinerChildren, SequencerPtr.Get());
	}

	PreviousLayoutNumRows = NumRows;
}

FOutlinerSizing FTrackModel::GetOutlinerSizing() const
{
	const TSharedPtr<FSequencerEditorViewModel> Editor = GetEditor();
	FViewDensityInfo Density = Editor ? Editor->GetViewDensity() : FViewDensityInfo();

	float Height = Density.UniformHeight.Get(SequencerLayoutConstants::SectionAreaDefaultHeight);
	if (auto It = SectionList.Iterate<FSectionModel>(); It)
	{
		TSharedPtr<FSectionModel> Section = *It;
		Height = Section->GetSectionInterface()->GetSectionHeight(Density);
	}
	return FOutlinerSizing(Height);
}

void FTrackModel::GetIdentifierForGrouping(TStringBuilder<128>& OutString) const
{
	FOutlinerItemModel::GetIdentifier().ToString(OutString);
}

FTrackAreaParameters FTrackModel::GetTrackAreaParameters() const
{
	FTrackAreaParameters Params;
	Params.LaneType = ETrackAreaLaneType::Nested;
	Params.TrackLanePadding.Bottom = 1.f;
	return Params;
}

FViewModelVariantIterator FTrackModel::GetTrackAreaModelList() const
{
	return &SectionList;
}

FViewModelVariantIterator FTrackModel::GetTopLevelChildTrackAreaModels() const
{
	return &TopLevelChannelList;
}

bool FTrackModel::CanRename() const
{
	UMovieSceneNameableTrack* NameableTrack = Cast<UMovieSceneNameableTrack>(GetTrack());
	return NameableTrack && NameableTrack->CanRename();
}

void FTrackModel::Rename(const FText& NewName)
{
	UMovieSceneNameableTrack* NameableTrack = ::Cast<UMovieSceneNameableTrack>(GetTrack());

	if (NameableTrack && !NameableTrack->GetDisplayName().EqualTo(NewName))
	{
		const FScopedTransaction Transaction(NSLOCTEXT("SequencerTrackNode", "RenameTrack", "Rename Track"));
		NameableTrack->SetDisplayName(NewName);

		SetIdentifier(FName(*NewName.ToString()));

		// HACK: this should not exist but is required to make renaming emitters work in niagara
		if (TSharedPtr<FSequenceModel> OwnerModel = FindAncestorOfType<FSequenceModel>())
		{
			OwnerModel->GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		}
	}
}

bool FTrackModel::IsRenameValidImpl(const FText& NewName, FText& OutErrorMessage) const
{
	UMovieSceneNameableTrack* NameableTrack = ::Cast<UMovieSceneNameableTrack>(GetTrack());
	if (NameableTrack)
	{
		return NameableTrack->ValidateDisplayName(NewName, OutErrorMessage);
	}
	return false;
}

void FTrackModel::SortChildren()
{
	// Nothing to do
}

FSortingKey FTrackModel::GetSortingKey() const
{
	FSortingKey SortingKey;

	if (UMovieSceneTrack* Track = GetTrack())
	{
		SortingKey.DisplayName = Track->GetDisplayName();
		SortingKey.CustomOrder = Track->GetSortingOrder();
	}

	// When inside object bindings, we come after other object bindings. Elsewhere, we come before object bindings.
	const bool bHasParentObjectBinding = (CastParent<IObjectBindingExtension>() != nullptr);
	SortingKey.PrioritizeBy(bHasParentObjectBinding ? 1 : 2);

	return SortingKey;
}

void FTrackModel::SetCustomOrder(int32 InCustomOrder)
{
	if (UMovieSceneTrack* Track = GetTrack())
	{
		Track->SetSortingOrder(InCustomOrder);
	}
}

bool FTrackModel::HasCurves() const
{
	FViewModelChildren TopLevelChannels = const_cast<FTrackModel*>(this)->GetTopLevelChannels();
	for (auto It(TopLevelChannels.IterateSubList<FChannelGroupModel>()); It; ++It)
	{
		if (It->HasCurves())
		{
			return true;
		}
	}
	return false;
}

void FTrackModel::CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels)
{
	TViewModelPtr<FChannelGroupModel> ChannelGroup = TopLevelChannelList.GetHead().ImplicitCast();
	if (ChannelGroup)
	{
		ChannelGroup->CreateCurveModels(OutCurveModels);
	}
}

bool FTrackModel::GetDefaultExpansionState() const
{
	TViewModelListIterator<ITrackRowExtension> It = GetChildrenOfType<ITrackRowExtension>();
	const bool bHasTrackRows = (bool)It;
	if (bHasTrackRows)
	{
		return true;
	}

	UMovieSceneTrack* Track = GetTrack();
	if (TrackEditor && Track)
	{
		return TrackEditor->GetDefaultExpansionState(Track);
	}

	return false;
}

bool FTrackModel::IsDimmed() const
{
	UMovieSceneTrack* Track = GetTrack();

	if (Track)
	{
		if (Track->IsEvalDisabled())
		{
			return true;
		}
		if (Track->ConditionContainer.Condition)
		{
			FGuid BindingID;
			FMovieSceneSequenceID SequenceID = MovieSceneSequenceID::Root;
			if (TViewModelPtr<FObjectBindingModel> ObjectBindingModel = FindAncestorOfType<FObjectBindingModel>())
			{
				BindingID = ObjectBindingModel->GetObjectGuid();
			}
			if (TViewModelPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>())
			{
				SequenceID = SequenceModel->GetSequenceID();

				if (TSharedPtr<FSequencerEditorViewModel> SequencerModel = SequenceModel->GetEditor())
				{
					if (!MovieSceneHelpers::EvaluateSequenceCondition(BindingID, SequenceID, Track->ConditionContainer.Condition, Track, SequencerModel->GetSequencer()->GetSharedPlaybackState()))
					{
						return true;
					}
				}
			}
		} 
	}

	return FOutlinerItemModel::IsDimmed();
}

FSlateFontInfo FTrackModel::GetLabelFont() const
{
	bool bAllAnimated = false;
	TViewModelPtr<FChannelGroupModel> TopLevelChannel = TopLevelChannelList.GetHead().ImplicitCast();
	if (TopLevelChannel)
	{
		for (const TViewModelPtr<FChannelModel>& ChannelModel : TopLevelChannel->GetTrackAreaModelListAs<FChannelModel>())
		{
			FMovieSceneChannel* Channel = ChannelModel->GetChannel();
			if (!Channel || Channel->GetNumKeys() == 0)
			{
				return FOutlinerItemModel::GetLabelFont();
			}
			else
			{
				bAllAnimated = true;
			}
		}
		if (bAllAnimated == true)
		{
			return FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.ItalicFont");
		}
	}
	return FOutlinerItemModel::GetLabelFont();
}

const FSlateBrush* FTrackModel::GetIconBrush() const
{
	return TrackEditor ? TrackEditor->GetIconBrush() : nullptr;
}

FText FTrackModel::GetLabel() const
{
	UMovieSceneTrack* Track = GetTrack();
	return Track ? Track->GetDisplayName() : FText::GetEmpty();
}

FSlateColor FTrackModel::GetLabelColor() const
{
	UMovieSceneTrack* Track = GetTrack();
	if (!Track)
	{
		return FSlateColor::UseForeground();
	}

	FMovieSceneLabelParams LabelParams;
	LabelParams.bIsDimmed = IsDimmed();
	if (TViewModelPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>())
	{
		if (TSharedPtr<FSequencerEditorViewModel> SequencerModel = SequenceModel->GetEditor())
		{
			LabelParams.SequenceID = SequenceModel->GetSequenceID();
			LabelParams.Player = SequencerModel->GetSequencer().Get();
			if (LabelParams.Player)
			{
				if (TViewModelPtr<FObjectBindingModel> ObjectBindingModel = FindAncestorOfType<FObjectBindingModel>())
				{
					LabelParams.BindingID = ObjectBindingModel->GetObjectGuid();

					// If the object binding model has an invalid binding, we want to use its label color, as it may be red or gray depending on situation
					// and we want the children of that to have the same color.
					// Otherwise, we can use the track's label color below
					TArrayView<TWeakObjectPtr<> > BoundObjects = LabelParams.Player->FindBoundObjects(LabelParams.BindingID, LabelParams.SequenceID);
					if (BoundObjects.Num() == 0)
					{
						return ObjectBindingModel->GetLabelColor();
					}
				}
			}
		}
	}

	return Track->GetLabelColor(LabelParams);
}

FText FTrackModel::GetLabelToolTipText() const
{
	UMovieSceneTrack* Track = GetTrack();
	if (!Track)
	{
		return FText();
	}

	FMovieSceneLabelParams LabelParams;
	LabelParams.bIsDimmed = IsDimmed();
	if (TViewModelPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>())
	{
		if (TSharedPtr<FSequencerEditorViewModel> SequencerModel = SequenceModel->GetEditor())
		{
			LabelParams.SequenceID = SequenceModel->GetSequenceID();
			LabelParams.Player = SequencerModel->GetSequencer().Get();
			if (LabelParams.Player)
			{
				if (TViewModelPtr<FObjectBindingModel> ObjectBindingModel = FindAncestorOfType<FObjectBindingModel>())
				{
					LabelParams.BindingID = ObjectBindingModel->GetObjectGuid();
				}
				return Track->GetDisplayNameToolTipText(LabelParams);
			}
		}
	}
	return FText();
}

TSharedPtr<SWidget> FTrackModel::CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName)
{
	FBuildColumnWidgetParams Params(SharedThis(this), InParams);
	return TrackEditor->BuildOutlinerColumnWidget(Params, InColumnName);
}

bool FTrackModel::IsResizable() const
{
	UMovieSceneTrack* Track = GetTrack();
	return Track && TrackEditor->IsResizable(Track);
}

void FTrackModel::Resize(float NewSize)
{
	UMovieSceneTrack* Track = GetTrack();

	if (Track && TrackEditor->IsResizable(Track))
	{
		TrackEditor->Resize(NewSize, Track);
	}
}

ELockableLockState FTrackModel::GetLockState() const
{
	int32 NumSections = 0;
	int32 NumLockedSections = 0;

	for (const TViewModelPtr<FSectionModel>& Section : SectionList.Iterate<FSectionModel>())
	{
		++NumSections;

		UMovieSceneSection* SectionObject = Section->GetSection();
		if (SectionObject && SectionObject->IsLocked())
		{
			++NumLockedSections;
		}
	}

	if (NumSections == 0 || NumLockedSections == 0)
	{
		return ELockableLockState::None;
	}
	return NumLockedSections == NumSections ? ELockableLockState::Locked : ELockableLockState::PartiallyLocked;
}

void FTrackModel::SetIsLocked(bool bInIsLocked)
{
	for (const TViewModelPtr<FSectionModel>& Section : SectionList.Iterate<FSectionModel>())
	{
		UMovieSceneSection* SectionObject = Section->GetSection();
		if (SectionObject)
		{
			SectionObject->Modify();
			SectionObject->SetIsLocked(bInIsLocked);
		}
	}
}

const UMovieSceneCondition* FTrackModel::GetCondition() const
{
	UMovieSceneTrack* const Track = GetTrack();
	if (IsValid(Track))
	{
		return Track->ConditionContainer.Condition;
	}
	return nullptr;
}

EConditionableConditionState FTrackModel::GetConditionState() const
{
	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	TSharedPtr<ISequencer> Sequencer = SequenceModel ? SequenceModel->GetSequencer() : nullptr;
	if (Sequencer)
	{
		FGuid BindingID;

		if (TSharedPtr<IObjectBindingExtension> ParentBinding = FindAncestorOfType<IObjectBindingExtension>())
		{
			BindingID = ParentBinding->GetObjectGuid();
		}
		UMovieSceneTrack* const Track = GetTrack();
		if (IsValid(Track))
		{
			if (Track->ConditionContainer.Condition)
			{
				if (Track->ConditionContainer.Condition->bEditorForceTrue)
				{
					return EConditionableConditionState::HasConditionEditorForceTrue;
				}

				if (MovieSceneHelpers::EvaluateSequenceCondition(BindingID, Sequencer->GetFocusedTemplateID(), Track->ConditionContainer.Condition, Track, Sequencer->GetSharedPlaybackState()))
				{
					return EConditionableConditionState::HasConditionEvaluatingTrue;
				}
				else
				{
					return EConditionableConditionState::HasConditionEvaluatingFalse;
				}
			}
		
			// Special case. If we support multiple rows, and there is only a single row, then we must also check track row metadata for a condition here, as there will be no track row model.
			if (Track->SupportsMultipleRows() && Track->GetMaxRowIndex() == 0)
			{
				if (const FMovieSceneTrackRowMetadata* TrackRowMetadata = Track->FindTrackRowMetadata(0))
				{
					if (TrackRowMetadata->ConditionContainer.Condition)
					{
						if (TrackRowMetadata->ConditionContainer.Condition->bEditorForceTrue)
						{
							return EConditionableConditionState::HasConditionEditorForceTrue;
						}
						else if (MovieSceneHelpers::EvaluateSequenceCondition(BindingID, Sequencer->GetFocusedTemplateID(), TrackRowMetadata->ConditionContainer.Condition, Track, Sequencer->GetSharedPlaybackState()))
						{
							return EConditionableConditionState::HasConditionEvaluatingTrue;
						}
						else
						{
							return EConditionableConditionState::HasConditionEvaluatingFalse;
						}
					}
				}
			}
		}
	}
	return EConditionableConditionState::None;
}

void FTrackModel::SetConditionEditorForceTrue(bool bEditorForceTrue)
{
	UMovieSceneTrack* const Track = GetTrack();
	if (IsValid(Track))
	{
		if (Track->ConditionContainer.Condition) 
		{
			const FScopedTransaction Transaction(NSLOCTEXT("SequencerTrackNode", "ConditionEditorForceTrue", "Set Condition Editor Force True"));
			Track->ConditionContainer.Condition->Modify();
			Track->ConditionContainer.Condition->bEditorForceTrue = bEditorForceTrue;
		}
	}
}

bool FTrackModel::CanDrag() const
{
	// Can only drag root tracks at the moment
	TSharedPtr<IObjectBindingExtension> ObjectBindingExtension = FindAncestorOfType<IObjectBindingExtension>();
	return ObjectBindingExtension == nullptr;
}

TOptional<EItemDropZone> FTrackModel::CanAcceptDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone)
{
	// Check if this is a track row reorder operation
	TSharedPtr<FTrackRowReorderDragDropOp> TrackRowReorderOp = DragDropEvent.GetOperationAs<FTrackRowReorderDragDropOp>();
	if (TrackRowReorderOp.IsValid())
	{
		// Ensure the dragged rows belong to this track
		TSharedPtr<FTrackModel> DraggedParent = TrackRowReorderOp->GetCommonParentTrack();
		if (DraggedParent.Get() == this)
		{
			// Accept drops above or below track rows for reordering
			if (InItemDropZone == EItemDropZone::AboveItem || InItemDropZone == EItemDropZone::BelowItem)
			{
				return InItemDropZone;
			}
		}
	}

	return TOptional<EItemDropZone>();
}

void FTrackModel::PerformDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone)
{
	TSharedPtr<FTrackRowReorderDragDropOp> TrackRowReorderOp = DragDropEvent.GetOperationAs<FTrackRowReorderDragDropOp>();
	if (TrackRowReorderOp.IsValid())
	{
		// Ensure the dragged rows belong to this track
		TSharedPtr<FTrackModel> DraggedParent = TrackRowReorderOp->GetCommonParentTrack();
		if (DraggedParent.Get() == this && TargetModel)
		{
			// Get the target ITrackRowExtension
			TViewModelPtr<ITrackRowExtension> TargetTrackRowItem = TargetModel.ImplicitCast();
			if (TargetTrackRowItem)
			{
				int32 TargetRowIndex = TargetTrackRowItem->GetRowIndex();

				// Adjust target based on drop zone
				if (InItemDropZone == EItemDropZone::BelowItem)
				{
					TargetRowIndex++;
				}

				const FScopedTransaction Transaction(NSLOCTEXT("SequencerTrackNode", "ReorderTrackRows", "Reorder Track Rows"));
				TrackRowReorderOp->ExecuteReorder(TargetRowIndex);
			}
		}
	}
}

void FTrackModel::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UMovieSceneTrack* const Track = GetTrack();
	if (!IsValid(Track))
	{
		return;
	}

	if (TrackEditor)
	{
		TrackEditor->BuildTrackContextMenu(MenuBuilder, Track);
	}

	TArray<TWeakObjectPtr<>> WeakTracks;
	WeakTracks.Add(Track);
	SequencerHelpers::BuildEditTrackMenu(Sequencer, WeakTracks, MenuBuilder, true);

	if (Track->GetSupportedBlendTypes().Num() > 0)
	{
		SequencerHelpers::BuildNewSectionMenu(Sequencer, 1, GetTrack(), MenuBuilder);
	}

	SequencerHelpers::BuildBlendingMenu(Sequencer, Track, MenuBuilder);

	const TArray<TWeakObjectPtr<>> TrackAreaModels = SequencerHelpers::GetSectionObjectsFromTrackAreaModels(GetTrackAreaModelList());
	SequencerHelpers::BuildEditSectionMenu(Sequencer, TrackAreaModels, MenuBuilder, true);

	if (const TViewModelPtr<FChannelGroupModel> ChannelGroup = TopLevelChannelList.GetHead().ImplicitCast())
	{
		ChannelGroup->BuildChannelOverrideMenu(MenuBuilder);
	}

	FOutlinerItemModel::BuildContextMenu(MenuBuilder);
}

void FTrackModel::BuildSidebarMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UMovieSceneTrack* const Track = GetTrack();
	if (!IsValid(Track))
	{
		return;
	}

	if (TrackEditor)
	{
		TrackEditor->BuildTrackSidebarMenu(MenuBuilder, Track);
	}

	TArray<TWeakObjectPtr<>> WeakTracks;
	WeakTracks.Add(Track);
	SequencerHelpers::BuildEditTrackMenu(Sequencer, WeakTracks, MenuBuilder, false);


	if (Track->GetSupportedBlendTypes().Num() > 0)
	{
		SequencerHelpers::BuildNewSectionMenu(Sequencer, 1, GetTrack(), MenuBuilder);
	}

	SequencerHelpers::BuildBlendingMenu(Sequencer, Track, MenuBuilder);

	const TArray<TWeakObjectPtr<>> TrackAreaModels = SequencerHelpers::GetSectionObjectsFromTrackAreaModels(GetTrackAreaModelList());
	SequencerHelpers::BuildEditSectionMenu(Sequencer, TrackAreaModels, MenuBuilder, false);

	if (const TViewModelPtr<FChannelGroupModel> ChannelGroup = TopLevelChannelList.GetHead().ImplicitCast())
	{
		ChannelGroup->BuildChannelOverrideMenu(MenuBuilder);
	}

	FOutlinerItemModel::BuildSidebarMenu(MenuBuilder);
}

bool FTrackModel::CanDelete(FText* OutErrorMessage) const
{
	return true;
}

void FTrackModel::Delete()
{
	UMovieSceneTrack* Track = GetTrack();
	if (!Track)
	{
		return;
	}

	// Remove from a parent folder if necessary.
	if (TViewModelPtr<FFolderModel> ParentFolder = CastParent<FFolderModel>())
	{
		ParentFolder->GetFolder()->Modify();
		ParentFolder->GetFolder()->RemoveChildTrack(Track);
	}

	TSharedPtr<FSequenceModel> OwnerModel = FindAncestorOfType<FSequenceModel>();
	TSharedPtr<IObjectBindingExtension> ParentObjectBinding = FindAncestorOfType<IObjectBindingExtension>();

	check(OwnerModel);

	UMovieScene* MovieScene = OwnerModel->GetMovieScene();

	MovieScene->Modify();
	if (ParentObjectBinding)
	{
		FMovieSceneBinding* Binding = MovieScene->FindBinding(ParentObjectBinding->GetObjectGuid());
		if (Binding)
		{
			Binding->RemoveTrack(*Track, MovieScene);
		}
	}
	else if (MovieScene->GetCameraCutTrack() == Track)
	{
		MovieScene->RemoveCameraCutTrack();
	}
	else
	{
		MovieScene->RemoveTrack(*Track);
	}
}

bool FTrackModel::IsMuted() const
{
	UMovieSceneTrack* Track = GetTrack();
	if (!Track)
	{
		return false;
	}

	if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = Track->FindDecoration<UMovieSceneMuteSoloDecoration>())
	{
		return MuteSoloDecoration->IsMuted();
	}

	return false;
}

void FTrackModel::SetIsMuted(bool bIsMuted)
{
	UMovieSceneTrack* Track = GetTrack();
	if (!Track)
	{
		return;
	}

	const bool bAlwaysMarkDirty = false;
	Track->Modify(bAlwaysMarkDirty);

	if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = Track->GetOrCreateDecoration<UMovieSceneMuteSoloDecoration>())
	{
		MuteSoloDecoration->SetMuted(bIsMuted);
	}
}

bool FTrackModel::IsSolo() const
{
	UMovieSceneTrack* Track = GetTrack();
	if (!Track)
	{
		return false;
	}

	if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = Track->FindDecoration<UMovieSceneMuteSoloDecoration>())
	{
		return MuteSoloDecoration->IsSoloed();
	}

	return false;
}

void FTrackModel::SetIsSoloed(bool bIsSoloed)
{
	UMovieSceneTrack* Track = GetTrack();
	if (!Track)
	{
		return;
	}

	const bool bAlwaysMarkDirty = false;
	Track->Modify(bAlwaysMarkDirty);

	if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = Track->GetOrCreateDecoration<UMovieSceneMuteSoloDecoration>())
	{
		MuteSoloDecoration->SetSoloed(bIsSoloed);
	}
}

bool FTrackModel::FindBoundObjects(TArray<UObject*>& OutBoundObjects) const
{
	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	TSharedPtr<ISequencer> Sequencer = SequenceModel ? SequenceModel->GetSequencer() : nullptr;
	if (!Sequencer)
	{
		return false;
	}

	TSharedPtr<IObjectBindingExtension> ParentBinding = FindAncestorOfType<IObjectBindingExtension>();
	if (!ParentBinding)
	{
		return false;
	}

	TArrayView<TWeakObjectPtr<>> FoundBoundObjects = Sequencer->FindBoundObjects(ParentBinding->GetObjectGuid(), Sequencer->GetFocusedTemplateID());
	OutBoundObjects.Reserve(OutBoundObjects.Num() + FoundBoundObjects.Num());
	for (TWeakObjectPtr<> WeakObject : FoundBoundObjects)
	{
		if (UObject* Object = WeakObject.Get())
		{
			OutBoundObjects.Add(Object);
		}
	}
	return true;
}

int32 FTrackModel::GetMaxRowIndexForSectionDrag(const TSet<UMovieSceneSection*>& ExcludeSections) const
{
	int32 MaxRow = INDEX_NONE;
	UMovieSceneTrack* Track = GetTrack();
	if (!Track)
	{
		return MaxRow;
	}

	for (UMovieSceneSection* Section : Track->GetAllSections())
	{
		if (!ExcludeSections.Contains(Section))
		{
			MaxRow = FMath::Max(MaxRow, Section->GetRowIndex() + 1);
		}
	}
	return MaxRow;
}

void FTrackModel::ShiftRowsDownFromIndex(int32 StartIndex, const TSet<UMovieSceneSection*>& ExcludeSections)
{
	UMovieSceneTrack* Track = GetTrack();
	if (!Track)
	{
		return;
	}

	Track->Modify();

	const TArray<UMovieSceneSection*>& AllSections = Track->GetAllSections();
	for (UMovieSceneSection* Section : AllSections)
	{
		if (!ExcludeSections.Contains(Section) && Section->GetRowIndex() >= StartIndex)
		{
			Section->Modify();
			Section->SetRowIndex(Section->GetRowIndex() + 1);
		}
	}
}

bool FTrackModel::OnSectionVerticalDrag(const FSectionVerticalDragContext& Context)
{
	if (!Context.Section || !Context.DraggedSections)
	{
		return false;
	}

	UMovieSceneTrack* Track = GetTrack();
	if (!Track)
	{
		return false;
	}

	// Don't allow multi-row selection to be moved vertically
	if (Context.bSectionsAreOnDifferentRows)
	{
		return false;
	}

	int32 CurrentRowIndex = Context.Section->GetRowIndex();
	int32 TargetRowIndex = CurrentRowIndex;
	int32 MaxRowIndex = GetMaxRowIndexForSectionDrag(*Context.DraggedSections);

	// Check if we have TrackRowModel children - determines which calculation method to use
	// (FTrackRowModel children mean the track has multiple rows with separate row models)
	bool bHasTrackRowChildren = false;
	for (TSharedPtr<FViewModel> ChildNode : GetChildren())
	{
		if (ChildNode->CastThis<FTrackRowModel>())
		{
			bHasTrackRowChildren = true;
			break;
		}
	}

	if (bHasTrackRowChildren)
	{
		// FTrackRowModel case - use child iteration to find target row based on geometry
		// This is used when the track has TrackRowModel children (expanded multi-row track)

		// First, find the TrackRowModel that contains the current section
		// (this is needed for checking non-dragged sections in the current row)
		TSharedPtr<FTrackRowModel> CurrentSectionRow;
		for (TSharedPtr<FViewModel> ChildNode : GetChildren())
		{
			if (TSharedPtr<FTrackRowModel> TrackRow = ChildNode->CastThisShared<FTrackRowModel>())
			{
				// Check if this row contains the section
				for (TSharedPtr<FSectionModel> SectionNode : TrackRow->GetTrackAreaModelListAs<FSectionModel>())
				{
					if (SectionNode->GetSection() == Context.Section)
					{
						CurrentSectionRow = TrackRow;
						break;
					}
				}
				if (CurrentSectionRow)
				{
					break;
				}
			}
		}

		int32 ChildIndex = 0;
		for (TSharedPtr<FViewModel> ChildNode : GetChildren())
		{
			FVirtualGeometry ChildVirtualGeometry;
			if (IGeometryExtension* ChildGeometryExtension = ChildNode->CastThis<IGeometryExtension>())
			{
				ChildVirtualGeometry = ChildGeometryExtension->GetVirtualGeometry();
			}

			float VirtualSectionTop = ChildVirtualGeometry.Top;
			float VirtualSectionBottom = ChildVirtualGeometry.NestedBottom;

			if (ChildIndex == 0 && (Context.VirtualMousePos.Y <= VirtualSectionTop || Context.LocalMousePos.Y <= 0))
			{
				// Mouse is above the first row
				TargetRowIndex = 0;

				// Check if the current row has non-dragged sections
				// (if so, we need to shift others down instead of just moving to row 0)
				if (CurrentSectionRow)
				{
					for (TSharedPtr<FSectionModel> SectionNode : CurrentSectionRow->GetTrackAreaModelListAs<FSectionModel>())
					{
						if (!Context.DraggedSections->Contains(SectionNode->GetSection()))
						{
							TargetRowIndex = -1;
							break;
						}
					}
				}
				break;
			}
			else if (Context.VirtualMousePos.Y < VirtualSectionBottom)
			{
				TargetRowIndex = ChildIndex;
				break;
			}
			else
			{
				TargetRowIndex = ChildIndex + 1;
			}

			++ChildIndex;
		}
	}
	else
	{
		// Find the total height of the track - this is necessary because tracks may contain key areas, but they will not use sub tracks unless there is more than one row
		const int32 NumRows = FMath::Max(CurrentRowIndex + 1, MaxRowIndex);
		const FVirtualGeometry TrackVirtualGeometry = GetVirtualGeometry();
		const float VirtualSectionHeight = TrackVirtualGeometry.NestedBottom - TrackVirtualGeometry.Top;
		const float VirtualRowHeight = VirtualSectionHeight / NumRows;

		const int32 NewIndex = FMath::FloorToInt((Context.VirtualMousePos.Y - TrackVirtualGeometry.Top) / VirtualRowHeight);
		TargetRowIndex = FMath::Clamp(NewIndex, 0, MaxRowIndex);

		// If close to the top of the track, signal to move everything else down
		if (Context.VirtualMousePos.Y <= TrackVirtualGeometry.Top || Context.LocalMousePos.Y <= 0)
		{
			TargetRowIndex = -1;
		}
	}

	// Check if anything changed
	if (TargetRowIndex == CurrentRowIndex)
	{
		return false;
	}

	// Prevent flickering by only moving sections if the user has actually made an effort to do so
	int32 TargetRowDelta = TargetRowIndex - CurrentRowIndex;
	if (TargetRowDelta != 0 && Context.PrevMousePosY.IsSet())
	{
		float MouseDeltaY = Context.LocalMousePos.Y - Context.PrevMousePosY.GetValue();
		// Check mouse has been moved in the direction of intended move
		if ((TargetRowDelta < 0 && MouseDeltaY > 1.0f) ||
			(TargetRowDelta > 0 && MouseDeltaY < 1.0f))
		{
			// Mouse was not moved in the direction the section wants to swap
			// Assume offset is due to UI relayout and block moving the section
			return false;
		}
	}

	// Build list of non-dragged sections for overlap checking
	TArray<UMovieSceneSection*> NonDraggedSections;
	for (UMovieSceneSection* TrackSection : Track->GetAllSections())
	{
		if (!Context.DraggedSections->Contains(TrackSection))
		{
			NonDraggedSections.Add(TrackSection);
		}
	}

	// Check if vertical movement is allowed (blend type allows overlap, or no overlap would occur)
	// Note: We pass 0 for horizontal delta since we're only checking vertical movement here
	if (!Context.Section->GetBlendType().IsValid() &&
		Context.Section->OverlapsWithSections(NonDraggedSections, TargetRowDelta, 0))
	{
		// Section would overlap and doesn't support blending - don't allow move
		return false;
	}

	bool bRowIndexChanged = false;

	// Reached the top, move everything else we're not moving downwards
	if (TargetRowIndex == -1)
	{
		// If the sections being moved are all at the top, and all others are below it, do nothing
		bool bSectionsBeingMovedAreAtTop = true;
		for (const FInitialRowIndex& InitialRowIndex : *Context.InitialSectionRowIndices)
		{
			if (!Context.DraggedSections->Contains(InitialRowIndex.Section))
			{
				if (Context.LowestDraggedRow.IsSet() && InitialRowIndex.RowIndex <= Context.LowestDraggedRow.GetValue())
				{
					bSectionsBeingMovedAreAtTop = false;
					break;
				}
			}
		}

		if (!bSectionsBeingMovedAreAtTop)
		{
			for (const FInitialRowIndex& InitialRowIndex : *Context.InitialSectionRowIndices)
			{
				if (!Context.DraggedSections->Contains(InitialRowIndex.Section))
				{
					InitialRowIndex.Section->Modify();
					InitialRowIndex.Section->SetRowIndex(InitialRowIndex.RowIndex + 1);
					bRowIndexChanged = true;
				}
			}
		}
	}
	else
	{
		// But first check if sections being moved are already at the bottom
		// If the sections being moved are all at the bottom, and all others are above it, do nothing
		bool bSectionsBeingMovedAreAtBottom = true;
		for (const FInitialRowIndex& InitialRowIndex : *Context.InitialSectionRowIndices)
		{
			if (!Context.DraggedSections->Contains(InitialRowIndex.Section))
			{
				if (Context.HighestDraggedRow.IsSet() && InitialRowIndex.RowIndex >= Context.HighestDraggedRow.GetValue())
				{
					bSectionsBeingMovedAreAtBottom = false;
					break;
				}
			}
		}

		if (!bSectionsBeingMovedAreAtBottom || TargetRowIndex < CurrentRowIndex)
		{
			Context.Section->Modify();
			Context.Section->SetRowIndex(TargetRowIndex);
			bRowIndexChanged = true;
		}
	}

	// Handle auto-expansion for tracks with child rows (TrackRowModels)
	// This matches the original FTrackRowModel expansion logic
	if (bHasTrackRowChildren)
	{
		if (TargetRowIndex > 0)
		{
			// Targeting a row > 0 - expand if collapsed
			if (!IsExpanded() && !bAutoExpandedDuringDrag)
			{
				SetExpansion(true);
				bAutoExpandedDuringDrag = true;
			}
		}
		else if (bAutoExpandedDuringDrag)
		{
			// Targeting row 0 or top insertion - collapse if we auto-expanded
			SetExpansion(false);
			bAutoExpandedDuringDrag = false;
		}
	}

	return bRowIndexChanged;
}

void FTrackModel::OnBeginSectionVerticalDrag()
{
	// Reset auto-expansion tracking at the start of each drag operation
	bAutoExpandedDuringDrag = false;
}

void FTrackModel::OnEndSectionVerticalDrag(const FSectionVerticalDragContext& Context)
{
	// If we auto-expanded this track during the drag, collapse it when drag ends
	if (bAutoExpandedDuringDrag)
	{
		SetExpansion(false);
		bAutoExpandedDuringDrag = false;
	}
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

