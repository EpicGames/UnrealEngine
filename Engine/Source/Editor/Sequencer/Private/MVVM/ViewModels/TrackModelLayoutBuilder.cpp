// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/TrackModelLayoutBuilder.h"

#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "ISequencerSection.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/IRecyclableExtension.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "SequencerCoreFwd.h"

class FText;

namespace UE
{
namespace Sequencer
{

FHierarchicalModelListRefresher::FListData::FListData(TSharedPtr<FViewModel> InParent, const FViewModelChildren& InExistingChildren)
	: Parent(InParent)
	, Children(InExistingChildren)
{
}

FHierarchicalModelListRefresher::FHierarchicalModelListRefresher()
{
}

FHierarchicalModelListRefresher::FHierarchicalModelListRefresher(TSharedPtr<FViewModel> InRoot, FViewModelChildren InExistingChildren)
{
	// Add a recycled child list to the model. This will get cleaned up when
	// this FHierarchicalModelListRefresher instance is destroyed
	ConditionalRecycleChildren(InRoot, InExistingChildren);

	ListData.Add(FListData(InRoot, InExistingChildren));
}

void FHierarchicalModelListRefresher::ConditionalRecycleChildren(const TSharedPtr<FViewModel>& InModel, FViewModelChildren InExistingChildren)
{
	TOptional<FViewModelChildren> RecycledChildren = InModel->FindChildList(EViewModelListType::Recycled);
	if (!RecycledChildren)
	{
		if (RecycledLists.Num() == 0 || RecycledLists.Last().GetSlack() == 0)
		{
			RecycledLists.Emplace();
			RecycledLists.Last().Reserve(16);
		}

		RecycledLists.Last().Emplace(InModel, EViewModelListType::Recycled);

		RecycledChildren = InModel->GetChildList(EViewModelListType::Recycled);

		check(RecycledChildren.IsSet());
		InExistingChildren.MoveChildrenTo<IRecyclableExtension>(RecycledChildren.GetValue(), IRecyclableExtension::CallOnRecycle);
	}
}

bool FHierarchicalModelListRefresher::IsValid() const
{
	return ListData.Num() > 0;
}

void FHierarchicalModelListRefresher::Reset()
{
	check(ListData.Num() >= 1);
	ListData.RemoveAt(1, ListData.Num() - 1, EAllowShrinking::No);

	FListData& Last = ListData.Last();
	Last.AttachTail = nullptr;
}

void FHierarchicalModelListRefresher::Link(TSharedPtr<FViewModel> Item)
{
	FListData& Last = ListData.Last();

	if (Item != Last.AttachTail)
	{
		// Attach this item to the attach tail, or the head of the list
		Last.Children.InsertChild(Item, Last.AttachTail);
		Last.AttachTail = Item;
	}
}

void FHierarchicalModelListRefresher::RecurseInto(TSharedPtr<FViewModel> Item, FViewModelChildren InExistingChildren)
{
	// This item may already have recycled children if we're refreshing a track
	// or row that has multiple sections with different channel layouts.
	// We ensure that children remain eligible for recycling for the duration
	// of all sections by adding a temporary recycle list to the model while
	// the layout is being refreshed, but we create this recycle list only on the
	// first try to not recycle "legitimate" children we added in the previous
	// loop.
	ConditionalRecycleChildren(Item, InExistingChildren);

	ListData.Add(FListData(Item, InExistingChildren));
}

TSharedPtr<FViewModel> FHierarchicalModelListRefresher::GetCurrentParent() const
{
	if (ensure(ListData.Num() >= 1))
	{
		return ListData.Last().Parent;
	}
	return nullptr;
}

EViewModelListType FHierarchicalModelListRefresher::GetCurrentType() const
{
	if (ensure(ListData.Num() >= 1))
	{
		return ListData.Last().Children.GetType();
	}
	return EViewModelListType::Invalid;
}

void FHierarchicalModelListRefresher::Pop()
{
	if (ensure(ListData.Num() > 1))
	{
		ListData.Pop();
	}
}

FTrackModelLayoutBuilder::FTrackModelLayoutBuilder(TSharedPtr<FViewModel> InSharedTreeRoot)
	: Root(InSharedTreeRoot)
	, OutlinerList(InSharedTreeRoot, InSharedTreeRoot->GetChildList(EViewModelListType::Outliner))
	, Section(nullptr)
{
}

FTrackModelLayoutBuilder::~FTrackModelLayoutBuilder()
{
	// Recompute shared sizing
	for (TSharedPtr<ICompoundOutlinerExtension> CompoundItem : Root->GetDescendantsOfType<ICompoundOutlinerExtension>())
	{
		CompoundItem->RecomputeSizing();
	}
	for (TSharedPtr<FChannelGroupOutlinerModel> OutlinerChannelGroup : Root->GetDescendantsOfType<FChannelGroupOutlinerModel>())
	{
		OutlinerChannelGroup->OnUpdated();
	}
}

void FTrackModelLayoutBuilder::RefreshLayout(TSharedPtr<FSectionModel> InSection)
{
	InSection->SetLinkedOutlinerItem(CastViewModelChecked<IOutlinerExtension>(Root));
	if (UMovieSceneSection* SectionObject = InSection->GetSection())
	{
		InSection->SetLayoutChannelProxy(SectionObject->GetChannelProxy().AsWeak());
		InSection->SetLayoutRowIndex(SectionObject->GetRowIndex());
	}
	else
	{
		InSection->SetLayoutChannelProxy(nullptr);
		InSection->SetLayoutRowIndex(-1);
	}

	// Reset everything
	OutlinerList.Reset();

	// Start off with the track area list adding to the top-level-channel list
	TrackAreaList = FHierarchicalModelListRefresher(InSection, InSection->GetChildList(FTrackModel::GetTopLevelChannelType()));
	Section = InSection;

	Section->GetSectionInterface()->GenerateSectionLayout(*this);

	Section = nullptr;
	TrackAreaList = FHierarchicalModelListRefresher();
}

void FTrackModelLayoutBuilder::PushCategory(FName CategoryName, const FText& DisplayLabel, FGetMovieSceneTooltipText GetGroupTooltipTextDelegate, TFunction<TSharedPtr<FCategoryModel>(FName, const FText&)> OptionalFactory)
{
	check(TrackAreaList.IsValid() && Section);

	auto CategoryNamePredicate = [CategoryName](const auto& InModel){ return InModel.GetCategoryName() == CategoryName; };

	// ------------------------------------
	// Add a category group to the outliner
	// Note: the outliner can comprise the categories and channels of multile sections, potentially with different combinations
	//       of channels. We have to be careful to ensure that we reuse existing items that may have been added to the outliner
	//       from a previously built section.

	TSharedPtr<FCategoryGroupModel> OutlinerModel = OutlinerList.FindItem<FCategoryGroupModel>(CategoryNamePredicate);
	if (!OutlinerModel)
	{
		OutlinerModel = MakeShared<FCategoryGroupModel>(CategoryName, DisplayLabel, GetGroupTooltipTextDelegate);
	}

	OutlinerList.Link(OutlinerModel);
	OutlinerList.RecurseInto(OutlinerModel, OutlinerModel->GetChildList(EViewModelListType::Outliner));

	// ------------------------------------------
	// Add the actual category to the track model
	TSharedPtr<FCategoryModel> TrackAreaModel = TrackAreaList.FindItem<FCategoryModel>(CategoryNamePredicate);
	if (!TrackAreaModel)
	{
		if (OptionalFactory)
		{
			TrackAreaModel = OptionalFactory(CategoryName, DisplayLabel);
		}

		if (!TrackAreaModel)
		{
			TrackAreaModel = MakeShared<FCategoryModel>(CategoryName);
		}
	}

	OutlinerModel->AddCategory(TrackAreaModel);
	TrackAreaModel->SetLinkedOutlinerItem(OutlinerModel);

	TrackAreaList.Link(TrackAreaModel);
	TrackAreaList.RecurseInto(TrackAreaModel, TrackAreaModel->GetChildList(EViewModelListType::Generic));
}

void FTrackModelLayoutBuilder::PopCategory()
{
	check(TrackAreaList.IsValid() && Section);

	TrackAreaList.Pop();
	OutlinerList.Pop();
}

void FTrackModelLayoutBuilder::SetTopLevelChannel(const FMovieSceneChannelHandle& Channel, TFunction<TSharedPtr<UE::Sequencer::FChannelModel>(FName, const FSectionModel&, const FMovieSceneChannelHandle&)> OptionalFactory)
{
	ensureAlwaysMsgf(
			OutlinerList.GetCurrentParent() == Root,
			TEXT("Attempting to assign a top level channel when a category node is active. Top level key nodes will always be added to the outermost track node."));

	FViewModelChildren TopLevelGroup = Root->GetChildList(FTrackModel::GetTopLevelChannelGroupType());

	TSharedPtr<FViewModel> CurrentModel = OutlinerList.GetCurrentParent();
	OutlinerList.RecurseInto(CurrentModel, TopLevelGroup);

	AddChannel(Channel, true, OptionalFactory);

	OutlinerList.Pop();
}

void FTrackModelLayoutBuilder::AddChannel(const FMovieSceneChannelHandle& Channel, TFunction<TSharedPtr<UE::Sequencer::FChannelModel>(FName, const FSectionModel&, const FMovieSceneChannelHandle&)> OptionalFactory)
{
	check(TrackAreaList.IsValid() && Section);

	// Since we always start off adding to the top level channel, point the track area list at the generic child list now
	if (TrackAreaList.GetCurrentType() == FTrackModel::GetTopLevelChannelType())
	{
		TSharedPtr<FViewModel> CurrentModel = TrackAreaList.GetCurrentParent();
		TrackAreaList.RecurseInto(CurrentModel, CurrentModel->GetChildList(EViewModelListType::Generic));
	}

	AddChannel(Channel, false, OptionalFactory);
}

void FTrackModelLayoutBuilder::AddChannel(const FMovieSceneChannelHandle& Channel, bool bIsTopLevel, TFunction<TSharedPtr<UE::Sequencer::FChannelModel>(FName, const FSectionModel&, const FMovieSceneChannelHandle&)> OptionalFactory)
{
	check(TrackAreaList.IsValid() && Section);

	// @todo: this is all pretty crusty - we're currently linear-searching for both the child node, and the IKeyArea within that node
	// Performance is generally acceptible however since we are dealing with small numbers of children, but this may need to be revisited.
	const FMovieSceneChannelMetaData* MetaData = Channel.GetMetaData();
	if (!ensureAlwaysMsgf(MetaData, TEXT("Attempting to add an expired channel handle to the node tree")))
	{
		return;
	}

	FName ChannelName = MetaData->Name;

	auto ChannelNamePredicate = [ChannelName](const auto& InModel){ return InModel.GetChannelName() == ChannelName; };

	// -----------------------------------
	// Add a channel group to the outliner
	// Note: the outliner can comprise the channels of multile sections, potentially with different combinations
	//       of channels. We have to be careful to ensure that we reuse existing items that may have been added
	//       to the outliner from a previously built section.

	TViewModelPtr<FChannelGroupModel> OutlinerModel;
	if (bIsTopLevel)
	{
		OutlinerModel = OutlinerList.FindItem<FChannelGroupModel>(ChannelNamePredicate);
	}
	else
	{
		OutlinerModel = OutlinerList.FindItem<FChannelGroupOutlinerModel>(ChannelNamePredicate);
	}

	if (!OutlinerModel)
	{
		if (bIsTopLevel)
		{
			OutlinerModel = MakeShared<FChannelGroupModel>(ChannelName, MetaData->DisplayText, MetaData->GetTooltipTextDelegate);
		}
		else
		{
			OutlinerModel = MakeShared<FChannelGroupOutlinerModel>(ChannelName, MetaData->DisplayText, MetaData->GetTooltipTextDelegate);
		}
	}

	OutlinerList.Link(OutlinerModel);

	// ----------------------------------------
	// Add the channel itself to the track area
	// Note: The track area model must only ever add a single category or channel for each unique name.
	//       Therefore we assert on the presence of an existing category in the track area to ensure that we highlight 
	//       bad FTrackModelLayoutBuilder logic or data that specifies duplicate names

	TSharedPtr<FChannelModel> TrackAreaModel = TrackAreaList.FindExistingItem<FChannelModel>(ChannelNamePredicate);
	if (ensureAlwaysMsgf(TrackAreaModel == nullptr,
		TEXT("Channel with identifier '%s' has already been added to the list for this section which is not allowed. Please give each channel a unique identifier."), *ChannelName.ToString()))
	{
		TrackAreaModel = TrackAreaList.FindRecycledItem<FChannelModel>(ChannelNamePredicate);
		if (!TrackAreaModel)
		{
			if (OptionalFactory)
			{
				TrackAreaModel = OptionalFactory(ChannelName, *Section, Channel);
			}

			if (!TrackAreaModel)
			{
				TrackAreaModel = MakeShared<FChannelModel>(ChannelName, Section->GetSectionInterface(), Channel);
			}
		}

		OutlinerModel->AddChannel(TrackAreaModel);
		if (bIsTopLevel)
		{
			TrackAreaModel->SetLinkedOutlinerItem(OutlinerModel->GetParent().ImplicitCastChecked());
		}
		else
		{
			TrackAreaModel->SetLinkedOutlinerItem(OutlinerModel.ImplicitCastChecked());
		}
		TrackAreaModel->Initialize(Section->GetSectionInterface(), Channel);

		TrackAreaList.Link(TrackAreaModel);
	}
}

} // namespace Sequencer
} // namespace UE

