// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/TrackRowModel.h"

#include "MVVM/Extensions/ITrackRowExtension.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/FolderModel.h"
#include "MVVM/ViewModels/TrackModelLayoutBuilder.h"
#include "MVVM/Views/SOutlinerTrackView.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/SectionModelStorageExtension.h"

#include "SequencerNodeTree.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "SequencerUtilities.h"
#include "SequencerCommonHelpers.h"
#include "SSequencer.h"

#include "MovieSceneTrack.h"
#include "MovieSceneFolder.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "Decorations/MovieSceneTrackRowDecoration.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieScenePrimitiveMaterialTrack.h"
#include "EntitySystem/IMovieSceneBlenderSystemSupport.h"

#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "TrackRowModel"

namespace UE::Sequencer
{

FTrackRowModel::FTrackRowModel(UMovieSceneTrack* Track, int32 InRowIndex)
	: SectionList(EViewModelListType::TrackArea)
	, TopLevelChannelList(FTrackModel::GetTopLevelChannelGroupType())
	, WeakTrack(Track)
	, RowIndex(InRowIndex)
{
	RegisterChildList(&SectionList);
	RegisterChildList(&TopLevelChannelList);

	FName Identifier = Track->GetFName();
	Identifier.SetNumber(InRowIndex);
	SetIdentifier(Identifier);
}

FTrackRowModel::~FTrackRowModel()
{
}

void FTrackRowModel::Initialize()
{
	UMovieSceneTrack* Track = GetParentTrack();
	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	if (Track && SequenceModel)
	{
		TSharedPtr<ISequencer> Sequencer = SequenceModel->GetSequencer();
		if (Sequencer)
		{
			TrackEditor = Sequencer->GetTrackEditor(Track);
			ensure(TrackEditor);
			SetExpansion(TrackEditor->GetDefaultExpansionState(Track));
		}
	}
}

FViewModelChildren FTrackRowModel::GetTopLevelChannels()
{
	return GetChildrenForList(&TopLevelChannelList);
}

UMovieSceneTrack* FTrackRowModel::GetParentTrack() const
{
	return WeakTrack.Get();
}

int32 FTrackRowModel::GetRowIndex() const
{
	return RowIndex;
}

FViewModelChildren FTrackRowModel::GetSectionModels()
{
	return GetChildrenForList(&SectionList);
}

FOutlinerSizing FTrackRowModel::GetOutlinerSizing() const
{
	const TSharedPtr<FSequencerEditorViewModel> Editor = GetEditor();
	FViewDensityInfo Density = Editor ? Editor->GetViewDensity() : FViewDensityInfo();

	float Height = Density.UniformHeight.Get(SequencerLayoutConstants::SectionAreaDefaultHeight);
	if (auto It = SectionList.Iterate<FSectionModel>(); It)
	{
		TSharedPtr<FSectionModel> Section = *It;
		Height = Section->GetSectionInterface()->GetSectionHeight(Density);
	}
	return FOutlinerSizing(Height);
}

FTrackAreaParameters FTrackRowModel::GetTrackAreaParameters() const
{
	FTrackAreaParameters Params;
	Params.LaneType = ETrackAreaLaneType::Nested;
	Params.TrackLanePadding.Bottom = 1.f;
	return Params;
}

FViewModelVariantIterator FTrackRowModel::GetTrackAreaModelList() const
{
	return &SectionList;
}

FViewModelVariantIterator FTrackRowModel::GetTopLevelChildTrackAreaModels() const
{
	return &TopLevelChannelList;
}

void FTrackRowModel::CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels)
{
	TViewModelPtr<FChannelGroupModel> ChannelGroup = TopLevelChannelList.GetHead().ImplicitCast();
	if (ChannelGroup)
	{
		ChannelGroup->CreateCurveModels(OutCurveModels);
	}
}

bool FTrackRowModel::IsDimmed() const
{
	UMovieSceneTrack* Track = GetParentTrack();

	if (Track)
	{
		if (Track->IsRowEvalDisabled(GetRowIndex()))
		{
			return true;
		}

		FGuid BindingID;
		FMovieSceneSequenceID SequenceID = MovieSceneSequenceID::Root;
		if (TViewModelPtr<FObjectBindingModel> ObjectBindingModel = FindAncestorOfType<FObjectBindingModel>())
		{
			BindingID = ObjectBindingModel->GetObjectGuid();
		}

		if (TViewModelPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>())
		{
			SequenceID = SequenceModel->GetSequenceID();

			if (TSharedPtr<FSequencerEditorViewModel> SequencerModel = SequenceModel->GetEditor())
			{
				if (const FMovieSceneTrackRowMetadata* TrackRowMetadata = Track->FindTrackRowMetadata(GetRowIndex()))
				{
					if (TrackRowMetadata->ConditionContainer.Condition)
					{
						if (!MovieSceneHelpers::EvaluateSequenceCondition(BindingID, SequenceID, TrackRowMetadata->ConditionContainer.Condition, Track, SequencerModel->GetSequencer()->GetSharedPlaybackState()))
						{
							return true;
						}
					}
				}

				if (Track->ConditionContainer.Condition)
				{
					if (!MovieSceneHelpers::EvaluateSequenceCondition(BindingID, SequenceID, Track->ConditionContainer.Condition, Track, SequencerModel->GetSequencer()->GetSharedPlaybackState()))
					{
						return true;
					}
				}
			}
		}
	}

	return FOutlinerItemModel::IsDimmed();
}

ELockableLockState FTrackRowModel::GetLockState() const
{
	int32 NumSections = 0;
	int32 NumLockedSections = 0;

	for (const TViewModelPtr<FSectionModel>& Section : SectionList.Iterate<FSectionModel>())
	{
		++NumSections;

		UMovieSceneSection* SectionObject = Section->GetSection();
		if (SectionObject && SectionObject->IsLocked())
		{
			++NumLockedSections;
		}
	}

	if (NumSections == 0 || NumLockedSections == 0)
	{
		return ELockableLockState::None;
	}
	return NumLockedSections == NumSections ? ELockableLockState::Locked : ELockableLockState::PartiallyLocked;
}

void FTrackRowModel::SetIsLocked(bool bInIsLocked)
{
	for (const TViewModelPtr<FSectionModel>& Section : SectionList.Iterate<FSectionModel>())
	{
		UMovieSceneSection* SectionObject = Section->GetSection();
		if (SectionObject)
		{
			SectionObject->Modify();
			SectionObject->SetIsLocked(bInIsLocked);
		}
	}
}

const UMovieSceneCondition* FTrackRowModel::GetCondition() const
{
	UMovieSceneTrack* Track = GetParentTrack();

	if (Track)
	{
		if (const FMovieSceneTrackRowMetadata* TrackRowMetadata = Track->FindTrackRowMetadata(GetRowIndex()))
		{
			return TrackRowMetadata->ConditionContainer.Condition;
		}
	}

	return nullptr;
}


EConditionableConditionState FTrackRowModel::GetConditionState() const
{
	UMovieSceneTrack* Track = GetParentTrack();

	if (Track)
	{
		FGuid BindingID;
		FMovieSceneSequenceID SequenceID = MovieSceneSequenceID::Root;
		if (TViewModelPtr<FObjectBindingModel> ObjectBindingModel = FindAncestorOfType<FObjectBindingModel>())
		{
			BindingID = ObjectBindingModel->GetObjectGuid();
		}

		if (TViewModelPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>())
		{
			SequenceID = SequenceModel->GetSequenceID();

			if (TSharedPtr<FSequencerEditorViewModel> SequencerModel = SequenceModel->GetEditor())
			{
				if (const FMovieSceneTrackRowMetadata* TrackRowMetadata = Track->FindTrackRowMetadata(GetRowIndex()))
				{
					if (TrackRowMetadata->ConditionContainer.Condition)
					{
						if (TrackRowMetadata->ConditionContainer.Condition->bEditorForceTrue)
						{
							return EConditionableConditionState::HasConditionEditorForceTrue;
						}
						else if (MovieSceneHelpers::EvaluateSequenceCondition(BindingID, SequencerModel->GetSequencer()->GetFocusedTemplateID(), TrackRowMetadata->ConditionContainer.Condition, Track, SequencerModel->GetSequencer()->GetSharedPlaybackState()))
						{
							return EConditionableConditionState::HasConditionEvaluatingTrue;
						}
						else
						{
							return EConditionableConditionState::HasConditionEvaluatingFalse;
						}
					}
				}
			}
		}
	}
	return EConditionableConditionState::None;
}

void FTrackRowModel::SetConditionEditorForceTrue(bool bEditorForceTrue)
{
	UMovieSceneTrack* Track = GetParentTrack();

	if (Track)
	{
		if (const FMovieSceneTrackRowMetadata* TrackRowMetadata = Track->FindTrackRowMetadata(GetRowIndex()))
		{
			if (TrackRowMetadata->ConditionContainer.Condition)
			{
				const FScopedTransaction Transaction(NSLOCTEXT("SequencerTrackRowNode", "ConditionEditorForceTrue", "Set Condition Editor Force True"));
				TrackRowMetadata->ConditionContainer.Condition->Modify();
				TrackRowMetadata->ConditionContainer.Condition->bEditorForceTrue = bEditorForceTrue;
			}
		}
	}
}

FSlateFontInfo FTrackRowModel::GetLabelFont() const
{
	bool bAllAnimated = false;
	TViewModelPtr<FChannelGroupModel> TopLevelChannel = TopLevelChannelList.GetHead().ImplicitCast();
	if (TopLevelChannel)
	{
		for (const TViewModelPtr<FChannelModel>& ChannelModel : TopLevelChannel->GetDescendantsOfType<FChannelModel>())
		{
			FMovieSceneChannel* Channel = ChannelModel->GetChannel();
			if (!Channel || Channel->GetNumKeys() == 0)
			{
				return FOutlinerItemModel::GetLabelFont();
			}
			else
			{
				bAllAnimated = true;
			}
		}
		if (bAllAnimated == true)
		{
			return FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.ItalicFont");
		}
	}
	return FOutlinerItemModel::GetLabelFont();
}

const FSlateBrush* FTrackRowModel::GetIconBrush() const
{
	return TrackEditor ? TrackEditor->GetIconBrush() : nullptr;
}

FText FTrackRowModel::GetLabel() const
{
	UMovieSceneTrack* Track = GetParentTrack();
	return Track ? Track->GetTrackRowDisplayName(RowIndex) : FText::GetEmpty();
}

FSlateColor FTrackRowModel::GetLabelColor() const
{
	UMovieSceneTrack* Track = GetParentTrack();
	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	if (!Track || !SequenceModel)
	{
		return FSlateColor::UseForeground();
	}

	TSharedPtr<ISequencer> Sequencer = SequenceModel->GetSequencer();
	if (!Sequencer)
	{
		return FSlateColor::UseForeground();
	}
	FMovieSceneLabelParams LabelParams;
	LabelParams.bIsDimmed = IsDimmed();
	LabelParams.Player = Sequencer.Get();
	LabelParams.SequenceID = SequenceModel->GetSequenceID();
	if (TViewModelPtr<FObjectBindingModel> ObjectBindingModel = FindAncestorOfType<FObjectBindingModel>())
	{
		LabelParams.BindingID = ObjectBindingModel->GetObjectGuid();					
		// If the object binding model has an invalid binding, we want to use its label color, as it may be red or gray depending on situation
		// and we want the children of that to have the same color.
		// Otherwise, we can use the track's label color below
		TArrayView<TWeakObjectPtr<> > BoundObjects = LabelParams.Player->FindBoundObjects(LabelParams.BindingID, LabelParams.SequenceID);
		if (BoundObjects.Num() == 0)
		{
			return ObjectBindingModel->GetLabelColor();
		}
	}
	return Track->GetLabelColor(LabelParams);
}

TSharedPtr<SWidget> FTrackRowModel::CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName)
{
	FBuildColumnWidgetParams Params(SharedThis(this), InParams);
	return TrackEditor->BuildOutlinerColumnWidget(Params, InColumnName);
}

bool FTrackRowModel::CanRename() const
{
	UMovieSceneNameableTrack* NameableTrack = Cast<UMovieSceneNameableTrack>(GetParentTrack());
	return NameableTrack && NameableTrack->CanRename();
}

void FTrackRowModel::Rename(const FText& NewName)
{
	UMovieSceneNameableTrack* NameableTrack = ::Cast<UMovieSceneNameableTrack>(GetParentTrack());

	if (NameableTrack && !NameableTrack->GetTrackRowDisplayName(GetRowIndex()).EqualTo(NewName))
	{
		const FScopedTransaction Transaction(NSLOCTEXT("SequencerTrackRowNode", "RenameTrackRow", "Rename Track Row"));
		NameableTrack->SetTrackRowDisplayName(NewName, GetRowIndex());

		SetIdentifier(FName(*NewName.ToString()));

		// HACK: this should not exist but is required to make renaming emitters work in niagara
		if (TSharedPtr<FSequenceModel> OwnerModel = FindAncestorOfType<FSequenceModel>())
		{
			OwnerModel->GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		}
	}
}

bool FTrackRowModel::IsRenameValidImpl(const FText& NewName, FText& OutErrorMessage) const
{
	UMovieSceneNameableTrack* NameableTrack = ::Cast<UMovieSceneNameableTrack>(GetParentTrack());
	if (NameableTrack)
	{
		return NameableTrack->ValidateDisplayName(NewName, OutErrorMessage);
	}
	return false;
}

bool FTrackRowModel::IsResizable() const
{
	UMovieSceneTrack* Track = GetParentTrack();
	TSharedPtr<ISequencerTrackEditor> TrackEditorPtr = GetTrackEditor();
	return Track && TrackEditorPtr && TrackEditorPtr->IsResizable(Track);
}

void FTrackRowModel::Resize(float NewSize)
{
	UMovieSceneTrack* Track = GetParentTrack();
	TSharedPtr<ISequencerTrackEditor> TrackEditorPtr = GetTrackEditor();
	if (Track && TrackEditorPtr && TrackEditorPtr->IsResizable(Track))
	{
		TrackEditorPtr->Resize(NewSize, Track);
	}
}

void FTrackRowModel::RefreshLayout(bool bChildrenNeedLayout)
{
	TSharedPtr<FViewModel> SectionsTail;
	TUniquePtr<FScopedViewModelListHead> RecycledModels;

	TSharedPtr<FSequenceModel> SequenceModel = FindAncestorOfType<FSequenceModel>();
	if (!SequenceModel)
	{
		// Not part of a full sequence hierarchy yet - wait for OnSetSharedData()
		return;
	}
	FGuid ObjectBinding;
	if (TSharedPtr<IObjectBindingExtension> ObjectBindingExtension = FindAncestorOfType<IObjectBindingExtension>())
	{
		ObjectBinding = ObjectBindingExtension->GetObjectGuid();
	}

	FSectionModelStorageExtension* SectionModelStorage = SequenceModel->CastDynamic<FSectionModelStorageExtension>();
	// Recycle sections, outliner children, and more, while keeping them alive.
	RecycledModels = MakeUnique<FScopedViewModelListHead>(SharedThis(this), EViewModelListType::Recycled);
	FViewModelChildren RecycledRowModels = RecycledModels->GetChildren();

	GetSectionModels().MoveChildrenTo<IRecyclableExtension>(RecycledRowModels, IRecyclableExtension::CallOnRecycle);

	if (UMovieSceneTrack* Track = WeakTrack.Get())
	{
		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			if (Section && Section->GetRowIndex() == RowIndex)
			{
				TSharedPtr<FSectionModel> SectionModel = SectionModelStorage->FindModelForSection(Section);
				if (!SectionModel && TrackEditor)
				{
					TSharedRef<ISequencerSection> SectionInterface = TrackEditor->MakeSectionInterface(*Section, *Track, ObjectBinding);
					SectionModel = SectionModelStorage->CreateModelForSection(Section, SectionInterface);
					bChildrenNeedLayout = true;
				}
				else if (SectionModel)
				{
					bChildrenNeedLayout |= SectionModel->NeedsLayout();
				}

				if (SectionModel)
				{
					GetSectionModels().InsertChild(SectionModel, SectionsTail);
					SectionsTail = SectionModel;
				}
			}
		}
	}
	
	const bool bStillHasRecycledChildren = RecycledModels && !RecycledModels->GetChildren().IsEmpty();
	if (bChildrenNeedLayout || bStillHasRecycledChildren)
	{
		GetChildList(EViewModelListType::Outliner).MoveChildrenTo<IRecyclableExtension>(RecycledRowModels, IRecyclableExtension::CallOnRecycle);
		GetTopLevelChannels().MoveChildrenTo<IRecyclableExtension>(RecycledRowModels, IRecyclableExtension::CallOnRecycle);

		FTrackModelLayoutBuilder LayoutBuilder(AsShared());
		for (TSharedPtr<FSectionModel> Section : GetChildrenOfType<FSectionModel>(EViewModelListType::TrackArea))
		{
			LayoutBuilder.RefreshLayout(Section);
		}
	}
}

void FTrackRowModel::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UMovieSceneTrack* const Track = GetParentTrack();
	if (!IsValid(Track))
	{
		return;
	}

	if (TrackEditor)
	{
		TrackEditor->BuildTrackContextMenu(MenuBuilder, Track);
	}

	if (Track->GetSupportedBlendTypes().Num() > 0)
	{
		SequencerHelpers::BuildNewSectionMenu(Sequencer, GetRowIndex() + 1, GetParentTrack(), MenuBuilder);
	}

	SequencerHelpers::BuildBlendingMenu(Sequencer, Track, MenuBuilder);

	TArray<TWeakObjectPtr<>> WeakTracks;
	WeakTracks.Add(Track);
	SequencerHelpers::BuildEditTrackMenu(Sequencer, WeakTracks, MenuBuilder, true);

	const TArray<TWeakObjectPtr<>> TrackAreaModels = SequencerHelpers::GetSectionObjectsFromTrackAreaModels(GetTrackAreaModelList());
	SequencerHelpers::BuildEditSectionMenu(Sequencer, TrackAreaModels, MenuBuilder, true);

	if (const TViewModelPtr<FChannelGroupModel> ChannelGroup = TopLevelChannelList.GetHead().ImplicitCast())
	{
		ChannelGroup->BuildChannelOverrideMenu(MenuBuilder);
	}

	FOutlinerItemModel::BuildContextMenu(MenuBuilder);
}

void FTrackRowModel::BuildSidebarMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = GetEditor();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UMovieSceneTrack* const Track = GetParentTrack();
	if (!IsValid(Track))
	{
		return;
	}

	if (TrackEditor)
	{
		TrackEditor->BuildTrackSidebarMenu(MenuBuilder, Track);
	}

	TArray<TWeakObjectPtr<>> WeakTracks;
	WeakTracks.Add(Track);
	SequencerHelpers::BuildEditTrackMenu(Sequencer, WeakTracks, MenuBuilder, false);

	if (Track->GetSupportedBlendTypes().Num() > 0)
	{
		SequencerHelpers::BuildNewSectionMenu(Sequencer, GetRowIndex() + 1, GetParentTrack(), MenuBuilder);
	}

	SequencerHelpers::BuildBlendingMenu(Sequencer, Track, MenuBuilder);

	const TArray<TWeakObjectPtr<>> TrackAreaModels = SequencerHelpers::GetSectionObjectsFromTrackAreaModels(GetTrackAreaModelList());
	SequencerHelpers::BuildEditSectionMenu(Sequencer, TrackAreaModels, MenuBuilder, false);

	if (const TViewModelPtr<FChannelGroupModel> ChannelGroup = TopLevelChannelList.GetHead().ImplicitCast())
	{
		ChannelGroup->BuildChannelOverrideMenu(MenuBuilder);
	}

	FOutlinerItemModel::BuildSidebarMenu(MenuBuilder);
}

bool FTrackRowModel::CanDelete(FText* OutErrorMessage) const
{
	return true;
}

void FTrackRowModel::Delete()
{
	UMovieSceneTrack* Track = GetParentTrack();
	if (!Track)
	{
		return;
	}

	// Remove from a parent folder if necessary.
	if (TViewModelPtr<FFolderModel> ParentFolder = CastParent<FFolderModel>())
	{
		ParentFolder->GetFolder()->Modify();
		ParentFolder->GetFolder()->RemoveChildTrack(Track);
	}

	// Remove sub tracks belonging to this row only
	Track->Modify();
	Track->SetFlags(RF_Transactional);

	for (TSharedPtr<FSectionModel> SectionModel : SectionList.Iterate<FSectionModel>())
	{
		UMovieSceneSection* Section = SectionModel->GetSection();
		if (Section)
		{
			Track->RemoveSection(*Section);
		}
	}

	Track->UpdateEasing();
	Track->FixRowIndices();
}
bool FTrackRowModel::IsMuted() const
{
	UMovieSceneTrack* Track = GetParentTrack();
	if (!Track)
	{
		return false;
	}

	if (UMovieSceneTrackRowDecoration* TrackRowDecoration = Track->FindDecoration<UMovieSceneTrackRowDecoration>())
	{
		return TrackRowDecoration->IsMuted(GetRowIndex());
	}

	return false;
}

void FTrackRowModel::SetIsMuted(bool bIsMuted)
{
	UMovieSceneTrack* Track = GetParentTrack();
	if (!Track)
	{
		return;
	}

	const bool bAlwaysMarkDirty = false;
	Track->Modify(bAlwaysMarkDirty);

	if (UMovieSceneTrackRowDecoration* TrackRowDecoration = Cast<UMovieSceneTrackRowDecoration>(Track->GetOrCreateDecoration(UMovieSceneTrackRowDecoration::StaticClass())))
	{
		TrackRowDecoration->SetMuted(GetRowIndex(), bIsMuted);
	}
}

bool FTrackRowModel::IsSolo() const
{
	UMovieSceneTrack* Track = GetParentTrack();
	if (!Track)
	{
		return false;
	}

	if (UMovieSceneTrackRowDecoration* TrackRowDecoration = Track->FindDecoration<UMovieSceneTrackRowDecoration>())
	{
		return TrackRowDecoration->IsSoloed(GetRowIndex());
	}

	return false;
}

void FTrackRowModel::SetIsSoloed(bool bIsSoloed)
{
	UMovieSceneTrack* Track = GetParentTrack();
	if (!Track)
	{
		return;
	}

	const bool bAlwaysMarkDirty = false;
	Track->Modify(bAlwaysMarkDirty);

	if (UMovieSceneTrackRowDecoration* TrackRowDecoration = Cast<UMovieSceneTrackRowDecoration>(Track->GetOrCreateDecoration(UMovieSceneTrackRowDecoration::StaticClass())))
	{
		TrackRowDecoration->SetSoloed(GetRowIndex(), bIsSoloed);
	}
}

bool FTrackRowModel::SetRowIndex(int32 NewRowIndex)
{
	if (RowIndex == NewRowIndex)
	{
		return false;
	}

	UMovieSceneTrack* Track = GetParentTrack();
	if (!Track)
	{
		return false;
	}

	Track->Modify();

	// Update all sections on this row to the new row index
	bool bAnyChanged = false;
	for (TSharedPtr<FSectionModel> SectionModel : SectionList.Iterate<FSectionModel>())
	{
		if (UMovieSceneSection* Section = SectionModel->GetSection())
		{
			if (Section->GetRowIndex() == RowIndex)
			{
				Section->SetRowIndex(NewRowIndex);
				bAnyChanged = true;
			}
		}
	}

	if (bAnyChanged)
	{
		// Update our cached row index
		RowIndex = NewRowIndex;
	}

	return bAnyChanged;
}

bool FTrackRowModel::CanDrag() const
{
	// For now, regular track rows cannot be dragged in the outliner
	return false;
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

