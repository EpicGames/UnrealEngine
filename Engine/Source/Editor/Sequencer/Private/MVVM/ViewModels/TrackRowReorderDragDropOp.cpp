// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/ViewModels/TrackRowReorderDragDropOp.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/ITrackRowExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"

namespace UE::Sequencer
{

TSharedRef<FTrackRowReorderDragDropOp> FTrackRowReorderDragDropOp::New(
	TArray<TWeakViewModelPtr<IOutlinerExtension>>&& InDraggedViewModels,
	FText InDefaultText,
	const FSlateBrush* InDefaultIcon)
{
	TSharedRef<FTrackRowReorderDragDropOp> Operation = MakeShareable(new FTrackRowReorderDragDropOp);

	Operation->WeakViewModels = MoveTemp(InDraggedViewModels);
	Operation->DefaultHoverText = Operation->CurrentHoverText = InDefaultText;
	Operation->DefaultHoverIcon = Operation->CurrentIconBrush = InDefaultIcon;

	Operation->Construct();

	return Operation;
}

TSharedPtr<FTrackModel> FTrackRowReorderDragDropOp::GetCommonParentTrack() const
{
	if (bCommonParentTrackCached)
	{
		return CachedCommonParentTrack.Pin().ImplicitCast();
	}

	bCommonParentTrackCached = true;
	TSharedPtr<FTrackModel> CommonParent;

	// Find the common parent track - all dragged items must belong to the same track
	for (const TWeakViewModelPtr<IOutlinerExtension>& WeakViewModel : WeakViewModels)
	{
		TSharedPtr<FViewModel> ViewModel = WeakViewModel.Pin().AsModel();
		if (!ViewModel)
		{
			// Non-view model in selection
			CachedCommonParentTrack = nullptr;
			return nullptr;
		}

		// Check if this is a track row item (implements ITrackRowExtension)
		TViewModelPtr<ITrackRowExtension> TrackRowItem = ViewModel->CastThisShared<ITrackRowExtension>();
		if (!TrackRowItem)
		{
			// Non-track-row item in selection
			CachedCommonParentTrack = nullptr;
			return nullptr;
		}

		// Get the parent track
		TViewModelPtr<ITrackExtension> ParentTrack = ViewModel->FindAncestorOfType<ITrackExtension>();
		if (!ParentTrack)
		{
			CachedCommonParentTrack = nullptr;
			return nullptr;
		}

		// Ensure all items belong to the same track/layer
		if (!CommonParent.IsValid())
		{
			CommonParent = ParentTrack.ImplicitCast();
		}
		else if (TSharedPtr<ITrackExtension> OtherParent = ParentTrack.ImplicitCast())
		{
			if (CommonParent != OtherParent)
			{
				// Mixed selection from different tracks - not allowed
				CachedCommonParentTrack = nullptr;
				return nullptr;
			}
		}
		else
		{
			CachedCommonParentTrack = nullptr;
			return nullptr;
		}
	}

	CachedCommonParentTrack = CommonParent;
	return CommonParent;
}

bool FTrackRowReorderDragDropOp::ExecuteReorder(int32 TargetRowIndex)
{
	TSharedPtr<FTrackModel> ParentTrack = GetCommonParentTrack();
	if (!ParentTrack.IsValid())
	{
		return false;
	}

	if (WeakViewModels.IsEmpty())
	{
		return false;
	}

	// Find min/max row indices of dragged items
	int32 MinDraggedRow = MAX_int32;
	int32 MaxDraggedRow = -1;
	for (const TWeakViewModelPtr<IOutlinerExtension>& Item : WeakViewModels)
	{
		if (TViewModelPtr<ITrackRowExtension> TrackRow = Item.ImplicitPin())
		{
			int32 RowIndex = TrackRow->GetRowIndex();
			MinDraggedRow = FMath::Min(MinDraggedRow, RowIndex);
			MaxDraggedRow = FMath::Max(MaxDraggedRow, RowIndex);
		}
	}

	// Check if already at target
	if (TargetRowIndex == MinDraggedRow)
	{
		return false;
	}

	// Collect all track items to shift them properly
	TArray<TViewModelPtr<ITrackRowExtension>> AllItems;
	for (TSharedPtr<FViewModel> Child : ParentTrack->GetChildren())
	{
		if (TViewModelPtr<ITrackRowExtension> TrackRowItem = Child->CastThisShared<ITrackRowExtension>())
		{
			AllItems.Add(TrackRowItem);
		}
	}

	bool bAnyChanged = false;

	// Shift items to make space, then place dragged items
	if (TargetRowIndex < MinDraggedRow)
	{
		// Moving up: shift items in range [TargetRowIndex, MinDraggedRow-1] down
		for (const TViewModelPtr<ITrackRowExtension>& Item : AllItems)
		{
			int32 CurrentRow = Item->GetRowIndex();
			bool bIsDragged = WeakViewModels.ContainsByPredicate([&Item](const TWeakViewModelPtr<IOutlinerExtension>& DraggedItem) { return DraggedItem.Pin() == Item.AsModel(); });

			if (!bIsDragged && CurrentRow >= TargetRowIndex && CurrentRow < MinDraggedRow)
			{
				// Shift this item down by number of dragged items
				if (Item->SetRowIndex(CurrentRow + WeakViewModels.Num()))
				{
					bAnyChanged = true;
				}
			}
		}

		// Place dragged items starting at target
		int32 NewRow = TargetRowIndex;
		for (const TWeakViewModelPtr<IOutlinerExtension>& Item : WeakViewModels)
		{
			if (TViewModelPtr<ITrackRowExtension> TrackRow = Item.ImplicitPin())
			{
				if (TrackRow->SetRowIndex(NewRow))
				{
					bAnyChanged = true;
				}
				NewRow++;
			}
		}
	}
	else if (TargetRowIndex > MaxDraggedRow)
	{
		// Moving down: shift items in range [MaxDraggedRow+1, TargetRowIndex-1] up
		for (const TViewModelPtr<ITrackRowExtension>& Item : AllItems)
		{
			int32 CurrentRow = Item->GetRowIndex();
			bool bIsDragged = WeakViewModels.ContainsByPredicate([&Item](const TWeakViewModelPtr<IOutlinerExtension>& DraggedItem) { return DraggedItem.Pin() == Item.AsModel(); });

			if (!bIsDragged && CurrentRow > MaxDraggedRow && CurrentRow < TargetRowIndex)
			{
				// Shift this item up by number of dragged items
				if (Item->SetRowIndex(CurrentRow - WeakViewModels.Num()))
				{
					bAnyChanged = true;
				}
			}
		}

		// Place dragged items ending at target-1
		int32 NewRow = TargetRowIndex - WeakViewModels.Num();
		for (const TWeakViewModelPtr<IOutlinerExtension>& Item : WeakViewModels)
		{
			if (TViewModelPtr<ITrackRowExtension> TrackRow = Item.ImplicitPin())
			{
				if (TrackRow->SetRowIndex(NewRow))
				{
					bAnyChanged = true;
				}
				NewRow++;
			}
		}
	}

	return bAnyChanged;
}

void FTrackRowReorderDragDropOp::GetTargetAndDropModel(const FViewModelPtr& CurrentViewModel, EItemDropZone InItemDropZone, FViewModelPtr& DropModel, FViewModelPtr& TargetModel) const
{
	// Use the base implementation first to find the current level's target model and drop model
	FOutlinerViewModelDragDropOp::GetTargetAndDropModel(CurrentViewModel, InItemDropZone, DropModel, TargetModel);

	// If the TargetModel is not a track row and we're trying to drop after it, change the target model to the parent and go again
	
	TViewModelPtr<ITrackRowExtension> TrackRow = TargetModel.ImplicitCast();
	if (!TrackRow && TargetModel && !TargetModel->GetNextSibling() && InItemDropZone == EItemDropZone::BelowItem)
	{
		GetTargetAndDropModel(TargetModel->GetParent(), InItemDropZone, DropModel, TargetModel);
	}
}

} // namespace UE::Sequencer
