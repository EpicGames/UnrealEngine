// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/SChannelView.h"

#include "Filters/SequencerFilterBar.h"
#include "Filters/Linking/Mode/LinkedFilterViewModel.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModelImpl.h"
#include "MVVM/Views/STrackAreaView.h"
#include "MVVM/Extensions/IMutableExtension.h"
#include "MVVM/Selection/Selection.h"
#include "IKeyArea.h"
#include "Misc/InputState.h"
#include "MovieScene.h"
#include "Sequencer.h"
#include "SequencerHotspots.h"
#include "ISequencerEditTool.h"
#include "ISequencerSection.h"
#include "SequencerAddKeyOperation.h"
#include "SSequencerSection.h"
#include "SequencerChannelTraits.h"
#include "SequencerCommands.h"

#include "ScopedTransaction.h"

#include "Algo/Transform.h"

#define LOCTEXT_NAMESPACE "SequencerChannelView"

namespace UE::Sequencer
{

/** Convert a SequencerCore FKeyRenderer hittest query to FSequencerSelectedKeys */
TArray<FSequencerSelectedKey> KeyRendererResultToSelectedKeys(TArrayView<const FKeyRenderer::FKeysForModel> InKeys, UMovieSceneSection* SectionObject)
{
	TArray<FSequencerSelectedKey> SelectedKeys;
	for (const FKeyRenderer::FKeysForModel& Result : InKeys)
	{
		TViewModelPtr<FChannelModel> Channel = Result.Model.ImplicitCast();
		if (Channel && Result.Keys.Num() > 0)
		{
			SelectedKeys.Reserve(SelectedKeys.Num() + Result.Keys.Num());
			for (FKeyHandle Key : Result.Keys)
			{
				SelectedKeys.Add(FSequencerSelectedKey(*SectionObject, Channel, Key));
			}
		}
	}
	return SelectedKeys;
}


/** Key renderer interface for defining selection and hover states. Should be removed in favor of SequencerCore knowing about selection in the future */
struct FSequencerKeyRendererInterface : IKeyRendererInterface
{
	const TSet<FKeyHandle>* SelectedKeys;
	const TMap<FKeyHandle, ESelectionPreviewState>* SelectionPreview;
	const TSet<FKeyHandle>* HoveredKeys;

	FSequencerKeyRendererInterface(FSequencer* Sequencer, TSharedPtr<ITrackAreaHotspot> Hotspot)
	{
		TSharedPtr<FKeyHotspot> KeyHotspot = HotspotCast<FKeyHotspot>(Hotspot);

		SelectedKeys     = &Sequencer->GetViewModel()->GetSelection()->KeySelection.GetSelected();
		SelectionPreview = &Sequencer->GetSelectionPreview().GetDefinedKeyStates();
		HoveredKeys      = (KeyHotspot.IsValid() && !CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport))
			? &KeyHotspot->RawKeys : nullptr;
	}

	bool HasAnySelectedKeys() const override
	{
		return SelectedKeys->Num() != 0;
	}
	bool HasAnyPreviewSelectedKeys() const override
	{
		return SelectionPreview->Num() != 0;
	}
	bool HasAnyHoveredKeys() const override
	{
		return HoveredKeys && HoveredKeys->Num() != 0;
	}

	bool IsKeySelected(const TViewModelPtr<IKeyExtension>& InOwner, FKeyHandle InKey) const override
	{
		return SelectedKeys->Contains(InKey);
	}
	bool IsKeyHovered(const TViewModelPtr<IKeyExtension>& InOwner, FKeyHandle InKey) const override
	{
		return HoveredKeys ? HoveredKeys->Contains(InKey) : false;
	}
	EKeySelectionPreviewState GetPreviewSelectionState(const TViewModelPtr<IKeyExtension>& InOwner, FKeyHandle InKey) const override
	{
		if (const ESelectionPreviewState* State = SelectionPreview->Find(InKey))
		{
			switch(*State)
			{
			case ESelectionPreviewState::Undefined:   return EKeySelectionPreviewState::Undefined;
			case ESelectionPreviewState::Selected:    return EKeySelectionPreviewState::Selected;
			case ESelectionPreviewState::NotSelected: return EKeySelectionPreviewState::NotSelected;
			}
		}

		return EKeySelectionPreviewState::Undefined;
	}
};


FChannelViewKeyCachedState::FChannelViewKeyCachedState()
	: ValidPlayRangeMin(TNumericLimits<int32>::Lowest())
	, ValidPlayRangeMax(TNumericLimits<int32>::Max())
	, VisibleRange(0, 1)
	, KeySizePx(12, 12)
	, SelectionSerial(0)
	, SelectionPreviewHash(0)
	, FilterSerial(0)
	, bShowCurve(false)
	, bShowKeyBars(true)
	, bCollapseChildren(false)
	, bIsChannelHovered(false)
{}

FChannelViewKeyCachedState::FChannelViewKeyCachedState(TRange<FFrameTime> InVisibleRange, TSharedPtr<ITrackAreaHotspot> Hotspot, FViewModelPtr Model, FSequencer* Sequencer)
{
	// Gather keys for a region larger than the view range to ensure we draw keys that are only just offscreen.
	// Compute visible range taking into account a half-frame offset for keys, plus half a key width for keys that are partially offscreen
	TRange<FFrameNumber> ValidKeyRange = Sequencer->GetSubSequenceRange().Get(Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange());

	TViewModelPtr<FChannelModel> Channel = Model.ImplicitCast();
	if (!Channel)
	{
		TOptional<FViewModelChildren> TopLevelChannels = Model->FindChildList(FTrackModel::GetTopLevelChannelType());
		if (TopLevelChannels.IsSet())
		{
			Channel = TopLevelChannels->FindFirstChildOfType<FChannelModel>();
		}
	}
	TViewModelPtr<FLinkedOutlinerExtension> Outliner = Model.ImplicitCast();

	ValidPlayRangeMin = UE::MovieScene::DiscreteInclusiveLower(ValidKeyRange);
	ValidPlayRangeMax = UE::MovieScene::DiscreteExclusiveUpper(ValidKeyRange);
	VisibleRange = InVisibleRange;
	KeySizePx = FVector2D(12, 12);
	SelectionSerial = Sequencer->GetViewModel()->GetSelection()->GetSerialNumber();
	SelectionPreviewHash = Sequencer->GetSelectionPreview().GetSelectionHash();
	const TSharedPtr<FLinkedFilterViewModel> LinkedFilterViewModel = Sequencer->GetViewModelImpl()->GetLinkedFilterViewModelImpl();
	FilterSerial = LinkedFilterViewModel ? LinkedFilterViewModel->GetActiveFilterBarImpl()->GetFilterSerial() : 0;
	bShowCurve = Channel && Channel->GetKeyArea()->ShouldShowCurve();
	bShowKeyBars = Sequencer->GetSequencerSettings()->GetShowKeyBars();
	bCollapseChildren = Outliner && Outliner->GetLinkedOutlinerItem() && Outliner->GetLinkedOutlinerItem()->IsExpanded() == false;
	bIsChannelHovered = false;
	if (!Hotspot || Hotspot->CastThis<FKeyHotspot>() || Hotspot->CastThis<FKeyBarHotspot>())
	{
		WeakHotspot = Hotspot;
		RawHotspotDoNotUse = Hotspot.Get();
	}
}

EViewDependentCacheFlags FChannelViewKeyCachedState::CompareTo(const FChannelViewKeyCachedState& Other) const
{
	EViewDependentCacheFlags Flags = EViewDependentCacheFlags::None;

	if (bIsChannelHovered || bIsChannelHovered != Other.bIsChannelHovered)
	{
		// Check if either hotspot expired
		if ( (WeakHotspot.Pin() == nullptr && RawHotspotDoNotUse != nullptr) ||
			 (Other.WeakHotspot.Pin() == nullptr && Other.RawHotspotDoNotUse != nullptr) )
		{
			Flags |= EViewDependentCacheFlags::KeyStateChanged;
		}

		// Check if the hotspot has changed (this is necessary because theoretically both hotspots
		// could still be _alive_ but Other.WeakHotspot is the active one)
		if (WeakHotspot != Other.WeakHotspot)
		{
			// Hotspot has changed - maybe we (un)hovered some keys or a key bar
			Flags |= EViewDependentCacheFlags::KeyStateChanged;
		}
	}

	if (Other.NonLinearTransform.Pin() != NonLinearTransform.Pin())
	{
		Flags |= EViewDependentCacheFlags::ViewChanged;
	}

	if (ValidPlayRangeMin != Other.ValidPlayRangeMin || ValidPlayRangeMax != Other.ValidPlayRangeMax)
	{
		// The valid key ranges for the data has changed
		Flags |= EViewDependentCacheFlags::KeyStateChanged;
	}

	if (SelectionSerial != Other.SelectionSerial || SelectionPreviewHash != Other.SelectionPreviewHash)
	{
		// Selection states have changed
		Flags |= EViewDependentCacheFlags::KeyStateChanged;
	}

	if (FilterSerial != Other.FilterSerial)
	{
		// Filters have re-run since the last paint, so the set of filtered-in descendants the
		// key renderer collapses into this section may have changed. Force CacheKeyExtensions
		// to walk the descendants again.
		Flags |= EViewDependentCacheFlags::DataChanged;
	}

	if (VisibleRange != Other.VisibleRange)
	{
		Flags |= EViewDependentCacheFlags::ViewChanged;

		const double RangeSize = VisibleRange.Size<FFrameTime>().AsDecimal();
		const double OtherRangeSize = Other.VisibleRange.Size<FFrameTime>().AsDecimal();

		if (!FMath::IsNearlyEqual(RangeSize, OtherRangeSize, RangeSize * 0.001))
		{
			Flags |= EViewDependentCacheFlags::ViewZoomed;
		}
	}

	if (bShowCurve != Other.bShowCurve)
	{
		Flags |= EViewDependentCacheFlags::DataChanged;
	}

	if (bShowKeyBars != Other.bShowKeyBars)
	{
		Flags |= EViewDependentCacheFlags::DataChanged;
	}

	if (bCollapseChildren != Other.bCollapseChildren)
	{
		Flags |= EViewDependentCacheFlags::DataChanged;
	}

	if (KeySizePx != Other.KeySizePx)
	{
		Flags |= EViewDependentCacheFlags::ViewChanged | EViewDependentCacheFlags::ViewZoomed;
	}

	return Flags;
}

void SChannelView::Construct(const FArguments& InArgs, const FViewModelPtr& InViewModel, TSharedPtr<STrackAreaView> InTrackAreaView)
{
	STrackAreaLaneView::Construct(
		STrackAreaLaneView::FArguments()
		[
			InArgs._Content.Widget
		]
		, InViewModel, InTrackAreaView);

	KeyBarColor = InArgs._KeyBarColor;

	KeyRenderer.Initialize(InViewModel);
}

FReply SChannelView::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<STrackAreaView>      TrackAreaView = WeakTrackAreaView.Pin();
	TSharedPtr<FTrackAreaViewModel> TrackArea     = TrackAreaView ? TrackAreaView->GetViewModel() : nullptr;
	if (!TrackArea)
	{
		return FReply::Unhandled();
	}

	TSet<FSequencerSelectedKey> HoveredKeys;

	// The hovered key is defined from the sequencer hotspot
	if (TSharedPtr<FKeyHotspot> KeyHotspot = HotspotCast<FKeyHotspot>(TrackArea->GetHotspot()))
	{
		HoveredKeys = KeyHotspot->Keys;
	}

	// Create keys with middle mouse button only if there are no modifier keys down (to prevent zoom scrolling and creating keys at the same time)
	if (!MouseEvent.GetModifierKeys().AnyModifiersDown() && MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton)
	{
		GEditor->BeginTransaction(LOCTEXT("CreateKey_Transaction", "Create Key"));
		bCreateKeyTransactionActive = true;

		// Generate a key and set it as the PressedKey
		TArray<FSequencerSelectedKey> NewKeys;
		CreateKeysUnderMouse(MouseEvent.GetScreenSpacePosition(), MyGeometry, HoveredKeys, NewKeys);

		if (NewKeys.Num())
		{
			TSharedPtr<FSequencer> Sequencer = LegacyGetSequencer();
			FKeySelection& KeySelection = Sequencer->GetViewModel()->GetSelection()->KeySelection;

			KeySelection.Empty();
			for (const FSequencerSelectedKey& NewKey : NewKeys)
			{
				if (TSharedPtr<FChannelModel> Channel = NewKey.WeakChannel.Pin())
				{
					KeySelection.Select(Channel, NewKey.KeyHandle);
				}
			}

			// Pass the event to the tool to copy the hovered key and move it
			TrackArea->SetHotspot(MakeShared<FKeyHotspot>(MoveTemp(NewKeys), Sequencer));

			// Return unhandled so that the EditTool can handle the mouse down based on the newly created keyframe and prepare to move it
			return FReply::Unhandled();
		}
	}

	return FReply::Unhandled();
}

FReply SChannelView::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<STrackAreaView>      TrackAreaView = WeakTrackAreaView.Pin();
	TViewModelPtr<FSectionModel>    Section       = GetSection();
	TSharedPtr<FTrackAreaViewModel> TrackArea     = TrackAreaView ? TrackAreaView->GetViewModel() : nullptr;
	if (!TrackArea || !Section)
	{
		return FReply::Unhandled();
	}

	if (CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport))
	{
		TrackArea->SetHotspot(nullptr);
		return FReply::Unhandled();
	}

	TSharedPtr<FSequencer> Sequencer = LegacyGetSequencer();
	

	// Don't fiddle with hotspots if there is an active drag
	const ISequencerEditTool* EditTool = TrackArea->GetEditTool();
	if (EditTool && EditTool->GetDragHotspot())
	{
		return FReply::Unhandled();
	}

	// Checked for hovered key
	TSharedPtr<FKeyHotspot>    KeyHotspot    = HotspotCast<FKeyHotspot>(TrackArea->GetHotspot());
	TSharedPtr<FKeyBarHotspot> KeyBarHotspot = HotspotCast<FKeyBarHotspot>(TrackArea->GetHotspot());
	
	const FTimeToPixel RelativeTimeToPixel = GetRelativeTimeToPixel();
	const FVector2D    MousePixel          = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	const float        HalfHeightPx        = MyGeometry.GetLocalSize().Y*.5f;


	// Perform validation of existing hotspots, and hittesting of new ones if necessary.
	// WARNING: This code will early-return once it has found a valid hotspot, so no 
	//          code should be added after here if it must always run
	if (KeyRendererCache.IsSet())
	{
		if (KeyHotspot)
		{
			const FFrameTime KeyTime       = KeyHotspot->GetTime().Get(0);
			const FVector2D  HalfKeySizePx = KeyRendererCache->KeySizePx * .5f;
			const float      KeyPixel      = RelativeTimeToPixel.FrameToPixel(KeyTime);

			if (FMath::Abs(MousePixel.Y - HalfHeightPx) <= HalfKeySizePx.Y &&
				FMath::Abs(MousePixel.X - KeyPixel) <= HalfKeySizePx.X)
			{
				// Key hotspot is still valid
				return FReply::Handled();
			}
		}
		else if (KeyBarHotspot)
		{
			if (FMath::Abs(MousePixel.Y - HalfHeightPx) <= 4.f &&
				KeyBarHotspot->Range.Contains(RelativeTimeToPixel.PixelToFrame(MousePixel.X)))
			{
				// Keybar hotspot is still valid
				return FReply::Handled();
			}
		}
	}

	TArray<FSequencerSelectedKey> KeysUnderMouse;
	GetKeysUnderMouse(MouseEvent.GetScreenSpacePosition(), MyGeometry, KeysUnderMouse);

	// If we are hovering over a new key, set that as the hotspot
	if (KeysUnderMouse.Num())
	{
		TrackArea->SetHotspot(MakeShared<FKeyHotspot>(MoveTemp(KeysUnderMouse), Sequencer));
		return FReply::Handled();
	}

	// Keys always render in the middle of the geometry so do a quick vertical test first
	// We actually hittest a larger vertical area than the bars render to make it easier for user
	// to hit key bars
	if (FMath::Abs(MousePixel.Y - HalfHeightPx) <= 4.f)
	{
		FKeyRenderer::FKeyBar KeyBar;
		if (KeyRenderer.HitTestKeyBar(RelativeTimeToPixel.PixelToFrame(MousePixel.X), KeyBar))
		{
			UMovieSceneSection* SectionObject = Section->GetSection();
			TrackArea->SetHotspot(MakeShared<FKeyBarHotspot>(
				KeyBar.Range,
				KeyRendererResultToSelectedKeys(KeyBar.LeadingKeys, SectionObject),
				KeyRendererResultToSelectedKeys(KeyBar.TrailingKeys, SectionObject),
				Sequencer)
			);

			return FReply::Handled();
		}
	}

	// Otherwise reset the hotspot if it is no longer valid
	if (KeyHotspot || KeyBarHotspot)
	{
		TrackArea->SetHotspot(nullptr);
	}
	return FReply::Unhandled();
}

FReply SChannelView::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Drive the end of the transaction off the flag set on mouse down rather than re-checking the button/modifier state, which can diverge by the time the release arrives (e.g. user grabs a modifier mid-drag, or a different button's up event reaches us first) and would otherwise strand the transaction. See UE-381007.
	EndCreateKeyTransactionIfActive();

	// Return unhandled so that the EditTool can handle the mouse up based on the newly created keyframe and finish moving it
	return FReply::Unhandled();
}

void SChannelView::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseLeave(MouseEvent);

	EndCreateKeyTransactionIfActive();

	TSharedPtr<STrackAreaView>      TrackAreaView = WeakTrackAreaView.Pin();
	TSharedPtr<FTrackAreaViewModel> TrackArea     = TrackAreaView ? TrackAreaView->GetViewModel() : nullptr;
	if (TrackArea)
	{
		TrackArea->SetHotspot(nullptr);
	}
}

void SChannelView::EndCreateKeyTransactionIfActive()
{
	if (!bCreateKeyTransactionActive)
	{
		return;
	}

	bCreateKeyTransactionActive = false;

	TSharedPtr<FSequencer> Sequencer = LegacyGetSequencer();

	// Snap keys on mouse up since we want to create keys at the exact mouse position (ie. to keep the newly created keys under the mouse
	// while dragging) but obey snapping rules if necessary
	if (Sequencer->GetSequencerSettings()->GetForceWholeFrames())
	{
		Sequencer->SnapToFrame();

		FSequencerSelection& Selection = *Sequencer->GetViewModel()->GetSelection();
		for (FKeyHandle Key : Selection.KeySelection)
		{
			TSharedPtr<FChannelModel> Channel = Selection.KeySelection.GetModelForKey(Key);
			if (Channel)
			{
				const FFrameNumber CurrentTime = Channel->GetKeyArea()->GetKeyTime(Key);
				Sequencer->SetLocalTime(CurrentTime, ESnapTimeMode::STM_Interval);
				break;
			}
		}
	}
	GEditor->EndTransaction();
}

FReply SChannelView::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<FSectionModel> Section = GetSection();
	if (!Section)
	{
		return FReply::Handled();
	}

	if(MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		TArray<FSequencerSelectedKey> Keys;
		GetKeysUnderMouse(MouseEvent.GetScreenSpacePosition(), MyGeometry, Keys);
		TArray<FKeyHandle> KeyHandles;
		for (FSequencerSelectedKey Key : Keys)
		{
			KeyHandles.Add(Key.KeyHandle);
		}
		if (KeyHandles.Num() > 0)
		{
			FReply Reply = Section->GetSectionInterface()->OnKeyDoubleClicked(KeyHandles);
			return Reply.Handled();
		}
	}

	return FReply::Unhandled();
}

int32 SChannelView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	static const FName SelectionColorName("SelectionColor");


	TSharedPtr<FSequencer> Sequencer = LegacyGetSequencer();
	if (!Sequencer.IsValid())
	{
		return LayerId;
	}

	FViewModelPtr Model = WeakModel.Pin();
	if (!Model)
	{
		return LayerId;
	}

	{
		IMutableExtension* Mutable = Model->CastThis<IMutableExtension>();
		bParentEnabled = bParentEnabled && (!Mutable || !Mutable->IsMuted());
	}

	LayerId = DrawLane(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	const bool bIncludeThis = true;

	TSharedPtr<STrackAreaView>      TrackAreaView = WeakTrackAreaView.Pin();
	TSharedPtr<FTrackAreaViewModel> TrackArea     = TrackAreaView ? TrackAreaView->GetViewModel() : nullptr;
	TViewModelPtr<FSectionModel>    Section       = Model->FindAncestorOfType<FSectionModel>(bIncludeThis);
	TViewModelPtr<FSequenceModel>   Sequence      = Model->FindAncestorOfType<FSequenceModel>();

	UMovieSceneSection* SectionObject = Section ? Section->GetSection() : nullptr;
	if (!Section || !Sequence || !SectionObject || !TrackArea)
	{
		return LayerId;
	}

	// Make the time <-> pixel converter relative to the section range rather than the view range
	FTimeToPixel RelativeTimeToPixel = GetRelativeTimeToPixel();

	FVector2D TopLeft = AllottedGeometry.AbsoluteToLocal(MyCullingRect.GetTopLeft());
	FVector2D BottomRight = AllottedGeometry.AbsoluteToLocal(MyCullingRect.GetBottomRight());

	TRange<FFrameTime> VisibleRange = TRange<FFrameTime>(
		RelativeTimeToPixel.PixelToFrame(TopLeft.X),
		RelativeTimeToPixel.PixelToFrame(BottomRight.X)
	);

	TViewModelPtr<FChannelModel> Channel = Model.ImplicitCast();
	if (!Channel)
	{
		TOptional<FViewModelChildren> TopLevelChannels = Model->FindChildList(FTrackModel::GetTopLevelChannelType());
		if (TopLevelChannels.IsSet())
		{
			Channel = TopLevelChannels->FindFirstChildOfType<FChannelModel>();
		}
	}
	if (Channel)
	{
		FSequencerChannelPaintArgs ChannelPaintArgs = {
			OutDrawElements,
			Args,
			AllottedGeometry,
			MyCullingRect,
			InWidgetStyle,
			RelativeTimeToPixel,
			bParentEnabled
		};
		LayerId = Channel->CustomPaint(ChannelPaintArgs, LayerId);
	}

	TSharedPtr<ISequencerSection> SectionInterface = Section->GetSectionInterface();

	FSequencerKeyRendererInterface KeyRenderInterface(Sequencer.Get(), TrackArea->GetHotspot());
	FChannelViewKeyCachedState NewCachedState(VisibleRange, TrackArea->GetHotspot(), Model, Sequencer.Get());
	NewCachedState.bIsChannelHovered = IsHovered();
	NewCachedState.KeySizePx = SectionInterface ? SectionInterface->GetKeySize() : FVector2D(1.0, 1.0);
	NewCachedState.NonLinearTransform = RelativeTimeToPixel.NonLinearTransform;

	FKeyBatchParameters Params(RelativeTimeToPixel);
	Params.CacheState = KeyRendererCache.IsSet() ? NewCachedState.CompareTo(KeyRendererCache.GetValue()) : EViewDependentCacheFlags::All;
	Params.VisibleRange = VisibleRange;
	Params.ValidPlayRangeMin = NewCachedState.ValidPlayRangeMin;
	Params.ValidPlayRangeMax = NewCachedState.ValidPlayRangeMax;
	Params.bShowCurve = NewCachedState.bShowCurve;
	Params.bShowKeyBars = NewCachedState.bShowKeyBars;
	Params.bCollapseChildren = NewCachedState.bCollapseChildren;
	Params.ClientInterface = &KeyRenderInterface;

	FKeyRendererPaintArgs PaintArgs;
	PaintArgs.DrawElements = &OutDrawElements;
	PaintArgs.KeyThrobValue = SSequencerSection::GetKeySelectionThrobValue();
	PaintArgs.DrawEffects = bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	PaintArgs.KeySelectionFillColor = FAppStyle::GetSlateColor(SelectionColorName).GetColor(InWidgetStyle);
	if (KeyBarColor.IsSet())
	{
		PaintArgs.KeyBarColor = KeyBarColor.Get();
	}

	USequencerSettings* SequencerSettings = Sequencer->GetSequencerSettings();
	if (SequencerSettings && SequencerSettings->GetShowChannelColors() && Channel)
	{
		if (TOptional<FLinearColor> ChannelColor = Channel->GetKeyArea()->GetColor())
		{
			PaintArgs.CurveColor = ChannelColor.GetValue();
		}
	}


	KeyRenderer.Update(Params, AllottedGeometry);
	LayerId = KeyRenderer.DrawCurve(Params, AllottedGeometry, MyCullingRect, PaintArgs, LayerId);
	LayerId = KeyRenderer.Draw(Params, AllottedGeometry, MyCullingRect, PaintArgs, LayerId + 10);

	KeyRendererCache = NewCachedState;

	return LayerId;
}

int32 SChannelView::DrawLane(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FViewModelPtr Model = WeakModel.Pin();
	TViewModelPtr<FSequenceModel> Sequence = Model ? Model->FindAncestorOfType<FSequenceModel>() : nullptr;
	TViewModelPtr<FLinkedOutlinerExtension> Outliner = WeakModel.ImplicitPin();
	if (!Sequence || !Outliner)
	{
		return LayerId;
	}

	TViewModelPtr<IOutlinerExtension> OutlinerItem = Outliner->GetLinkedOutlinerItem();
	if (!OutlinerItem)
	{
		return LayerId;
	}

	FSequencer* Sequencer = Sequence->GetSequencerImpl().Get();
	FSequencerSelection& Selection = Sequencer->GetSelection();

	const ESlateDrawEffect DrawEffects = bParentEnabled
		? ESlateDrawEffect::None
		: ESlateDrawEffect::DisabledEffect;

	FLinearColor HighlightColor;
	bool bDrawHighlight = false;
	if (Sequencer->GetSelection().NodeHasSelectedKeysOrSections(OutlinerItem))
	{
		bDrawHighlight = true;
		HighlightColor = FLinearColor(1.0f, 1.0f, 1.0f, 0.15f);
	}
	else if (TViewModelPtr<IHoveredExtension> Hovered = OutlinerItem.ImplicitCast())
	{
		if (Hovered->IsHovered())
		{
			bDrawHighlight = true;
			HighlightColor = FLinearColor(1.0f, 1.0f, 1.0f, 0.05f);
		}
	}

	// --------------------------------------------
	// Draw hover or selection highlight
	if (bDrawHighlight)
	{
		static const FName HighlightBrushName("Sequencer.AnimationOutliner.DefaultBorder");

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId++,
			AllottedGeometry.ToPaintGeometry(),
			FAppStyle::GetBrush(HighlightBrushName),
			DrawEffects,
			HighlightColor
		);
	}

	// --------------------------------------------
	// Draw outliner selection tint
	if (Selection.Outliner.IsSelected(OutlinerItem))
	{
		static const FName SelectionColorName("SelectionColor");
		static const FName HighlightBrushName("Sequencer.AnimationOutliner.DefaultBorder");

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId++,
			AllottedGeometry.ToPaintGeometry(),
			FAppStyle::GetBrush(HighlightBrushName),
			DrawEffects,
			FAppStyle::GetSlateColor(SelectionColorName).GetColor(InWidgetStyle).CopyWithNewOpacity(0.2f)
		);
	}

	return LayerId;
}

TViewModelPtr<FSectionModel> SChannelView::GetSection() const
{
	FViewModelPtr Model = WeakModel.Pin();
	if (Model)
	{
		constexpr bool bIncludeThis = true;
		return Model->FindAncestorOfType<FSectionModel>(bIncludeThis);
	}
	return nullptr;
}

TSharedPtr<FSequencer> SChannelView::LegacyGetSequencer() const
{
	FViewModelPtr Model = WeakModel.Pin();
	if (Model)
	{
		TViewModelPtr<FSequenceModel> Sequence = Model->FindAncestorOfType<FSequenceModel>();
		if (Sequence)
		{
			return Sequence->GetSequencerImpl();
		}
	}
	ensureMsgf(false, TEXT("No valid Sequencer exists"));
	return nullptr;
}

void SChannelView::GetKeysUnderMouse(const FVector2D& MousePosition, const FGeometry& AllottedGeometry, TArray<FSequencerSelectedKey>& OutKeys) const
{
	TViewModelPtr<FSectionModel> Section = GetSection();
	if (!Section || !KeyRendererCache.IsSet())
	{
		return;
	}

	const float HalfHeightPx = AllottedGeometry.GetLocalSize().Y*.5f;
	const FVector2D MousePixel = AllottedGeometry.AbsoluteToLocal(MousePosition);

	// Keys always render in the middle of the geometry so do a quick vertical test first
	if (FMath::Abs(MousePixel.Y - HalfHeightPx) <= KeyRendererCache->KeySizePx.Y*.5f)
	{
		GetKeysAtPixelX(MousePixel.X, OutKeys);
	}
}

void SChannelView::GetKeysAtPixelX(float LocalMousePixelX, TArray<FSequencerSelectedKey>& OutKeys) const
{
	TViewModelPtr<FSectionModel> Section = GetSection();
	UMovieSceneSection* SectionObject = Section ? Section->GetSection() : nullptr;

	if (!SectionObject)
	{
		return;
	}

	// Make the time <-> pixel converter relative to the section range rather than the view range
	FTimeToPixel RelativeTimeToPixel = GetRelativeTimeToPixel();

	TArray<FKeyRenderer::FKeysForModel> Results;
	KeyRenderer.HitTestKeys(RelativeTimeToPixel.PixelToFrame(LocalMousePixelX), Results);
	OutKeys = KeyRendererResultToSelectedKeys(Results, SectionObject);
}

void SChannelView::CreateKeysUnderMouse(const FVector2D& MousePosition, const FGeometry& AllottedGeometry, const TSet<FSequencerSelectedKey>& InPressedKeys, TArray<FSequencerSelectedKey>& OutKeys)
{
	FViewModelPtr Model = WeakModel.Pin();
	TViewModelPtr<FSectionModel> Section = GetSection();
	if (!Section || !Model)
	{
		return;
	}

	UMovieSceneSection* SectionObject = Section->GetSection();
	if (!SectionObject)
	{
		return;
	}

	if (SectionObject->IsReadOnly())
	{
		return;
	}

	FTimeToPixel RelativeTimeToPixel = GetRelativeTimeToPixel();

	// If the pressed key exists, offset the new key and look for it in the newly laid out key areas
	if (InPressedKeys.Num())
	{
		SectionObject->Modify();

		// Offset by 1 pixel worth of time if possible
		const FFrameTime TimeFuzz = RelativeTimeToPixel.PixelDeltaToFrame(1.f);

		for (const FSequencerSelectedKey& PressedKey : InPressedKeys)
		{
			const FFrameNumber CurrentTime = PressedKey.WeakChannel.Pin()->GetKeyArea()->GetKeyTime(PressedKey.KeyHandle);
			const FKeyHandle NewHandle = PressedKey.WeakChannel.Pin()->GetKeyArea()->DuplicateKey(PressedKey.KeyHandle);

			PressedKey.WeakChannel.Pin()->GetKeyArea()->SetKeyTime(NewHandle, CurrentTime + TimeFuzz.FrameNumber);
			OutKeys.Add(FSequencerSelectedKey(*SectionObject, PressedKey.WeakChannel, NewHandle));
		}
	}
	else
	{
		TViewModelPtr<ITrackExtension>      TrackModel             = Section->FindAncestorOfType<ITrackExtension>();
		TSharedPtr<IObjectBindingExtension> ObjectBindingExtension = Section->FindAncestorOfType<IObjectBindingExtension>();

		FVector2D LocalSpaceMousePosition = AllottedGeometry.AbsoluteToLocal(MousePosition);
		const FFrameTime CurrentTime = RelativeTimeToPixel.PixelToFrame(LocalSpaceMousePosition.X);

		TArray<TSharedRef<IKeyArea>> ValidKeyAreasUnderCursor;
		for (TViewModelPtr<FChannelModel> Channel : Model->GetDescendantsOfType<FChannelModel>(true))
		{
			ValidKeyAreasUnderCursor.Add(Channel->GetKeyArea().ToSharedRef());
		}

		FScopedTransaction Transaction(LOCTEXT("CreateKeysUnderMouse", "Create keys under mouse"));
		TArray<FAddKeyResult> Results;
		FAddKeyOperation::FromKeyAreas(TrackModel->GetTrackEditor().Get(), ValidKeyAreasUnderCursor).Commit(CurrentTime.FrameNumber, *LegacyGetSequencer(), &Results);

		for (FAddKeyResult& Result : Results)
		{
			for (TViewModelPtr<FChannelModel> Channel : Model->GetDescendantsOfType<FChannelModel>(true))
			{
				if (Channel->GetChannel() == Result.KeyArea->GetChannel().Get())
				{
					OutKeys.Add(FSequencerSelectedKey(*SectionObject, Channel, Result.KeyHandle));
					break;
				}
			}
		}
	}
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

