// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/SOutlinerObjectBindingView.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "SequencerUtilities.h"

#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "ISequencerModule.h"
#include "ISequencerTrackEditor.h"

#include "Sequencer.h"
#include "EditorStyleSet.h"
#include "GameFramework/Actor.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Containers/ArrayBuilder.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "SOutlinerObjectBindingView"

namespace UE
{
namespace Sequencer
{

void SOutlinerObjectBindingView::Construct(
		const FArguments& InArgs, 
		TSharedPtr<FObjectBindingModel> InModel, 
		TSharedPtr<FSequencerEditorViewModel> InEditor,
		const TSharedRef<ISequencerTreeViewRow>& InTableRow)
{
	SOutlinerItemViewBase::Construct(InArgs, TWeakViewModelPtr<IOutlinerExtension>(InModel), InEditor, InTableRow);
}

} // namespace Sequencer
} // namespace UE

#undef LOCTEXT_NAMESPACE
