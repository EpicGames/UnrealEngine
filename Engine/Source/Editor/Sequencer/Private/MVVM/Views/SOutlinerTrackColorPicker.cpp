// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/SOutlinerTrackColorPicker.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/Views/STrackAreaView.h"

#include "MovieSceneTrack.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Colors/SColorPicker.h"

#include "Styling/AppStyle.h"
#include "Engine/Engine.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SOutlinerTrackColorPicker"

namespace UE::Sequencer
{

void SOutlinerTrackColorPicker::Construct(const FArguments& InArgs, TWeakViewModelPtr<IOutlinerExtension> InWeakOutlinerExtension, const TSharedPtr<FEditorViewModel>& EditorViewModel)
{
	TAttribute<bool> IsEnabled = MakeAttributeLambda(
		[WeakEditor = TWeakPtr<FEditorViewModel>(EditorViewModel)]
		{
			TSharedPtr<FEditorViewModel> EditorPinned = WeakEditor.Pin();
			return EditorPinned && !EditorPinned->IsReadOnly();
		}
	);

	WeakOutlinerExtension = InWeakOutlinerExtension;

	ChildSlot
	[
		SNew(SButton)
		.ContentPadding(0)
		.VAlign(VAlign_Fill)
		.IsFocusable(false) // Intentionally false so that it's easier to tab to the next numeric input
		.IsEnabled(IsEnabled)
		.ButtonStyle(FAppStyle::Get(), "Sequencer.AnimationOutliner.ColorStrip")
		.OnClicked(this, &SOutlinerTrackColorPicker::OnSetTrackColor)
		.Content()
		[
			SNew(SBox)
			.WidthOverride(6.f)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("WhiteBrush"))
				.ColorAndOpacity(this, &SOutlinerTrackColorPicker::GetTrackColorTint)
			]
		]
	];
}

// We store these for when the Color Picker is canceled so we can restore the old value.
namespace AnimationOutlinerTreeNode
{
	FLinearColor InitialTrackColor;
	bool bFolderPickerWasCancelled;
}

FReply SOutlinerTrackColorPicker::OnSetTrackColor()
{
	AnimationOutlinerTreeNode::InitialTrackColor = GetTrackColorTint().GetSpecifiedColor();
	AnimationOutlinerTreeNode::bFolderPickerWasCancelled = false;

	FColorPickerArgs PickerArgs;
	PickerArgs.bUseAlpha = false;
	PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
	PickerArgs.InitialColor = AnimationOutlinerTreeNode::InitialTrackColor;
	PickerArgs.ParentWidget = GetParentWidget();
	PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &SOutlinerTrackColorPicker::OnColorPickerPicked);
	PickerArgs.OnColorPickerWindowClosed = FOnWindowClosed::CreateSP(this, &SOutlinerTrackColorPicker::OnColorPickerClosed);
	PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &SOutlinerTrackColorPicker::OnColorPickerCancelled);

	OpenColorPicker(PickerArgs);
	return FReply::Handled();
}

void SOutlinerTrackColorPicker::OnColorPickerPicked(FLinearColor NewFolderColor)
{
	UMovieSceneTrack* Track = GetTrackFromNode();
	if (Track)
	{
		// This is called every time the user adjusts the UI so we don't want to create a transaction for it, just directly
		// modify the track so we can see the change immediately.
		Track->SetColorTint(NewFolderColor.ToFColor(true));
	}
}

void SOutlinerTrackColorPicker::OnColorPickerClosed(const TSharedRef<SWindow>& Window)
{
	// Under Unreal UX terms, closing the Color Picker (via the UI) is the same as confirming it since we've been live updating
	// the color. The track already has the latest color change so we undo the change before calling Modify so that Undo sets us
	// to the original color. This is also called in the event of pressing cancel so we need to detect if it was canceled or not.
	if (!AnimationOutlinerTreeNode::bFolderPickerWasCancelled)
	{
		UMovieSceneTrack* Track = GetTrackFromNode();
		if(Track)
		{
			const FScopedTransaction Transaction(LOCTEXT("SetTrackColor", "Set Track Color"));
			FSlateColor CurrentColor = GetTrackColorTint();
			Track->SetColorTint(AnimationOutlinerTreeNode::InitialTrackColor.ToFColor(true));
			Track->Modify();
			Track->SetColorTint(CurrentColor.GetSpecifiedColor().ToFColor(true));
		}
	}
}

void SOutlinerTrackColorPicker::OnColorPickerCancelled(FLinearColor NewFolderColor)
{
	AnimationOutlinerTreeNode::bFolderPickerWasCancelled = true;

	// Restore the original color of the track. No transaction will be created when the OnColorPickerClosed callback is called.
	UMovieSceneTrack* Track = GetTrackFromNode();
	if (Track)
	{
		Track->SetColorTint(AnimationOutlinerTreeNode::InitialTrackColor.ToFColor(true));
	}
}

UMovieSceneTrack* SOutlinerTrackColorPicker::GetTrackFromNode() const
{
	TViewModelPtr<IOutlinerExtension> Outliner = WeakOutlinerExtension.Pin();
	if (Outliner)
	{
		constexpr bool bIncludeThis = true;
		for (TSharedPtr<ITrackExtension> TrackExtension : Outliner.AsModel()->GetAncestorsOfType<ITrackExtension>(bIncludeThis))
		{
			if (UMovieSceneTrack* Track = TrackExtension->GetTrack())
			{
				return Track;
			}
		}
	}

	return nullptr;
}

FSlateColor SOutlinerTrackColorPicker::GetTrackColorTint() const
{
	UMovieSceneTrack* Track = GetTrackFromNode();
	if (Track)
	{
		return STrackAreaView::BlendDefaultTrackColor(Track->GetColorTint());
	}

	return FLinearColor::Transparent;
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

