// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/SOutlinerTrackPreserveRatio.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/Views/OutlinerColumns/SOutlinerColumnButton.h"

#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneChannelEditorData.h"
#include "IKeyArea.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"

#include "CoreGlobals.h"
#include "Engine/Engine.h"
#include "Misc/ConfigCacheIni.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SOutlinerTrackPreserveRatio"

namespace UE::Sequencer
{

void SOutlinerTrackPreserveRatio::Construct(const FArguments& InArgs, TWeakViewModelPtr<IOutlinerExtension> InWeakOutlinerExtension, const TSharedPtr<FEditorViewModel>& EditorViewModel)
{
	WeakOutlinerExtension = InWeakOutlinerExtension;

	bool bPreserveRatio = false;
	GConfig->GetBool(TEXT("SelectionDetails"), TEXT("PreserveScaleRatio"), bPreserveRatio, GEditorPerProjectIni);

	SetPreserveRatio(bPreserveRatio);

	TAttribute<bool> IsEnabled = MakeAttributeLambda(
		[WeakEditor = TWeakPtr<FEditorViewModel>(EditorViewModel)]
		{
			TSharedPtr<FEditorViewModel> EditorPinned = WeakEditor.Pin();
			return EditorPinned && !EditorPinned->IsReadOnly();
		}
	);

	ChildSlot
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		.Visibility(CanPreserveRatio() ? EVisibility::Visible : EVisibility::Hidden)
		[
			SNew(SOutlinerColumnButton)
			.IsFocusable(false)
			.IsEnabled(IsEnabled)
			.IsChecked(this, &SOutlinerTrackPreserveRatio::OnGetPreserveRatio)
			.ToolTipText(LOCTEXT("PreserveRatioTooltip", "When enabled, all axis values scale together so the object maintains its proportions in all directions."))
			.Image(FAppStyle::GetBrush("Icons.Lock"))
			.UncheckedImage(FAppStyle::GetBrush("Icons.Unlock"))
			.OnClicked(this, &SOutlinerTrackPreserveRatio::OnSetPreserveRatio)
		]
	];
}

bool SOutlinerTrackPreserveRatio::OnGetPreserveRatio() const
{
	bool bPreserveRatio = true;

	int32 Count = 0;
	for (const FMovieSceneChannelMetaData* ChannelMetaData : GetMetaData())
	{
		Count++;
		if (ChannelMetaData && ChannelMetaData->bPreserveRatio != bPreserveRatio)
		{
			bPreserveRatio = false;
			break;
		}
	}

	if (Count > 0)
	{
		return bPreserveRatio;
	}
	return false;
}

FReply SOutlinerTrackPreserveRatio::OnSetPreserveRatio()
{
	const bool bPreserveRatio = !OnGetPreserveRatio();
	GConfig->SetBool(TEXT("SelectionDetails"), TEXT("PreserveScaleRatio"), bPreserveRatio, GEditorPerProjectIni);

	SetPreserveRatio(bPreserveRatio);

	return FReply::Handled();
}

TArray<const FMovieSceneChannelMetaData*> SOutlinerTrackPreserveRatio::GetMetaData() const
{
	TArray<const FMovieSceneChannelMetaData*> AllChannelMetaData;

	TViewModelPtr<IOutlinerExtension> Outliner = WeakOutlinerExtension.Pin();
	if (!Outliner)
	{
		return AllChannelMetaData;
	}

	TViewModelPtr<ITrackAreaExtension> TrackArea = Outliner.ImplicitCast();
	if (!TrackArea)
	{
		return AllChannelMetaData;
	}

	for (const FViewModelPtr& TrackAreaModel : TrackArea->GetTrackAreaModelList())
	{
		for (TSharedPtr<FChannelModel> ChannelModel : TrackAreaModel->GetDescendantsOfType<FChannelModel>())
		{
			if (TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea())
			{
				const FMovieSceneChannelHandle& ChannelHandle = KeyArea->GetChannel();
				const FMovieSceneChannelMetaData* ChannelMetaData = ChannelHandle.GetMetaData();
				if (ChannelMetaData)
				{
					AllChannelMetaData.Add(ChannelMetaData);
				}
			}
		}
	}

	return AllChannelMetaData;
}

void SOutlinerTrackPreserveRatio::SetPreserveRatio(bool bInPreserveRatio)
{
	for (const FMovieSceneChannelMetaData* ChannelMetaData : GetMetaData())
	{
		if (ChannelMetaData)
		{
			ChannelMetaData->bPreserveRatio = bInPreserveRatio;
		}
	}
}

bool SOutlinerTrackPreserveRatio::CanPreserveRatio() const
{
	for (const FMovieSceneChannelMetaData* ChannelMetaData : GetMetaData())
	{
		if (ChannelMetaData && ChannelMetaData->bCanPreserveRatio)
		{
			return true;
		}
	}
	return false;
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

