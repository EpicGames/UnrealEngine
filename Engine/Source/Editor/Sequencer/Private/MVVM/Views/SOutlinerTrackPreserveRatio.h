// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerCoreFwd.h"
#include "Widgets/SCompoundWidget.h"
#include "MVVM/ViewModelPtr.h"

struct FMovieSceneChannelMetaData;

namespace UE::Sequencer
{

class IOutlinerExtension;
class FEditorViewModel;

class SOutlinerTrackPreserveRatio
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SOutlinerTrackPreserveRatio){}
	SLATE_END_ARGS()

	SEQUENCER_API void Construct(const FArguments& InArgs, TWeakViewModelPtr<IOutlinerExtension> InWeakOutlinerExtension, const TSharedPtr<FEditorViewModel>& EditorViewModel);

private:

	/** Whether this button is seto to preserve ratio */
	bool OnGetPreserveRatio() const;

	/** Called when the user clicks on the preserve ratio button */
	FReply OnSetPreserveRatio();

	/** Retrieve the channel metadata for this channel group */
	TArray<const FMovieSceneChannelMetaData*> GetMetaData() const;

	/** Set the channel meta data to enable/disable preserve ratio */
	void SetPreserveRatio(bool bInPreserveRatio);

	/** Gets whether any of the channels can preserve ratio */
	bool CanPreserveRatio() const;

private:

	TWeakViewModelPtr<IOutlinerExtension> WeakOutlinerExtension;
};

} // namespace UE::Sequencer
