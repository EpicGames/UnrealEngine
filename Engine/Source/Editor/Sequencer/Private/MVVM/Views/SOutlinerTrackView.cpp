// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/SOutlinerTrackView.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"

namespace UE::Sequencer
{

void SOutlinerTrackView::Construct(
		const FArguments& InArgs,
		TWeakViewModelPtr<IOutlinerExtension> InWeakOutlinerExtension,
		TWeakPtr<FSequencerEditorViewModel> InWeakEditor,
		const TSharedRef<ISequencerTreeViewRow>& InTableRow)
{
	SOutlinerItemViewBase::Construct(InArgs, InWeakOutlinerExtension, InWeakEditor, InTableRow);
}

} // namespace UE::Sequencer

