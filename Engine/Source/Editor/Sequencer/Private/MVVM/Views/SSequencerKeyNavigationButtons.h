// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Views/SKeyNavigationButtons.h"
#include "IKeyArea.h"
#include "ISequencer.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneSection.h"
#include "SequencerAddKeyOperation.h"
#include "SequencerCommands.h"
#include "SequencerCommonHelpers.h"
#include "ScopedTransaction.h"
#include "MVVM/Extensions/ITimeDomainExtension.h"

#define LOCTEXT_NAMESPACE "SSequencerKeyNavigationButtons"

namespace UE
{
namespace Sequencer
{

class IOutlinerExtension;

/**
 * A widget for navigating between keys on a sequencer track
 */
class SSequencerKeyNavigationButtons
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSequencerKeyNavigationButtons) : _Buttons(EKeyNavigationButtons::All) {}
		SLATE_ARGUMENT(EKeyNavigationButtons, Buttons)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<FViewModel>& InModel, TSharedPtr<ISequencer> InSequencer)
	{
		WeakModel = InModel;
		WeakSequencer = InSequencer;

		ITimeDomainExtension* TimeDomain = InModel->CastThis<ITimeDomainExtension>();
		Domain = TimeDomain ? TimeDomain->GetDomain() : ETimeDomain::Warped;

		ChildSlot
		[
			SNew(SKeyNavigationButtons, InModel)
			.Buttons(InArgs._Buttons)
			.AddKeyToolTip(FText::Format(LOCTEXT("AddKeyButton", "Add a new key at the current time ({0})"), FSequencerCommands::Get().SetKey->GetInputText()))
			.PreviousKeyToolTip(FText::Format(LOCTEXT("PreviousKeyButton", "Set the time to the previous key ({0})"), FSequencerCommands::Get().StepToPreviousKey->GetInputText()))
			.NextKeyToolTip(FText::Format(LOCTEXT("NextKeyButton", "Set the time to the next key ({0})"), FSequencerCommands::Get().StepToNextKey->GetInputText()))
			.GetNavigatableTimes(this, &SSequencerKeyNavigationButtons::GetNavigatableTimes)
			.Time(this, &SSequencerKeyNavigationButtons::GetTime)
			.OnSetTime(this, &SSequencerKeyNavigationButtons::SetTime)
			.OnAddKey(this, &SSequencerKeyNavigationButtons::HandleAddKey)
			.IsEnabled_Lambda([this] () { return WeakSequencer.IsValid() ? !WeakSequencer.Pin()->IsReadOnly() : false; })
		];
	}

	void GetNavigatableTimes(TArray<FFrameNumber>& NavigatableTimes)
	{
		TSharedPtr<FViewModel> DataModel = WeakModel.Pin();
		if (!DataModel)
		{
			return;
		}

		TSet<TSharedPtr<IKeyArea>> KeyAreas;
		SequencerHelpers::GetAllKeyAreas( DataModel, KeyAreas );

		for ( TSharedPtr<IKeyArea> Keyarea : KeyAreas )
		{
			Keyarea->GetKeyTimes(NavigatableTimes);
		}

		TSet<TWeakObjectPtr<UMovieSceneSection> > Sections;
		SequencerHelpers::GetAllSections( DataModel, Sections );
		for ( TWeakObjectPtr<UMovieSceneSection> Section : Sections )
		{
			if (Section.IsValid())
			{
				Section->GetSnapTimes(NavigatableTimes, true);
			}
		}
	}

	FFrameTime GetTime() const
	{
		return Domain == ETimeDomain::Warped
			? WeakSequencer.Pin()->GetLocalTime().Time
			: WeakSequencer.Pin()->GetUnwarpedLocalTime().Time;
	}

	void SetTime(FFrameTime InTime)
	{
		if (Domain == ETimeDomain::Unwarped)
		{
			InTime = WeakSequencer.Pin()->GetLocalTimeWarpTransform().TransformTime(InTime);
		}
		WeakSequencer.Pin()->SetLocalTime(InTime);
	}

	void HandleAddKey(FFrameTime Time, TSharedPtr<FViewModel> InModel)
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		if (InModel && Sequencer)
		{
			FScopedTransaction Transaction(LOCTEXT("AddKeys", "Add Keys at Current Time"));
			FAddKeyOperation::FromNode(InModel).Commit(Time.RoundToFrame(), *Sequencer);
		}
	}

	TWeakPtr<FViewModel> WeakModel;
	TWeakPtr<ISequencer> WeakSequencer;
	UE::Sequencer::ETimeDomain Domain;
};

} // namespace Sequencer
} // namespace UE

#undef LOCTEXT_NAMESPACE
