// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/SSequencerOutlinerView.h"
#include "MVVM/Views/SOutlinerViewRow.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModels/OutlinerSpacer.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/Views/SOutlinerItemViewBase.h"

#include "SequencerCoreFwd.h"
#include "Styling/StyleColors.h"

namespace UE::Sequencer
{

class SSequencerOutlinerViewRow : public SOutlinerViewRow
{
public:

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, TWeakViewModelPtr<IOutlinerExtension> InWeakModel, TWeakPtr<FOutlinerViewModel> InWeakOutliner)
	{
		WeakOutliner = InWeakOutliner;

		SOutlinerViewRow::Construct(InArgs, OwnerTableView, InWeakModel);
	}

	virtual const FSlateBrush* GetBorder() const override
	{
		const bool bEvenEntryIndex = (IndexInList % 2 == 0);

		TSharedPtr<FOutlinerViewModel>    Outliner = WeakOutliner.Pin();
		TViewModelPtr<IOutlinerExtension> OutlinerItem = WeakModel.Pin();

		if (!Outliner || !OutlinerItem)
		{
			return SOutlinerViewRow::GetBorder();
		}

		// If selected or highlighted, early out and return the default selected or highlighted border
		if (IsSelected() || IsHighlighted())
		{
			return SOutlinerViewRow::GetBorder();
		}

		EOutlinerSelectionState SelectionState = OutlinerItem->GetSelectionState();

		if (EnumHasAnyFlags(SelectionState, EOutlinerSelectionState::SelectedDirectly))
		{
			return &Style->ActiveBrush;
		}

		// If keys or the track area are selected, highlight this track row
		if (EnumHasAnyFlags(SelectionState, EOutlinerSelectionState::HasSelectedKeys | EOutlinerSelectionState::HasSelectedTrackAreaItems | EOutlinerSelectionState::DescendentHasSelectedTrackAreaItems | EOutlinerSelectionState::DescendentHasSelectedKeys))
		{
			return IsHovered()
				? (bEvenEntryIndex ? &Style->EvenRowBackgroundHoveredBrush : &Style->OddRowBackgroundHoveredBrush)
				: &Style->ActiveHighlightedBrush;
		}

		// If this is at the root level, return it as a parent row
		if (GetIndentLevel() == 0 && !OutlinerItem.AsModel()->IsA<FOutlinerSpacer>())
		{
			return IsHovered()
				? &Style->ParentRowBackgroundHoveredBrush
				: &Style->ParentRowBackgroundBrush;
		}

		return SOutlinerViewRow::GetBorder();
	}

private:

	TWeakPtr<FOutlinerViewModel> WeakOutliner;
};

TSharedRef<ITableRow> SSequencerOutlinerView::OnGenerateRow(TWeakViewModelPtr<IOutlinerExtension> InWeakModel, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<SOutlinerViewRow> Row =
		SNew(SSequencerOutlinerViewRow, OwnerTable, InWeakModel, WeakOutliner)
		.OnDetectDrag(this, &SSequencerOutlinerView::OnDragRow)
		.OnGetColumnVisibility(this, &SSequencerOutlinerView::IsColumnVisible)
		.OnGenerateWidgetForColumn(this, &SSequencerOutlinerView::GenerateWidgetForColumn);

	if (TViewModelPtr<IOutlinerExtension> ViewModel = InWeakModel.Pin())
	{
		CreateTrackLanesForRow(Row, ViewModel);
	}

	return Row;
}


} // namespace UE::Sequencer