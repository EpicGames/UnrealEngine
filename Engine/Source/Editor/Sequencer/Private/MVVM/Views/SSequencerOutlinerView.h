// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Views/SOutlinerView.h"

namespace UE::Sequencer
{

class SSequencerOutlinerView : public SOutlinerView
{
public:

	TSharedRef<ITableRow> OnGenerateRow(TWeakViewModelPtr<IOutlinerExtension> InWeakModel, const TSharedRef<STableViewBase>& OwnerTable) override;
};


} // namespace UE::Sequencer