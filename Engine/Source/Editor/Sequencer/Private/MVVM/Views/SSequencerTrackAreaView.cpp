// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVM/Views/SSequencerTrackAreaView.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Extensions/ITrackRowExtension.h"
#include "MVVM/ViewModels/OutlinerSpacer.h"
#include "MVVM/ViewModels/SequencerTrackAreaViewModel.h"
#include "MVVM/Views/SOutlinerView.h"

#include "ITimeSlider.h"
#include "Misc/InputState.h"
#include "SequencerCommands.h"
#include "SequencerTimeSliderController.h"
#include "Tools/SequencerEditTool_Selection.h"

#include "Sequencer.h"
#include "ISequencerTrackEditor.h"
#include "Framework/Application/SlateApplication.h"

namespace UE
{
namespace Sequencer
{

void SSequencerTrackAreaView::Construct(const FArguments& InArgs, TWeakPtr<FTrackAreaViewModel> InWeakViewModel, TSharedRef<ITimeSliderController> InTimeSliderController)
{
	TimeSliderController = InTimeSliderController;

	STrackAreaView::Construct(InArgs, InWeakViewModel);

	// Add the time slider controller to the input stack
	InputStack.AddHandler(TimeSliderController.Get());
}

int32 SSequencerTrackAreaView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	return STrackAreaView::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

FReply SSequencerTrackAreaView::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const TSharedPtr<FTrackAreaViewModel> ViewModel = WeakViewModel.Pin();
	if (ViewModel.IsValid() && CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport))
	{
		ViewModel->SetHotspot(nullptr);
		ViewModel->AttemptToActivateTool(FSequencerEditTool_Selection::Identifier);
	}

	return STrackAreaView::OnMouseButtonDown(MyGeometry, MouseEvent);
}

FReply SSequencerTrackAreaView::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	//FContextMenuSuppressor SuppressContextMenus(TimeSliderController.ToSharedRef());
	return STrackAreaView::OnMouseButtonUp(MyGeometry, MouseEvent);
}

void SSequencerTrackAreaView::UpdateHoverStates(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const TSharedPtr<FTrackAreaViewModel> ViewModel = WeakViewModel.Pin();

	if (ViewModel.IsValid() && CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport))
	{
		ViewModel->SetHotspot(nullptr);
		ViewModel->AttemptToActivateTool(FSequencerEditTool_Selection::Identifier);
		return;
	}

	STrackAreaView::UpdateHoverStates(MyGeometry, MouseEvent);

	TSharedPtr<ITrackAreaHotspot>   Hotspot   = ViewModel ? ViewModel->GetHotspot() : nullptr;
	if (!Hotspot && ViewModel)
	{
		// Any other region implies selection mode
		ViewModel->AttemptToActivateTool(FSequencerEditTool_Selection::Identifier);
	}
}

void SSequencerTrackAreaView::OnResized(const FVector2D& OldSize, const FVector2D& NewSize)
{
	// Zoom by the difference in horizontal size
	const float Difference = OldSize.X - SizeLastFrame->X;
	TRange<double> OldRange = TimeSliderController->GetViewRange().GetAnimationTarget();

	double NewRangeMin = OldRange.GetLowerBoundValue();
	double NewRangeMax = OldRange.GetUpperBoundValue() + (Difference * OldRange.Size<double>() / SizeLastFrame->X);

	TRange<double> ClampRange = TimeSliderController->GetClampRange();

	if (NewRangeMin < ClampRange.GetLowerBoundValue() || NewRangeMax > ClampRange.GetUpperBoundValue())
	{
		double NewClampRangeMin = NewRangeMin < ClampRange.GetLowerBoundValue() ? NewRangeMin : ClampRange.GetLowerBoundValue();
		double NewClampRangeMax = NewRangeMax > ClampRange.GetUpperBoundValue() ? NewRangeMax : ClampRange.GetUpperBoundValue();

		TimeSliderController->SetClampRange(NewClampRangeMin, NewClampRangeMax);
	}

	TimeSliderController->SetViewRange(
		NewRangeMin, NewRangeMax,
		EViewRangeInterpolation::Immediate
	);
}

FReply SSequencerTrackAreaView::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	using namespace UE::Sequencer;

	TSharedPtr<SOutlinerView> Outliner = WeakOutliner.Pin();

	TViewModelPtr<IOutlinerExtension> DroppedItem = Outliner->HitTestNode(MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition()).Y);
	if (DroppedItem && DroppedItem.AsModel()->IsA<FOutlinerSpacer>())
	{
		DroppedItem = nullptr;
	}

	WeakDroppedItem = DroppedItem;

	bAllowDrop = false;
	DropFrameRange.Reset();

	TSharedPtr<FTrackAreaViewModel> ViewModel = WeakViewModel.Pin();
	FSequencerEditorViewModel* EditorViewModel = ViewModel->GetEditor()->CastThisChecked<FSequencerEditorViewModel>();
	TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();

	if (Sequencer && DroppedItem)
	{
		TSharedPtr<ITrackExtension> TrackDestination = DroppedItem.ImplicitCast();
		TSharedPtr<ITrackRowExtension> TrackRowDestination = DroppedItem.ImplicitCast();
		if (TrackDestination || TrackRowDestination)
		{
			UMovieSceneTrack* Track    = TrackDestination ? TrackDestination->GetTrack() : TrackRowDestination->GetParentTrack();
			const int32       TrackRow = TrackRowDestination ? TrackRowDestination->GetRowIndex() : 0;

			TSharedPtr<IObjectBindingExtension> BindingExtension = DroppedItem.AsModel()->FindAncestorOfType<IObjectBindingExtension>();
			FGuid ObjectBinding = BindingExtension ? BindingExtension->GetObjectGuid() : FGuid();

			// give track editors a chance to accept the drag event
			const TArray<TSharedPtr<ISequencerTrackEditor>>& TrackEditors = Sequencer->GetTrackEditors();

			FVector2D LocalPos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());
			FFrameNumber DropFrameNumber = TimeToPixel->PixelToFrame(LocalPos.X).FrameNumber;
			if (Sequencer->GetSequencerSettings()->GetForceWholeFrames())
			{
				DropFrameNumber = FFrameRate::Snap(DropFrameNumber, Sequencer->GetFocusedTickResolution(), Sequencer->GetFocusedDisplayRate()).FrameNumber;
			}

			// If shift is pressed, drop onto the current time
			if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
			{
				DropFrameNumber = Sequencer->GetLocalTime().Time.FrameNumber;
			}

			FSequencerDragDropParams DragDropParams(Track, TrackRow, ObjectBinding, DropFrameNumber, TRange<FFrameNumber>());

			for (const TSharedPtr<ISequencerTrackEditor>& TrackEditor : TrackEditors)
			{
				if (TrackEditor->OnAllowDrop(DragDropEvent, DragDropParams))
				{
					bAllowDrop = true;
					DropFrameRange = DragDropParams.FrameRange;
					return FReply::Handled();
				}
			}
		}
	}

	return SPanel::OnDragOver(MyGeometry, DragDropEvent);
}

FReply SSequencerTrackAreaView::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	WeakDroppedItem = WeakOutliner.Pin()->HitTestNode(MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition()).Y);
	TSharedPtr<FViewModel> DroppedItem  = WeakDroppedItem.Pin();

	TSharedPtr<FTrackAreaViewModel> ViewModel = WeakViewModel.Pin();
	FSequencerEditorViewModel* EditorViewModel = ViewModel->GetEditor()->CastThisChecked<FSequencerEditorViewModel>();
	TSharedPtr<FSequencer> Sequencer = EditorViewModel->GetSequencerImpl();

	if (Sequencer && DroppedItem)
	{
		ITrackExtension* TrackDestination = DroppedItem->CastThis<ITrackExtension>();
		ITrackRowExtension* TrackRowDestination = DroppedItem->CastThis<ITrackRowExtension>();
		if (TrackDestination || TrackRowDestination)
		{
			UMovieSceneTrack* Track    = TrackDestination ? TrackDestination->GetTrack() : TrackRowDestination->GetParentTrack();
			const int32       TrackRow = TrackRowDestination ? TrackRowDestination->GetRowIndex() : 0;

			TSharedPtr<IObjectBindingExtension> BindingExtension = DroppedItem->FindAncestorOfType<IObjectBindingExtension>();
			FGuid ObjectBinding = BindingExtension ? BindingExtension->GetObjectGuid() : FGuid();

			// give track editors a chance to process the drag event
			const TArray<TSharedPtr<ISequencerTrackEditor>>& TrackEditors = Sequencer->GetTrackEditors();

			FVector2D LocalPos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());
			FFrameNumber DropFrameNumber = TimeToPixel->PixelToFrame(LocalPos.X).FrameNumber;
			if (Sequencer->GetSequencerSettings()->GetForceWholeFrames())
			{
				DropFrameNumber = FFrameRate::Snap(DropFrameNumber, Sequencer->GetFocusedTickResolution(), Sequencer->GetFocusedDisplayRate()).FrameNumber;
			}

			// If shift is pressed, drop onto the current time
			if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
			{
				DropFrameNumber = Sequencer->GetLocalTime().Time.FrameNumber;
			}

			FSequencerDragDropParams DragDropParams(Track, TrackRow, ObjectBinding, DropFrameNumber, TRange<FFrameNumber>());

			for (const TSharedPtr<ISequencerTrackEditor>& TrackEditor : TrackEditors)
			{
				if (TrackEditor->OnAllowDrop(DragDropEvent, DragDropParams))
				{
					WeakDroppedItem = nullptr;

					return TrackEditor->OnDrop(DragDropEvent, DragDropParams);
				}
			}
		}
	}

	WeakDroppedItem = nullptr;

	return SPanel::OnDrop(MyGeometry, DragDropEvent);
}

} // namespace Sequencer
} // namespace UE

