// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MVVM/Views/STrackAreaView.h"

class ITimeSliderController;

namespace UE
{
namespace Sequencer
{

class SSequencerTrackAreaView
	: public STrackAreaView
{
public:

	void Construct(const FArguments& InArgs, TWeakPtr<FTrackAreaViewModel> InWeakViewModel, TSharedRef<ITimeSliderController> InTimeSliderController);

	/*~ SWidget */
	int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/*~ STrackAreaView */
	void OnResized(const FVector2D& OldSize, const FVector2D& NewSize) override;
	void UpdateHoverStates(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

private:

	TSharedPtr<ITimeSliderController> TimeSliderController;
};

} // namespace Sequencer
} // namespace UE

