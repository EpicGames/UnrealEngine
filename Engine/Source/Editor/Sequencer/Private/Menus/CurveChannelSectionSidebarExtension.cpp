// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveChannelSectionSidebarExtension.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Curves/RealCurve.h"
#include "IKeyArea.h"
#include "ISequencer.h"
#include "ScopedTransaction.h"
#include "SequencerSettings.h"
#include "SSequencer.h"
#include "GameFramework/Actor.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "UObject/StructOnScope.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Curves/RealCurve.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "CurveChannelSectionSidebarExtension"

FCurveChannelSectionSidebarExtension::FCurveChannelSectionSidebarExtension(const TWeakPtr<ISequencer>& InWeakSequencer)
	: WeakSequencer(InWeakSequencer)
{
}

void FCurveChannelSectionSidebarExtension::AddSections(const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections)
{
	WeakSections = TSet(InWeakSections);
}

TSharedPtr<ISidebarChannelExtension> FCurveChannelSectionSidebarExtension::ExtendMenu(FMenuBuilder& MenuBuilder, const bool bInSubMenu)
{
	return SharedThis(this);
}

void FCurveChannelSectionSidebarExtension::AddDisplayOptionsMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}
	
	MenuBuilder.BeginSection(TEXT("DisplayOptions"), LOCTEXT("DisplayOptionsTooltip", "Display Options"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ToggleShowCurve", "Show Curve"),
		LOCTEXT("ToggleShowCurveTooltip", "Toggle showing the curve in the track area"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &FCurveChannelSectionSidebarExtension::ToggleShowCurve),
			FCanExecuteAction(),
			FGetActionCheckState::CreateSP(this, &FCurveChannelSectionSidebarExtension::IsShowCurve)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);	
	
	FString KeyAreaName;
	TArray<const IKeyArea*> SelectedKeyAreas;
	Sequencer->GetSelectedKeyAreas(SelectedKeyAreas);
	for (const IKeyArea* KeyArea : SelectedKeyAreas)
	{			
		if (KeyArea)
		{
			KeyAreaName = KeyArea->GetName().ToString();
			break;
		}
	}

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ToggleKeyAreaCurveNormalized", "Key Area Curve Normalized"),
		LOCTEXT("ToggleKeyAreaCurveNormalizedTooltip", "Toggle showing the curve in the track area as normalized"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &FCurveChannelSectionSidebarExtension::OnKeyAreaCurveNormalized, KeyAreaName),
			FCanExecuteAction::CreateSP(this, &FCurveChannelSectionSidebarExtension::IsAnyShowCurve),
			FIsActionChecked::CreateSP(this, &FCurveChannelSectionSidebarExtension::GetKeyAreaCurveNormalized, KeyAreaName)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);	

	MenuBuilder.AddWidget(
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SSpacer)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(50.f)
			.IsEnabled_Lambda([this, KeyAreaName]()
				{
					if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
					{
						return IsAnyShowCurve() && Sequencer->GetSequencerSettings()->HasKeyAreaCurveExtents(KeyAreaName);
					}
					return false;
				})
			[
				SNew(SSpinBox<double>)
				.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>(TEXT("Sequencer.HyperlinkSpinBox")))
				.Value(this, &FCurveChannelSectionSidebarExtension::GetKeyAreaCurveMin, KeyAreaName)
				.OnValueChanged(this, &FCurveChannelSectionSidebarExtension::OnKeyAreaCurveMinChanged, KeyAreaName)
				.OnValueCommitted_Lambda([this, KeyAreaName](const double InNewValue, const ETextCommit::Type InCommitType)
					{
						OnKeyAreaCurveMinChanged(InNewValue, KeyAreaName);
					})
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(50.f)
			.IsEnabled_Lambda([this, KeyAreaName]()
				{
					if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
					{
						return IsAnyShowCurve() && Sequencer->GetSequencerSettings()->HasKeyAreaCurveExtents(KeyAreaName);
					}
					return false;
				})
			[
				SNew(SSpinBox<double>)
				.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>(TEXT("Sequencer.HyperlinkSpinBox")))
				.Value(this, &FCurveChannelSectionSidebarExtension::GetKeyAreaCurveMax, KeyAreaName)
				.OnValueChanged(this, &FCurveChannelSectionSidebarExtension::OnKeyAreaCurveMaxChanged, KeyAreaName)
				.OnValueCommitted_Lambda([this, KeyAreaName](const double InNewValue, const ETextCommit::Type InCommitType)
					{
						OnKeyAreaCurveMaxChanged(InNewValue, KeyAreaName);
					})
			]
		],
		LOCTEXT("KeyAreaCurveRangeText", "Key Area Curve Range")
	);		
	
	MenuBuilder.AddWidget(
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SSpacer)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(50.f)
			[
				SNew(SSpinBox<int32>)
				.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>(TEXT("Sequencer.HyperlinkSpinBox")))
				.MinValue(15)
				.MaxValue(300)
				.Value(this, &FCurveChannelSectionSidebarExtension::GetKeyAreaHeight)
				.OnValueChanged(this, &FCurveChannelSectionSidebarExtension::OnKeyAreaHeightChanged)
				.OnValueCommitted_Lambda([this](const int32 InValue, const ETextCommit::Type InCommitType)
					{
						OnKeyAreaHeightChanged(InValue);
					})
			]
		],
		LOCTEXT("KeyAreaHeightText", "Key Area Height")
	);

	MenuBuilder.EndSection();
}

void FCurveChannelSectionSidebarExtension::AddExtrapolationMenu(FMenuBuilder& MenuBuilder, const bool bInPreInfinity)
{
	auto CreateUIAction = [this, bInPreInfinity](const ERichCurveExtrapolation InExtrapolation)
	{
		return FUIAction(
				FExecuteAction::CreateSP(this, &FCurveChannelSectionSidebarExtension::SetExtrapolationMode, InExtrapolation, bInPreInfinity),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FCurveChannelSectionSidebarExtension::IsExtrapolationModeSelected, InExtrapolation, bInPreInfinity)
			);
	};

	if (bInPreInfinity)
	{
		MenuBuilder.BeginSection(TEXT("PreInfinityExtrapolation"), LOCTEXT("SetPreInfinityExtrapolation", "Pre-Infinity"));
	}
	else
	{
		MenuBuilder.BeginSection(TEXT("PostInfinityExtrapolation"), LOCTEXT("SetPostInfinityExtrapolation", "Post-Infinity"));
	}

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SetExtrapConstant", "Constant"),
		LOCTEXT("SetExtrapConstantTooltip", "Set extrapolation constant"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("GenericCurveEditor.SetPreInfinityExtrapConstant")),
		CreateUIAction(RCCE_Constant),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SetExtrapCycle", "Cycle"),
		LOCTEXT("SetExtrapCycleTooltip", "Set extrapolation cycle"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("GenericCurveEditor.SetPreInfinityExtrapCycle")),
		CreateUIAction(RCCE_Cycle),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SetExtrapCycleWithOffset", "Cycle with Offset"),
		LOCTEXT("SetExtrapCycleWithOffsetTooltip", "Set extrapolation cycle with offset"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("GenericCurveEditor.SetPreInfinityExtrapCycleWithOffset")),
		CreateUIAction(RCCE_CycleWithOffset),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SetExtrapLinear", "Linear"),
		LOCTEXT("SetExtrapLinearTooltip", "Set extrapolation linear"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("GenericCurveEditor.SetPreInfinityExtrapLinear")),
		CreateUIAction(RCCE_Linear),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SetExtrapOscillate", "Oscillate"),
		LOCTEXT("SetExtrapOscillateTooltip", "Set extrapolation oscillate"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("GenericCurveEditor.SetPreInfinityExtrapOscillate")),
		CreateUIAction(RCCE_Oscillate),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.EndSection();
}

void FCurveChannelSectionSidebarExtension::GetChannels(TArray<FMovieSceneFloatChannel*>& FloatChannels, TArray<FMovieSceneDoubleChannel*>& DoubleChannels,
	TArray<FMovieSceneIntegerChannel*>& IntegerChannels, TArray<FMovieSceneBoolChannel*>& BoolChannels,
	TArray<FMovieSceneByteChannel*>& ByteChannels) const
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	// Get selected channels
	TArray<const IKeyArea*> KeyAreas;
	Sequencer->GetSelectedKeyAreas(KeyAreas);
	for (const IKeyArea* const KeyArea : KeyAreas)
	{
		FMovieSceneChannelHandle Handle = KeyArea->GetChannel();
		if (Handle.GetChannelTypeName() == FMovieSceneFloatChannel::StaticStruct()->GetFName())
		{
			FMovieSceneFloatChannel* const Channel = static_cast<FMovieSceneFloatChannel*>(Handle.Get());
			FloatChannels.Add(Channel);
		}
		else if (Handle.GetChannelTypeName() == FMovieSceneDoubleChannel::StaticStruct()->GetFName())
		{
			FMovieSceneDoubleChannel* const Channel = static_cast<FMovieSceneDoubleChannel*>(Handle.Get());
			DoubleChannels.Add(Channel);
		}
		else if (Handle.GetChannelTypeName() == FMovieSceneIntegerChannel::StaticStruct()->GetFName())
		{
			FMovieSceneIntegerChannel* const Channel = static_cast<FMovieSceneIntegerChannel*>(Handle.Get());
			IntegerChannels.Add(Channel);
		}
		else if (Handle.GetChannelTypeName() == FMovieSceneBoolChannel::StaticStruct()->GetFName())
		{
			FMovieSceneBoolChannel* const Channel = static_cast<FMovieSceneBoolChannel*>(Handle.Get());
			BoolChannels.Add(Channel);
		}
		else if (Handle.GetChannelTypeName() == FMovieSceneByteChannel::StaticStruct()->GetFName())
		{
			FMovieSceneByteChannel* const Channel = static_cast<FMovieSceneByteChannel*>(Handle.Get());
			ByteChannels.Add(Channel);
		}
	}

	// Otherwise, the channels of all the sections
	if (FloatChannels.Num() + DoubleChannels.Num() + IntegerChannels.Num() + BoolChannels.Num() + ByteChannels.Num() == 0)
	{
		for (const TWeakObjectPtr<UMovieSceneSection>& WeakSection : WeakSections)
		{
			if (const UMovieSceneSection* const Section = WeakSection.Get())
			{
				FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
				for (FMovieSceneFloatChannel* const Channel : ChannelProxy.GetChannels<FMovieSceneFloatChannel>())
				{
					FloatChannels.Add(Channel);
				}
				for (FMovieSceneDoubleChannel* const Channel : ChannelProxy.GetChannels<FMovieSceneDoubleChannel>())
				{
					DoubleChannels.Add(Channel);
				}
				for (FMovieSceneIntegerChannel* const Channel : ChannelProxy.GetChannels<FMovieSceneIntegerChannel>())
				{
					IntegerChannels.Add(Channel);
				}
				for (FMovieSceneBoolChannel* const Channel : ChannelProxy.GetChannels<FMovieSceneBoolChannel>())
				{
					BoolChannels.Add(Channel);
				}
				for (FMovieSceneByteChannel* const Channel : ChannelProxy.GetChannels<FMovieSceneByteChannel>())
				{
					ByteChannels.Add(Channel);
				}
			}
		}
	}
}

void FCurveChannelSectionSidebarExtension::SetExtrapolationMode(const ERichCurveExtrapolation InExtrapolation, const bool bInPreInfinity)
{
	TArray<FMovieSceneFloatChannel*> FloatChannels;
	TArray<FMovieSceneDoubleChannel*> DoubleChannels;
	TArray<FMovieSceneIntegerChannel*> IntegerChannels;
	TArray<FMovieSceneBoolChannel*> BoolChannels;
	TArray<FMovieSceneByteChannel*> ByteChannels;

	GetChannels(FloatChannels, DoubleChannels, IntegerChannels, BoolChannels, ByteChannels);

	if (FloatChannels.Num() + DoubleChannels.Num() + IntegerChannels.Num() + BoolChannels.Num() + ByteChannels.Num() == 0)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SetExtrapolationMode_Transaction", "Set Extrapolation Mode"));

	bool bAnythingChanged = false;

	// Modify all sections
	for (TWeakObjectPtr<UMovieSceneSection> WeakSection : WeakSections)
	{
		UMovieSceneSection* const Section = WeakSection.Get();
		if (IsValid(Section))
		{
			Section->Modify();
		}
	}

	// Apply to all channels
	for (FMovieSceneFloatChannel* const Channel : FloatChannels)
	{
		TEnumAsByte<ERichCurveExtrapolation>& DestExtrap = bInPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
		DestExtrap = InExtrapolation;
		bAnythingChanged = true;
	}
	for (FMovieSceneDoubleChannel* const Channel : DoubleChannels)
	{
		TEnumAsByte<ERichCurveExtrapolation>& DestExtrap = bInPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
		DestExtrap = InExtrapolation;
		bAnythingChanged = true;
	}
	for (FMovieSceneIntegerChannel* const Channel : IntegerChannels)
	{
		TEnumAsByte<ERichCurveExtrapolation>& DestExtrap = bInPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
		DestExtrap = InExtrapolation;
		bAnythingChanged = true;
	}
	for (FMovieSceneBoolChannel* const Channel : BoolChannels)
	{
		TEnumAsByte<ERichCurveExtrapolation>& DestExtrap = bInPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
		DestExtrap = InExtrapolation;
		bAnythingChanged = true;
	}
	for (FMovieSceneByteChannel* const Channel : ByteChannels)
	{
		TEnumAsByte<ERichCurveExtrapolation>& DestExtrap = bInPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
		DestExtrap = InExtrapolation;
		bAnythingChanged = true;
	}

	if (bAnythingChanged)
	{
		if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
		{
			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		}
	}
	else
	{
		Transaction.Cancel();
	}
}

bool FCurveChannelSectionSidebarExtension::IsExtrapolationModeSelected(const ERichCurveExtrapolation InExtrapolation, const bool bInPreInfinity) const
{
	TArray<FMovieSceneFloatChannel*> FloatChannels;
	TArray<FMovieSceneDoubleChannel*> DoubleChannels;
	TArray<FMovieSceneIntegerChannel*> IntegerChannels;
	TArray<FMovieSceneBoolChannel*> BoolChannels;
	TArray<FMovieSceneByteChannel*> ByteChannels;

	GetChannels(FloatChannels, DoubleChannels, IntegerChannels, BoolChannels, ByteChannels);

	for (FMovieSceneFloatChannel* const Channel : FloatChannels)
	{
		const ERichCurveExtrapolation SourceExtrapolation = bInPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
		if (SourceExtrapolation != InExtrapolation)
		{
			return false;
		}
	}
	for (FMovieSceneDoubleChannel* const Channel : DoubleChannels)
	{
		const ERichCurveExtrapolation SourceExtrapolation = bInPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
		if (SourceExtrapolation != InExtrapolation)
		{
			return false;
		}
	}
	for (FMovieSceneIntegerChannel* const Channel : IntegerChannels)
	{
		const ERichCurveExtrapolation SourceExtrapolation = bInPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
		if (SourceExtrapolation != InExtrapolation)
		{
			return false;
		}
	}
	for (FMovieSceneBoolChannel* const Channel : BoolChannels)
	{
		const ERichCurveExtrapolation SourceExtrapolation = bInPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
		if (SourceExtrapolation != InExtrapolation)
		{
			return false;
		}
	}
	for (FMovieSceneByteChannel* const Channel : ByteChannels)
	{
		const ERichCurveExtrapolation SourceExtrapolation = bInPreInfinity ? Channel->PreInfinityExtrap : Channel->PostInfinityExtrap;
		if (SourceExtrapolation != InExtrapolation)
		{
			return false;
		}
	}

	return true;
}

void FCurveChannelSectionSidebarExtension::ToggleShowCurve()
{
	const ECheckBoxState CurrentState = IsShowCurve();
	const bool bShowCurve = (CurrentState != ECheckBoxState::Checked); // If unchecked or mixed, check it

	FScopedTransaction Transaction(LOCTEXT("ToggleShowCurve_Transaction", "Toggle Show Curve"));

	bool bAnythingChanged = false;

	// Modify all sections
	for (const TWeakObjectPtr<UMovieSceneSection>& WeakSection : WeakSections)
	{
		UMovieSceneSection* const Section = WeakSection.Get();
		if (IsValid(Section))
		{
			Section->Modify();
		}
	}

	// Apply to all channels
	for (const TWeakObjectPtr<UMovieSceneSection>& WeakSection : WeakSections)
	{
		if (const UMovieSceneSection* const Section = WeakSection.Get())
		{
			const FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();

			for (FMovieSceneFloatChannel* const Channel : ChannelProxy.GetChannels<FMovieSceneFloatChannel>())
			{
				if (Channel)
				{
					Channel->SetShowCurve(bShowCurve);
					bAnythingChanged = true;
				}
			}

			for (FMovieSceneDoubleChannel* const Channel : ChannelProxy.GetChannels<FMovieSceneDoubleChannel>())
			{
				if (Channel)
				{
					Channel->SetShowCurve(bShowCurve);
					bAnythingChanged = true;
				}
			}
		}
	}

	if (!bAnythingChanged)
	{
		Transaction.Cancel();
	}
}

ECheckBoxState FCurveChannelSectionSidebarExtension::IsShowCurve() const
{
	int32 NumShowedAndHidden[2] = { 0, 0 };
	for (const TWeakObjectPtr<UMovieSceneSection>& WeakSection : WeakSections)
	{
		if (const UMovieSceneSection* const Section = WeakSection.Get())
		{
			const FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();

			for (FMovieSceneFloatChannel* const Channel : ChannelProxy.GetChannels<FMovieSceneFloatChannel>())
			{
				if (Channel)
				{
					NumShowedAndHidden[Channel->GetShowCurve() ? 0 : 1]++;
				}
			}

			for (FMovieSceneDoubleChannel* const Channel : ChannelProxy.GetChannels<FMovieSceneDoubleChannel>())
			{
				if (Channel)
				{
					NumShowedAndHidden[Channel->GetShowCurve() ? 0 : 1]++;
				}
			}
		}
	}

	if (NumShowedAndHidden[0] == 0 && NumShowedAndHidden[1] > 0)  // No curve showed, some hidden
	{
		return ECheckBoxState::Unchecked;
	}
	else if (NumShowedAndHidden[0] > 0 && NumShowedAndHidden[1] == 0) // Some curves showed, none hidden
	{
		return ECheckBoxState::Checked;
	}
	return ECheckBoxState::Undetermined;  // Mixed states, or no curves
}

bool FCurveChannelSectionSidebarExtension::IsAnyShowCurve() const
{
	for (const TWeakObjectPtr<UMovieSceneSection>& WeakSection : WeakSections)
	{
		const UMovieSceneSection* const Section = WeakSection.Get();
		if (IsValid(Section))
		{
			const FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();

			for (const FMovieSceneFloatChannel* const Channel : ChannelProxy.GetChannels<FMovieSceneFloatChannel>())
			{
				if (Channel && Channel->GetShowCurve())
				{
					return true;
				}
			}

			for (const FMovieSceneDoubleChannel* const Channel : ChannelProxy.GetChannels<FMovieSceneDoubleChannel>())
			{
				if (Channel && Channel->GetShowCurve())
				{
					return true;
				}
			}
		}
	}
	
	return false;
}

int32 FCurveChannelSectionSidebarExtension::GetKeyAreaHeight() const
{
	if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		return (int32)Sequencer->GetSequencerSettings()->GetKeyAreaHeightWithCurves();
	}
	return 0;
}

void FCurveChannelSectionSidebarExtension::OnKeyAreaHeightChanged(const int32 InNewValue)
{
	USequencerSettings* const SequencerSettings = GetSequencerSettings();
	if (IsValid(SequencerSettings))
	{
		SequencerSettings->SetKeyAreaHeightWithCurves((float)InNewValue);
	}
}

bool FCurveChannelSectionSidebarExtension::GetKeyAreaCurveNormalized(const FString InKeyAreaName) const
{
	USequencerSettings* const SequencerSettings = GetSequencerSettings();
	if (!IsValid(SequencerSettings))
	{
		return !SequencerSettings->HasKeyAreaCurveExtents(InKeyAreaName);
	}
	return false;
}

void FCurveChannelSectionSidebarExtension::OnKeyAreaCurveNormalized(const FString InKeyAreaName) 
{
	USequencerSettings* const SequencerSettings = GetSequencerSettings();
	if (!IsValid(SequencerSettings))
	{
		return;
	}

	if (SequencerSettings->HasKeyAreaCurveExtents(InKeyAreaName))
	{
		SequencerSettings->RemoveKeyAreaCurveExtents(InKeyAreaName);
	}
	else
	{
		// Initialize to some arbitrary value
		SequencerSettings->SetKeyAreaCurveExtents(InKeyAreaName, 0.f, 6.f); 
	}
}

double FCurveChannelSectionSidebarExtension::GetKeyAreaCurveMin(const FString InKeyAreaName) const
{
	USequencerSettings* const SequencerSettings = GetSequencerSettings();
	if (!IsValid(SequencerSettings))
	{
		return 0.0;
	}

	double CurveMin = 0.f;
	double CurveMax = 0.f;
	SequencerSettings->GetKeyAreaCurveExtents(InKeyAreaName, CurveMin, CurveMax);

	return CurveMin;
}

void FCurveChannelSectionSidebarExtension::OnKeyAreaCurveMinChanged(const double InNewValue, const FString InKeyAreaName)
{
	USequencerSettings* const SequencerSettings = GetSequencerSettings();
	if (!IsValid(SequencerSettings))
	{
		return;
	}

	double CurveMin = 0.f;
	double CurveMax = 0.f;
	SequencerSettings->GetKeyAreaCurveExtents(InKeyAreaName, CurveMin, CurveMax);

	SequencerSettings->SetKeyAreaCurveExtents(InKeyAreaName, InNewValue, CurveMax);
}
	
double FCurveChannelSectionSidebarExtension::GetKeyAreaCurveMax(const FString InKeyAreaName) const
{
	USequencerSettings* const SequencerSettings = GetSequencerSettings();
	if (!IsValid(SequencerSettings))
	{
		return 0.0;
	}

	double CurveMin = 0.f;
	double CurveMax = 0.f;
	SequencerSettings->GetKeyAreaCurveExtents(InKeyAreaName, CurveMin, CurveMax);

	return CurveMax;
}

void FCurveChannelSectionSidebarExtension::OnKeyAreaCurveMaxChanged(const double InNewValue, const FString InKeyAreaName)
{
	USequencerSettings* const SequencerSettings = GetSequencerSettings();
	if (!IsValid(SequencerSettings))
	{
		return;
	}

	double CurveMin = 0.f;
	double CurveMax = 0.f;
	SequencerSettings->GetKeyAreaCurveExtents(InKeyAreaName, CurveMin, CurveMax);

	SequencerSettings->SetKeyAreaCurveExtents(InKeyAreaName, CurveMin, InNewValue);
}

USequencerSettings* FCurveChannelSectionSidebarExtension::GetSequencerSettings() const
{
	if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		return Sequencer->GetSequencerSettings();
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
