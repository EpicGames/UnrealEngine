// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerChannelTraits.h"
#include "MovieSceneSection.h"

struct FMovieSceneBoolChannel;
struct FMovieSceneByteChannel;
struct FMovieSceneDoubleChannel;
struct FMovieSceneFloatChannel;
struct FMovieSceneIntegerChannel;
enum class ECheckBoxState : uint8;
enum ERichCurveExtrapolation : int;

struct FCurveChannelSectionSidebarExtension : TSharedFromThis<FCurveChannelSectionSidebarExtension>, ISidebarChannelExtension
{
	FCurveChannelSectionSidebarExtension(const TWeakPtr<ISequencer>& InWeakSequencer);
	virtual ~FCurveChannelSectionSidebarExtension() = default;

	void AddSections(const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections);

	virtual TSharedPtr<ISidebarChannelExtension> ExtendMenu(FMenuBuilder& MenuBuilder, const bool bInSubMenu) override;

private:
	void AddDisplayOptionsMenu(FMenuBuilder& MenuBuilder);
	void AddExtrapolationMenu(FMenuBuilder& MenuBuilder, const bool bInPreInfinity);

	void GetChannels(TArray<FMovieSceneFloatChannel*>& FloatChannels, TArray<FMovieSceneDoubleChannel*>& DoubleChannels,
		TArray<FMovieSceneIntegerChannel*>& IntegerChannels, TArray<FMovieSceneBoolChannel*>& BoolChannels, 
		TArray<FMovieSceneByteChannel*>& ByteChannels) const;

	void SetExtrapolationMode(const ERichCurveExtrapolation InExtrapolation, const bool bInPreInfinity);
	bool IsExtrapolationModeSelected(const ERichCurveExtrapolation InExtrapolation, const bool bInPreInfinity) const;

	void ToggleShowCurve();
	ECheckBoxState IsShowCurve() const;
	bool IsAnyShowCurve() const;

	int32 GetKeyAreaHeight() const;
	void OnKeyAreaHeightChanged(const int32 InNewValue);
	
	bool GetKeyAreaCurveNormalized(const FString InKeyAreaName) const;
	void OnKeyAreaCurveNormalized(const FString InKeyAreaName);

	double GetKeyAreaCurveMin(const FString InKeyAreaName) const;
	void OnKeyAreaCurveMinChanged(const double InNewValue, const FString InKeyAreaName);

	double GetKeyAreaCurveMax(const FString InKeyAreaName) const;
	void OnKeyAreaCurveMaxChanged(const double InNewValue, const FString InKeyAreaName);

	USequencerSettings* GetSequencerSettings() const;

	TWeakPtr<ISequencer> WeakSequencer;

	TSet<TWeakObjectPtr<UMovieSceneSection>> WeakSections;
};
