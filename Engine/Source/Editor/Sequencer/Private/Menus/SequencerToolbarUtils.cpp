// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerToolbarUtils.h"

#include "IVREditorModule.h"
#include "LevelSequence.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Sequencer.h"
#include "SequencerCommands.h"
#include "SequencerSettings.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"

#define LOCTEXT_NAMESPACE "Sequencer"

namespace UE::Sequencer
{
const FName GSequencerToolbarStyleName("SequencerToolbar");
	
namespace ToolbarUtilsDetail
{
static TSharedRef<SWidget> MakeKeyGroupMenu(const TWeakPtr<FSequencer> InWeakSequencer)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InWeakSequencer.Pin()->GetCommandBindings());

	if (InWeakSequencer.Pin()->IsLevelEditorSequencer())
	{
		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().SetKeyAll);
		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().SetKeyGroup);
		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().SetKeyChanged);
	}

	// Interpolation
	MenuBuilder.BeginSection("SequencerInterpolation", LOCTEXT("KeyInterpolationMenu", "Initial Key Interpolation"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("SetKeyInterpolationSmartAuto", "Cubic (Smart Auto)"),
			LOCTEXT("SetKeyInterpolationSmartAutoTooltip", "Set key interpolation to smart auto"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.IconKeySmartAuto"),
			FUIAction(
				FExecuteAction::CreateLambda([InWeakSequencer] { InWeakSequencer.Pin()->SetKeyInterpolation(EMovieSceneKeyInterpolation::SmartAuto); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([InWeakSequencer] { return InWeakSequencer.Pin()->GetKeyInterpolation() == EMovieSceneKeyInterpolation::SmartAuto; })),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("SetKeyInterpolationAuto", "Cubic (Auto)"),
			LOCTEXT("SetKeyInterpolationAutoTooltip", "Set key interpolation to auto"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.IconKeyAuto"),
			FUIAction(
				FExecuteAction::CreateLambda([InWeakSequencer] { InWeakSequencer.Pin()->SetKeyInterpolation(EMovieSceneKeyInterpolation::Auto); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([InWeakSequencer] { return InWeakSequencer.Pin()->GetKeyInterpolation() == EMovieSceneKeyInterpolation::Auto; })),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("SetKeyInterpolationUser", "Cubic (User)"),
			LOCTEXT("SetKeyInterpolationUserTooltip", "Set key interpolation to user"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.IconKeyUser"),
			FUIAction(
				FExecuteAction::CreateLambda([InWeakSequencer] { InWeakSequencer.Pin()->SetKeyInterpolation(EMovieSceneKeyInterpolation::User); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([InWeakSequencer] { return InWeakSequencer.Pin()->GetKeyInterpolation() == EMovieSceneKeyInterpolation::User; })),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("SetKeyInterpolationBreak", "Cubic (Break)"),
			LOCTEXT("SetKeyInterpolationBreakTooltip", "Set key interpolation to break"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.IconKeyBreak"),
			FUIAction(
				FExecuteAction::CreateLambda([InWeakSequencer] { InWeakSequencer.Pin()->SetKeyInterpolation(EMovieSceneKeyInterpolation::Break); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([InWeakSequencer] { return InWeakSequencer.Pin()->GetKeyInterpolation() == EMovieSceneKeyInterpolation::Break; })),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("SetKeyInterpolationLinear", "Linear"),
			LOCTEXT("SetKeyInterpolationLinearTooltip", "Set key interpolation to linear"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.IconKeyLinear"),
			FUIAction(
				FExecuteAction::CreateLambda([InWeakSequencer] { InWeakSequencer.Pin()->SetKeyInterpolation(EMovieSceneKeyInterpolation::Linear); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([InWeakSequencer] { return InWeakSequencer.Pin()->GetKeyInterpolation() == EMovieSceneKeyInterpolation::Linear; })),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("SetKeyInterpolationConstant", "Constant"),
			LOCTEXT("SetKeyInterpolationConstantTooltip", "Set key interpolation to constant"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.IconKeyConstant"),
			FUIAction(
				FExecuteAction::CreateLambda([InWeakSequencer] { InWeakSequencer.Pin()->SetKeyInterpolation(EMovieSceneKeyInterpolation::Constant); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([InWeakSequencer] { return InWeakSequencer.Pin()->GetKeyInterpolation() == EMovieSceneKeyInterpolation::Constant; })),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection(); // SequencerInterpolation

	return MenuBuilder.MakeWidget();
}

template<typename TCallback>
static void GetKeyGroupMenuEntryArgs(const TWeakPtr<FSequencer>& InWeakSequencer, TCallback&& InConsumer)
{
	TAttribute<FSlateIcon> KeyGroupModeIcon;
	KeyGroupModeIcon.Bind(TAttribute<FSlateIcon>::FGetter::CreateLambda([InWeakSequencer] 
	{
		switch (InWeakSequencer.Pin()->GetKeyGroupMode())
		{
		case EKeyGroupMode::KeyAll:
			return FSequencerCommands::Get().SetKeyAll->GetIcon();
		case EKeyGroupMode::KeyGroup:
			return FSequencerCommands::Get().SetKeyGroup->GetIcon();
		default: // EKeyGroupMode::KeyChanged
			return FSequencerCommands::Get().SetKeyChanged->GetIcon();
		}
	}));

	TAttribute<FText> KeyGroupModeToolTip;
	KeyGroupModeToolTip.Bind(TAttribute<FText>::FGetter::CreateLambda([InWeakSequencer] 
	{
		switch (InWeakSequencer.Pin()->GetKeyGroupMode())
		{
		case EKeyGroupMode::KeyAll:
			return FSequencerCommands::Get().SetKeyAll->GetDescription();
		case EKeyGroupMode::KeyGroup:
			return FSequencerCommands::Get().SetKeyGroup->GetDescription();
		default: // EKeyGroupMode::KeyChanged
			return FSequencerCommands::Get().SetKeyChanged->GetDescription();
		}
	}));

	InConsumer(
		"KeyGroup", FUIAction(), FOnGetContent::CreateStatic(&MakeKeyGroupMenu, InWeakSequencer),
		LOCTEXT("KeyGroup", "Key All"), KeyGroupModeToolTip, KeyGroupModeIcon
		);
}
}
	
FToolMenuEntry MakeKeyGroupMenuEntry_ToolMenus(const TWeakPtr<FSequencer>& InWeakSequencer)
{
	FToolMenuEntry KeyGroupEntry;
	ToolbarUtilsDetail::GetKeyGroupMenuEntryArgs(InWeakSequencer,
		[&KeyGroupEntry](const FName InName, const FUIAction& InAction, const FOnGetContent& InMenuContentGenerator,
		const TAttribute<FText>& InLabelOverride, const TAttribute<FText>& ToolTipAttr, const TAttribute<FSlateIcon>& IconAttr)
	{
		KeyGroupEntry = FToolMenuEntry::InitComboButton(
			InName, InAction, InMenuContentGenerator, InLabelOverride, ToolTipAttr, IconAttr
		);
		KeyGroupEntry.StyleNameOverride = GSequencerToolbarStyleName;
	});
	return KeyGroupEntry;
}
	
namespace ToolbarUtilsDetail
{
static TSharedRef<SWidget> MakeAutoChangeMenu(const TWeakPtr<FSequencer> InWeakSequencer)
{
	FMenuBuilder MenuBuilder(false, InWeakSequencer.Pin()->GetCommandBindings());

	MenuBuilder.AddMenuEntry(FSequencerCommands::Get().SetAutoKey);

	if (InWeakSequencer.Pin()->IsLevelEditorSequencer())
	{
		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().SetAutoTrack);
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (IVREditorModule::Get().IsVREditorModeActive()
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		|| (InWeakSequencer.Pin()->IsLevelEditorSequencer() && !ExactCast<ULevelSequence>(InWeakSequencer.Pin()->GetFocusedMovieSceneSequence())))
	{
		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().SetAutoChangeAll);
	}

	MenuBuilder.AddMenuEntry(FSequencerCommands::Get().SetAutoChangeNone);

	return MenuBuilder.MakeWidget();
}

template<typename TCallback, typename TCallback2>
static void GetAutoKeyMenuEntryArgs(const TSharedPtr<FSequencer>& InSequencer, TCallback&& InConsumer, TCallback2&& InConsumer2)
{
	const TWeakPtr<FSequencer> WeakSequencer = InSequencer.ToWeakPtr();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (IVREditorModule::Get().IsVREditorModeActive()
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		|| (InSequencer->IsLevelEditorSequencer() && ExactCast<ULevelSequence>(InSequencer->GetFocusedMovieSceneSequence()) == nullptr))
	{
		TAttribute<FSlateIcon> AutoChangeModeIcon;
		AutoChangeModeIcon.Bind(TAttribute<FSlateIcon>::FGetter::CreateLambda([WeakSequencer] 
		{
			switch ( WeakSequencer.Pin()->GetAutoChangeMode() )
			{
			case EAutoChangeMode::AutoKey:
				return FSequencerCommands::Get().SetAutoKey->GetIcon();
			case EAutoChangeMode::AutoTrack:
				return FSequencerCommands::Get().SetAutoTrack->GetIcon();
			case EAutoChangeMode::All:
				return FSequencerCommands::Get().SetAutoChangeAll->GetIcon();
			default: // EAutoChangeMode::None
				return FSequencerCommands::Get().SetAutoChangeNone->GetIcon();
			}
		} ) );

		TAttribute<FText> AutoChangeModeToolTip;
		AutoChangeModeToolTip.Bind( TAttribute<FText>::FGetter::CreateLambda([WeakSequencer] 
		{
			switch ( WeakSequencer.Pin()->GetAutoChangeMode() )
			{
			case EAutoChangeMode::AutoKey:
				return FSequencerCommands::Get().SetAutoKey->GetDescription();
			case EAutoChangeMode::AutoTrack:
				return FSequencerCommands::Get().SetAutoTrack->GetDescription();
			case EAutoChangeMode::All:
				return FSequencerCommands::Get().SetAutoChangeAll->GetDescription();
			default: // EAutoChangeMode::None
				return FSequencerCommands::Get().SetAutoChangeNone->GetDescription();
			}
		} ) );
		
		InConsumer(
			"AutoChange", FUIAction(), FOnGetContent::CreateStatic(&ToolbarUtilsDetail::MakeAutoChangeMenu, WeakSequencer),
			LOCTEXT("AutoChangeMode", "Auto-Change Mode"), AutoChangeModeToolTip, AutoChangeModeIcon
		);
	}
	else	
	{
		InConsumer2(FSequencerCommands::Get().ToggleAutoKeyEnabled);
	}
}
}
	
FToolMenuEntry MakeAutoKeyMenuEntry(const TSharedPtr<FSequencer>& InSequencer)
{
	FToolMenuEntry KeyGroupEntry;
	ToolbarUtilsDetail::GetAutoKeyMenuEntryArgs(InSequencer,
		[&KeyGroupEntry](const FName InName, const FUIAction& InAction, const FOnGetContent& InMenuContentGenerator,
		const TAttribute<FText>& InLabelOverride, const TAttribute<FText>& ToolTipAttr, const TAttribute<FSlateIcon>& IconAttr)
		{
		KeyGroupEntry = FToolMenuEntry::InitComboButton(
			InName, InAction, InMenuContentGenerator, InLabelOverride, ToolTipAttr, IconAttr
		);
		KeyGroupEntry.StyleNameOverride = GSequencerToolbarStyleName;
		},
		[&KeyGroupEntry](TSharedPtr<FUICommandInfo> ToggleAutoKeyEnabled)
		{
			KeyGroupEntry = FToolMenuEntry::InitMenuEntry(ToggleAutoKeyEnabled);
		});
	KeyGroupEntry.StyleNameOverride = GSequencerToolbarStyleName;
	return KeyGroupEntry;
}

namespace ToolbarUtilsDetail
{
static TSharedRef<SWidget> MakeAllowEditsMenu(const TWeakPtr<FSequencer> InWeakSequencer)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InWeakSequencer.Pin()->GetCommandBindings());

	MenuBuilder.AddMenuEntry(FSequencerCommands::Get().AllowAllEdits);
	MenuBuilder.AddMenuEntry(FSequencerCommands::Get().AllowSequencerEditsOnly);
	MenuBuilder.AddMenuEntry(FSequencerCommands::Get().AllowLevelEditsOnly);

	return MenuBuilder.MakeWidget();
}

template<typename TCallback>
static bool GetAllowEditsModeMenuArgs(const TSharedPtr<FSequencer>& InSequencer, TCallback&& InConsumer)
{
	const TWeakPtr<FSequencer> WeakSequencer = InSequencer.ToWeakPtr();
	if (InSequencer->IsLevelEditorSequencer())
	{
		TAttribute<FSlateIcon> AllowEditsModeIcon;
		AllowEditsModeIcon.Bind(TAttribute<FSlateIcon>::FGetter::CreateLambda([WeakSequencer] 
		{
			switch ( WeakSequencer.Pin()->GetAllowEditsMode() )
			{
			case EAllowEditsMode::AllEdits:
				return FSequencerCommands::Get().AllowAllEdits->GetIcon();
			case EAllowEditsMode::AllowSequencerEditsOnly:
				return FSequencerCommands::Get().AllowSequencerEditsOnly->GetIcon();
			default: // EAllowEditsMode::AllowLevelEditsOnly
				return FSequencerCommands::Get().AllowLevelEditsOnly->GetIcon();
			}
		} ) );

		TAttribute<FText> AllowEditsModeToolTip;
		AllowEditsModeToolTip.Bind( TAttribute<FText>::FGetter::CreateLambda([WeakSequencer] 
		{
			switch ( WeakSequencer.Pin()->GetAllowEditsMode() )
			{
			case EAllowEditsMode::AllEdits:
				return FSequencerCommands::Get().AllowAllEdits->GetDescription();
			case EAllowEditsMode::AllowSequencerEditsOnly:
				return FSequencerCommands::Get().AllowSequencerEditsOnly->GetDescription();
			default: // EAllowEditsMode::AllowLevelEditsOnly
				return FSequencerCommands::Get().AllowLevelEditsOnly->GetDescription();
			}
		} ) );

		InConsumer(
			"AllowEditsMode",
			FUIAction(),
			FOnGetContent::CreateStatic(&ToolbarUtilsDetail::MakeAllowEditsMenu, WeakSequencer),
			LOCTEXT("AllowEditsMode", "Allow Edits"),
			AllowEditsModeToolTip,
			AllowEditsModeIcon
		);
		return true;
	}
	return false;
}
}
	
TOptional<FToolMenuEntry> MakeAllowEditsModeMenuEntry(const TSharedPtr<FSequencer>& InSequencer)
{
	FToolMenuEntry KeyGroupEntry;
	const bool bMade = ToolbarUtilsDetail::GetAllowEditsModeMenuArgs(InSequencer,
		[&KeyGroupEntry](const FName InName, const FUIAction& InAction, const FOnGetContent& InMenuContentGenerator,
		const TAttribute<FText>& InLabelOverride, const TAttribute<FText>& ToolTipAttr, const TAttribute<FSlateIcon>& IconAttr)
	{
		KeyGroupEntry = FToolMenuEntry::InitComboButton(
			InName, InAction, InMenuContentGenerator, InLabelOverride, ToolTipAttr, IconAttr
		);
		KeyGroupEntry.StyleNameOverride = GSequencerToolbarStyleName;
	});
	if (bMade)
	{
		return KeyGroupEntry;
	}
	return {};
}

void AppendSequencerToolbarEntries(const TSharedPtr<FSequencer>& InSequencer, FToolBarBuilder& InToolbarBuilder, const EToolbarItemFlags InFlags)
{
	const TWeakPtr<FSequencer> WeakSequencer = InSequencer.ToWeakPtr();

	InToolbarBuilder.BeginStyleOverride(GSequencerToolbarStyleName);
	
	if (EnumHasAnyFlags(InFlags, EToolbarItemFlags::KeyGroup))
	{
		ToolbarUtilsDetail::GetKeyGroupMenuEntryArgs(WeakSequencer,
			[&InToolbarBuilder](const FName, const FUIAction& InAction, const FOnGetContent& InMenuContentGenerator,
			const TAttribute<FText>& InLabelOverride, const TAttribute<FText>& ToolTipAttr, const TAttribute<FSlateIcon>& IconAttr)
		{
			InToolbarBuilder.AddComboButton(InAction, InMenuContentGenerator, InLabelOverride, ToolTipAttr, IconAttr);
		});
	}
	
	if (EnumHasAnyFlags(InFlags, EToolbarItemFlags::AutoKey))
	{
		ToolbarUtilsDetail::GetAutoKeyMenuEntryArgs(InSequencer,
			[&InToolbarBuilder](const FName, const FUIAction& InAction, const FOnGetContent& InMenuContentGenerator,
			const TAttribute<FText>& InLabelOverride, const TAttribute<FText>& ToolTipAttr, const TAttribute<FSlateIcon>& IconAttr)
			{
				InToolbarBuilder.AddComboButton(InAction, InMenuContentGenerator, InLabelOverride, ToolTipAttr, IconAttr);
			},
			[&InToolbarBuilder](TSharedPtr<FUICommandInfo> ToggleAutoKeyEnabled)
			{
				InToolbarBuilder.AddToolBarButton(ToggleAutoKeyEnabled);
			});
	}
	
	if (EnumHasAnyFlags(InFlags, EToolbarItemFlags::AllowEditsMode))
	{
		ToolbarUtilsDetail::GetAllowEditsModeMenuArgs(InSequencer,
			[&InToolbarBuilder](const FName, const FUIAction& InAction, const FOnGetContent& InMenuContentGenerator,
			const TAttribute<FText>& InLabelOverride, const TAttribute<FText>& ToolTipAttr, const TAttribute<FSlateIcon>& IconAttr)
		{
			InToolbarBuilder.AddComboButton(InAction, InMenuContentGenerator, InLabelOverride, ToolTipAttr, IconAttr);
		});
	}
	
	InToolbarBuilder.EndStyleOverride();
}
}

#undef LOCTEXT_NAMESPACE