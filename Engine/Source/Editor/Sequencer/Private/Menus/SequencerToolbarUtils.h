// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"
#include "Templates//SharedPointerFwd.h"
#include "UObject/NameTypes.h"

class FSequencer;
class FToolBarBuilder;
struct FButtonArgs;
struct FToolMenuEntry;
template<typename OptionalType>struct TOptional;

namespace UE::Sequencer
{
extern const FName GSequencerToolbarStyleName;

/** Makes the combo button for changing the key group settings. */
FToolMenuEntry MakeKeyGroupMenuEntry_ToolMenus(const TWeakPtr<FSequencer>& InWeakSequencer);
	
/** Makes the button for toggling auto keying. */
FToolMenuEntry MakeAutoKeyMenuEntry(const TSharedPtr<FSequencer>& InSequencer);
	
/** Makes the combo button for changing the types of edits driving auto keying. */
TOptional<FToolMenuEntry> MakeAllowEditsModeMenuEntry(const TSharedPtr<FSequencer>& InSequencer);

enum class EToolbarItemFlags : uint32
{
	None = 0,
	
	KeyGroup = 1 << 0,
	AutoKey = 1 << 1,
	AllowEditsMode = 1 << 2,
	
	All = KeyGroup | AutoKey | AllowEditsMode
};
ENUM_CLASS_FLAGS(EToolbarItemFlags);
/**
 * Appends the flagged items to InToolbarBuilder.
 * This function is effectively an adapter for converting the UToolMenus API to FToolBarBuilder. It can e.g. be used to inject items to CurveEditor.
 */
void AppendSequencerToolbarEntries(
	const TSharedPtr<FSequencer>& InSequencer, FToolBarBuilder& InToolbarBuilder, const EToolbarItemFlags InFlags = EToolbarItemFlags::All
	);
}
