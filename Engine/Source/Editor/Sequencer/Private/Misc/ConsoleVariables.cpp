// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleVariables.h"

namespace UE::Sequencer
{
	TAutoConsoleVariable<bool> CVarEnableRelevantThumbnails(
		TEXT("Sequencer.EnableRelevantThumbnails"),
		true,
		TEXT("If enabled, upon saving a sequence, tries to find a suitable frame, scrubs to it, renders a thumbnail, and saves it as thumbnail.")
	);
}