// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"

namespace UE::Sequencer
{
	/** Whether to enable the feature that, upon saving a sequence, tries to find a suitable frame, scrubs to it, renders a thumbnail, and saves it as thumbnail. */
	extern TAutoConsoleVariable<bool> CVarEnableRelevantThumbnails;
}
