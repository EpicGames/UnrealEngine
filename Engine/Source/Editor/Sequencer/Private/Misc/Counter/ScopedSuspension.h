// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SuspendCounter.h"
#include "Templates/UnrealTemplate.h"

namespace UE::Sequencer
{
/** Increments FSuspendCounter for the lifetime of the instance.*/
class FScopedSuspension : public FNoncopyable
{
public:
	
	explicit FScopedSuspension(FSuspendCounter& InCounter UE_LIFETIMEBOUND)
		: Counter(&InCounter)
	{
		Counter->Increment();
	}
	
	FScopedSuspension(FScopedSuspension&& InOther)
		: Counter(InOther.Counter)
	{
		InOther.Counter = nullptr;
	}
	
	~FScopedSuspension()
	{
		if (Counter)
		{
			Counter->Decrement();
		}
	}
	
private:
	
	FSuspendCounter* Counter;
};
}

