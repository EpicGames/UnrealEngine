// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/UnrealMathUtility.h"

namespace UE::Sequencer
{
/** 
 * A counter that is used for suspending a certain action temporarily while the counter is positive.
 * E.g. suspending event broadcast, etc.
 */
class FSuspendCounter
{
public:
	
	/** Starts suspending the action. */
	void Increment() { ++Counter; }
	/** Decrements the counter. If the counter reaches 0, the action is no longer suspended. */
	void Decrement() { Counter = FMath::Max(0, Counter - 1); }
	
	/** @return Whether the action is suspended. */
	bool IsSuspended() const { return Counter > 0; }
	
private:
	
	int32 Counter = 0;
};
}

