// Copyright Epic Games, Inc. All Rights Reserved.

#include "KeyHelperUtils.h"
#include "IKeyArea.h"
#include "ISequencer.h"
#include "ISequencerTrackEditor.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTrack.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "ScopedTransaction.h"
#include "Sequencer.h"
#include "SequencerCommonHelpers.h"
#include "SequencerSelectedKey.h"
#include "Tracks/MovieScene3DTransformTrack.h"

#define LOCTEXT_NAMESPACE "KeyHelperUtils"

namespace UE::Sequencer::KeyHelperUtils
{

bool HasKeysAtTime(const TSet<TWeakViewModelPtr<FChannelModel>>& InChannelModels, const FFrameNumber InTime)
{
	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : InChannelModels)
	{
		const TSharedPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
		if (!ChannelModel.IsValid())
		{
			continue;
		}

		const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea();
		if (!KeyArea.IsValid())
		{
			continue;
		}

		TArray<FKeyHandle> KeyHandles;
		KeyArea->GetKeyHandles(KeyHandles);
		if (KeyHandles.IsEmpty())
		{
			continue;
		}

		TArray<FFrameNumber> KeyTimes;
		KeyTimes.SetNumUninitialized(KeyHandles.Num());
		KeyArea->GetKeyTimes(KeyHandles, KeyTimes);

		for (const FFrameNumber KeyTime : KeyTimes)
		{
			if (KeyTime == InTime)
			{
				return true;
			}
		}
	}

	return false;
}

bool DeleteKeysAtTime(ISequencer& InSequencer, const TSet<TWeakViewModelPtr<FChannelModel>>& InChannelModels, const FFrameNumber InTime)
{
	bool bChanged = false;

	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : InChannelModels)
	{
		const TSharedPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
		if (!ChannelModel.IsValid())
		{
			continue;
		}

		const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea();
		if (!KeyArea.IsValid())
		{
			continue;
		}

		TArray<FKeyHandle> KeyHandles;
		KeyArea->GetKeyHandles(KeyHandles);
		if (KeyHandles.IsEmpty())
		{
			continue;
		}

		TArray<FFrameNumber> KeyTimes;
		KeyTimes.SetNumUninitialized(KeyHandles.Num());
		KeyArea->GetKeyTimes(KeyHandles, KeyTimes);

		TArray<FKeyHandle> MatchingKeyHandles;
		for (int32 KeyIndex = 0; KeyIndex < KeyHandles.Num(); ++KeyIndex)
		{
			if (KeyTimes[KeyIndex] == InTime)
			{
				MatchingKeyHandles.Add(KeyHandles[KeyIndex]);
			}
		}

		if (MatchingKeyHandles.IsEmpty())
		{
			continue;
		}

		if (UMovieSceneSection* Section = ChannelModel->GetSection())
		{
			Section->Modify();
		}

		KeyArea->DeleteKeys(MatchingKeyHandles, InTime);
		bChanged = true;
	}

	if (bChanged)
	{
		InSequencer.NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}

	return bChanged;
}

bool HasKeyableChannelModels(const TSet<TWeakViewModelPtr<FChannelModel>>& InChannelModels)
{
	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : InChannelModels)
	{
		const TSharedPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
		if (ChannelModel.IsValid() && ChannelModel->GetKeyArea().IsValid())
		{
			return true;
		}
	}
	return false;
}

bool DoesChannelNameMatchTransformMask(const FName InChannelName, const EMovieSceneTransformChannel InChannel)
{
	if (InChannel == EMovieSceneTransformChannel::All || InChannel == EMovieSceneTransformChannel::AllTransform)
	{
		return true;
	}

	const FString ChannelName = InChannelName.ToString();

	auto HasAnyFlagsAndMatchesAnySuffix = [InChannel, &ChannelName]
		(const EMovieSceneTransformChannel InFlags, const TSet<FString>& InSuffixes)
	{
		if (!EnumHasAnyFlags(InChannel, InFlags))
		{
			return false;
		}

		for (const FString& Suffix : InSuffixes)
		{
			if (ChannelName == Suffix || ChannelName.EndsWith(Suffix))
			{
				return true;
			}
		}

		return false;
	};

	return HasAnyFlagsAndMatchesAnySuffix(EMovieSceneTransformChannel::TranslationX, { TEXT("Location.X"), TEXT("Translation.X") })
		|| HasAnyFlagsAndMatchesAnySuffix(EMovieSceneTransformChannel::TranslationY, { TEXT("Location.Y"), TEXT("Translation.Y") })
		|| HasAnyFlagsAndMatchesAnySuffix(EMovieSceneTransformChannel::TranslationZ, { TEXT("Location.Z"), TEXT("Translation.Z") })
		|| HasAnyFlagsAndMatchesAnySuffix(EMovieSceneTransformChannel::RotationX, { TEXT("Rotation.X"), TEXT("Rotation.Roll") })
		|| HasAnyFlagsAndMatchesAnySuffix(EMovieSceneTransformChannel::RotationY, { TEXT("Rotation.Y"), TEXT("Rotation.Pitch") })
		|| HasAnyFlagsAndMatchesAnySuffix(EMovieSceneTransformChannel::RotationZ, { TEXT("Rotation.Z"), TEXT("Rotation.Yaw") })
		|| HasAnyFlagsAndMatchesAnySuffix(EMovieSceneTransformChannel::ScaleX, { TEXT("Scale.X") })
		|| HasAnyFlagsAndMatchesAnySuffix(EMovieSceneTransformChannel::ScaleY, { TEXT("Scale.Y") })
		|| HasAnyFlagsAndMatchesAnySuffix(EMovieSceneTransformChannel::ScaleZ, { TEXT("Scale.Z") })
		|| HasAnyFlagsAndMatchesAnySuffix(EMovieSceneTransformChannel::Weight, { TEXT("Weight") });
}

TSet<TWeakViewModelPtr<FChannelModel>> GetTransformChannelModels(const TSet<TWeakViewModelPtr<FChannelModel>>& InChannelModels
	, const EMovieSceneTransformChannel InChannel)
{
	TSet<TWeakViewModelPtr<FChannelModel>> OutChannelModels;

	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : InChannelModels)
	{
		const TSharedPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
		if (!ChannelModel.IsValid())
		{
			continue;
		}

		const TViewModelPtr<ITrackExtension> TrackModel = ChannelModel->FindAncestorOfType<ITrackExtension>(/*bIncludeThis=*/true);
		if (!TrackModel.IsValid())
		{
			continue;
		}

		const TSharedPtr<ISequencerTrackEditor> TrackEditor = TrackModel->GetTrackEditor();
		if (TrackEditor.IsValid()
			&& TrackEditor->HasTransformKeyBindings()
			&& ChannelModel->GetKeyArea().IsValid()
			&& DoesChannelNameMatchTransformMask(ChannelModel->GetChannelName(), InChannel))
		{
			OutChannelModels.Add(WeakChannelModel);
		}
	}

	return OutChannelModels;
}

TSet<TWeakViewModelPtr<FChannelModel>> GetKeyableChannelModels(const TSet<TWeakViewModelPtr<IOutlinerExtension>>& InNodes)
{
	TSet<TWeakViewModelPtr<FChannelModel>> OutChannelModels;

	for (const TWeakViewModelPtr<IOutlinerExtension>& WeakNode : InNodes)
	{
		const TViewModelPtr<IOutlinerExtension> Node = WeakNode.Pin();
		if (!Node.IsValid())
		{
			continue;
		}

		TSet<TSharedPtr<FChannelModel>> Channels;
		SequencerHelpers::GetAllChannels(Node.AsModel(), Channels);

		for (const TSharedPtr<FChannelModel>& Channel : Channels)
		{
			if (!Channel.IsValid() || !Channel->GetKeyArea().IsValid())
			{
				continue;
			}

			const TViewModelPtr<IOutlinerExtension> LinkedOutlinerItem = Channel->GetLinkedOutlinerItem();
			if (LinkedOutlinerItem.IsValid() && LinkedOutlinerItem->IsFilteredOut())
			{
				continue;
			}

			OutChannelModels.Add(Channel);
		}
	}

	return OutChannelModels;
}

void CollectKeysRelativeToTime(const TSet<TWeakViewModelPtr<FChannelModel>>& InWeakChannelModels
	, const TRange<FFrameNumber>& InRange, TSet<FSequencerSelectedKey>& OutSelectedKeys)
{
	TArray<FKeyHandle> KeyHandles;

	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : InWeakChannelModels)
	{
		const TSharedPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
		if (!ChannelModel)
		{
			continue;
		}

		const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea();
		if (!KeyArea)
		{
			continue;
		}

		KeyHandles.Reset();
		KeyArea->GetKeyHandles(KeyHandles, InRange);
		
		if (KeyHandles.IsEmpty())
		{
			continue;
		}

		UMovieSceneSection* const Section = KeyArea->GetOwningSection();
		if (!Section)
		{
			continue;
		}

		for (const FKeyHandle KeyHandle : KeyHandles)
		{
			OutSelectedKeys.Add(FSequencerSelectedKey(*Section, ChannelModel, KeyHandle));
		}
	}
}

bool TransformKeySelection(ISequencer& InSequencer
	, const TSet<FSequencerSelectedKey>& InKeys
	, const FFrameTime& InDeltaTime
	, const float InScale
	, const bool bInTransact)
{
	using namespace UE::Sequencer;

	if (InKeys.IsEmpty())
	{
		return false;
	}

	bool bAnythingChanged = false;

	const FFrameTime OriginTime = InSequencer.GetLocalTime().Time;
	const bool bApplyScale = (InScale != 0.f);

	TArray<FSequencerSelectedKey> SelectedKeys;
	SelectedKeys.Reserve(InKeys.Num());
	SelectedKeys.Append(InKeys.Array());
	if (SelectedKeys.IsEmpty())
	{
		return false;
	}

	FSelectedKeysByChannel KeysByChannel(MakeArrayView(SelectedKeys));

	if (KeysByChannel.SelectedChannels.IsEmpty())
	{
		return false;
	}

	TMap<UMovieSceneSection*, TRange<FFrameNumber>> SectionToNewBounds;
	TArray<TPair<FSelectedChannelInfo*, TArray<FFrameNumber>>> DeferredNewTimesByChannel;

	SectionToNewBounds.Reserve(KeysByChannel.SelectedChannels.Num());
	if (bApplyScale)
	{
		DeferredNewTimesByChannel.Reserve(KeysByChannel.SelectedChannels.Num());
	}

	auto UpdateSectionBounds = [&SectionToNewBounds]
		(UMovieSceneSection* InSection, FFrameTime InLowestFrameTime, FFrameTime InHighestFrameTime)
	{
		TRange<FFrameNumber>* NewSectionBounds = SectionToNewBounds.Find(InSection);
		if (!NewSectionBounds)
		{
			InSection->Modify();
			NewSectionBounds = &SectionToNewBounds.Add(InSection, InSection->GetRange());
		}
		*NewSectionBounds = TRange<FFrameNumber>::Hull(
			*NewSectionBounds,
			TRange<FFrameNumber>(InLowestFrameTime.GetFrame(), InHighestFrameTime.GetFrame() + 1)
		);
	};

	auto TransformTimesAndComputeBounds = [OriginTime, InDeltaTime, InScale, bApplyScale]
		(TArray<FFrameNumber>& InOutTimes) -> TPair<FFrameTime, FFrameTime>
	{
		check(!InOutTimes.IsEmpty());

		FFrameTime LowestFrameTime = InOutTimes[0];
		FFrameTime HighestFrameTime = InOutTimes[0];

		for (FFrameNumber& Time : InOutTimes)
		{
			const FFrameTime KeyTime = Time;

			Time = bApplyScale
				? (OriginTime + InDeltaTime + (KeyTime - OriginTime) * InScale).FloorToFrame()
				: (KeyTime + InDeltaTime).FloorToFrame();

			if (Time < LowestFrameTime)
			{
				LowestFrameTime = Time;
			}
			if (Time > HighestFrameTime)
			{
				HighestFrameTime = Time;
			}
		}

		return TPair<FFrameTime, FFrameTime>(LowestFrameTime, HighestFrameTime);
	};

	const FScopedTransaction TransformSelectionTransaction(LOCTEXT("TransformSelection_Transaction", "Transform Selection"), bInTransact);

	for (FSelectedChannelInfo& ChannelInfo : KeysByChannel.SelectedChannels)
	{
		if (!ChannelInfo.OwningSection || ChannelInfo.KeyHandles.IsEmpty())
		{
			continue;
		}

		UMovieSceneSection* const Section = ChannelInfo.OwningSection;
		if (Section->IsLocked() || Section->IsReadOnly())
		{
			continue;
		}

		FMovieSceneChannel* const Channel = ChannelInfo.Channel.Get();
		if (!Channel)
		{
			continue;
		}

		TArray<FFrameNumber> ImmediateNewTimes;
		TArray<FFrameNumber>* NewTimes = &ImmediateNewTimes;
		if (bApplyScale)
		{
			DeferredNewTimesByChannel.Add(TPair<FSelectedChannelInfo*, TArray<FFrameNumber>>(&ChannelInfo, TArray<FFrameNumber>()));
			NewTimes = &DeferredNewTimesByChannel.Last().Value;
		}

		NewTimes->SetNum(ChannelInfo.KeyHandles.Num());
		Channel->GetKeyTimes(ChannelInfo.KeyHandles, *NewTimes);

		const TPair<FFrameTime, FFrameTime> Bounds = TransformTimesAndComputeBounds(*NewTimes);

		UpdateSectionBounds(Section, Bounds.Key, Bounds.Value);
		bAnythingChanged = true;

		if (!bApplyScale)
		{
			Channel->SetKeyTimes(ChannelInfo.KeyHandles, *NewTimes);
		}
	}

	if (bApplyScale)
	{
		for (TPair<FSelectedChannelInfo*, TArray<FFrameNumber>>& Pair : DeferredNewTimesByChannel)
		{
			FSelectedChannelInfo* const ChannelInfo = Pair.Key;
			if (!ChannelInfo || ChannelInfo->KeyHandles.IsEmpty())
			{
				continue;
			}

			FMovieSceneChannel* const Channel = ChannelInfo->Channel.Get();
			if (!Channel)
			{
				continue;
			}

			Channel->SetKeyTimes(ChannelInfo->KeyHandles, Pair.Value);
		}
	}

	for (TPair<UMovieSceneSection*, TRange<FFrameNumber>>& Pair : SectionToNewBounds)
	{
		Pair.Key->SetRange(Pair.Value);
	}

	for (TPair<UMovieSceneSection*, TRange<FFrameNumber>>& Pair : SectionToNewBounds)
	{
		if (UMovieSceneTrack* Track = Pair.Key->GetTypedOuter<UMovieSceneTrack>())
		{
			Track->OnSectionMoved(*Pair.Key, FMovieSceneSectionMovedParams(EPropertyChangeType::ValueSet));
		}
	}

	if (bAnythingChanged)
	{
		InSequencer.NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}

	return bAnythingChanged;
}

bool TransformSelectedOrRelativeKeys(ISequencer& InSequencer
	, const TSet<TWeakViewModelPtr<FChannelModel>>& InWeakChannels
	, const TSet<FSequencerSelectedKey>& InSelectedKeys
	, const FFrameTime InDeltaTime
	, const float InScale
	, const bool bInTransact)
{
	TSet<FSequencerSelectedKey> KeysToTransform = InSelectedKeys;

	if (KeysToTransform.IsEmpty())
	{
		const FFrameNumber ScrubFrame = InSequencer.GetLocalTime().Time.FloorToFrame();
		const TRange<FFrameNumber> Range(
			TRangeBound<FFrameNumber>::Exclusive(ScrubFrame),
			TRangeBound<FFrameNumber>::Open()
		);

		CollectKeysRelativeToTime(InWeakChannels, Range, KeysToTransform);
	}

	if (KeysToTransform.IsEmpty())
	{
		return false;
	}

	return TransformKeySelection(InSequencer, KeysToTransform, InDeltaTime, InScale, bInTransact);
}

bool TransformMarkedFrameSelection(FSequencer& InSequencer
	, const TSet<int32>& InMarkedFrames
	, const FFrameTime InDeltaTime
	, const float InScale)
{
	if (InMarkedFrames.IsEmpty())
	{
		return false;
	}

	UMovieSceneSequence* const FocusedMovieSequence = InSequencer.GetFocusedMovieSceneSequence();
	UMovieScene* const FocusedMovieScene = FocusedMovieSequence ? FocusedMovieSequence->GetMovieScene() : nullptr;
	if (!FocusedMovieScene || FocusedMovieScene->AreMarkedFramesLocked() || FocusedMovieScene->IsReadOnly())
	{
		return false;
	}

	const FFrameTime OriginTime = InSequencer.GetLocalTime().Time;
	const bool bApplyScale = (InScale != 0.f);
	const TArray<FMovieSceneMarkedFrame>& AllMarkedFrames = FocusedMovieScene->GetMarkedFrames();

	FocusedMovieScene->Modify();

	bool bAnythingChanged = false;

	for (const int32 MarkIndex : InMarkedFrames)
	{
		if (!AllMarkedFrames.IsValidIndex(MarkIndex))
		{
			continue;
		}

		const FFrameTime MarkTime = AllMarkedFrames[MarkIndex].FrameNumber;
		const FFrameNumber NewMarkTime = bApplyScale
			? (OriginTime + InDeltaTime + (MarkTime - OriginTime) * InScale).FloorToFrame()
			: (MarkTime + InDeltaTime).FloorToFrame();

		if (AllMarkedFrames[MarkIndex].FrameNumber != NewMarkTime)
		{
			FocusedMovieScene->SetMarkedFrame(MarkIndex, NewMarkTime);
			bAnythingChanged = true;
		}
	}

	return bAnythingChanged;
}

bool TransformSectionSelection(ISequencer& InSequencer
	, const TSet<UMovieSceneSection*>& InSections
	, const FFrameTime InDeltaTime
	, const float InScale
	, const bool bInTransact)
{
	if (InSections.IsEmpty())
	{
		return false;
	}

	TSet<UMovieSceneSection*> SectionsToTransform;
	SectionsToTransform.Reserve(InSections.Num());

	auto AddSection = [&SectionsToTransform](UMovieSceneSection* InSection)
	{
		if (InSection && !InSection->IsLocked() && !InSection->IsReadOnly())
		{
			SectionsToTransform.Add(InSection);
		}
	};

	for (UMovieSceneSection* const Section : InSections)
	{
		AddSection(Section);

		if (const UMovieScene* const MovieScene = Section ? Section->GetTypedOuter<UMovieScene>() : nullptr)
		{
			if (const FMovieSceneSectionGroup* const SectionGroup = MovieScene->GetSectionGroup(*Section))
			{
				for (const TWeakObjectPtr<UMovieSceneSection>& WeakGroupedSection : *SectionGroup)
				{
					AddSection(WeakGroupedSection.Get());
				}
			}
		}
	}

	if (SectionsToTransform.IsEmpty())
	{
		return false;
	}

	const FScopedTransaction TransformSelectionTransaction(LOCTEXT("TransformSelection_Transaction", "Transform Selection"), bInTransact);
	const FFrameTime OriginTime = InSequencer.GetLocalTime().Time;
	const bool bScaleSection = !FMath::IsNearlyEqual(InScale, 1.f);

	auto TransformFrame = [OriginTime, InDeltaTime, InScale](FFrameTime InTime) -> FFrameNumber
	{
		return (OriginTime + InDeltaTime + (InTime - OriginTime) * InScale).FloorToFrame();
	};

	bool bAnythingChanged = false;
	EMovieSceneSectionMovedResult SectionMovedResult = EMovieSceneSectionMovedResult::None;
	TSet<UMovieSceneTrack*> Tracks;

	for (UMovieSceneSection* Section : SectionsToTransform)
	{
		if (!Section)
		{
			continue;
		}

		const TRange<FFrameNumber> InitialRange = Section->GetRange();
		bool bSectionChanged = false;

		if (!bScaleSection)
		{
			const FFrameNumber DeltaFrame = InDeltaTime.FloorToFrame();
			if (DeltaFrame != 0)
			{
				Section->MoveSection(DeltaFrame);
			}

			bSectionChanged = Section->GetRange() != InitialRange;
		}
		else
		{
			TOptional<FFrameNumber> NewStartFrame;
			TOptional<FFrameNumber> NewEndFrame;

			if (InitialRange.GetLowerBound().IsClosed())
			{
				NewStartFrame = TransformFrame(InitialRange.GetLowerBoundValue());
			}
			if (InitialRange.GetUpperBound().IsClosed())
			{
				NewEndFrame = TransformFrame(InitialRange.GetUpperBoundValue());
			}

			TRange<FFrameNumber> NewRange = InitialRange;

			if (NewStartFrame.IsSet())
			{
				NewRange.SetLowerBoundValue(NewStartFrame.GetValue());
			}
			if (NewEndFrame.IsSet())
			{
				NewRange.SetUpperBoundValue(NewEndFrame.GetValue());
			}

			if (NewRange.IsEmpty() || NewRange.IsDegenerate())
			{
				continue;
			}

			Section->Modify();

			const FFrameNumber OldStartFrame = Section->HasStartFrame() ? Section->GetInclusiveStartFrame() : 0;

			FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
			for (const FMovieSceneChannelEntry& Entry : ChannelProxy.GetAllEntries())
			{
				TArrayView<FMovieSceneChannel* const> Channels = Entry.GetChannels();
				TArrayView<const FMovieSceneChannelMetaData> MetaData = Entry.GetMetaData();

				for (int32 ChannelIndex = 0; ChannelIndex < Channels.Num(); ++ChannelIndex)
				{
					FMovieSceneChannel* const Channel = Channels[ChannelIndex];
					if (!Channel)
					{
						continue;
					}

					TArray<FKeyHandle> KeyHandles;
					TArray<FFrameNumber> KeyTimes;
					Channel->GetKeys(TRange<FFrameNumber>::All(), &KeyTimes, &KeyHandles);
					if (KeyHandles.IsEmpty())
					{
						continue;
					}

					const bool bRelativeToSection = Section->HasStartFrame() && MetaData.IsValidIndex(ChannelIndex) && MetaData[ChannelIndex].bRelativeToSection;
					const FFrameNumber NewStartFrameValue = NewStartFrame.Get(OldStartFrame);
					TArray<FFrameNumber> NewKeyTimes = KeyTimes;

					for (FFrameNumber& KeyTime : NewKeyTimes)
					{
						const FFrameNumber AbsoluteKeyTime = bRelativeToSection ? OldStartFrame + KeyTime : KeyTime;
						const FFrameNumber NewAbsoluteKeyTime = TransformFrame(AbsoluteKeyTime);
						KeyTime = bRelativeToSection ? NewAbsoluteKeyTime - NewStartFrameValue : NewAbsoluteKeyTime;
					}

					if (NewKeyTimes != KeyTimes)
					{
						Channel->SetKeyTimes(KeyHandles, NewKeyTimes);
						bSectionChanged = true;
					}
				}
			}

			if (NewRange != InitialRange)
			{
				Section->SetRange(NewRange);
				bSectionChanged = true;
			}
		}

		bAnythingChanged |= bSectionChanged;

		if (bSectionChanged)
		{
			if (UMovieSceneTrack* const Track = Section->GetTypedOuter<UMovieSceneTrack>())
			{
				SectionMovedResult |= Track->OnSectionMoved(*Section, FMovieSceneSectionMovedParams(EPropertyChangeType::ValueSet));
				Tracks.Add(Track);
			}
		}
	}

	for (UMovieSceneTrack* const Track : Tracks)
	{
		Track->UpdateEasing();
	}

	if (bAnythingChanged)
	{
		const EMovieSceneDataChangeType DataChangeType = EnumHasAnyFlags(SectionMovedResult, EMovieSceneSectionMovedResult::SectionsChanged)
			? EMovieSceneDataChangeType::MovieSceneStructureItemsChanged
			: EMovieSceneDataChangeType::TrackValueChanged;
		InSequencer.NotifyMovieSceneDataChanged(DataChangeType);
	}

	return bAnythingChanged;
}

} // namespace UE::Sequencer::KeyHelperUtils

#undef LOCTEXT_NAMESPACE
