// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Range.h"
#include "MVVM/ViewModelPtr.h"
#include "ToolableTimeline/Caches/ToolableTimelineChannelCache.h"

class FSequencer;
class ISequencer;
class UMovieSceneSection;
enum class EMovieSceneTransformChannel : uint32;
struct FFrameNumber;
struct FFrameTime;
struct FSequencerSelectedKey;

namespace UE::Sequencer
{
	class FChannelModel;
	class IOutlinerExtension;
}

namespace UE::Sequencer::KeyHelperUtils
{

/**  */
bool HasKeysAtTime(const TSet<TWeakViewModelPtr<FChannelModel>>& InChannelModels, FFrameNumber InTime);

/**  */
bool DeleteKeysAtTime(ISequencer& InSequencer, const TSet<TWeakViewModelPtr<FChannelModel>>& InChannelModels, FFrameNumber InTime);

/**
 * Determines if there are any keyable channel models in the provided set of channel models.
 *
 * @param InChannelModels The set of weak pointers to channel models to evaluate for keyability.
 *
 * @return True if at least one channel model in the set is keyable, otherwise false.
 */
bool HasKeyableChannelModels(const TSet<TWeakViewModelPtr<FChannelModel>>& InChannelModels);

/**
 * Filters the provided set of channel models to those that match the specified transform channel mask.
 */
TSet<TWeakViewModelPtr<FChannelModel>> GetTransformChannelModels(const TSet<TWeakViewModelPtr<FChannelModel>>& InChannelModels
	, EMovieSceneTransformChannel InChannel);

/**
 * Gathers all keyable channel models underneath the specified outliner nodes.
 */
TSet<TWeakViewModelPtr<FChannelModel>> GetKeyableChannelModels(const TSet<TWeakViewModelPtr<IOutlinerExtension>>& InNodes);

/**
 * Collects sequencer keys within the specified time range from the given channel models
 * and adds them to the output set of selected keys.
 *
 * @param InWeakChannelModels The set of weak pointers to channel models from which keys will be collected.
 * @param InRange The range of time used to filter the keys for collection.
 * @param OutSelectedKeys The set where the collected keys will be stored.
 */
void CollectKeysRelativeToTime(const TSet<TWeakViewModelPtr<FChannelModel>>& InWeakChannelModels
	, const TRange<FFrameNumber>& InRange, TSet<FSequencerSelectedKey>& OutSelectedKeys);

/**
 * Transforms the selected keys by applying a time delta and an optional scaling factor.
 * Updates the affected sections' bounds and notifies the sequencer of changes.
 *
 * @param InSequencer The sequencer instance used to transform the keys.
 * @param InKeys The set of selected keys to transform.
 * @param InDeltaTime The time delta used for translating the keys.
 * @param InScale The scaling factor applied to the keys.
 * @param bInTransact Whether to create a transaction for the transform.
 *
 * @return True if any keys were changed, false otherwise.
 */
bool TransformKeySelection(ISequencer& InSequencer
	, const TSet<FSequencerSelectedKey>& InKeys
	, const FFrameTime& InDeltaTime
	, const float InScale
	, const bool bInTransact = true);

/**
 * Transforms either the selected keys or a relative set of keys based on a time delta and scaling factor.
 * If no selected keys are provided, keys relative to the specified time range will be collected and transformed.
 *
 * @param InSequencer The sequencer instance used to transform the keys.
 * @param InWeakChannels A set of weak pointers to channel models that define the scope for collecting keys.
 * @param InSelectedKeys A set of keys explicitly selected for transformation.
 * @param InDeltaTime The time delta used for translating the keys.
 * @param InScale The scaling factor applied to the keys.
 * @param bInTransact Whether to create a transaction for the transform.
 *
 * @return True if any keys were changed, false otherwise.
 */
bool TransformSelectedOrRelativeKeys(ISequencer& InSequencer
	, const TSet<TWeakViewModelPtr<FChannelModel>>& InWeakChannels
	, const TSet<FSequencerSelectedKey>& InSelectedKeys
	, const FFrameTime InDeltaTime
	, const float InScale
	, const bool bInTransact = true);

/**
 * Transforms the selected marked frames by applying a time delta and an optional scaling factor.
 *
 * @param InSequencer The sequencer instance used to transform the marked frames.
 * @param InMarkedFrames The selected marked frame indices to transform.
 * @param InDeltaTime The time delta used for translating the marked frames.
 * @param InScale The scaling factor applied to the marked frames.
 *
 * @return True if any marked frames were changed, false otherwise.
 */
bool TransformMarkedFrameSelection(FSequencer& InSequencer
	, const TSet<int32>& InMarkedFrames
	, const FFrameTime InDeltaTime
	, const float InScale);

/**
 * Transforms the selected sections by applying a time delta and an optional scaling factor.
 *
 * @param InSequencer The sequencer instance used to transform the sections.
 * @param InSections The selected sections to transform.
 * @param InDeltaTime The time delta used for translating the sections.
 * @param InScale The scaling factor applied to the sections.
 * @param bInTransact Whether to create a transaction for the transform.
 *
 * @return True if any sections were changed, false otherwise.
 */
bool TransformSectionSelection(ISequencer& InSequencer
	, const TSet<UMovieSceneSection*>& InSections
	, const FFrameTime InDeltaTime
	, const float InScale
	, const bool bInTransact = true);

} // namespace UE::Sequencer::KeyHelperUtils
