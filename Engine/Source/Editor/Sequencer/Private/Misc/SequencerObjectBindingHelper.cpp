// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/SequencerObjectBindingHelper.h"

#include "KeyPropertyParams.h"
#include "ISequencer.h"
#include "StructUtils/PropertyBag.h"

void FSequencerObjectBindingHelper::GetKeyablePropertyPaths(const UObject* Object, TSharedRef<ISequencer> Sequencer, TArray<FPropertyPath>& KeyablePropertyPaths)
{
	FPropertyPath PropertyPath;
	GetKeyablePropertyPaths(Object->GetClass(), Object, Object->GetClass(), PropertyPath, Sequencer, KeyablePropertyPaths);
}

void FSequencerObjectBindingHelper::GetKeyablePropertyPaths(const UClass* Class, const void* ValuePtr, const UStruct* PropertySource, FPropertyPath PropertyPath, TSharedRef<ISequencer> Sequencer, TArray<FPropertyPath>& KeyablePropertyPaths)
{
	for (TFieldIterator<FProperty> PropertyIterator(PropertySource); PropertyIterator; ++PropertyIterator)
	{
		FProperty* Property = *PropertyIterator;

		if (Property && !Property->HasAnyPropertyFlags(CPF_Deprecated) && !Property->GetBoolMetaData(TEXT("SequencerHideProperty")))
		{
			PropertyPath.AddProperty(FPropertyInfo(Property));

			if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				// If this is an array property, add property paths for each item in the array. If we don't know how to key that item,
				// like for instance it's a custom struct for which we don't have a track editor, then we recurse into it and add
				// property paths for its own properties.
				FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(ValuePtr));
				for (int32 Index = 0; Index < ArrayHelper.Num(); ++Index)
				{
					PropertyPath.AddProperty(FPropertyInfo(ArrayProperty->Inner, Index));

					if (Sequencer->CanKeyProperty(FCanKeyPropertyParams(Class, PropertyPath)))
					{
						KeyablePropertyPaths.Add(PropertyPath);
					}
					else if (FStructProperty* StructProperty = CastField<FStructProperty>(ArrayProperty->Inner))
					{
						GetKeyablePropertyPaths(Class, ArrayHelper.GetRawPtr(Index), StructProperty->Struct, PropertyPath, Sequencer, KeyablePropertyPaths);
					}

					PropertyPath = *PropertyPath.TrimPath(1);
				}
			}
			else if (Sequencer->CanKeyProperty(FCanKeyPropertyParams(Class, PropertyPath)))
			{
				// This is a property that we can key directly. That is: we have a track editor specifically for that property type,
				// such as FVector or FLinearColor or other well known structs. This also includes custom/system-specific structs
				// like FMargin, for which UMG registers a custom track.
				KeyablePropertyPaths.Add(PropertyPath);
			}
			else if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				if (StructProperty->Struct == FInstancedPropertyBag::StaticStruct() && StructProperty->GetBoolMetaData(TEXT("InterpBagProperties")))
				{
					// It's a property bag. Show the properties inside if we have been explicitly allowed to do so. Note that we are
					// using a custom metadata ("InterpBagProperties") because the built-in "Interp" tag only works on BlueprintTypes.
					const FInstancedPropertyBag* PropertyBag = StructProperty->ContainerPtrToValuePtr<FInstancedPropertyBag>(ValuePtr);
					FStructProperty* PropertyBagValueProperty = CastFieldChecked<FStructProperty>(StructProperty->Struct->FindPropertyByName(TEXT("Value")));
					PropertyPath.AddProperty(FPropertyInfo(PropertyBagValueProperty));
					const UPropertyBag* PropertyBagStruct = PropertyBag->GetPropertyBagStruct();
					GetKeyablePropertyPaths(Class, PropertyBag, const_cast<UPropertyBag*>(PropertyBagStruct), PropertyPath, Sequencer, KeyablePropertyPaths);
				}
				else if (StructProperty->Struct == FInstancedStruct::StaticStruct() && StructProperty->GetBoolMetaData(TEXT("InterpStructProperties")))
				{
					// As above, but for an instanced struct.
					const FInstancedStruct* InstancedStruct = StructProperty->ContainerPtrToValuePtr<FInstancedStruct>(ValuePtr);
					const UScriptStruct* InstancedStructType = InstancedStruct->GetScriptStruct();
					GetKeyablePropertyPaths(Class, InstancedStruct, const_cast<UScriptStruct*>(InstancedStructType), PropertyPath, Sequencer, KeyablePropertyPaths);
				}
				else
				{
					// It's a struct property that we don't know how to key directly, so add property paths for its own properties.
					// The user will have to key them individually.
					GetKeyablePropertyPaths(Class, StructProperty->ContainerPtrToValuePtr<void>(ValuePtr), StructProperty->Struct, PropertyPath, Sequencer, KeyablePropertyPaths);
				}
			}

			PropertyPath = *PropertyPath.TrimPath(1);
		}
	}
}
