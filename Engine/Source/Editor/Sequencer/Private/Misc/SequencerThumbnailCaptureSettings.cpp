// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/SequencerThumbnailCaptureSettings.h"

#include "ISequencer.h"
#include "MovieScene.h"
#include "Math/Range.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequencerThumbnailCaptureSettings)

namespace UE::Sequencer
{
	FFrameNumber GetFrameByRule(const ISequencer& Sequencer, ESequencerThumbnailCaptureTimeLocation Rule)
	{
		switch (Rule)
		{
		case ESequencerThumbnailCaptureTimeLocation::FirstFrame:
			return Sequencer.GetRootMovieSceneSequence()->GetMovieScene()->GetPlaybackRange().GetLowerBoundValue();
		case ESequencerThumbnailCaptureTimeLocation::MiddleFrame:
			{
				const FFrameNumber FirstFrame = GetFrameByRule(Sequencer, ESequencerThumbnailCaptureTimeLocation::FirstFrame);
				const FFrameNumber LastFrame = GetFrameByRule(Sequencer, ESequencerThumbnailCaptureTimeLocation::LastFrame);
				return FirstFrame + (LastFrame - FirstFrame) / 2;
			}
		case ESequencerThumbnailCaptureTimeLocation::LastFrame:
			return Sequencer.GetRootMovieSceneSequence()->GetMovieScene()->GetPlaybackRange().GetUpperBoundValue();
			
		case ESequencerThumbnailCaptureTimeLocation::CurrentFrame:
			return Sequencer.GetGlobalTime().Time.FrameNumber;
			
		default:
			checkNoEntry();
			return GetFrameByRule(Sequencer, ESequencerThumbnailCaptureTimeLocation::CurrentFrame);
		}
	}
}
