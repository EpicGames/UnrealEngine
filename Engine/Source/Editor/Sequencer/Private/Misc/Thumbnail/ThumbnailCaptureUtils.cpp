// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailCaptureUtils.h"

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Editor.h"
#include "Misc/ConsoleVariables.h"
#include "Misc/ScopeExit.h"
#include "ObjectTools.h"
#include "Templates/Function.h"
#include "TextureResource.h"

namespace UE::Sequencer
{
	void CaptureThumbnailForAssetBlocking(UObject& Asset, FSequencer& Sequencer, const FSequencerThumbnailCaptureSettings& Settings)
	{
		if (!CVarEnableRelevantThumbnails.GetValueOnGameThread()
			|| !CaptureThumbnailFromCameraCutBlocking(Asset, Sequencer, Settings))
		{
			CaptureThumbnailFromViewportBlocking(Asset);
		}
	}
	
	void SetAssetThumbnail(UObject& Asset, const TArrayView64<FColor>& Bitmap)
	{
		const int32 ThumbnailSize  = ThumbnailTools::DefaultThumbnailSize;
		const int32 NumPixels = ThumbnailSize * ThumbnailSize;
		check(Bitmap.Num() == NumPixels);
		
		FObjectThumbnail TempThumbnail;
		TempThumbnail.SetImageSize(ThumbnailSize, ThumbnailSize);
		TArray<uint8>& ThumbnailByteArray = TempThumbnail.AccessImageData();
		constexpr int32 MemorySize = NumPixels * sizeof(FColor);
		ThumbnailByteArray.AddUninitialized(MemorySize);
		FMemory::Memcpy(&ThumbnailByteArray[0], &Bitmap[0], MemorySize);
		
		const FString ObjectFullName = FAssetData(&Asset).GetFullName();
		UPackage* Package = Asset.GetPackage();
		FObjectThumbnail* NewThumbnail = ThumbnailTools::CacheThumbnail(ObjectFullName, &TempThumbnail, Package);
		if (ensure(NewThumbnail))
		{
			// We need to indicate that the package needs to be resaved
			Package->MarkPackageDirty();
			// Let the content browser know that we've changed the thumbnail
			NewThumbnail->MarkAsDirty();
			// Signal that the asset was changed so thumbnail pools will update
			Asset.PostEditChange();
			// Set that thumbnail as a valid custom thumbnail so it'll be saved out
			NewThumbnail->SetCreatedAfterCustomThumbsEnabled();
		}
	}
}