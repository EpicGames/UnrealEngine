// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieScene.h"
#include "Templates/FunctionFwd.h"

class UCameraComponent;
class FSequencer;
class UMovieScene;
class UObject;
struct FSequencerThumbnailCaptureSettings;

namespace UE::Sequencer
{
	using FCaptureFrame = TFunctionRef<void(const FFrameNumber& Frame, UCameraComponent& Camera)>;
	
	/**
	 * Tries to capture a relevant thumbnail and saves it into Asset.
	 * This function may block the game thread.
	 *
	 * The order of evaluation is as follows:
	 * 1. If sequence has a camera cut, determine a suitable frame using Settings, and capture.
	 * 2. If sequence has a camera track, determine a suitable frame using Settings, and capture
	 * 3. Try steps 1-2 recursively on sub-sequences.
	 * 4. Fall back to capturing the viewport content.
	 *
	 * Post-call: The sequencer's focus state is the same as before the call.
	 * 
	 * @param Asset The asset into which the thumbnail is supposed to be saved.
	 * @param RootMovieScene The root movie scene of Asset.
	 * @param Sequencer Used to animate all objects for capturing the thumbnail
	 * @param Settings The settings to use for capturing the thumbnail
	 */
	void CaptureThumbnailForAssetBlocking(
		UObject& Asset,
		FSequencer& Sequencer,
		const FSequencerThumbnailCaptureSettings& Settings
		);
	
	/**
	 * Attempts to locate a camera from any camera cut track within the specified movie scene and invokes 
	 * CaptureFrameCallback with the identified camera and the corresponding frame based on the provided Settings.
	 *
	 * @param Asset The asset into which the thumbnail is supposed to be saved.
	 * @param Sequencer The sequencer used to animate objects during the thumbnail capture process.
	 * @param Settings The configuration settings to be used for capturing the thumbnail.
	 *
	 * Pre-call: The sequencer's focus state is focused on MovieScene.
	 * Post-call: The sequencer's focus state is focused on MovieScene.
	 * 
	 * @return True if CaptureFrameCallback was invoked; otherwise, false.
	 */
	bool CaptureThumbnailFromCameraCutBlocking(
		UObject& Asset,
		FSequencer& Sequencer,
		const FSequencerThumbnailCaptureSettings& Settings
		);
	
	/** Captures the first active viewport as thumbnail for Asset. */
	void CaptureThumbnailFromViewportBlocking(UObject& Asset);

	/** Saves Bitmap as thumbnail of Asset. */
	void SetAssetThumbnail(UObject& Asset, const TArrayView64<FColor>& Bitmap);
}
