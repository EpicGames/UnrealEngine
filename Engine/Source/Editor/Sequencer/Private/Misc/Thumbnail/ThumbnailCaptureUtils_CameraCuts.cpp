// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailCaptureUtils.h"

#include "AssetRegistry/AssetData.h"
#include "Camera/CameraComponent.h"
#include "Containers/Array.h"
#include "Editor.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "Misc/ScopeExit.h"
#include "MovieSceneToolsUserSettings.h"
#include "ObjectTools.h"
#include "Sequencer.h"
#include "Templates/Function.h"
#include "TextureResource.h"
#include "TrackEditorThumbnail/TrackThumbnailUtils.h"

namespace UE::Sequencer::Private
{
	static UTextureRenderTarget2D* CaptureThumbnail(
		FSequencer& Sequencer,
		const FFrameNumber& Frame
		)
	{
		constexpr int32 ThumbnailSize  = ThumbnailTools::DefaultThumbnailSize;

		// Before calling PreDrawThumbnailSetupSequencer, save where the user had scrubbed, so we can revert it later.
		// This step differs from FTrackEditorThumbnailPool::DrawThumbnails, which does not need to do this... may want to investigate. 
		const FQualifiedFrameTime RestoreTime = Sequencer.GetGlobalTime();
		// Positions all animated objects by jumping to the right frame...
		MoveSceneTools::PreDrawThumbnailSetupSequencer(Sequencer, Frame);
		ON_SCOPE_EXIT
		{
			// It's important to reset the time after the GPU has captured the texture, or we'll simply be capturing the current frame!
			if (RestoreTime.Time != Sequencer.GetGlobalTime().Time)
			{
				Sequencer.SetGlobalTime(RestoreTime.Time);
			}
			
			MoveSceneTools::PostDrawThumbnailCleanupSequencer(Sequencer);
		};
		
		UCameraComponent* Component = Sequencer.GetLastEvaluatedCameraCut().Get();
		if (!Component)
		{
			return nullptr;
		}
		
		// Important to get camera view after PreDrawThumbnailSetupSequencer since it may have set its transform!
		FMinimalViewInfo ViewInfo;
		Component->GetCameraView(FApp::GetDeltaTime(), ViewInfo);
		// We're rendering square, so lets make sure we set the aspect ratio here.
		ViewInfo.AspectRatio = 1.0f;
		// It's important to capture with sRGB so the thumbnail has the right brightness
		UTextureRenderTarget2D* RenderTarget2D = UKismetRenderingLibrary::CreateRenderTarget2D(
			Component, ThumbnailSize, ThumbnailSize, RTF_RGBA8_SRGB
			);
		FTextureRenderTargetResource* RenderTarget = RenderTarget2D->GameThread_GetRenderTargetResource();
		
		// ... enqueues rendering commands to fill RenderTarget
		MoveSceneTools::DrawViewportThumbnail(
			*RenderTarget, { ThumbnailSize, ThumbnailSize }, *GWorld->Scene, ViewInfo, EThumbnailQuality::Best, &Component->PostProcessSettings
			);
		// ... cleans up any important state modified by PreDrawThumbnailSetupSequencer.

		// Need to wait on the GPU to execute the commands above
		FRenderCommandFence Fence;
		Fence.BeginFence();
		Fence.Wait();

		return RenderTarget2D;
	}
}

namespace UE::Sequencer
{
	bool CaptureThumbnailFromCameraCutBlocking(
		UObject& Asset,
		FSequencer& Sequencer,
		const FSequencerThumbnailCaptureSettings& Settings
		)
	{
		const FFrameNumber Frame = GetFrameByRule(Sequencer, Settings.CaptureFrameLocationRule);
		if (UTextureRenderTarget2D* RenderTarget2D = Private::CaptureThumbnail(Sequencer, Frame))
		{
			ON_SCOPE_EXIT{ RenderTarget2D->ReleaseResource(); };
			FTextureRenderTargetResource* RenderTarget = RenderTarget2D->GameThread_GetRenderTargetResource();
		
			TArray<FColor> Color;
			RenderTarget->ReadPixels(Color);
			SetAssetThumbnail(Asset, Color);
			return true;
		}
		
		return false;
	}
}