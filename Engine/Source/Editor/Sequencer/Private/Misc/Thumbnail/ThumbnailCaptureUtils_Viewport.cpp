// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThumbnailCaptureUtils.h"

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "IContentBrowserSingleton.h"
#include "LevelEditorViewport.h"
#include "UnrealClient.h"

namespace UE::Sequencer::PrivateThumbnailCapture
{
	/** @return The active viewport if valid and otherwise any viewport that allows cinematic preview. */
	static FViewport* FindActiveViewportThenAnyWithCinematicPreview()
	{
		FViewport* Viewport = GEditor->GetActiveViewport();
		if (Viewport)
		{
			return Viewport;
		}

		// If there's no active viewport, find any other viewport that allows cinematic preview.
		for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
		{
			if (LevelVC && LevelVC->AllowsCinematicControl() && LevelVC->Viewport)
			{
				return LevelVC->Viewport;
			}
		}
		
		return nullptr;
	}
}

namespace UE::Sequencer
{
	void CaptureThumbnailFromViewportBlocking(UObject& Asset)
	{
		if (!GCurrentLevelEditingViewportClient)
		{
			return;
		}
		FViewport* Viewport = PrivateThumbnailCapture::FindActiveViewportThenAnyWithCinematicPreview();
		if (!Viewport)
		{
			return;
		}
		
		const bool bIsInGameView = GCurrentLevelEditingViewportClient->IsInGameView();
		FLevelEditorViewportClient* const OldViewportClient = GCurrentLevelEditingViewportClient;
		// Remove editor widgets from the render
		GCurrentLevelEditingViewportClient->SetGameView(true);
		// Remove selection box around client during render
		GCurrentLevelEditingViewportClient = nullptr;
		ON_SCOPE_EXIT
		{
			// Redraw viewport to have the yellow highlight again
			GCurrentLevelEditingViewportClient = OldViewportClient;
			GCurrentLevelEditingViewportClient->SetGameView(bIsInGameView);
				
			// If turn off game view now need to make sure widget/gizmo is on
			if (!bIsInGameView )
			{
				GCurrentLevelEditingViewportClient->ShowWidget(true);
			}
				
			Viewport->Draw();
		};
			
		// Have to re-render the requested viewport before capturing a thumbnail
		Viewport->Draw();
		
		IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
		TArray AssetDataList { FAssetData(&Asset) };
		ContentBrowser.CaptureThumbnailFromViewport(Viewport, AssetDataList);
	}
}