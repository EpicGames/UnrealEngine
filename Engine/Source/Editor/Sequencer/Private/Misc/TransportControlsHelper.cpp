// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransportControlsHelper.h"
#include "Framework/Docking/TabManager.h"
#include "ISequencer.h"
#include "LevelSequence.h"
#include "SequencerSettings.h"
#include "SPopoutTabInlineContent.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "TransportControlsHelper"

namespace UE::Sequencer
{

TSharedRef<SPopoutTabInlineContent> MakePopoutTransportControls(const TSharedRef<ISequencer>& InSequencer
	, const TSharedPtr<FTabManager>& InTabManager, const TSharedPtr<SWidget>& InOverriddenContent)
{
	const USequencerSettings* const SequencerSettings = InSequencer->GetSequencerSettings();
	check(SequencerSettings);

	const FName TabId = FName(*FString::Printf(TEXT("Sequencer.TransportControls.%s"), *SequencerSettings->GetName()));
	const TWeakPtr<ISequencer> WeakSequencer = InSequencer;
	const TSharedPtr<SWidget> Content = InOverriddenContent.IsValid()
		? InOverriddenContent : InSequencer->MakeTransportControls(true);

	return SNew(SPopoutTabInlineContent)
		.TabManager(InTabManager)
		.TabRole(ETabRole::PanelTab)
		.TabId(TabId)
		.TabDisplayName(LOCTEXT("TransportControlsTabName", "Transport"))
		.TabIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.Transport.Looping")))
		.TabGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCinematicsCategory())
		.AutosizeWhenFloating(true)
		.HideDockTabStackTab(true)
		.ContentHAlign(HAlign_Fill)
		.InitiallyPoppedOut(SequencerSettings->ShouldRestoreTransportControlsPoppedOut())
		.OnPopoutStateChanged_Lambda([WeakSequencer](const bool bInPoppedOut)
			{
				const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
				USequencerSettings* const SequencerSettings = Sequencer.IsValid()
					? Sequencer->GetSequencerSettings() : nullptr;
				if (!SequencerSettings)
				{
					return;
				}

				// Live current state
				SequencerSettings->SetTransportControlsPoppedOut(bInPoppedOut);

				// Only update the restore preference when transport controls are actually being shown.
				// If they are being hidden, the popout is closed as a side effect and should not erase
				// the user's preferred popped-out mode.
				if (SequencerSettings->ShouldShowTransportControls())
				{
					SequencerSettings->SetRestoreTransportControlsPoppedOut(bInPoppedOut);
				}
			})
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				Content.ToSharedRef()
			]
		];
}

bool IsPopoutTransportControlsVisible(const TSharedRef<ISequencer>& InSequencer
	, const bool bWantsToShowTransportControls)
{
	// Only support level sequences for now. Always show transport controls otherwise.
	const UMovieSceneSequence* const RootSequence = InSequencer->GetRootMovieSceneSequence();
	if (!RootSequence || !RootSequence->IsA<ULevelSequence>())
	{
		return true;
	}

	// Sequencer transport popped out state overrides curve editor transport controls visibility
	const USequencerSettings* const SequencerSettings = InSequencer->GetSequencerSettings();
	if (!SequencerSettings
		|| !SequencerSettings->ShouldShowTransportControls()
		|| SequencerSettings->ShouldRestoreTransportControlsPoppedOut())
	{
		return false;
	}

	return bWantsToShowTransportControls;
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
