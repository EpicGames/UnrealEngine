// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FTabManager;
class ISequencer;
class SPopoutTabInlineContent;
class SWidget;
struct EVisibility;

namespace UE::Sequencer
{

/**
 * Create a transport controls panel that can be popped out.
 * This ensures the Sequencer and Curve Editor use the same example popout widget settings.
 */
TSharedRef<SPopoutTabInlineContent> MakePopoutTransportControls(const TSharedRef<ISequencer>& InSequencer
	, const TSharedPtr<FTabManager>& InTabManager, const TSharedPtr<SWidget>& InOverriddenContent = nullptr);

/**
 * Determines if a transport control that can be popped out is visible in its inline content, or hidden 
 * due to being popped out or bWantsToShowTransportControls is false. This function is set up like this to
 * enable its use with other systems that want a transport control that can be popped out with its own
 * visibility for the transport controls. The popped out state overrides visibility everywhere, but when not
 * popped out, visibility should be taken from the system that defines the inline content.
 */
bool IsPopoutTransportControlsVisible(const TSharedRef<ISequencer>& InSequencer
	, const bool bWantsToShowTransportControls);

} // namespace UE::Sequencer
