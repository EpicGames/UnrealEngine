// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneSpawnable.h"
#include "MovieSceneBinding.h"
#include "MovieScenePossessable.h"
#include "MovieSceneCopyableBinding.generated.h"

class UMovieSceneTrack;
class UMovieSceneCustomBinding;

UCLASS(Transient)
class UMovieSceneCopyableBinding : public UObject
{
public:
	GENERATED_BODY()
	
	/** 
	* Spawnables need to know about their Object Template but we cannot rely on automatic serialization due to the object
	* template belonging to the Movie Scene (it gets serialized as a reference). Instead we manually serialize the object
	* so that we can duplicate it into a new object (which is stored in this variable).
	*/
	UPROPERTY()
	TArray<TObjectPtr<UObject>> SpawnableObjectTemplates;

	UPROPERTY()
	int32 NumSpawnableObjectTemplates;

	/**
	 * Tracks are also owned by the owning Movie Sequence. We manually copy the tracks out of a binding when we copy,
	 * because the binding stores them as a reference to a privately owned object. We store these copied tracks here,
	 * and then restore them upon paste to re-create the tracks with the correct owner.
	 */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneTrack>> Tracks;

	UPROPERTY()
	FMovieSceneBinding Binding;

	UPROPERTY()
	FMovieSceneSpawnable Spawnable;

	UPROPERTY()
	FMovieScenePossessable Possessable;

	UPROPERTY()
	TArray<FString> BoundObjectNames;

	UPROPERTY()
	TArray<FName> FolderPath;

	UPROPERTY()
	TArray<FName> Tags;

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneCustomBinding>> CustomBindings;	
	
	UPROPERTY()
	int32 NumCustomBindings;

	UPROPERTY()
	TArray<int32> PreviewSpawnableBindings;
};
