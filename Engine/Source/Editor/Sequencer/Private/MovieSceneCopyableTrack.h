// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneCopyableTrack.generated.h"

class UMovieSceneTrack;

UCLASS(Transient)
class UMovieSceneCopyableTrack : public UObject
{
public:
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UMovieSceneTrack> Track;

	UPROPERTY()
	bool bIsRootTrack;

	UPROPERTY()
	bool bIsCameraCutTrack;

	UPROPERTY()
	TArray<FName> FolderPath;
};
