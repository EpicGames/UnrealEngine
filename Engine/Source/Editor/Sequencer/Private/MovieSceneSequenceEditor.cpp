// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSequenceEditor.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneSequence.h"
#include "ISequencerModule.h"
#include "BlueprintActionDatabase.h"

FMovieSceneSequenceEditor* FMovieSceneSequenceEditor::Find(UMovieSceneSequence* InSequence)
{
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	return InSequence ? SequencerModule.FindSequenceEditor(InSequence->GetClass()) : nullptr;
}

bool FMovieSceneSequenceEditor::SupportsEvents(UMovieSceneSequence* InSequence) const
{
	return CanCreateEvents(InSequence);
}

bool FMovieSceneSequenceEditor::SupportsDirectorBlueprint(UMovieSceneSequence* InSequence) const
{
	return CanCreateDirectorBlueprint(InSequence);
}

UBlueprint* FMovieSceneSequenceEditor::FindDirectorBlueprint(UMovieSceneSequence* Sequence) const
{
	return GetBlueprintForSequence(Sequence);
}

UBlueprint* FMovieSceneSequenceEditor::GetOrCreateDirectorBlueprint(UMovieSceneSequence* Sequence) const
{
	if (!SupportsDirectorBlueprint(Sequence))
	{
		return nullptr;
	}

	UBlueprint* Blueprint = GetBlueprintForSequence(Sequence);
	if (!Blueprint)
	{
		Blueprint = CreateBlueprintForSequence(Sequence);

		// This asset now has a blueprint where before it did not: refresh asset actions for this asset
		UObject* AssetObject = Sequence;
		while (AssetObject)
		{
			if (AssetObject->IsAsset())
			{
				break;
			}
			AssetObject = AssetObject->GetOuter();
		}

		if (AssetObject)
		{
			FBlueprintActionDatabase::Get().RefreshAssetActions(AssetObject);
		}
	}
	return Blueprint;
}
