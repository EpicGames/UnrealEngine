// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTrackEditor.h"
#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "ISequencer.h"
#include "Framework/Commands/UICommandList.h"
#include "ScopedTransaction.h"
#include "MovieSceneTrack.h"
#include "ISequencerTrackEditor.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "Sequencer.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SequencerUtilities.h"
#include "SequencerCommonHelpers.h"
#include "MVVM/Views/ViewUtilities.h"
#include "Framework/Application/SlateApplication.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "MVVM/Views/SSequencerKeyNavigationButtons.h"
#include "MVVM/Views/SOutlinerTrackColorPicker.h"
#include "MVVM/Views/SOutlinerItemViewBase.h"
#include "MVVM/Extensions/ITrackRowExtension.h"
#include "SKeyAreaEditorSwitcher.h"

namespace UE::Sequencer
{
	bool HasKeyableAreas(const FViewModel& ViewModel)
	{
		for (TSharedPtr<FChannelGroupModel> ChannelGroup : ViewModel.GetDescendantsOfType<FChannelGroupModel>())
		{
			for (const TWeakViewModelPtr<FChannelModel>& WeakChannel : ChannelGroup->GetChannels())
			{
				if (TViewModelPtr<FChannelModel> Channel = WeakChannel.Pin())
				{
					if (Channel->GetKeyArea())
					{
						return true;
					}
				}
			}
		}

		return false;
	}
}

FMovieSceneTrackEditor::FMovieSceneTrackEditor(TSharedRef<ISequencer> InSequencer)
	: WeakSequencerPtr(InSequencer)
{ 
}

FMovieSceneTrackEditor::~FMovieSceneTrackEditor() 
{ 
}

UMovieSceneSequence* FMovieSceneTrackEditor::GetMovieSceneSequence() const
{
	return WeakSequencerPtr.Pin()->GetFocusedMovieSceneSequence();
}

TOptional<FFrameNumber> FMovieSceneTrackEditor::NextKeyTime;
bool FMovieSceneTrackEditor::bKeying;

FFrameNumber FMovieSceneTrackEditor::GetTimeForKey()
{ 
	if (bKeying && NextKeyTime.IsSet())
	{
		return NextKeyTime.GetValue();
	}

	// Otherwise, key at the current time
	TSharedPtr<ISequencer> SequencerPin = WeakSequencerPtr.Pin();
	return SequencerPin.IsValid() ? SequencerPin->GetLocalTime().Time.FrameNumber : FFrameNumber(0);
}

void FMovieSceneTrackEditor::BeginKeying(FFrameNumber InFrameNumber)
{
	bKeying = true;
	NextKeyTime = InFrameNumber;
}

void FMovieSceneTrackEditor::EndKeying()
{
	bKeying = false;
	NextKeyTime.Reset();
}

void FMovieSceneTrackEditor::UpdatePlaybackRange()
{
	TSharedPtr<ISequencer> SequencerPin = WeakSequencerPtr.Pin();
	if( SequencerPin.IsValid()  )
	{
		SequencerPin->UpdatePlaybackRange();
	}
}

TSharedRef<ISequencerSection> FMovieSceneTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	return MakeShared<FSequencerSection>(SectionObject);
}

void FMovieSceneTrackEditor::AnimatablePropertyChanged( FOnKeyProperty OnKeyProperty )
{
	check(OnKeyProperty.IsBound());

	// Get the movie scene we want to autokey
	UMovieSceneSequence* MovieSceneSequence = GetMovieSceneSequence();
	if (MovieSceneSequence)
	{
		FFrameNumber KeyTime = GetTimeForKey();

		// @todo Sequencer - The sequencer probably should have taken care of this
		MovieSceneSequence->SetFlags(RF_Transactional);
	
		// Create a transaction record because we are about to add keys
		const bool bShouldActuallyTransact = !GIsTransacting;		// Don't transact if we're recording in a PIE world.  That type of keyframe capture cannot be undone.
		FScopedTransaction AutoKeyTransaction( NSLOCTEXT("AnimatablePropertyTool", "PropertyChanged", "Animatable Property Changed"), bShouldActuallyTransact );

		FKeyPropertyResult KeyPropertyResult = OnKeyProperty.Execute( KeyTime );

		// When shift is down, enable adding keys/sections one after the other
		const bool bInProperSlateThread = (IsInGameThread() || IsInSlateThread() || IsInAsyncLoadingThread());
		if (bInProperSlateThread && FSlateApplication::Get().GetModifierKeys().IsShiftDown())
		{
			for (TWeakObjectPtr<UMovieSceneSection> NewSection : KeyPropertyResult.SectionsCreated)
			{
				if (NewSection.IsValid() && NewSection.Get()->HasEndFrame())
				{
					NextKeyTime = NewSection.Get()->GetExclusiveEndFrame();
					break;
				}
			}
		}

		if (KeyPropertyResult.bTrackCreated)
		{
			// If a track is created evaluate immediately so that the pre-animated state can be stored
			WeakSequencerPtr.Pin()->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::RefreshAllImmediately );
		}
		else if (KeyPropertyResult.bTrackModified)
		{
			WeakSequencerPtr.Pin()->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemAdded );
		}
		else if (KeyPropertyResult.bKeyCreated)
		{
			WeakSequencerPtr.Pin()->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::TrackValueChanged );
		}
		else
		{
			// If the only thing we changed as a result of the external change were channel defaults, we suppress automatic
			// re-evaluation of the sequence for this change to ensure that the object does not have the change immediately overwritten
			// by animated channels that have keys, but did not have keys added
			UMovieSceneSequence* FocusedSequence = WeakSequencerPtr.Pin()->GetFocusedMovieSceneSequence();
			if (FocusedSequence)
			{
				WeakSequencerPtr.Pin()->SuppressAutoEvaluation(FocusedSequence, FocusedSequence->GetSignature());
			}
		}

		// If a handle was created, something new has been added to Sequencer, so its selection state must be synchronized
		if (KeyPropertyResult.bHandleCreated)
		{
			WeakSequencerPtr.Pin()->ExternalSelectionHasChanged();
		}

		UpdatePlaybackRange();
	}
}


FMovieSceneTrackEditor::FFindOrCreateHandleResult FMovieSceneTrackEditor::FindOrCreateHandleToObject( UObject* Object, bool bCreateHandleIfMissing, const FName& CreatedFolderName)
{
	FGuid ObjectBindingID = GetSequencer()->GetHandleToObject(Object, false);
	if (ObjectBindingID.IsValid() || !bCreateHandleIfMissing)
	{
		return FFindOrCreateHandleResult{ ObjectBindingID, false };
	}

	FFindOrCreateHandleResult Result;
	Result.Handle      = GetSequencer()->GetHandleToObject(Object, true, CreatedFolderName);
	Result.bWasCreated = Result.Handle.IsValid();
	return Result;
}


FMovieSceneTrackEditor::FFindOrCreateTrackResult FMovieSceneTrackEditor::FindOrCreateTrackForObject( const FGuid& ObjectHandle, TSubclassOf<UMovieSceneTrack> TrackClass, FName PropertyName, bool bCreateTrackIfMissing )
{
	FFindOrCreateTrackResult Result;
	bool bTrackExisted;

	UMovieScene* MovieScene = GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene();
	Result.Track = MovieScene->FindTrack( TrackClass, ObjectHandle, PropertyName );
	bTrackExisted = Result.Track != nullptr;

	if (!Result.Track && bCreateTrackIfMissing)
	{
		Result.Track = AddTrack(MovieScene, ObjectHandle, TrackClass, PropertyName);
	}

	Result.bWasCreated = bTrackExisted == false && Result.Track != nullptr;

	return Result;
}

TSharedPtr<ISequencer> FMovieSceneTrackEditor::GetSequencer() const
{
	return WeakSequencerPtr.IsValid() ? WeakSequencerPtr.Pin() : nullptr;
}

FText FMovieSceneTrackEditor::GetDisplayName() const
{
	return FText::GetEmpty();
}

UMovieSceneTrack* FMovieSceneTrackEditor::AddTrack(UMovieScene* FocusedMovieScene, const FGuid& ObjectHandle, TSubclassOf<class UMovieSceneTrack> TrackClass, FName UniqueTypeName)
{
	return FocusedMovieScene->AddTrack(TrackClass, ObjectHandle);
}

void FMovieSceneTrackEditor::BindCommands(TSharedRef<FUICommandList> SequencerCommandBindings) 
{ 
}

void FMovieSceneTrackEditor::BuildPinnedAddTrackMenu(FMenuBuilder& MenuBuilder)
{
}

void FMovieSceneTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder) 
{ 
}

void FMovieSceneTrackEditor::BuildObjectBindingColumnWidgets(TFunctionRef<TSharedRef<SHorizontalBox>()> GetEditBox, const UE::Sequencer::TViewModelPtr<UE::Sequencer::FObjectBindingModel>& ObjectBinding, const UE::Sequencer::FCreateOutlinerViewParams& InParams, const FName& InColumnName)
{
}

void FMovieSceneTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
}

TSharedPtr<SWidget> FMovieSceneTrackEditor::BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName)
{
	using namespace UE::Sequencer;

	TViewModelPtr<ITrackExtension> TrackModel = Params.ViewModel->FindAncestorOfType<ITrackExtension>(true);
	TViewModelPtr<ITrackRowExtension> TrackRowModel = Params.ViewModel.ImplicitCast();
	UMovieSceneTrack* Track = TrackModel ? TrackModel->GetTrack() : nullptr;

	TViewModelPtr<FSequencerEditorViewModel> Editor       = Params.Editor->CastThisShared<FSequencerEditorViewModel>();
	TViewModelPtr<IOutlinerExtension>        OutlinerItem = Params.ViewModel.ImplicitCast();
	if (!Track || !Editor || !OutlinerItem)
	{
		return SNullWidget::NullWidget;
	}

	if (ColumnName == FCommonOutlinerNames::Label)
	{
		return SNew(SOutlinerItemViewBase, OutlinerItem, Params.Editor, Params.TreeViewRow);
	}

	if (ColumnName == FCommonOutlinerNames::Edit)
	{
		TOptional<FViewModelChildren>  TopLevelChannels = Params.ViewModel->FindChildList(FTrackModel::GetTopLevelChannelGroupType());
		TSharedPtr<FChannelGroupModel> TopLevelChannel  = TopLevelChannels ? TopLevelChannels->FindFirstChildOfType<FChannelGroupModel>() : nullptr;
		if (TopLevelChannel)
		{
			return SNew(SKeyAreaEditorSwitcher, TopLevelChannel, Editor);
		}

		TAttribute<bool> IsHovered = MakeAttributeSP(Params.TreeViewRow, &ISequencerTreeViewRow::IsHovered);

		FBuildEditWidgetParams EditWidgetParams(Params);
		EditWidgetParams.NodeIsHovered = IsHovered;
		EditWidgetParams.RowIndex = TrackRowModel ? TrackRowModel->GetRowIndex() : 0;
		if (Params.ViewModel->FindAncestorOfType<ITrackExtension>())
		{
			EditWidgetParams.TrackInsertRowIndex = TrackRowModel ? TrackRowModel->GetRowIndex() : 0;
		}
		else if (Track && Track->SupportsMultipleRows())
		{
			EditWidgetParams.TrackInsertRowIndex = Track->GetMaxRowIndex() + 1;
		}
		TViewModelPtr<IObjectBindingExtension> ObjectBinding = Params.ViewModel->FindAncestorOfType<IObjectBindingExtension>();
		return BuildOutlinerEditColumnWidget(ObjectBinding ? ObjectBinding->GetObjectGuid() : FGuid(), Track, EditWidgetParams);
	}

	if (ColumnName == FCommonOutlinerNames::Add)
	{
		TAttribute<bool> IsHovered = MakeAttributeSP(Params.TreeViewRow, &ISequencerTreeViewRow::IsHovered);

		// This function is a misnomer but is called this way for backwards compatibility
		FBuildEditWidgetParams EditWidgetParams(Params);
		EditWidgetParams.NodeIsHovered = IsHovered;
		EditWidgetParams.RowIndex = TrackRowModel ? TrackRowModel->GetRowIndex() : 0;
		if (Params.ViewModel->FindAncestorOfType<ITrackExtension>())
		{
			EditWidgetParams.TrackInsertRowIndex = TrackRowModel ? TrackRowModel->GetRowIndex() : 0;
		}
		else if (Track && Track->SupportsMultipleRows())
		{
			EditWidgetParams.TrackInsertRowIndex = Track->GetMaxRowIndex() + 1;
		}
		TViewModelPtr<IObjectBindingExtension> ObjectBinding = Params.ViewModel->FindAncestorOfType<IObjectBindingExtension>();
		return BuildOutlinerEditWidget(ObjectBinding ? ObjectBinding->GetObjectGuid() : FGuid(), Track, EditWidgetParams);
	}

	if (ColumnName == FCommonOutlinerNames::KeyFrame)
	{
		if (HasKeyableAreas(*Params.ViewModel))
		{
			EKeyNavigationButtons Buttons = EKeyNavigationButtons::AddKey;

			return SNew(SSequencerKeyNavigationButtons, Params.ViewModel, Editor->GetSequencer())
				.Buttons(Buttons);
		}

		return nullptr;
	}

	if (ColumnName == FCommonOutlinerNames::Nav)
	{
		if (HasKeyableAreas(*Params.ViewModel))
		{
			EKeyNavigationButtons Buttons = Params.TreeViewRow->IsColumnVisible(FCommonOutlinerNames::KeyFrame)
				? EKeyNavigationButtons::NavOnly
				: EKeyNavigationButtons::All;

			return SNew(SSequencerKeyNavigationButtons, Params.ViewModel, Editor->GetSequencer())
				.Buttons(Buttons);
		}

		return nullptr;
	}

	if (ColumnName == FCommonOutlinerNames::ColorPicker)
	{
		return SNew(SOutlinerTrackColorPicker, Params.ViewModel.ImplicitCast(), Params.Editor);
	}

	return nullptr;
}

TSharedPtr<SWidget> FMovieSceneTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	const bool bHasSections = Track && Track->GetSupportedBlendTypes().Num() > 0;
	const bool bHasDecorations = SequencerHelpers::HasCompatibleDecorations(Track);

	if (bHasSections || bHasDecorations)
	{
		TWeakPtr<ISequencer> WeakSequencer = GetSequencer();

		const int32 RowIndex = Params.TrackInsertRowIndex;
		auto SubMenuCallback = [=, this]() -> TSharedRef<SWidget>
		{
			FMenuBuilder MenuBuilder(true, nullptr);

			// Add section creation entries if track supports sections
			if (bHasSections)
			{
				FSequencerUtilities::PopulateMenu_CreateNewSection(MenuBuilder, RowIndex, Track, WeakSequencer);
			}

			// Add decoration entries if track supports decorations
			if (bHasDecorations)
			{
				SequencerHelpers::BuildDecorationMenu(MenuBuilder, Track, Track->FindObjectBindingGuid(), WeakSequencer);
			}

			return MenuBuilder.MakeWidget();
		};

		return UE::Sequencer::MakeAddButton(NSLOCTEXT("MovieSceneTrackEditor", "AddSection", "Section"), FOnGetContent::CreateLambda(SubMenuCallback), Params.ViewModel);
	}
	else
	{
		return TSharedPtr<SWidget>();
	}
}

TSharedPtr<SWidget> FMovieSceneTrackEditor::BuildOutlinerEditColumnWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	return TSharedPtr<SWidget>();
}

void FMovieSceneTrackEditor::BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track)
{
}

void FMovieSceneTrackEditor::BuildTrackSidebarMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) 
{
	BuildTrackContextMenu(MenuBuilder, Track);
}

bool FMovieSceneTrackEditor::HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) 
{
	return false; 
}

bool FMovieSceneTrackEditor::OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams)
{
	return false;
}

FReply FMovieSceneTrackEditor::OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams)
{
	return FReply::Unhandled();
}

void FMovieSceneTrackEditor::OnInitialize() 
{ 
}

void FMovieSceneTrackEditor::OnRelease() 
{ 
}

void FMovieSceneTrackEditor::Tick(float DeltaTime) 
{ 
}

UMovieScene* FMovieSceneTrackEditor::GetFocusedMovieScene() const
{
	UMovieSceneSequence* FocusedSequence = GetSequencer()->GetFocusedMovieSceneSequence();
	return FocusedSequence->GetMovieScene();
}
