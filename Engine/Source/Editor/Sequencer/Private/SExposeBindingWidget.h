// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class ISequencer;

class SExposeBindingWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SExposeBindingWidget){}
	SLATE_END_ARGS()


	void Construct(const FArguments& InArgs, TWeakPtr<ISequencer> InWeakSequencer);

private:

	void Reconstruct();

	void OnNewTextCommitted(const FText& InNewText, ETextCommit::Type CommitType);

	void ExposeAsName(FName InNewName);

	FReply RemoveFromExposedName(FName InNameToRemove);

private:

	/** The sequencer UI instance that is currently open */
	TWeakPtr<ISequencer> WeakSequencer;
};