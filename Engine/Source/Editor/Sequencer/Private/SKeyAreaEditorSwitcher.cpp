// Copyright Epic Games, Inc. All Rights Reserved.

#include "SKeyAreaEditorSwitcher.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Sequencer.h"
#include "SequencerCommonHelpers.h"
#include "IKeyArea.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Views/SKeyNavigationButtons.h"

namespace UE::Sequencer
{

void SKeyAreaEditorSwitcher::Construct(const FArguments& InArgs, TSharedPtr<FChannelGroupModel> InModel, TWeakPtr<FSequencerEditorViewModel> InWeakEditorModel)
{
	WeakModel = InModel;
	WeakEditorModel = InWeakEditorModel;
	CachedChannelsSerialNumber = 0;

	SetVisibility(MakeAttributeSP(this, &SKeyAreaEditorSwitcher::ComputeVisibility));
}

int32 SKeyAreaEditorSwitcher::GetWidgetIndex() const
{
	return VisibleIndex;
}

EVisibility SKeyAreaEditorSwitcher::ComputeVisibility() const
{
	return VisibleIndex != INDEX_NONE
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

void SKeyAreaEditorSwitcher::Rebuild()
{
	TSharedPtr<FChannelGroupModel>        Model = WeakModel.Pin();
	TSharedPtr<FSequencerEditorViewModel> Editor = WeakEditorModel.Pin();
	if (!Model || !Editor)
	{
		// Empty our cache so we don't persistently rebuild
		CachedKeyAreas.Empty();

		// Node is no longer valid so just make this a null widget
		ChildSlot
		[
			SNullWidget::NullWidget
		];
		return;
	}

	SetEnabled(MakeAttributeSP(Editor.ToSharedRef(), &FEditorViewModel::IsEditable));
	VisibleIndex = INDEX_NONE;

	TSharedRef<SWidgetSwitcher> Switcher = SNew(SWidgetSwitcher)
		.WidgetIndex(this, &SKeyAreaEditorSwitcher::GetWidgetIndex);

	TSharedPtr<IObjectBindingExtension> ParentObjectBinding = Model->FindAncestorOfType<IObjectBindingExtension>();
	FGuid ObjectBindingID = ParentObjectBinding.IsValid() ? ParentObjectBinding->GetObjectGuid() : FGuid();

	for (TSharedRef<IKeyArea> KeyArea : CachedKeyAreas)
	{
		Switcher->AddSlot()
		.HAlign(HAlign_Left)
		[
			KeyArea->CreateKeyEditor(Editor->GetSequencer(), ObjectBindingID)
		];
	}

	ChildSlot
	[
		Switcher
	];
}

void SKeyAreaEditorSwitcher::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	TSharedPtr<FChannelGroupModel>        Model  = WeakModel.Pin();
	TSharedPtr<FSequencerEditorViewModel> Editor = WeakEditorModel.Pin();
	if (!Model || !Editor)
	{
		if (CachedKeyAreas.Num() != 0)
		{
			CachedKeyAreas.Empty();

			// Node is not valid but we have a valid cache - we need to rebuild the switcher now
			Rebuild();
		}
		return;
	}
	else
	{
		const uint32 NewChannelsSerialNumber = Model->GetChannelsSerialNumber();
		const bool bRebuild = (NewChannelsSerialNumber != CachedChannelsSerialNumber);
		if (bRebuild)
		{
			CachedChannelsSerialNumber = NewChannelsSerialNumber;
			CachedKeyAreas = Model->GetAllKeyAreas();
			for (int32 Index = CachedKeyAreas.Num()-1; Index >= 0; --Index)
			{
				if (!CachedKeyAreas[Index]->CanCreateKeyEditor())
				{
					CachedKeyAreas.RemoveAtSwap(Index);
				}
			}
			Rebuild();
		}

		// Figure out which widget is active based on which section the current time is at.
		TArray<UMovieSceneSection*> AllSections;
		for (const TSharedRef<IKeyArea>& KeyArea : CachedKeyAreas)
		{
			AllSections.Add(KeyArea->GetOwningSection());
		}

		VisibleIndex = SequencerHelpers::GetSectionFromTime(AllSections, Editor->GetSequencer()->GetLocalTime().Time.FrameNumber);
	}
}

} // namespace UE::Sequencer
