// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SOverlay;
class IKeyArea;
class ISequencer;

namespace UE
{
namespace Sequencer
{

class FChannelGroupModel;
class FSequencerEditorViewModel;

class SKeyAreaEditorSwitcher : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SKeyAreaEditorSwitcher){}
	SLATE_END_ARGS()

	/** Construct the widget */
	void Construct(const FArguments& InArgs, TSharedPtr<FChannelGroupModel> InModel, TWeakPtr<FSequencerEditorViewModel> InWeakEditorViewModel);

	/** Rebuild this widget from its cached key area node */
	void Rebuild();

private:

	/** Tick this widget. Updates the currently visible key editor */
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	int32 GetWidgetIndex() const;

	EVisibility ComputeVisibility() const;

private:

	/** Our overlay widget */
	TSharedPtr<SOverlay> Overlay;
	/** Index of the currently visible key editor */
	int32 VisibleIndex;
	/** The key area to which we relate */
	TWeakPtr<FChannelGroupModel> WeakModel;
	/** Weak editor view model */
	TWeakPtr<FSequencerEditorViewModel> WeakEditorModel;
	/** Key areas cached from the node */
	TArray<TSharedRef<IKeyArea>> CachedKeyAreas;
	/** Serial cached from the node */
	uint32 CachedChannelsSerialNumber;
};


} // namespace Sequencer
} // namespace UE
