// Copyright Epic Games, Inc. All Rights Reserved.

#include "SObjectBindingTagManager.h"

#include "Sequencer.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "SequenceBindingTree.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneObjectBindingIDPicker.h"
#include "ObjectBindingTagCache.h"
#include "SObjectBindingTag.h"

#include "SlateOptMacros.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"

#include "Styling/AppStyle.h"
#include "EditorFontGlyphs.h"
#include "ScopedTransaction.h"

#include "Algo/Sort.h"

#define LOCTEXT_NAMESPACE "SObjectBindingTagManager"

class SSequenceBindingNodeRow : public SMultiColumnTableRow<TSharedRef<FSequenceBindingNode>>
{
	SLATE_BEGIN_ARGS(SSequenceBindingNodeRow) {}
	SLATE_END_ARGS()

	static FName Column_Label;
	static FName Column_Tags;

public:

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TWeakPtr<FSequenceBindingNode> InBindingNode, TWeakPtr<SObjectBindingTagManager> InWeakTabManager)
	{
		WeakBindingNode = InBindingNode;
		WeakTabManager  = InWeakTabManager;

		SMultiColumnTableRow<TSharedRef<FSequenceBindingNode>>
			::Construct(FSuperRowType::FArguments().Padding(5.f), InOwnerTableView);
	}

public:

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		TSharedPtr<FSequenceBindingNode> BindingNode = WeakBindingNode.Pin();
		if (!BindingNode)
		{
			return SNullWidget::NullWidget;
		}

		if (ColumnName == Column_Label)
		{
			return GenerateLabelContent(BindingNode);
		}
		else if (ColumnName == Column_Tags)
		{
			TSharedRef<SWidget> TagContent = SNullWidget::NullWidget;
			if (BindingNode->BindingID.Guid.IsValid())
			{
				TagContent = GenerateTagsContent(BindingNode);
			}

			return SNew(SBox)
				.HeightOverride(24.f)
				[
					TagContent
				];
		}

		return SNullWidget::NullWidget;
	}

	TSharedRef<SWidget> GenerateTagsContent(TSharedPtr<FSequenceBindingNode> BindingNode)
	{
		TSharedPtr<FSequencer> Sequencer = WeakTabManager.Pin()->GetSequencer();
		if (!Sequencer)
		{
			return SNullWidget::NullWidget;
		}

		FObjectBindingTagCache* BindingCache = Sequencer->GetObjectBindingTagCache();

		TArray<FName, TInlineAllocator<4>> AllTags;

		for (auto It = BindingCache->IterateTags(BindingNode->BindingID); It; ++It)
		{
			AllTags.Add(It.Value());
		}

		if (AllTags.Num() == 0)
		{
			return SNullWidget::NullWidget;
		}

		Algo::Sort(AllTags, [](const FName& A, const FName& B) { return A.Compare(B) < 0; });

		TSharedRef<SHorizontalBox> PillBox = SNew(SHorizontalBox);

		FMargin Padding(0.f, 0.f, 4.f, 0.f);

		bool bReadOnly = Sequencer->GetRootMovieSceneSequence()->GetMovieScene()->IsReadOnly();
		for (FName TagName : AllTags)
		{
			FSimpleDelegate OnDeleted;
			if (!bReadOnly)
			{
				OnDeleted = FSimpleDelegate::CreateSP(this, &SSequenceBindingNodeRow::HandleDeleteClicked, TagName);
			}

			PillBox->AddSlot()
			.AutoWidth()
			.Padding(Padding)
			[
				SNew(SObjectBindingTag)
				.ColorTint(BindingCache->GetTagColor(TagName))
				.Text(FText::FromName(TagName))
				.OnDeleted(OnDeleted)
			];

			Padding = FMargin(4.f, 0.f);
		}

		return PillBox;
	}

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	TSharedRef<SWidget> GenerateLabelContent(TSharedPtr<FSequenceBindingNode> BindingNode)
	{
		return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SExpanderArrow, SharedThis(this))
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(FMargin(5.f, 0.f, 5.f, 0.f))
		.AutoWidth()
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(BindingNode->Icon.GetIcon())
			]

			+ SOverlay::Slot()
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Right)
			[
				SNew(SImage)
				.Visibility(BindingNode->bIsSpawnable ? EVisibility::Visible : EVisibility::Collapsed)
				.Image(FAppStyle::GetBrush("Sequencer.SpawnableIconOverlay"))
			]
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(BindingNode->DisplayString)
		];
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

	void HandleDeleteClicked(FName TagName)
	{
		TSharedPtr<SObjectBindingTagManager> TabManager  = WeakTabManager.Pin();
		if (TabManager)
		{
			TabManager->UntagSelection(TagName, WeakBindingNode.Pin());
		}
	}

private:

	TWeakPtr<FSequencer> WeakSequencer;
	TWeakPtr<SObjectBindingTagManager> WeakTabManager;
	TWeakPtr<FSequenceBindingNode> WeakBindingNode;
};

FName SSequenceBindingNodeRow::Column_Label("Label");
FName SSequenceBindingNodeRow::Column_Tags("Tags");

void SObjectBindingTagManager::Construct(const FArguments& InArgs, TWeakPtr<FSequencer> InWeakSequencer)
{
	WeakSequencer = InWeakSequencer;

	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	UMovieSceneSequence*   Sequence = Sequencer ? Sequencer->GetRootMovieSceneSequence() : nullptr;
	if (!ensure(Sequence))
	{
		return;
	}

	BindingTree = MakeShared<FSequenceBindingTree>();
	BindingTree->ForceRebuild(Sequence, Sequence, MovieSceneSequenceID::Root);

	TWeakPtr<SObjectBindingTagManager> WeakTabManager = SharedThis(this);
	auto HandleGenerateRow = [WeakTabManager](TSharedRef<FSequenceBindingNode> InNode, const TSharedRef<STableViewBase>& InOwnerTableView) -> TSharedRef<ITableRow>
	{
		return SNew(SSequenceBindingNodeRow, InOwnerTableView, InNode, WeakTabManager);
	};

	auto HandleGetChildren = [](TSharedRef<FSequenceBindingNode> InParent, TArray<TSharedRef<FSequenceBindingNode>>& OutChildren)
	{
		OutChildren.Append(InParent->Children);
	};

	TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow)
		+ SHeaderRow::Column(SSequenceBindingNodeRow::Column_Label)
		.DefaultLabel(LOCTEXT("HeaderLabel_Text", "Object Binding"))
		.VAlignHeader(VAlign_Center)
		.FillWidth(0.4)

		+ SHeaderRow::Column(SSequenceBindingNodeRow::Column_Tags)
		.DefaultLabel(LOCTEXT("HeaderTags_Text", "Tags"))
		.VAlignHeader(VAlign_Center)
		.FillWidth(0.6);

	TreeView = SNew(STreeView<TSharedRef<FSequenceBindingNode>>)
		.OnGenerateRow_Lambda(HandleGenerateRow)
		.OnGetChildren_Lambda(HandleGetChildren)
		.TreeItemsSource(&BindingTree->GetRootNode()->Children)
		.OnContextMenuOpening( this, &SObjectBindingTagManager::OnContextMenuOpening)
		.HeaderRow(HeaderRow);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			[
				SNew(SScrollBorder, TreeView.ToSharedRef())
				[
					TreeView.ToSharedRef()
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(10.f))
			[
				SAssignNew(Tags, SHorizontalBox)
			]
		]
	];

	FObjectBindingTagCache* BindingCache = Sequencer->GetObjectBindingTagCache();
	BindingCache->OnUpdatedEvent.AddSP(this, &SObjectBindingTagManager::OnBindingCacheUpdated);
	OnBindingCacheUpdated(BindingCache);

	ExpandAllItems();
}


void SObjectBindingTagManager::OnBindingCacheUpdated(const FObjectBindingTagCache* BindingCache)
{
	Tags->ClearChildren();

	// BindingCache is owned by FSequencer so it must still be valid
	UMovieSceneSequence* Sequence = WeakSequencer.Pin()->GetRootMovieSceneSequence();
	if (!Sequence)
	{
		return;
	}

	float LeftPadding = 0.f;
	for (const TTuple<FName, FMovieSceneObjectBindingIDs>& Pair : Sequence->GetMovieScene()->AllTaggedBindings())
	{
		FName TagName = Pair.Key;

		Tags->AddSlot()
		.Padding(FMargin(LeftPadding, 0.f, 0.f, 0.f))
		.AutoWidth()
		[
			SNew(SObjectBindingTag)
			.OnDeleted(this, &SObjectBindingTagManager::RemoveTag, TagName)
			.ColorTint(BindingCache->GetTagColor(TagName))
			.Text(FText::FromName(TagName))
		];

		LeftPadding = 5.f;
	}
}

void SObjectBindingTagManager::ExpandAllItems()
{
	struct FInternal
	{
		static void ExpandAll(TSharedRef<FSequenceBindingNode> Node, STreeView<TSharedRef<FSequenceBindingNode>>* InTreeView)
		{
			InTreeView->SetItemExpansion(Node, true);
			for (TSharedRef<FSequenceBindingNode> Child : Node->Children)
			{
				ExpandAll(Child, InTreeView);
			}
		}
	};
	FInternal::ExpandAll(BindingTree->GetRootNode(), TreeView.Get());
}

void SObjectBindingTagManager::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	TSharedPtr<ISequencer> Sequencer  = WeakSequencer.Pin();
	UMovieSceneSequence*   Sequence   = Sequencer ? Sequencer->GetRootMovieSceneSequence() : nullptr;
	if (Sequence)
	{
		const bool bRebuilt = BindingTree->ConditionalRebuild(Sequence, Sequence, MovieSceneSequenceID::Root);
		if (bRebuilt)
		{
			ExpandAllItems();
			TreeView->SetTreeItemsSource(&BindingTree->GetRootNode()->Children);
			TreeView->RequestTreeRefresh();
		}
	}
}

TSharedPtr<SWidget> SObjectBindingTagManager::OnContextMenuOpening()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("AddNewHeader", "Add Tag"));
	{
		MenuBuilder.AddWidget(
			SNew(SBox)
			.Padding(FMargin(7.f, 2.f))
			[
				SNew(SObjectBindingTag)
				.ToolTipText(LOCTEXT("AddNew_Tooltip", "Adds a new persistent tag to the selected bindings"))
				.OnCreateNew(this, &SObjectBindingTagManager::TagSelectionAs)
			]
			, FText()
			, true
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SObjectBindingTagManager::TagSelectionAs(FName NewName)
{
	TSharedPtr<ISequencer> Sequencer  = WeakSequencer.Pin();
	UMovieSceneSequence*   Sequence   = Sequencer ? Sequencer->GetRootMovieSceneSequence() : nullptr;
	UMovieScene*           MovieScene = Sequence  ? Sequence->GetMovieScene()              : nullptr;

	if (!MovieScene)
	{
		return;
	}
	else if (MovieScene->IsReadOnly())
	{
		FNotificationInfo NotificationInfo(LOCTEXT("SequenceLocked", "Unable to change bindings while the sequence is locked."));
		NotificationInfo.ExpireDuration = 4.0f;
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		return;
	}

	FScopedTransaction Transaction(FText::Format(LOCTEXT("TagBinding", "Tag binding(s) as {0}"), FText::FromName(NewName)));

	bool bMadeAnyChanges = false;

	MovieScene->Modify();
	for (TSharedRef<FSequenceBindingNode> SelectedItem : TreeView->GetSelectedItems())
	{
		if (SelectedItem->BindingID.Guid.IsValid())
		{
			bMadeAnyChanges = true;
			MovieScene->TagBinding(NewName, SelectedItem->BindingID);
		}
	}

	if (!bMadeAnyChanges)
	{
		Transaction.Cancel();
	}
}

void SObjectBindingTagManager::UntagSelection(FName TagName, TSharedPtr<FSequenceBindingNode> InstigatorNode)
{
	TSharedPtr<ISequencer> Sequencer  = WeakSequencer.Pin();
	UMovieSceneSequence*   Sequence   = Sequencer ? Sequencer->GetRootMovieSceneSequence() : nullptr;
	UMovieScene*           MovieScene = Sequence  ? Sequence->GetMovieScene()              : nullptr;

	if (!MovieScene)
	{
		return;
	}
	else if (MovieScene->IsReadOnly())
	{
		FNotificationInfo NotificationInfo(LOCTEXT("SequenceLocked", "Unable to change bindings while the sequence is locked."));
		NotificationInfo.ExpireDuration = 4.0f;
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		return;
	}

	FScopedTransaction Transaction(FText::Format(LOCTEXT("UntagBinding", "Untag {0} from binding(s)"), FText::FromName(TagName)));

	bool bMadeAnyChanges = false;

	MovieScene->Modify();
	for (TSharedRef<FSequenceBindingNode> SelectedItem : TreeView->GetSelectedItems())
	{
		if (SelectedItem->BindingID.Guid.IsValid())
		{
			bMadeAnyChanges = true;
			MovieScene->UntagBinding(TagName, SelectedItem->BindingID);
		}
	}
	if (InstigatorNode && !TreeView->IsItemSelected(InstigatorNode.ToSharedRef()))
	{
		if (InstigatorNode->BindingID.Guid.IsValid())
		{
			bMadeAnyChanges = true;
			MovieScene->UntagBinding(TagName, InstigatorNode->BindingID);
		}
	}

	if (!bMadeAnyChanges)
	{
		Transaction.Cancel();
	}
}

void SObjectBindingTagManager::RemoveTag(FName TagName)
{
	TSharedPtr<ISequencer> Sequencer  = WeakSequencer.Pin();
	UMovieSceneSequence*   Sequence   = Sequencer ? Sequencer->GetRootMovieSceneSequence() : nullptr;
	UMovieScene*           MovieScene = Sequence  ? Sequence->GetMovieScene()              : nullptr;

	if (!MovieScene)
	{
		return;
	}
	else if (MovieScene->IsReadOnly())
	{
		FNotificationInfo NotificationInfo(LOCTEXT("SequenceLocked", "Unable to change bindings while the sequence is locked."));
		NotificationInfo.ExpireDuration = 4.0f;
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);
	}
	else
	{
		FScopedTransaction Transaction(FText::Format(LOCTEXT("RemoveTag", "Remove tag {0}"), FText::FromName(TagName)));

		MovieScene->Modify();
		MovieScene->RemoveTag(TagName);
	}
}

#undef LOCTEXT_NAMESPACE