// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "MovieSceneObjectBindingID.h"

struct FSequenceBindingNode;
struct FSequenceBindingTree;

class FSequencer;
class FMenuBuilder;
class FObjectBindingTagCache;

class SObjectBindingTagManager : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SObjectBindingTagManager){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FSequencer> InWeakSequencer);

	TSharedPtr<FSequencer> GetSequencer() const
	{
		return WeakSequencer.Pin();
	}

	void TagSelectionAs(FName NewName);

	void UntagSelection(FName TagName, TSharedPtr<FSequenceBindingNode> InstigatorNode);

	void RemoveTag(FName TagName);

private:

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void OnBindingCacheUpdated(const FObjectBindingTagCache* BindingCache);

	void ExpandAllItems();

	TSharedPtr<SWidget> OnContextMenuOpening();

private:

	/** The sequencer UI instance that is currently open */
	TWeakPtr<FSequencer> WeakSequencer;

	TSharedPtr<FSequenceBindingTree> BindingTree;

	TSharedPtr<STreeView<TSharedRef<FSequenceBindingNode>>> TreeView;

	TSharedPtr<SHorizontalBox> Tags;
};