// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencer.h"
#include "MVVM/Extensions/ISectionOwnerExtension.h"
#include "MovieSceneSection.h"
#include "BakingAnimationKeySettings.h"
#include "Engine/Blueprint.h"
#include "Filters/Filters/SequencerTrackFilter_CustomText.h"
#include "Filters/Filters/SequencerTrackFilters.h"
#include "Filters/Menus/SequencerViewOptionsMenu.h"
#include "Filters/SequencerFilterBar.h"
#include "Filters/SFilterSearchBox.h"
#include "Filters/Widgets/SFilterBarClippingHorizontalBox.h"
#include "Filters/Widgets/SFilterBarIsolateHideShow.h"
#include "Filters/Widgets/SSequencerCustomTextFilterDialog.h"
#include "Filters/Widgets/SSequencerFilterBar.h"
#include "MovieSceneSequence.h"
#include "MVVM/CurveEditorExtension.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/Extensions/IRenameableExtension.h"
#include "MVVM/Extensions/HierarchicalCacheExtension.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/SequencerTrackAreaViewModel.h"
#include "MVVM/ViewModels/SequencerOutlinerViewModel.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/SharedViewModelData.h"
#include "Sections/MovieSceneSubSection.h"
#include "MovieScene.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "Editor.h"
#include "IDetailsView.h"
#include "IKeyArea.h"
#include "Widgets/Layout/SBorder.h"
#include "ISequencerEditTool.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Overlays/STopSliderAreaOverlay.h"
#include "Widgets/Overlays/STrackAreaOverlay.h"
#include "SPositiveActionButton.h"
#include "Styling/AppStyle.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "MovieSceneSequenceEditor.h"
#include "Engine/Selection.h"
#include "LevelEditorViewport.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "SequencerCommands.h"
#include "Widgets/Input/SNumericDropDown.h"
#include "SequencerCommonHelpers.h"
#include "ISequencerWidgetsModule.h"
#include "ScopedTransaction.h"
#include "SequencerTimeSliderController.h"
#include "SequencerToolMenuContext.h"
#include "SSequencerSectionOverlay.h"
#include "STemporarilyFocusedSpinBox.h"
#include "MVVM/Views/STrackAreaView.h"
#include "MVVM/Views/SSequencerTrackAreaView.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "DragAndDrop/ClassDragDropOp.h"
#include "DragAndDrop/FolderDragDropOp.h"
#include "DragAndDrop/CompositeDragDropOp.h"
#include "Widgets/Input/SSearchBox.h"
#include "MVVM/Views/SSequencerOutlinerView.h"
#include "MVVM/ViewModels/OutlinerColumns/IOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "MovieSceneTrackEditor.h"
#include "SSequencerSplitterOverlay.h"
#include "SequencerHotspots.h"
#include "SSequencerTimePanel.h"
#include "MVVM/ViewModels/VirtualTrackArea.h"
#include "Framework/Commands/GenericCommands.h"
#include "SequencerContextMenus.h"
#include "Math/UnitConversion.h"
#include "Widgets/Input/NumericUnitTypeInterface.inl"
#include "FrameNumberDetailsCustomization.h"
#include "SequencerSettings.h"
#include "SSequencerTransformBox.h"
#include "SSequencerStretchBox.h"
#include "SSequencerDebugVisualizer.h"
#include "SSequencerTreeFilterStatusBar.h"
#include "ISequencerModule.h"
#include "IMovieRendererInterface.h"
#include "IVREditorModule.h"
#include "EditorFontGlyphs.h"
#include "HAL/PlatformApplicationMisc.h"
#include "SSequencerPlayRateCombo.h"
#include "Camera/CameraActor.h"
#include "SCurveEditorPanel.h"
#include "Tree/SCurveEditorTree.h"
#include "Tree/CurveEditorTreeFilter.h"
#include "Tree/SCurveEditorTreeTextFilter.h"
#include "SequencerSelectionCurveFilter.h"
#include "SCurveKeyDetailPanel.h"
#include "MovieSceneTimeHelpers.h"
#include "FrameNumberNumericInterface.h"
#include "LevelSequence.h"
#include "SequencerLog.h"
#include "MovieSceneCopyableBinding.h"
#include "SObjectBindingTagManager.h"
#include "SSequencerGroupManager.h"
#include "SSequencerHierarchyBrowser.h"
#include "MovieSceneCopyableTrack.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "IPropertyRowGenerator.h"
#include "Fonts/FontMeasure.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "SequencerCustomizationManager.h"
#include "EditorActorFolders.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "ToolMenus.h"
#include "MovieSceneFolder.h"
#include "MovieSceneToolHelpers.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "UniversalObjectLocators/ActorLocatorFragment.h"
#include "SequencerUtilities.h"
#include "Sidebar/SidebarDrawerConfig.h"
#include "Sidebar/SSidebar.h"
#include "Sidebar/SSidebarContainer.h"
#include "Widgets/SOverlay.h"
#include "Filters/Widgets/SSequencerSearchBox.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Widgets/Sequencer/SSequencerSearchAndFilterRow.h"
#include "Menus/SequencerToolbarUtils.h"
#include "Misc/TransportControlsHelper.h"
#include "MVVM/Extensions/ITopTimeSliderOverlayExtension.h"
#include "MVVM/Extensions/ITrackAreaOverlayExtension.h"
#include "Settings/EditorExperimentalSettings.h"
#include "SimpleView/SimpleViewTimeline.h"
#include "SimpleView/SimpleViewUtils.h"
#include "SPopoutTabInlineContent.h"
#include "ToolableTimeline/ToolableTimeline.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Sidebar/SequencerSelectionDrawer.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Filters/FilterConfigIdentifiers.h"
#include "Filters/Linking/Mode/LinkedFilterFactoryViewModel.h"
#include "Filters/Utils/LinkedFilteringViewUtils.h"
#include "Filters/Widgets/Linking/SFilterBarSwitcher.h"
#include "MVVM/ViewModels/OutlinerFilterAreaManager.h"
#include "MVVM/ViewModels/SequencerEditorViewModelImpl.h"
#include "SimpleView/SequencerSimpleViewSettings.h"
#include "ToolableTimeline/Menus/ToolableTimelineMenuContext.h"

#define LOCTEXT_NAMESPACE "Sequencer"

FSequencerOutlinerColumnVisibility::FSequencerOutlinerColumnVisibility(TSharedPtr<UE::Sequencer::IOutlinerColumn> InColumn)
	: Column(InColumn)
	, bIsColumnVisible(InColumn->IsColumnVisibleByDefault())
{}

FSequencerOutlinerColumnVisibility::FSequencerOutlinerColumnVisibility(TSharedPtr<UE::Sequencer::IOutlinerColumn> InColumn, bool bInIsColumnVisible)
	: Column(InColumn)
	, bIsColumnVisible(bInIsColumnVisible)
{}

bool FSequencerOutlinerColumnVisibility::operator==(const FSequencerOutlinerColumnVisibility& InOther) const
{
	return Column->GetColumnName() == InOther.Column->GetColumnName();
}

/* SSequencer interface
 *****************************************************************************/
UE_DISABLE_OPTIMIZATION_SHIP
void SSequencer::Construct(const FArguments& InArgs, TSharedRef<FSequencer> InSequencer)
{
	using namespace UE::Sequencer;

	SequencerPtr = InSequencer;
	bIsActiveTimerRegistered = false;
	CachedClampRange = TRange<double>::Empty();
	CachedViewRange = TRange<double>::Empty();
	PendingFocus = FPendingWidgetFocus::MakeNoTextEdit();

	OnPlaybackRangeBeginDrag = InArgs._OnPlaybackRangeBeginDrag;
	OnPlaybackRangeEndDrag = InArgs._OnPlaybackRangeEndDrag;
	OnSelectionRangeBeginDrag = InArgs._OnSelectionRangeBeginDrag;
	OnSelectionRangeEndDrag = InArgs._OnSelectionRangeEndDrag;
	OnMarkBeginDrag = InArgs._OnMarkBeginDrag;
	OnMarkEndDrag = InArgs._OnMarkEndDrag;

	OnReceivedFocus = InArgs._OnReceivedFocus;
	OnInitToolMenuContext = InArgs._OnInitToolMenuContext;

	RootCustomization.OnReceivedDragOver = InArgs._OnReceivedDragOver;
	RootCustomization.OnReceivedDrop = InArgs._OnReceivedDrop;
	RootCustomization.OnAssetsDrop = InArgs._OnAssetsDrop;
	RootCustomization.OnClassesDrop = InArgs._OnClassesDrop;
	RootCustomization.OnActorsDrop = InArgs._OnActorsDrop;
	RootCustomization.OnFoldersDrop = InArgs._OnFoldersDrop;

	TWeakPtr<SSequencer> WeakSelf = StaticCastSharedRef<SSequencer>(AsShared());
	// Get the desired display format from the user's settings each time.
	TAttribute<EFrameNumberDisplayFormats> GetDisplayFormatAttr = MakeAttributeLambda(
		[WeakSelf]
		{
			if(TSharedPtr<SSequencer> Target = WeakSelf.Pin())
			{
				if (USequencerSettings* Settings = Target->GetSequencerSettings())
				{
					return Settings->GetTimeDisplayFormat();
				}
			}
			return EFrameNumberDisplayFormats::Frames;
		}
	);

	// Get the number of zero pad frames from the user's settings as well.
	TAttribute<uint8> GetZeroPadFramesAttr = MakeAttributeLambda(
		[WeakSelf]()->uint8
		{
			if(TSharedPtr<SSequencer> Target = WeakSelf.Pin())
			{
				if (USequencerSettings* Settings = Target->GetSequencerSettings())
				{
					return Settings->GetZeroPadFrames();
				}
			}
			return 0;
		}
	);

	FTimeSliderArgs TimeSliderArgs;
	{
		TimeSliderArgs.ViewRange = InArgs._ViewRange;
		TimeSliderArgs.ClampRange = InArgs._ClampRange;
		TimeSliderArgs.PlaybackRange = MakeAttributeSP(this, &SSequencer::GetViewSpacePlaybackRange, InArgs._PlaybackRange);
		TimeSliderArgs.TimeBounds = InArgs._TimeBounds;
		TimeSliderArgs.DisplayRate = TAttribute<FFrameRate>(InSequencer, &FSequencer::GetFocusedDisplayRate);
		TimeSliderArgs.TickResolution = TAttribute<FFrameRate>(InSequencer, &FSequencer::GetFocusedTickResolution);
		TimeSliderArgs.SelectionRange = InArgs._SelectionRange;
		TimeSliderArgs.OnPlaybackRangeChanged = FOnFrameRangeChanged::CreateSP(this, &SSequencer::OnViewSpacePlaybackRangeChanged, InArgs._OnPlaybackRangeChanged);
		TimeSliderArgs.OnPlaybackRangeBeginDrag = OnPlaybackRangeBeginDrag;
		TimeSliderArgs.OnPlaybackRangeEndDrag = OnPlaybackRangeEndDrag;
		TimeSliderArgs.OnSelectionRangeChanged = InArgs._OnSelectionRangeChanged;
		TimeSliderArgs.OnSelectionRangeBeginDrag = OnSelectionRangeBeginDrag;
		TimeSliderArgs.OnSelectionRangeEndDrag = OnSelectionRangeEndDrag;
		TimeSliderArgs.OnMarkBeginDrag = OnMarkBeginDrag;
		TimeSliderArgs.OnMarkEndDrag = OnMarkEndDrag;
		TimeSliderArgs.OnViewRangeChanged = InArgs._OnViewRangeChanged;
		TimeSliderArgs.OnClampRangeChanged = InArgs._OnClampRangeChanged;
		TimeSliderArgs.OnGetNearestKey = InArgs._OnGetNearestKey;
		TimeSliderArgs.IsPlaybackRangeLocked = InArgs._IsPlaybackRangeLocked;
		TimeSliderArgs.OnTogglePlaybackRangeLocked = InArgs._OnTogglePlaybackRangeLocked;
		TimeSliderArgs.ScrubPosition = InArgs._ScrubPosition;
		TimeSliderArgs.ScrubPositionText = InArgs._ScrubPositionText;
		TimeSliderArgs.ScrubPositionParent = InArgs._ScrubPositionParent;
		TimeSliderArgs.ScrubPositionParentChain = InArgs._ScrubPositionParentChain;
		TimeSliderArgs.OnScrubPositionParentChanged = InArgs._OnScrubPositionParentChanged;
		TimeSliderArgs.OnBeginScrubberMovement = InArgs._OnBeginScrubbing;
		TimeSliderArgs.OnEndScrubberMovement = InArgs._OnEndScrubbing;
		TimeSliderArgs.OnScrubPositionChanged = InArgs._OnScrubPositionChanged;
		TimeSliderArgs.PlaybackStatus = InArgs._PlaybackStatus;
		TimeSliderArgs.SubSequenceRange = InArgs._SubSequenceRange;
		TimeSliderArgs.VerticalFrames = InArgs._VerticalFrames;
		TimeSliderArgs.MarkedFrames = InArgs._MarkedFrames;
		TimeSliderArgs.GlobalMarkedFrames = InArgs._GlobalMarkedFrames;
		TimeSliderArgs.OnSetMarkedFrame = InArgs._OnSetMarkedFrame;
		TimeSliderArgs.OnAddMarkedFrame = InArgs._OnAddMarkedFrame;
		TimeSliderArgs.OnDeleteAllMarkedFrames = InArgs._OnDeleteAllMarkedFrames;
		TimeSliderArgs.AreMarkedFramesLocked = InArgs._AreMarkedFramesLocked;
		TimeSliderArgs.OnToggleMarkedFramesLocked = InArgs._OnToggleMarkedFramesLocked;
	}

	OnGetPlaybackSpeeds = InArgs._OnGetPlaybackSpeeds;

	RootCustomization.AddMenuExtender = InArgs._AddMenuExtender;
	RootCustomization.ToolbarExtender = InArgs._ToolbarExtender;

	PlayTimeDisplay = SequencerPtr.Pin()->MakePlayTimeDisplay();

	check(SequencerPtr.IsValid());
	const TSharedRef<FSequencer> SequencerRef = SequencerPtr.Pin().ToSharedRef();

	TimeSliderController = MakeShared<FSequencerTimeSliderController>(TimeSliderArgs, SequencerPtr);
	TSharedRef<FSequencerTimeSliderController> TimeSliderControllerRef = TimeSliderController.ToSharedRef();

	ISequencerWidgetsModule& SequencerWidgets = FModuleManager::Get().LoadModuleChecked<ISequencerWidgetsModule>(TEXT("SequencerWidgets"));

	ScrollBar = SNew(SScrollBar)
		.Thickness(FVector2D(9.f, 9.f));

	PinnedAreaScrollBar = SNew(SScrollBar)
		.Thickness(FVector2D(9.f, 9.f));

	TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = SequencerRef->GetViewModel()->CastThisShared<FSequencerEditorViewModel>();
	SAssignNew(PinnedTrackArea, SSequencerTrackAreaView, SequencerViewModel->GetTrackArea(), TimeSliderControllerRef);
	SAssignNew(PinnedTreeView, SSequencerOutlinerView, SequencerViewModel->GetOutliner(), PinnedTrackArea.ToSharedRef())
		.Selection(SequencerRef->GetViewModel()->GetSelection())
		.ExternalScrollbar(PinnedAreaScrollBar)
		.Clipping(EWidgetClipping::ClipToBounds);

	PinnedTrackArea->SetOutliner(PinnedTreeView);
	PinnedTrackArea->SetShowPinned(true);
	PinnedTrackArea->SetIsPinned(true);
	PinnedTreeView->SetShowPinned(true);

	SAssignNew(TrackArea, SSequencerTrackAreaView, SequencerViewModel->GetTrackArea(), TimeSliderControllerRef);
	SAssignNew(TreeView, SSequencerOutlinerView, SequencerViewModel->GetOutliner(), TrackArea.ToSharedRef())
		.Selection(SequencerRef->GetViewModel()->GetSelection())
		.ExternalScrollbar(ScrollBar)
		.Clipping(EWidgetClipping::ClipToBounds);

	TrackArea->SetOutliner(TreeView);

	TreeView->AddPinnedTreeView(PinnedTreeView);

	USequencerSettings* const SequencerSettings = GetSequencerSettings();

	if (SequencerSettings)
	{
		InitializeOutlinerColumns();
		
		SequencerViewModel->SetViewDensity(SequencerSettings->GetViewDensity());
	}

	SequencerViewModel->GetTrackArea()->CastThisChecked<FSequencerTrackAreaViewModel>()->InitializeDefaultEditTools(*TrackArea);

	// Create the top and bottom sliders
	bool bMirrorLabels = false;
	TopTimeSlider = SequencerWidgets.CreateTimeSlider(TimeSliderControllerRef, bMirrorLabels);
	bMirrorLabels = true;
	BottomTimeSlider = SequencerWidgets.CreateTimeSlider(TimeSliderControllerRef, TAttribute<EVisibility>(this, &SSequencer::GetBottomTimeSliderVisibility), bMirrorLabels);

	// Create bottom time range slider
	EShowRange Ranges = EShowRange(EShowRange::WorkingRange | EShowRange::ViewRange);
	if(InArgs._ShowPlaybackRangeInTimeSlider)
	{
		Ranges |= EShowRange::PlaybackRange;
	}

	const FTimeRangeArgs BottomTimeRangeArgs = FTimeRangeArgs(
		Ranges,
		TimeSliderControllerRef,
		TAttribute<EVisibility>(this, &SSequencer::GetTimeRangeVisibility),
		MakeAttributeSP(SequencerRef, &FSequencer::GetNumericTypeInterface, UE::Sequencer::ENumericIntent::Position),
		GetSequencerSettings()->GetPlaybackRangeStartColor(),
		GetSequencerSettings()->GetPlaybackRangeEndColor()
	);

	BottomTimeRange = SequencerWidgets.CreateTimeRange(
		BottomTimeRangeArgs,
		SequencerWidgets.CreateTimeRangeSlider(TimeSliderControllerRef)
	);

	OutlinerAndTrackAreaContent = ConstructOutlinerAndTrackArea(InArgs, TimeSliderArgs);

	const TSharedRef<FSequencerEditorViewModelImpl> RootViewModel = SequencerRef->GetViewModelImpl();
	const auto IsLinkedToggleButtonVisible = [WeakRootViewModel = RootViewModel.ToWeakPtr()]()
	{
		// The linked filtering toggle button should only be displayed if there is another UI system that can link... if not, the feature is pointless
		const TSharedPtr<FSequencerEditorViewModelImpl> RootViewModel = WeakRootViewModel.Pin();
		const TSharedPtr<FLinkedFilterFactoryViewModel> Factory = RootViewModel ? RootViewModel->GetLinkedFilterViewModelFactoryImpl() : nullptr;
		// Check for more than 1 because Sequencer's instance counts as a user (@see FSequencerEditorViewModelImpl::SequenceFilterInstanceViewModel)
		const bool bIsAnySystemUsingLinkedFiltering = Factory && Factory->GetFilterFeatureUseCount() > 1;
			
		const bool bShouldBeVisible = bIsAnySystemUsingLinkedFiltering;
		return bShouldBeVisible ? EVisibility::Visible : EVisibility::Collapsed;
	};
	
	FilterRow = SNew(SSequencerSearchAndFilterRow)
		.ExtenderForAddMenu_Lambda([this]{ return FExtender::Combine(AddMenuExtenders); })
		.FilterViewModel(RootViewModel->GetLinkedFilterViewModelImpl())
		.FilterToggleButtonVisibility_Lambda(IsLinkedToggleButtonVisible)
		.Sequencer(SequencerPtr);

	if (SequencerRef->GetHostCapabilities().bSupportsSimpleView)
	{
		SimpleViewTimeline = ToolableTimeline::FToolableTimeline::MakeInstance<SimpleView::FSimpleViewTimeline>(
			SequencerRef, TimeSliderArgs, BottomTimeRangeArgs
		);
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		// Toolbar
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			ConstructToolBar()
		]

		// Main content body
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SAssignNew(MainContentContainer, SBox)
		]
	];

	RebuildMainContent();

	if (InSequencer->GetHostCapabilities().bSupportsCurveEditor)
	{
		FCurveEditorExtension* CurveEditorExtension = SequencerViewModel->CastDynamicChecked<FCurveEditorExtension>();
		CurveEditorExtension->CreateCurveEditor(TimeSliderArgs);
	}

	ApplySequencerCustomization(RootCustomization);

	InSequencer->GetViewModel()->GetSelection()->KeySelection.OnChanged.AddSP(this, &SSequencer::HandleKeySelectionChanged);
	InSequencer->GetViewModel()->GetSelection()->Outliner.OnChanged.AddSP(this, &SSequencer::HandleOutlinerNodeSelectionChanged);

	ResetBreadcrumbs();

	// Make sure the saved state (full or simplified view) is restored correctly.
	// This needs to be done after the geometry is constructed, so we do it next tick.
	if (GEditor)
	{
		const TWeakPtr<SSequencer> ThisWeak = SharedThis(this);
		GEditor->GetTimerManager()->SetTimerForNextTick([ThisWeak]()
			{
				if (const TSharedPtr<SSequencer> This = ThisWeak.Pin())
				{
					This->EnableSimpleView(This->IsSimpleView());
				}
			});
	}
}

void SSequencer::RebuildMainContent()
{
	const TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	check(Sequencer.IsValid());

	TSharedPtr<SWidget> MainContent;

	if (Sequencer->GetHostCapabilities().bSupportsSimpleView)
	{
		MainContent = SNew(SWidgetSwitcher)
			.WidgetIndex_Lambda([this]()
				{
					return GetSequencerSettings()->IsSimpleView() ? 1 : 0;
				})
			+ SWidgetSwitcher::Slot()
			.VAlign(VAlign_Fill)
			[
				ConstructSidebarContent()
			]
			+ SWidgetSwitcher::Slot()
			.VAlign(VAlign_Top)
			[
				SNew(SBox)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				[
					SimpleViewTimeline->GenerateWidget()
				]
			];
	}
	else
	{
		MainContent = ConstructSidebarContent();
	}

	MainContentContainer->SetContent(MainContent.ToSharedRef());
}

TSharedRef<SWidget> SSequencer::ConstructToolBar()
{
	const TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();

	const TSharedRef<SHorizontalBox> Content = SNew(SHorizontalBox)
		// Right Aligned Breadcrumbs
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SSpacer)
		]
		// History Back Button
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SButton)
				.Visibility(this, &SSequencer::GetBreadcrumbNavigateVisibility)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText_Lambda([this] { return SequencerPtr.Pin()->GetNavigateBackwardTooltip(); })
				.ContentPadding(FMargin(1, 0))
				.OnClicked_Lambda([this] { return SequencerPtr.Pin()->NavigateBackward(); })
				.IsEnabled_Lambda([this] { return SequencerPtr.Pin()->CanNavigateBackward(); })
				[
					SNew(SBox) // scale up since the default icons are 16x16
					.WidthOverride(20)
					.HeightOverride(20)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("Icons.ArrowLeft"))
					]
				]
			]
		]
		// History Forward Button
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SButton)
				.Visibility(this, &SSequencer::GetBreadcrumbNavigateVisibility)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText_Lambda([this] { return SequencerPtr.Pin()->GetNavigateForwardTooltip(); })
				.ContentPadding(FMargin(1, 0))
				.OnClicked_Lambda([this] { return SequencerPtr.Pin()->NavigateForward(); })
				.IsEnabled_Lambda([this] { return SequencerPtr.Pin()->CanNavigateForward(); })
				[
					SNew(SBox) // scale up since the default icons are 16x16
					.WidthOverride(20)
					.HeightOverride(20)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("Icons.ArrowRight"))
					]
				]
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			// Separator
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(3, 0)
			[
				SNew(SSeparator)
				.Visibility(this, &SSequencer::GetBreadcrumbNavigateVisibility)
				.Orientation(Orient_Vertical)
			]
			// Breadcrumb Picker
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SAssignNew(BreadcrumbPickerButton, SComboButton)
				.Visibility(this, &SSequencer::GetBreadcrumbNavigateVisibility)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnGetMenuContent_Lambda([this]
					{
						return SNew(SSequencerHierarchyBrowser, SequencerPtr);
					})
				.HasDownArrow(false)
				.ContentPadding(FMargin(3, 3))
				.ButtonContent()
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.FolderOpen"))
				]
			]
			// Right Aligned Breadcrumbs
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SAssignNew(BreadcrumbTrail, SBreadcrumbTrail<FSequencerBreadcrumb>)
				.Visibility(this, &SSequencer::GetBreadcrumbTrailVisibility)
				.OnCrumbClicked(this, &SSequencer::OnCrumbClicked)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.DelimiterImage(FAppStyle::Get().GetBrush("Sequencer.BreadcrumbIcon"))
				.TextStyle(FAppStyle::Get(), "Sequencer.BreadcrumbText")
			]
			// Sequence Locking symbol
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0, 0, 6, 0)
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(),"ToggleButtonCheckBoxAlt")
				.ForegroundColor_Lambda([this]()
					{
						return IsReadOnly() ? FStyleColors::AccentRed : FStyleColors::Foreground;
					})
				.Type(ESlateCheckBoxType::CheckBox) // Use CheckBox instead of ToggleType since we're not putting ohter widget inside
				.Padding(FMargin(0.f))
				.IsFocusable(false)
				.IsChecked_Lambda([this]
					{
						return IsReadOnly() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged_Lambda([this](const ECheckBoxState InNewState)
					{
						SetReadOnly(InNewState == ECheckBoxState::Checked);
					})
				.ToolTipText_Lambda([this]
					{
						return IsReadOnly()
							? LOCTEXT("UnlockSequence", "Unlock the animation so that it is editable")
							: LOCTEXT("LockSequence", "Lock the animation so that it is not editable");
					})
				.CheckedImage(FAppStyle::Get().GetBrush("Icons.Lock"))
				.CheckedHoveredImage(FAppStyle::Get().GetBrush("Icons.Lock"))
				.CheckedPressedImage(FAppStyle::Get().GetBrush("Icons.Lock"))
				.UncheckedImage(FAppStyle::Get().GetBrush("Icons.Unlock"))
				.UncheckedHoveredImage(FAppStyle::Get().GetBrush("Icons.Unlock"))
				.UncheckedPressedImage(FAppStyle::Get().GetBrush("Icons.Unlock"))
			]
		];

	// Toggle button to switch between full and simplified views
	if (SimpleViewTimeline.IsValid())
	{
		Content->AddSlot()
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 3, 0)
				[
					SNew(SSeparator)
					.Orientation(Orient_Vertical)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(3, 0, 6, 0)
				[
					UE::Sequencer::SimpleView::Utils::CreateToggleButton(SimpleViewTimeline)
				]
			];
	}

	return SNew(SBox)
		.Visibility(this, &SSequencer::GetShowSequencerToolbar)
		.Padding(FMargin(CommonPadding,0.0f,0.0f,0.f))
		[
			SNew(SWrapBox)
			.UseAllottedSize(true)
			.InnerSlotPadding(FVector2D(5, 0))
			+ SWrapBox::Slot()
			.FillEmptySpace(true)
			.FillLineWhenSizeLessThan(600)
			[
				SAssignNew(ToolbarContainer, SBox)
			]
			+ SWrapBox::Slot()
			.FillEmptySpace(true)
			[
				Content
			]
		];
}

TSharedRef<SWidget> SSequencer::ConstructSidebarContent()
{
	const TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	check(Sequencer.IsValid());
	const bool bSupportsSidebar = Sequencer->GetHostCapabilities().bSupportsSidebar;

	USequencerSettings* const SequencerSettings = GetSequencerSettings();
	check(SequencerSettings);
	const FSidebarState& SidebarState = SequencerSettings->GetSidebarState();

	TSharedPtr<SWidget> OutWidget;

	FilterBarSplitterContainer = SNew(SBox);

	// Create the details sidebar only once to avoid having to re-register drawers
	if (bSupportsSidebar)
	{
		if (!DetailsSidebar.IsValid())
		{
			SidebarContainer = SNew(SSidebarContainer);

			DetailsSidebar = SNew(SSidebar, SidebarContainer.ToSharedRef())
				.Visibility(EVisibility::SelfHitTestInvisible)
				.TabLocation(ESidebarTabLocation::Right)
				.InitialDrawerSize(SidebarState.GetDrawerSize())
				.OnStateChanged(this, &SSequencer::OnSidebarStateChanged)
				.OnGetContent(FOnGetContent::CreateLambda([this]()
					{
						return FilterBarSplitterContainer.ToSharedRef();
					}));

			SelectionDetailsDrawer = MakeShared<FSequencerSelectionDrawer>(GetSequencer());
		}

		SidebarContainer->RebuildSidebar(DetailsSidebar.ToSharedRef(), SidebarState);
	}
	else
	{
		DetailsSidebar.Reset();
		SidebarContainer.Reset();
		SelectionDetailsDrawer.Reset();
	}
	
	RebuildFilterBarContent();

	if (SidebarState.IsHidden() || !bSupportsSidebar)
	{
		OutWidget = FilterBarSplitterContainer.ToSharedRef();
	}
	else if (SidebarState.IsVisible())
	{
		OutWidget = SidebarContainer.ToSharedRef();
	}

	return OutWidget.ToSharedRef();
}

void SSequencer::RebuildFilterBarContent()
{
	FilterBarSplitterContainer->SetContent(ConstructFilterBarContent());
}

TSharedRef<SWidget> SSequencer::ConstructFilterBarContent()
{
	RebuildSearchAndFilterRow();
	return OutlinerAndTrackAreaContent.ToSharedRef();
}

TSharedRef<SSplitter> SSequencer::ConstructOutlinerAndTrackArea(const FArguments& InArgs, const FTimeSliderArgs& InTimeSliderArgs)
{
	using namespace UE::Sequencer;

	// This will add a thin, light gray, horizontal line to match the right side of the sequencer widget
	const FMargin ResizeBarPadding(1.f, 0, 0, 0);

	const TSharedRef<FSequencer> SequencerRef = SequencerPtr.Pin().ToSharedRef();
	const TSharedRef<FSequencerTimeSliderController> TimeSliderControllerRef = TimeSliderController.ToSharedRef();
	USequencerSettings* const SequencerSettings = GetSequencerSettings();

	TreeViewWidth = SequencerSettings->GetTreeViewWidth();

	return SNew(SSplitter)
		.OnSplitterFinishedResizing_Lambda([this]()
			{
				GetSequencerSettings()->SetTreeViewWidth(TreeViewWidth);
			})
		+ SSplitter::Slot()
		.Resizable(true)
		.SizeRule(SSplitter::SizeToContent)
		.OnSlotResized_Lambda([this](const float InSlotSize)
			{
				TreeViewWidth = InSlotSize;
			})
		[
			SNew(SBox)
			.MinDesiredWidth(MinOutlinerWidth)
			.WidthOverride(this, &SSequencer::GetOutlinerWidth)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Expose(OutlinerSearchAndFilterSlot)
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush(TEXT("ToolPanel.GroupBorder")))
					.Padding(FAppStyle::GetMargin(TEXT("Sequencer.FilterAreaMargin")))
					.Clipping(EWidgetClipping::ClipToBounds)
					[
						SAssignNew(SearchAndFilterRow, SVerticalBox)
					]
				]
				+ SVerticalBox::Slot()
				[
					ConstructMainTreeView(SequencerRef)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					ConstructTransportPanel(SequencerRef)
				]
			]
		]
		+ SSplitter::Slot()
		[
			SNew(SBox)
			.MinDesiredWidth(MinOutlinerWidth)
			.Padding(ResizeBarPadding)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBox)
					.HeightOverride_Lambda([this]()
						{
							return OutlinerSearchAndFilterSlot
								? OutlinerSearchAndFilterSlot->GetWidget()->GetCachedGeometry().GetLocalSize().Y
								: 0.;
						})
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush(TEXT("ToolPanel.GroupBorder")))
						.BorderBackgroundColor(FLinearColor(.5f, .5f, .5f, 1.f))
						.Padding(0)
						.Clipping(EWidgetClipping::ClipToBounds)
						[
							SNew(SOverlay)
							+ SOverlay::Slot().ZOrder(1) [ SNew(STopSliderAreaOverlay, SequencerRef->GetViewModel()) ]
							+ SOverlay::Slot().ZOrder(2) [ TopTimeSlider.ToSharedRef() ]
						]
					]
				]
				+ SVerticalBox::Slot()
				[
					SNew(SOverlay)

					+ SOverlay::Slot()
					.ZOrder(0)
					[
						SNew(SSequencerSectionOverlay, TimeSliderControllerRef)
						.Visibility(this, &SSequencer::GetShowTickLines)
						.DisplayScrubPosition(false)
						.DisplayTickLines(true)
						.Clipping(EWidgetClipping::ClipToBounds)
					]

					+ SOverlay::Slot()
					.ZOrder(10)
					[
						SNew(SVerticalBox)

						// Pinned Track Area
						+ SVerticalBox::Slot()
						.AutoHeight()
						.MaxHeight(TAttribute<float>::Create(TAttribute<float>::FGetter::CreateSP(this, &SSequencer::GetPinnedAreaMaxHeight)))
						[
							SNew(SBox)
							.HeightOverride_Lambda([this]()
								{
									return OutlinerPinnedSlot
										? OutlinerPinnedSlot->GetWidget()->GetCachedGeometry().GetLocalSize().Y
										: 0.;
								})
							[
								SNew(SOverlay)
								.Visibility(this, &SSequencer::GetPinnedAreaVisibility)

								+ SOverlay::Slot()
								[
									SNew(SScrollBorder, PinnedTreeView.ToSharedRef())
									[
										SNew(SBox)
										.Padding(ResizeBarPadding)
										.Clipping(EWidgetClipping::ClipToBounds)
										[
											PinnedTrackArea.ToSharedRef()
										]
									]
								]

								+ SOverlay::Slot()
								.HAlign(HAlign_Right)
								[
									PinnedAreaScrollBar.ToSharedRef()
								]
							]
						]

						// Track Area
						+ SVerticalBox::Slot()
						[
							SNew(SOverlay)

							+ SOverlay::Slot()
							[
								SNew(SScrollBorder, TreeView.ToSharedRef())
								[
									SNew(SBox)
									.Padding(ResizeBarPadding)
									.Clipping(EWidgetClipping::ClipToBounds)
									[
										TrackArea.ToSharedRef()
									]
								]
							]

							+ SOverlay::Slot()
							.HAlign(HAlign_Right)
							[
								ScrollBar.ToSharedRef()
							]

							+ SOverlay::Slot()
							.VAlign(VAlign_Bottom)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SAssignNew(SequencerTreeFilterStatusBar, SSequencerTreeFilterStatusBar, SequencerRef)
									.Visibility(EVisibility::Hidden) // Initially hidden, visible on hover of the info button
								]
							]
						]
					]

					// Overlay that allows drawing over the (unpinned) track area
					+ SOverlay::Slot()
					.ZOrder(15)
					[
						SNew(STrackAreaOverlay, SequencerRef->GetViewModel())
					]

					// Overlay that draws the scrub position
					+ SOverlay::Slot()
					.ZOrder(20)
					[
						SNew(SSequencerSectionOverlay, TimeSliderControllerRef)
						.Visibility(EVisibility::HitTestInvisible)
						.DisplayScrubPosition(true)
						.DisplayTickLines(false)
						.DisplayMarkedFrames(true)
						.PaintPlaybackRangeArgs(this, &SSequencer::GetSectionPlaybackRangeArgs)
						.Clipping(EWidgetClipping::ClipToBounds)
					]

					+ SOverlay::Slot()
					.ZOrder(30)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Top)
					[
						// Transform box
						SAssignNew(TransformBox, SSequencerTransformBox, SequencerRef, *SequencerSettings, SequencerRef->GetNumericTypeInterface().ToSharedRef())
					]

					+ SOverlay::Slot()
					.ZOrder(40)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Top)
					[
						// Stretch box
						SAssignNew(StretchBox, SSequencerStretchBox, SequencerRef, *SequencerSettings, SequencerRef->GetNumericTypeInterface().ToSharedRef())
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSequencerDebugVisualizer, SequencerRef)
					.ViewRange(FAnimatedRange::WrapAttribute(InArgs._ViewRange))
					.Visibility(this, &SSequencer::GetDebugVisualizerVisibility)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush(TEXT("ToolPanel.GroupBorder")))
					.BorderBackgroundColor(FLinearColor(.5f, .5f, .5f, 1.f))
					.Clipping(EWidgetClipping::ClipToBounds)
					.Padding(0)
					
					[
						SNew(SOverlay)
						+ SOverlay::Slot()
						[
							SNew(SBox)
							.Visibility_Lambda([this]()
								{
									if (const USequencerSettings* const SequencerSettings = GetSequencerSettings())
									{
										return SequencerSettings->GetShowBottomTimeSlider()
											? EVisibility::Visible : EVisibility::Collapsed;
									}
									return EVisibility::Collapsed;
								})
							[
								BottomTimeSlider.ToSharedRef()
							]
						]
						+ SOverlay::Slot()
						[
							BottomTimeRange.ToSharedRef()
						]
					]
				]
			]
		];
}

TSharedRef<SWidget> SSequencer::ConstructMainTreeView(const TSharedRef<FSequencer>& InSequencer)
{
	return SAssignNew(MainSequencerArea, SVerticalBox)

		// Pinned Tree View
		+ SVerticalBox::Slot()
		.Expose(OutlinerPinnedSlot)
		.AutoHeight()
		.MaxHeight(TAttribute<float>::Create(TAttribute<float>::FGetter::CreateSP(this, &SSequencer::GetPinnedAreaMaxHeight)))
		.Padding(FMargin(0.f, 0.f, 0.f, CommonPadding))
		[
			SNew(SScrollBorder, PinnedTreeView.ToSharedRef())
			.Visibility(this, &SSequencer::GetPinnedAreaVisibility)
			[
				PinnedTreeView.ToSharedRef()
			]
		]

		// Tree View
		+ SVerticalBox::Slot()
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			[
				SNew(SScrollBorder, TreeView.ToSharedRef())
				[
					TreeView.ToSharedRef()
				]
			]

			+ SOverlay::Slot()
			.VAlign(VAlign_Bottom)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SAssignNew(SequencerTreeFilterStatusBar, SSequencerTreeFilterStatusBar, InSequencer)
					.Visibility(EVisibility::Hidden) // Initially hidden, visible on hover of the info button
				]
			]
		];
}

TSharedRef<SWidget> SSequencer::ConstructTransportPanel(const TSharedRef<ISequencer>& InSequencer)
{
	const USequencerSettings* const SequencerSettings = GetSequencerSettings();
	if (!ensure(SequencerSettings))
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(TEXT("ToolPanel.GroupBorder")))
		.Clipping(EWidgetClipping::ClipToBounds)
		.Visibility(this, &SSequencer::GetPopoutTransportControlsVisibility)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(SButton)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
				.ToolTipText_Lambda([this] { return LOCTEXT("ShowStatus", "Show Status"); })
				.ContentPadding(FMargin(1, 0))
				.Visibility(this, &SSequencer::GetInfoButtonVisibility)
				.OnHovered_Lambda([this] { SequencerTreeFilterStatusBar->ShowStatusBar(); })
				.OnUnhovered_Lambda([this] { SequencerTreeFilterStatusBar->FadeOutStatusBar(); })
				.OnClicked_Lambda([this] { SequencerTreeFilterStatusBar->HideStatusBar(); return FReply::Handled(); })
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush(TEXT("Icons.Info.Small")))
				]
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			[
				ConstructTransportControls(InSequencer)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.ButtonStyle(FAppStyle::Get(), TEXT("NoBorder"))
				.ContentPadding(FMargin(1, 0))
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Right)
					.Padding(FMargin(CommonPadding, 0.f, 0.f, 0.f))
					[
						PlayTimeDisplay.ToSharedRef()
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Right)
					.Padding(FMargin(CommonPadding, 0.f, 0.f, 0.f))
					[
						// Current loop index, if any
						SAssignNew(LoopIndexDisplay, STextBlock)
						.Font(FCoreStyle::GetDefaultFontStyle(TEXT("Regular"), 8))
						.Text_Lambda([this]() -> FText {
							TOptional<int32> LoopIndex = SequencerPtr.Pin()->GetLocalLoopIndex();
							return LoopIndex ? FText::AsNumber(LoopIndex.GetValue()) : FText();
						})
					]
				]
			]
		];
}

TSharedRef<SWidget> SSequencer::ConstructTransportControls(const TSharedRef<ISequencer>& InSequencer)
{
	const USequencerSettings* const SequencerSettings = GetSequencerSettings();
	if (!ensure(SequencerSettings))
	{
		return SNullWidget::NullWidget;
	}

	const TSharedPtr<FTabManager> TabManager = GetTabManager();
	const TSharedPtr<FUICommandList> CommandBindings = InSequencer->GetCommandBindings();

	static const FName TransportControlsLeft("Sequencer.TransportControls.Left");
	static const FName TransportControlsRight("Sequencer.TransportControls.Right");
	if (!UToolMenus::Get()->IsMenuRegistered(TransportControlsLeft))
	{
		UToolMenus::Get()->RegisterMenu(TransportControlsLeft, NAME_None, EMultiBoxType::ToolBar);
		UToolMenus::Get()->RegisterMenu(TransportControlsRight, NAME_None, EMultiBoxType::ToolBar);
	}

	USequencerToolMenuContext* ContextObject = NewObject<USequencerToolMenuContext>();
	ContextObject->WeakSequencer = SequencerPtr;

	const TSharedRef<SScrollBox> TransportControlsWidget = SNew(SScrollBox)
		.Orientation(Orient_Horizontal)
		.ScrollBarAlwaysVisible(false)
		.ScrollBarVisibility(EVisibility::Collapsed)
		+ SScrollBox::Slot()
		.HAlign(HAlign_Fill)
		.FillContentSize(1.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FMargin(CommonPadding, 0.f, 0.f, 0.f))
			[
				UToolMenus::Get()->GenerateWidget(TransportControlsLeft
					, FToolMenuContext(CommandBindings, nullptr, ContextObject))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			[
				InSequencer->MakeTransportControls(/*bExtended*/true)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(FMargin(CommonPadding, 0.f, 0.f, 0.f))
			[
				UToolMenus::Get()->GenerateWidget(TransportControlsRight
					, FToolMenuContext(CommandBindings, nullptr, ContextObject))
			]
		];

	const UMovieSceneSequence* const RootSequence = InSequencer->GetRootMovieSceneSequence();
	if (!RootSequence || !RootSequence->IsA<ULevelSequence>())
	{
		return TransportControlsWidget;
	}

	TransportControlsPopoutTab = UE::Sequencer::MakePopoutTransportControls(InSequencer, TabManager, TransportControlsWidget);

	return TransportControlsPopoutTab.ToSharedRef();
}
UE_ENABLE_OPTIMIZATION_SHIP

void SSequencer::BindCommands(TSharedRef<FUICommandList> SequencerCommandBindings)
{
	auto CanPasteFromHistory = [this]{
		if (!HasFocusedDescendants() && !HasKeyboardFocus())
		{
			return false;
		}

		return SequencerPtr.Pin()->GetClipboardStack().Num() != 0;
	};

	auto CanOpenDirectorBlueprint = [this]{
		UMovieSceneSequence* RootSequence = SequencerPtr.Pin()->GetRootMovieSceneSequence();
		if (RootSequence && RootSequence->GetTypedOuter<UBlueprint>() == nullptr && UMovieScene::IsTrackClassAllowed(UMovieSceneEventTrack::StaticClass()))
		{
			return true;
		}
		return false;
	};

	SequencerCommandBindings->MapAction(
		FSequencerCommands::Get().AddSelectionFromContentBrowser,
		FExecuteAction::CreateSP(this, &SSequencer::AddFromContentBrowser),
		FCanExecuteAction::CreateSP(this, &SSequencer::CanAddFromContentBrowser)
	);

	SequencerCommandBindings->MapAction(
		FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &SSequencer::OnPaste),
		FCanExecuteAction::CreateSP(this, &SSequencer::CanPaste)
	);

	SequencerCommandBindings->MapAction(
		FSequencerCommands::Get().PasteFromHistory,
		FExecuteAction::CreateSP(this, &SSequencer::PasteFromHistory),
		FCanExecuteAction::CreateLambda(CanPasteFromHistory)
	);

	SequencerCommandBindings->MapAction(
		FSequencerCommands::Get().ToggleShowGotoBox,
		FExecuteAction::CreateLambda([this] { PlayTimeDisplay->Setup(); FSlateApplication::Get().SetKeyboardFocus(PlayTimeDisplay, EFocusCause::SetDirectly); })
	);

	SequencerCommandBindings->MapAction(
		FSequencerCommands::Get().ToggleShowTransformBox,
		FExecuteAction::CreateLambda([this] { TransformBox->ToggleVisibility(); })
	);

	// Allow jumping to the Sequencer tree search if you have Sequencer focused
	SequencerCommandBindings->MapAction(
		FSequencerCommands::Get().QuickTreeSearch,
		FExecuteAction::CreateLambda([this] { FSlateApplication::Get().SetKeyboardFocus(FilterRow->GetActiveSearchBox(), EFocusCause::SetDirectly); })
	);
	
	SequencerCommandBindings->MapAction(
		FSequencerCommands::Get().ToggleShowStretchBox,
		FExecuteAction::CreateLambda([this] { StretchBox->ToggleVisibility(); })
	);

	auto OpenDirectorBlueprint = [WeakSequencer = SequencerPtr]
	{
		UMovieSceneSequence*       Sequence       = WeakSequencer.Pin()->GetFocusedMovieSceneSequence();
		FMovieSceneSequenceEditor* SequenceEditor = Sequence ? FMovieSceneSequenceEditor::Find(Sequence) : nullptr;
		if (SequenceEditor)
		{
			UBlueprint* DirectorBP = SequenceEditor->GetOrCreateDirectorBlueprint(Sequence);
			if (DirectorBP)
			{
				FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(DirectorBP);
			}
		}
	};

	SequencerCommandBindings->MapAction(
		FSequencerCommands::Get().OpenDirectorBlueprint,
		FExecuteAction::CreateLambda(OpenDirectorBlueprint),
		FCanExecuteAction::CreateLambda(CanOpenDirectorBlueprint)
	);

	SequencerCommandBindings->MapAction(
		FSequencerCommands::Get().OpenTaggedBindingManager,
		FExecuteAction::CreateSP(this, &SSequencer::OpenTaggedBindingManager)
	);

	SequencerCommandBindings->MapAction(
		FSequencerCommands::Get().OpenNodeGroupsManager,
		FExecuteAction::CreateSP(this, &SSequencer::OpenNodeGroupsManager)
	);

	SequencerCommandBindings->MapAction(
		FSequencerCommands::Get().ToggleToolBarVisible,
		FExecuteAction::CreateSP(this, &SSequencer::ToggleToolBarVisible)
	);
}

void SSequencer::OpenTickResolutionOptions()
{
	if (TSharedPtr<SWindow> Window = WeakTickResolutionOptionsWindow.Pin())
	{
		Window->DrawAttention(FWindowDrawAttentionParameters());
		return;
	}

	TSharedRef<SWindow> TickResolutionOptionsWindow = SNew(SWindow)
		.Title(LOCTEXT("TickResolutionOptions_Title", "Advanced Time Properties"))
		.SupportsMaximize(false)
		.ClientSize(FVector2D(600.f, 510.f))
		.Content()
		[
			SNew(SSequencerTimePanel, SequencerPtr)
		];

	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (ParentWindow)
	{
		FSlateApplication::Get().AddWindowAsNativeChild(TickResolutionOptionsWindow, ParentWindow.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(TickResolutionOptionsWindow);
	}

	WeakTickResolutionOptionsWindow = TickResolutionOptionsWindow;
}

void SSequencer::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged)
{
	// @todo sequencer: is this still needed?
}


/* SSequencer implementation
 *****************************************************************************/

TRange<FFrameNumber> SSequencer::GetViewSpacePlaybackRange(TAttribute<TRange<FFrameNumber>> RangeAttribute) const
{
	TRange<FFrameNumber> Range = RangeAttribute.Get();

	TSharedPtr<FTimeToPixel> TimeToPixel = TrackArea->GetTimeToPixel();
	if (TimeToPixel->NonLinearTransform)
	{
		double       EndSeconds = Range.GetUpperBoundValue() / TimeToPixel->GetTickResolution();
		FFrameNumber EndFrame   = (TimeToPixel->NonLinearTransform->SourceToView(EndSeconds) * TimeToPixel->GetTickResolution()).RoundToFrame();
		Range.SetUpperBoundValue(EndFrame);
	}

	return Range;
}

void SSequencer::OnViewSpacePlaybackRangeChanged(TRange<FFrameNumber> NewRange, FOnFrameRangeChanged OnPlaybackRangeChanged)
{
	TSharedPtr<FTimeToPixel> TimeToPixel = TrackArea->GetTimeToPixel();
	if (TimeToPixel->NonLinearTransform)
	{
		double       EndSeconds = NewRange.GetUpperBoundValue() / TimeToPixel->GetTickResolution();
		FFrameNumber EndFrame   = (TimeToPixel->NonLinearTransform->ViewToSource(EndSeconds) * TimeToPixel->GetTickResolution()).RoundToFrame();
		NewRange.SetUpperBoundValue(EndFrame);
	}

	OnPlaybackRangeChanged.ExecuteIfBound(NewRange);
}

void SSequencer::UpdateOutlinerViewColumns()
{
	using namespace UE::Sequencer;

	// Save updated column list in settings
	TArray<FColumnVisibilitySetting> ColumnVisibilitySettings;

	for (FSequencerOutlinerColumnVisibility ColumnVisibility : OutlinerColumnVisibilities)
	{
		ColumnVisibilitySettings.AddUnique(FColumnVisibilitySetting(ColumnVisibility.Column->GetColumnName(), ColumnVisibility.bIsColumnVisible));
	}

	GetSequencerSettings()->SetOutlinerColumnVisibility(ColumnVisibilitySettings);

	// Filter out hidden columns to create a list of visible columns for the outliner views
	TArray<TSharedPtr<IOutlinerColumn>> VisibleColumns;
	for (FSequencerOutlinerColumnVisibility ColumnVisibility : OutlinerColumnVisibilities)
	{
		if (ColumnVisibility.bIsColumnVisible)
		{
			VisibleColumns.Add(ColumnVisibility.Column);
		}
	}

	// Update both Outliner Views with updated visible outliner columns
	PinnedTreeView->SetOutlinerColumns(VisibleColumns);
	TreeView->SetOutlinerColumns(VisibleColumns);
}

void SSequencer::InitializeOutlinerColumns()
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	OutlinerColumnVisibilities.Reset();

	const TMap<FName, TSharedPtr<IOutlinerColumn>>& RegisteredColumns = Sequencer->GetOutlinerColumns();
	
	// Retrieve previously saved column names and visibilities
	TArray<FColumnVisibilitySetting> ColumnSettings = GetSequencerSettings()->GetOutlinerColumnSettings();
	TSet<FName> ColumnNamesFoundInSettings;

	// Add registered columns found in settings with their saved visibility state
	for (const FColumnVisibilitySetting& ColumnVisibility : ColumnSettings)
	{
		const TSharedPtr<IOutlinerColumn>* OutlinerColumn = RegisteredColumns.Find(ColumnVisibility.ColumnName);
		if (OutlinerColumn)
		{
			ColumnNamesFoundInSettings.Add(ColumnVisibility.ColumnName);
			OutlinerColumnVisibilities.AddUnique(FSequencerOutlinerColumnVisibility(*OutlinerColumn, ColumnVisibility.bIsVisible));
		}
	}

	// Add registered columns not found in settings with their default visibility state
	for (const TTuple<FName, TSharedPtr<IOutlinerColumn>>& RegisteredColumn : RegisteredColumns)
	{
		if (!ColumnNamesFoundInSettings.Contains(RegisteredColumn.Key))
		{
			OutlinerColumnVisibilities.AddUnique(FSequencerOutlinerColumnVisibility(RegisteredColumn.Value));
		}
	}

	Algo::Sort(OutlinerColumnVisibilities, [](const FSequencerOutlinerColumnVisibility& A, const FSequencerOutlinerColumnVisibility& B){
		return A.Column->GetPosition() < B.Column->GetPosition();
	});

	UpdateOutlinerViewColumns();
}

/* SSequencer callbacks
 *****************************************************************************/

void SSequencer::HandleKeySelectionChanged()
{
}


void SSequencer::HandleOutlinerNodeSelectionChanged()
{
	if (NodeGroupManager.IsValid())
	{
		NodeGroupManager->SelectItemsSelectedInSequencer();
	}

	if (SequencerTreeFilterStatusBar.IsValid())
	{
		SequencerTreeFilterStatusBar->UpdateText();
	}
}

namespace UE::Sequencer::ToolMenus
{
const FLazyName ToolbarMenuName("Sequencer.MainToolBar");
const FLazyName ViewOptionsMenuName("Sequencer.MainToolBar.ViewOptions");
}

TSharedRef<SWidget> SSequencer::MakeToolBar()
{
	using namespace UE::Sequencer::ToolMenus;

	if (!UToolMenus::Get()->IsMenuRegistered(ToolbarMenuName))
	{
		UToolMenu* Toolbar = UToolMenus::Get()->RegisterMenu(ToolbarMenuName, NAME_None, EMultiBoxType::ToolBar);
		Toolbar->AddDynamicSection("PopulateToolBar", FNewToolMenuDelegate::CreateStatic(&SSequencer::PopulateToolBar));
	}

	RegisterViewOptionsToolMenu();

	TArray<TSharedPtr<FExtender>> AllExtenders;
	ISequencerModule& SequencerModule = FModuleManager::GetModuleChecked<ISequencerModule>("Sequencer");
	AllExtenders.Add(SequencerModule.GetToolBarExtensibilityManager()->GetAllExtenders());
	AllExtenders.Append(ToolbarExtenders);

	TSharedPtr<FExtender> Extender = FExtender::Combine(AllExtenders);

	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();

	USequencerToolMenuContext* ContextObject = NewObject<USequencerToolMenuContext>();
	ContextObject->WeakSequencer = SequencerPtr;

	FToolMenuContext Context(SequencerPtr.Pin()->GetCommandBindings(), Extender, ContextObject);

	if (Sequencer.IsValid() && Sequencer->GetHostCapabilities().bSupportsSimpleView)
	{
		UToolableTimelineMenuContext* const SimpleViewContext = NewObject<UToolableTimelineMenuContext>();
		SimpleViewContext->WeakTimeline = SimpleViewTimeline;
		Context.AddObject(SimpleViewContext);
	}

	// Allow any toolkits to initialize their menu context
	OnInitToolMenuContext.ExecuteIfBound(Context);

	return UToolMenus::Get()->GenerateWidget(ToolbarMenuName, Context);
}

void SSequencer::PopulateToolBar(UToolMenu* InMenu)
{
	const FName SequencerToolbarStyleName = UE::Sequencer::GSequencerToolbarStyleName;
	
	USequencerToolMenuContext* ContextObject = InMenu->FindContext<USequencerToolMenuContext>();
	if (!ContextObject)
	{
		return;
	}

	TWeakPtr<FSequencer> WeakSequencer = StaticCastWeakPtr<FSequencer>(ContextObject->WeakSequencer);
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	const TSharedRef<SSequencer> SequencerWidget = Sequencer->GetUnderlyingSequencerWidget();

	const TAttribute<EVisibility> FullViewVisibility = TAttribute<EVisibility>::CreateLambda([WeakSequencer]()
		{
			const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
			return (Sequencer.IsValid() && !Sequencer->IsSimpleView()) ? EVisibility::Visible : EVisibility::Collapsed;
		});

	{
		FToolMenuSection& Section = InMenu->AddSection("BaseCommands");

		if (Sequencer->IsLevelEditorSequencer())
		{
			TAttribute<FSlateIcon> SaveIcon;
			SaveIcon.Bind(TAttribute<FSlateIcon>::FGetter::CreateLambda([WeakSequencer] 
			{
				if (WeakSequencer.IsValid())
				{
					TArray<UMovieScene*> MovieScenesToSave;
					MovieSceneHelpers::GetDescendantMovieScenes(WeakSequencer.Pin()->GetRootMovieSceneSequence(), MovieScenesToSave);
					for (UMovieScene* MovieSceneToSave : MovieScenesToSave)
					{
						UPackage* MovieScenePackageToSave = MovieSceneToSave->GetOuter()->GetOutermost();
						if (MovieScenePackageToSave->IsDirty())
						{
							return FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SaveChanged");
						}
					}
				}
				return FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save");
			}));

			if (Sequencer->GetHostCapabilities().bSupportsSaveMovieSceneAsset)
			{
				FToolMenuEntry SaveEntry = FToolMenuEntry::InitToolBarButton(
					"Save",
					FUIAction(FExecuteAction::CreateSP(SequencerWidget, &SSequencer::OnSaveMovieSceneClicked)),
					LOCTEXT("SaveDirtyPackages", "Save"),
					LOCTEXT("SaveDirtyPackagesTooltip", "Saves the current sequence and any subsequences"),
					SaveIcon
				);
				SaveEntry.StyleNameOverride = SequencerToolbarStyleName;
				Section.AddEntry(SaveEntry);				
			}

			FToolMenuEntry FindInContentBrowserEntry = FToolMenuEntry::InitToolBarButton(FSequencerCommands::Get().FindInContentBrowser, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::Get().GetStyleSetName(), "SystemWideCommands.FindInContentBrowser"));
			FindInContentBrowserEntry.StyleNameOverride = SequencerToolbarStyleName;
			FindInContentBrowserEntry.Visibility = FullViewVisibility;
			Section.AddEntry(FindInContentBrowserEntry);

			FToolMenuEntry CreateCameraEntry = FToolMenuEntry::InitToolBarButton(FSequencerCommands::Get().CreateCamera);
			CreateCameraEntry.StyleNameOverride = SequencerToolbarStyleName;
			CreateCameraEntry.Visibility = FullViewVisibility;
			Section.AddEntry(CreateCameraEntry);

			if (Sequencer->GetHostCapabilities().bSupportsRenderMovie)
			{
				FToolMenuEntry RenderMovieEntry = FToolMenuEntry::InitToolBarButton(FSequencerCommands::Get().RenderMovie, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.OpenCinematic"));
				RenderMovieEntry.StyleNameOverride = SequencerToolbarStyleName;
				RenderMovieEntry.Visibility = FullViewVisibility;
				Section.AddEntry(RenderMovieEntry);
			
				FToolMenuEntry RenderMovieOptionsEntry = FToolMenuEntry::InitComboButton(
					"RenderMovieOptions",
					FUIAction(),
					FOnGetContent::CreateSP(SequencerWidget, &SSequencer::MakeRenderMovieMenu),
					LOCTEXT("RenderMovieOptions", "Render Movie Options"),
					LOCTEXT("RenderMovieOptionsToolTip", "Render Movie Options"),
					TAttribute<FSlateIcon>(),
					true
				);
				RenderMovieOptionsEntry.StyleNameOverride = SequencerToolbarStyleName;
				RenderMovieOptionsEntry.Visibility = FullViewVisibility;
				Section.AddEntry(RenderMovieOptionsEntry);
			}

			UMovieSceneSequence* RootSequence = Sequencer->GetRootMovieSceneSequence();
			if (RootSequence->GetTypedOuter<UBlueprint>() == nullptr && UMovieScene::IsTrackClassAllowed(UMovieSceneEventTrack::StaticClass()))
			{
				// Only show this button where it makes sense (ie, if the sequence is not contained within a blueprint already)
				FToolMenuEntry OpenDirectorBlueprintEntry = FToolMenuEntry::InitToolBarButton(FSequencerCommands::Get().OpenDirectorBlueprint, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.OpenLevelBlueprint"));
				OpenDirectorBlueprintEntry.StyleNameOverride = SequencerToolbarStyleName;
				OpenDirectorBlueprintEntry.Visibility = FullViewVisibility;
				Section.AddEntry(OpenDirectorBlueprintEntry);
			}

			Section.AddSeparator(NAME_None);
		}

		FToolMenuEntry ActionsEntry = FToolMenuEntry::InitComboButton(
			"Actions",
			FUIAction(),
			FOnGetContent::CreateSP(SequencerWidget, &SSequencer::MakeActionsMenu),
			LOCTEXT("Actions", "Actions"),
			LOCTEXT("ActionsToolTip", "Actions"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Actions")
		);
		ActionsEntry.StyleNameOverride = SequencerToolbarStyleName;
		ActionsEntry.Visibility = FullViewVisibility;
		Section.AddEntry(ActionsEntry);

		FToolMenuEntry ViewOptionsEntry = FToolMenuEntry::InitComboButton(
			"ViewOptions",
			FUIAction(),
			FNewToolMenuWidget::CreateSP(SequencerWidget, &SSequencer::MakeViewMenu),
			LOCTEXT("ViewOptions", "View Options"),
			LOCTEXT("ViewOptionsToolTip", "View Options"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Visibility")
		);
		ViewOptionsEntry.StyleNameOverride = SequencerToolbarStyleName;
		Section.AddEntry(ViewOptionsEntry);

		FToolMenuEntry PlaybackOptionsEntry = FToolMenuEntry::InitComboButton(
			"PlaybackOptions",
			FUIAction(),
			FOnGetContent::CreateSP(SequencerWidget, &SSequencer::MakePlaybackMenu),
			LOCTEXT("PlaybackOptions", "Playback Options"),
			LOCTEXT("PlaybackOptionsToolTip", "Playback Options"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.PlaybackOptions")
		);
		PlaybackOptionsEntry.StyleNameOverride = SequencerToolbarStyleName;
		Section.AddEntry(PlaybackOptionsEntry);

		Section.AddSeparator(NAME_None);
		
		Section.AddEntry(UE::Sequencer::MakeKeyGroupMenuEntry_ToolMenus(WeakSequencer));
		Section.AddEntry(UE::Sequencer::MakeAutoKeyMenuEntry(Sequencer));
		if (TOptional<FToolMenuEntry> AllowEditsEntry = UE::Sequencer::MakeAllowEditsModeMenuEntry(Sequencer))
		{
			AllowEditsEntry->Visibility = FullViewVisibility;
			Section.AddEntry(*AllowEditsEntry);
		}
		
		Section.AddSeparator(NAME_None);
	}

	{
		FToolMenuSection& SnappingSection = InMenu->AddSection("Snapping");

		FToolMenuEntry ToggleIsSnapEnabledEntry = FToolMenuEntry::InitToolBarButton(FSequencerCommands::Get().ToggleIsSnapEnabled, TAttribute<FText>(FText::GetEmpty()));
		ToggleIsSnapEnabledEntry.StyleNameOverride = SequencerToolbarStyleName;
		SnappingSection.AddEntry(ToggleIsSnapEnabledEntry);

		FToolMenuEntry SnapOptionsEntry = FToolMenuEntry::InitComboButton(
			"SnapOptions",
			FUIAction(),
			FOnGetContent::CreateSP(SequencerWidget, &SSequencer::MakeSnapMenu),
			LOCTEXT("SnapOptions", "Options"),
			LOCTEXT("SnapOptionsToolTip", "Snapping Options"),
			TAttribute<FSlateIcon>(),
			true
		);
		SnapOptionsEntry.StyleNameOverride = SequencerToolbarStyleName;
		SnappingSection.AddEntry(SnapOptionsEntry);

		FToolMenuEntry ToggleWholeFramesEntry =
			FToolMenuEntry::InitToolBarButton(FSequencerCommands::Get().ToggleForceWholeFrames);
		ToggleWholeFramesEntry.StyleNameOverride = SequencerToolbarStyleName;
		SnappingSection.AddEntry(ToggleWholeFramesEntry);

		SnappingSection.AddSeparator("PlayRate");

		FToolMenuEntry PlayRateEntry = FToolMenuEntry::InitWidget(
			"PlayRate",
			SNew(SSequencerPlayRateCombo, Sequencer, SequencerWidget)
				.Visibility(Sequencer.Get(), &FSequencer::GetPlayRateComboVisibility),
		   LOCTEXT("PlayRate", "PlayRate"));
		PlayRateEntry.StyleNameOverride = SequencerToolbarStyleName;
		SnappingSection.AddEntry(PlayRateEntry);
	}

	{
		FToolMenuSection& CurveEditorSection = InMenu->AddSection("CurveEditor");

		// Only add the button if supported
		if (Sequencer->GetHostCapabilities().bSupportsCurveEditor)
		{
			FToolMenuEntry ShowCurveEditorEntry = FToolMenuEntry::InitToolBarButton(FSequencerCommands::Get().ToggleShowCurveEditor, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCurveEditor"));
			ShowCurveEditorEntry.StyleNameOverride = SequencerToolbarStyleName;
			CurveEditorSection.AddEntry(ShowCurveEditorEntry);
		}
	}
}

void SSequencer::RegisterViewOptionsToolMenu()
{
	using namespace UE::Sequencer::ToolMenus;
	
	if (!UToolMenus::Get()->IsMenuRegistered(ViewOptionsMenuName))
	{
		UToolMenu* ViewOptions = UToolMenus::Get()->RegisterMenu(ViewOptionsMenuName, NAME_None, EMultiBoxType::Menu);
		ViewOptions->AddDynamicSection("Sequencer", FNewToolMenuDelegate::CreateLambda([](UToolMenu* Menu)
		{
			USequencerToolMenuContext* Context = Menu->FindContext<USequencerToolMenuContext>();
			const TSharedPtr<ISequencer> SequencerPin = Context ? Context->WeakSequencer.Pin() : nullptr;
			if (SequencerPin)
			{
				const TSharedRef<SSequencer> SequencerWidget = StaticCastSharedPtr<FSequencer>(SequencerPin)->GetUnderlyingSequencerWidget();
				SequencerWidget->PopulateViewOptionMenu(Menu);
			}
		}));
	}	
}

TSharedRef<SWidget> SSequencer::MakeAddMenu()
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();

	TSharedPtr<FExtender> Extender = FExtender::Combine(AddMenuExtenders);
	FMenuBuilder MenuBuilder(true, Sequencer->GetCommandBindings(), Extender);

	if (SequencerPtr.Pin()->GetHostCapabilities().bSupportsAddFromContentBrowser)
	{
		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().AddSelectionFromContentBrowser, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Use"));
	}

	{
		Sequencer->GetViewModel()->GetOutliner()->CastThisChecked<FSequencerOutlinerViewModel>()->BuildContextMenu(MenuBuilder);
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SSequencer::MakeActionsMenu()
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = Sequencer->GetViewModel();

	TArray<TSharedPtr<FExtender>> AllExtenders;
	ISequencerModule& SequencerModule = FModuleManager::GetModuleChecked<ISequencerModule>("Sequencer");
	AllExtenders.Add(SequencerModule.GetActionsMenuExtensibilityManager()->GetAllExtenders());
	AllExtenders.Append(ActionsMenuExtenders);

	TSharedPtr<FExtender> Extender = FExtender::Combine(AllExtenders);
	FMenuBuilder MenuBuilder(true, Sequencer->GetCommandBindings(), Extender);

	if (Sequencer->GetHostCapabilities().bSupportsSaveMovieSceneAsset)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("SaveAs", "Save As..."),
			LOCTEXT("SaveAsTooltip", "Saves the current sequence under a different name"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Save"),
			FUIAction(FExecuteAction::CreateRaw(this, &SSequencer::OnSaveMovieSceneAsClicked)));
	}

	MenuBuilder.BeginSection("SequenceOptions", LOCTEXT("SequenceOptionsHeader", "Sequence"));
	{
		UMovieSceneSequence* RootSequence = Sequencer->GetRootMovieSceneSequence();
		if (RootSequence->GetTypedOuter<UBlueprint>() == nullptr)
		{
			// Only show this button where it makes sense (ie, if the sequence is not contained within a blueprint already)
			MenuBuilder.AddMenuEntry(FSequencerCommands::Get().OpenDirectorBlueprint, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.OpenLevelBlueprint"));
		}

		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().OpenTaggedBindingManager);
		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().OpenNodeGroupsManager);

		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().RestoreAnimatedState);

		MenuBuilder.AddSubMenu(LOCTEXT("AdvancedHeader", "Advanced"), FText::GetEmpty(), FNewMenuDelegate::CreateRaw(this, &SSequencer::FillAdvancedMenu));
	}
	MenuBuilder.EndSection();

	// transform actions
	MenuBuilder.BeginSection("Transform", LOCTEXT("TransformHeader", "Transform"));
	
	MenuBuilder.AddMenuEntry(FSequencerCommands::Get().ToggleShowTransformBox);
	MenuBuilder.AddMenuEntry(FSequencerCommands::Get().ToggleShowStretchBox);
	MenuBuilder.AddMenuEntry(FSequencerCommands::Get().TranslateLeft);
	MenuBuilder.AddMenuEntry(FSequencerCommands::Get().TranslateRight);
	MenuBuilder.AddMenuEntry(FSequencerCommands::Get().TrimOrExtendSectionLeft);
	MenuBuilder.AddMenuEntry(FSequencerCommands::Get().TrimOrExtendSectionRight);
	MenuBuilder.AddMenuEntry(FSequencerCommands::Get().AlignSelectionToPlayhead);

	MenuBuilder.EndSection();

	// selection range actions
	MenuBuilder.BeginSection("SelectionRange", LOCTEXT("SelectionRangeHeader", "Selection Range"));
	{
		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().SetSelectionRangeStart);
		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().SetSelectionRangeEnd);
		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().ClearSelectionRange);
		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().SelectKeysInSelectionRange);
		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().SelectSectionsInSelectionRange);
		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().SelectMarkersInSelectionRange);
		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().SelectAllInSelectionRange);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SSequencer::FillAdvancedMenu(FMenuBuilder& MenuBuilder)
{
	if (SequencerPtr.Pin()->IsLevelEditorSequencer())
	{
		MenuBuilder.BeginSection("Bindings", LOCTEXT("BindingsMenuHeader", "Bindings"));

		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().RebindPossessableReferences);
		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().FixPossessableObjectClass);

		MenuBuilder.EndSection();
	}

	MenuBuilder.BeginSection( "NetworkingOptions", LOCTEXT( "NetworkingOptionsHeader", "Networking" ) );
	{
		auto SetNetworkMode = [WeakSequencer = SequencerPtr](EMovieSceneServerClientMask InMode)
		{
			TSharedPtr<FSequencer> SequencerPin = WeakSequencer.Pin();
			if (SequencerPin)
			{
				// When changing the enumlated network mode, we have to re-initialize and re-compile the 
				// sequence data to ensure that the emulation is reading the correct client/server/all data
				FMovieSceneRootEvaluationTemplateInstance& Template = SequencerPin->GetEvaluationTemplate();

				UMovieSceneSequence* RootSequence = Template.GetRootSequence();

				// Set the new emulation mode
				Template.SetEmulatedNetworkMask(InMode);
				// Since sequencer owns its own compiled data manager, it's ok to override the mask here and reset everything
				Template.GetCompiledDataManager()->SetEmulatedNetworkMask(InMode);
				// Reinitialize the template again
				Template.Initialize(*RootSequence, *SequencerPin, Template.GetCompiledDataManager());
			}
		};
		auto IsNetworkModeChecked = [WeakSequencer = SequencerPtr](EMovieSceneServerClientMask InMode)
		{
			TSharedPtr<FSequencer> SequencerPin = WeakSequencer.Pin();
			if (SequencerPin)
			{
				return InMode == SequencerPin->GetEvaluationTemplate().GetEmulatedNetworkMask();
			}
			return false;
		};

		MenuBuilder.AddMenuEntry(
			LOCTEXT("NetworkEmulationAllLabel", "Do not emulate (default)"),
			LOCTEXT("NetworkEmulationAllTooltip", "Play this sequence with all sub sequences, regardless of their network mask."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(SetNetworkMode, EMovieSceneServerClientMask::All),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda(IsNetworkModeChecked, EMovieSceneServerClientMask::All)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("NetworkEmulationClientLabel", "Emulate as Client"),
			LOCTEXT("NetworkEmulationClientTooltip", "Plays this sequence as if it were being played on a client (excludes server only cinematics)."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(SetNetworkMode, EMovieSceneServerClientMask::Client),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda(IsNetworkModeChecked, EMovieSceneServerClientMask::Client)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("NetworkEmulationServerLabel", "Emulate as Server"),
			LOCTEXT("NetworkEmulationServerTooltip", "Plays this sequence as if it were being played on a server (excludes client only cinematics)."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(SetNetworkMode, EMovieSceneServerClientMask::Server),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda(IsNetworkModeChecked, EMovieSceneServerClientMask::Server)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection( "VolatilityOptions", LOCTEXT( "VolatilityOptionsHeader", "Volatility" ) );
	{
		auto ToggleVolatility = [WeakSequencer = SequencerPtr](EMovieSceneSequenceFlags InFlags)
		{
			TSharedPtr<FSequencer> SequencerPin = WeakSequencer.Pin();
			if (SequencerPin)
			{
				const FScopedTransaction Transaction(LOCTEXT("ToggleVolatility", "Toggle Volatility"));

				UMovieSceneSequence* RootSequence = SequencerPin->GetRootMovieSceneSequence();

				RootSequence->Modify();

				RootSequence->SetSequenceFlags(RootSequence->GetFlags() ^ InFlags);
			}
		};
		auto IsVolatilityChecked = [WeakSequencer = SequencerPtr](EMovieSceneSequenceFlags InFlags)
		{
			TSharedPtr<FSequencer> SequencerPin = WeakSequencer.Pin();
			if (SequencerPin)
			{
				UMovieSceneSequence* RootSequence = SequencerPin->GetRootMovieSceneSequence();
				return ((uint8)RootSequence->GetFlags() & (uint8)InFlags) != 0;
			}
			return false;
		};

		MenuBuilder.AddMenuEntry(
			LOCTEXT("VolatilityVolatileLabel", "Volatile"),
			LOCTEXT("VolatilityVolatileTooltip", "Flag signifying that this sequence can change dynamically at runtime or during the game so the template must be checked for validity and recompiled as necessary before each evaluation.  The absence of this flag will result in the same compiled data being used for the duration of the program, as well as being pre-built during cook. As such, any dynamic changes to the sequence will not be reflected in the evaluation itself. This flag *must* be set if *any* procedural changes will be made to the source sequence data in-game."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(ToggleVolatility, EMovieSceneSequenceFlags::Volatile),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda(IsVolatilityChecked, EMovieSceneSequenceFlags::Volatile)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}
	MenuBuilder.EndSection();
}

TSharedRef<SWidget> SSequencer::MakeViewMenu(const FToolMenuContext& InContext)
{
	// This was created by the toolbar builder...
	USequencerToolMenuContext* RootContextObject = InContext.FindContext<USequencerToolMenuContext>();

	// ... and passing along RootContextObject allows for external modules to add view options that depend on state of ISequencer.
	const TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	FToolMenuContext Context(Sequencer->GetCommandBindings());
	Context.AddObject(RootContextObject);

	// We want to support the legacy FExtender API for now
	TArray<TSharedPtr<FExtender>> AllExtenders;
	ISequencerModule& SequencerModule = FModuleManager::GetModuleChecked<ISequencerModule>("Sequencer");
	AllExtenders.Add(SequencerModule.GetViewMenuExtensibilityManager()->GetAllExtenders());
	AllExtenders.Append(ViewMenuExtenders);
	Context.AddExtender(FExtender::Combine(AllExtenders));
	
	return UToolMenus::Get()->GenerateWidget(UE::Sequencer::ToolMenus::ViewOptionsMenuName, Context);
}

void SSequencer::PopulateViewOptionMenu(UToolMenu* InToolMenu)
{
	const TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	if (Sequencer->GetHostCapabilities().bSupportsSidebar)
	{
		InToolMenu->AddMenuEntry(TEXT("SidebarVisible"), FToolMenuEntry::InitMenuEntry(FSequencerCommands::Get().ToggleSidebarVisible));
		InToolMenu->AddMenuEntry(TEXT("SelectionDetailsPoppedOut"), FToolMenuEntry::InitMenuEntry(FSequencerCommands::Get().ToggleSelectionDetailsPoppedOut));
	}

	if (Sequencer->IsLevelEditorSequencer())
	{
		FToolMenuSection& PilotCameraSection = InToolMenu->AddSection("PilotCamera", LOCTEXT("PilotCamera", "Pilot Camera"));
		PilotCameraSection.AddMenuEntry(FSequencerCommands::Get().TogglePilotCamera);
		PilotCameraSection.AddMenuEntry(FSequencerCommands::Get().ToggleRestoreOriginalViewportOnCameraCutUnlock);
		PilotCameraSection.AddMenuEntry(FSequencerCommands::Get().TogglePreviewCameraCutsInSimulate);
	}

	FToolMenuSection& SequencerSettingsSection = InToolMenu->AddSection("SequencerSettings", LOCTEXT("SequencerSettings", "Sequencer Settings"));
	{
		SequencerSettingsSection.AddMenuEntry( FSequencerCommands::Get().ToggleAutoScroll );
		SequencerSettingsSection.AddMenuEntry( FSequencerCommands::Get().ToggleShowTransportControl );
		SequencerSettingsSection.AddMenuEntry( FSequencerCommands::Get().ToggleTransportControlsPoppedOut );
		SequencerSettingsSection.AddMenuEntry( FSequencerCommands::Get().ToggleShowRangeSlider );
		SequencerSettingsSection.AddMenuEntry( FSequencerCommands::Get().ToggleShowBottomTimeSlider );
		SequencerSettingsSection.AddMenuEntry( FSequencerCommands::Get().ToggleLayerBars );
		SequencerSettingsSection.AddMenuEntry( FSequencerCommands::Get().ToggleKeyBars );
		SequencerSettingsSection.AddMenuEntry( FSequencerCommands::Get().ToggleChannelColors );
		SequencerSettingsSection.AddMenuEntry( FSequencerCommands::Get().ToggleShowInfoButton );
		SequencerSettingsSection.AddMenuEntry( FSequencerCommands::Get().ToggleShowPreAndPostRoll );

		SequencerSettingsSection.AddSubMenu("ViewDensityMenu", LOCTEXT("ViewDensityMenuLabel", "View Density"), FText::GetEmpty(), FNewMenuDelegate::CreateRaw(this, &SSequencer::FillViewDensityMenu));

		// Menu Entry for Outliner Column Visibilities
		if (OutlinerColumnVisibilities.Num() > 0)
		{
			SequencerSettingsSection.AddSubMenu("ColumnVisibility", LOCTEXT("ColumnVisibilityHeader", "Columns"), FText::GetEmpty(), FNewMenuDelegate::CreateRaw(this, &SSequencer::FillColumnVisibilityMenu));
		}

		// Menu entry for zero padding
		const auto OnZeroPadChanged = [this](uint8 NewValue) {
			GetSequencerSettings()->SetZeroPadFrames(NewValue);
		};
		const TSharedRef<SWidget> ZeroPaddingWidget = 
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				[
					SNew(SSpacer)
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SSpinBox<uint8>)
					.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
					.OnValueCommitted_Lambda([=](uint8 Value, ETextCommit::Type) { OnZeroPadChanged(Value); })
					.OnValueChanged_Lambda(OnZeroPadChanged)
					.MinValue(0)
					.MaxValue(8)
					.Value_Lambda([this]() -> uint8 {
						return GetSequencerSettings()->GetZeroPadFrames();
					})
				];
		SequencerSettingsSection.AddEntry(
			FToolMenuEntry::InitWidget("ZeroPadding", ZeroPaddingWidget, LOCTEXT("ZeroPaddingText", "Zero Pad Frame Numbers"))
			);
	}

	FToolMenuSection& MarkedFramesSection = InToolMenu->AddSection("MarkedFrames", LOCTEXT("MarkedFramesHeader", "Marked Frames"));
	{
		MarkedFramesSection.AddMenuEntry(FSequencerCommands::Get().ToggleShowMarkedFrames);
		MarkedFramesSection.AddMenuEntry(FSequencerCommands::Get().ToggleShowMarkedFramesGlobally);
		MarkedFramesSection.AddMenuEntry(FSequencerCommands::Get().ClearGlobalMarkedFrames);
	}
}

void SSequencer::OpenTaggedBindingManager()
{
	if (TSharedPtr<SWindow> Window = WeakExposedBindingsWindow.Pin())
	{
		Window->DrawAttention(FWindowDrawAttentionParameters());
		return;
	}

	TSharedRef<SWindow> ExposedBindingsWindow = SNew(SWindow)
		.Title(FText::Format(LOCTEXT("ExposedBindings_Title", "Bindings Exposed in {0}"), FText::FromName(SequencerPtr.Pin()->GetRootMovieSceneSequence()->GetFName())))
		.SupportsMaximize(false)
		.ClientSize(FVector2D(600.f, 500.f))
		.Content()
		[
			SNew(SObjectBindingTagManager, SequencerPtr)
		];

	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (ParentWindow)
	{
		FSlateApplication::Get().AddWindowAsNativeChild(ExposedBindingsWindow, ParentWindow.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(ExposedBindingsWindow);
	}

	WeakExposedBindingsWindow = ExposedBindingsWindow;
}

void SSequencer::OpenNodeGroupsManager()
{
	if (TSharedPtr<SWindow> Window = WeakNodeGroupWindow.Pin())
	{
		Window->DrawAttention(FWindowDrawAttentionParameters());
		return;
	}

	TSharedRef<SWindow> NodeGroupManagerWindow = SNew(SWindow)
		.Title(FText::Format(LOCTEXT("NodeGroup_Title", "Groups in {0}"), FText::FromName(SequencerPtr.Pin()->GetRootMovieSceneSequence()->GetFName())))
		.SupportsMaximize(false)
		.ClientSize(FVector2D(600.f, 500.f))
		.Content()
		[
			SAssignNew(NodeGroupManager, SSequencerGroupManager, SequencerPtr)
		];

	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (ParentWindow)
	{
		FSlateApplication::Get().AddWindowAsNativeChild(NodeGroupManagerWindow, ParentWindow.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(NodeGroupManagerWindow);
	}

	WeakNodeGroupWindow = NodeGroupManagerWindow;
}

void SSequencer::FillPlaybackSpeedMenu(FMenuBuilder& InMenuBarBuilder)
{
	TArray<float> PlaybackSpeeds = OnGetPlaybackSpeeds.Execute();

	InMenuBarBuilder.BeginSection("PlaybackSpeed");
	for( int32 PlaybackSpeedIndex = 0; PlaybackSpeedIndex < PlaybackSpeeds.Num(); ++PlaybackSpeedIndex )
	{
		float PlaybackSpeed = PlaybackSpeeds[PlaybackSpeedIndex];
		const FText MenuStr = FText::Format( LOCTEXT("PlaybackSpeedStr", "{0}"), FText::AsNumber( PlaybackSpeed ) );
		InMenuBarBuilder.AddMenuEntry(MenuStr, FText(), FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda( [this, PlaybackSpeed]
				{
					SequencerPtr.Pin()->SetPlaybackSpeed(PlaybackSpeed);
				}),
				FCanExecuteAction::CreateLambda([] { return true; }),
				FIsActionChecked::CreateLambda( [this, PlaybackSpeed]{ return SequencerPtr.Pin()->GetPlaybackSpeed() == PlaybackSpeed; })
				),
			NAME_None,
			EUserInterfaceActionType::RadioButton
			);
	}
	InMenuBarBuilder.EndSection();
}

void SSequencer::FillViewDensityMenu(FMenuBuilder& InMenuBuilder)
{
	using namespace UE::Sequencer;

	auto SetViewDensity = [this](EViewDensity InViewDensity){
		TSharedPtr<FEditorViewModel> Editor = this->SequencerPtr.Pin()->GetViewModel();
		Editor->SetViewDensity(InViewDensity);

		if (USequencerSettings* Settings = this->GetSequencerSettings())
		{
			if (InViewDensity == EViewDensity::Compact)
			{
				Settings->SetViewDensity("Compact");
			}
			else if (InViewDensity == EViewDensity::Relaxed)
			{
				Settings->SetViewDensity("Relaxed");
			}
			else
			{
				Settings->SetViewDensity("Variable");
			}
		}
	};
	auto IsCurrentViewDensity = [this](EViewDensity InViewDensity){
		TSharedPtr<FEditorViewModel> Editor = this->SequencerPtr.Pin()->GetViewModel();
		return Editor->GetViewDensity().Density == InViewDensity;
	};

	InMenuBuilder.AddMenuEntry(
		LOCTEXT("VariableViewDensity_Label", "Variable"),
		LOCTEXT("VariableViewDensity_Tooltip", "Change Sequencer to use a variable height view mode withe inner items displaying more condensed than outer items"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(SetViewDensity, EViewDensity::Variable),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda(IsCurrentViewDensity, EViewDensity::Variable)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	InMenuBuilder.AddMenuEntry(
		LOCTEXT("CompactViewDensity_Label", "Compact"),
		LOCTEXT("CompactViewDensity_Tooltip", "Change Sequencer to use a compact view mode with uniform track heights"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(SetViewDensity, EViewDensity::Compact),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda(IsCurrentViewDensity, EViewDensity::Compact)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	InMenuBuilder.AddMenuEntry(
		LOCTEXT("RelaxedViewDensity_Label", "Relaxed"),
		LOCTEXT("RelaxedViewDensity_Tooltip", "Change Sequencer to use a relaxed view mode with larger uniform track heights"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(SetViewDensity, EViewDensity::Relaxed),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda(IsCurrentViewDensity, EViewDensity::Relaxed)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);
}

void SSequencer::FillColumnVisibilityMenu(FMenuBuilder& InMenuBuilder)
{
	using namespace UE::Sequencer;

	const bool bShouldCloseWindowAfterMenuSelection = true;

	for (FSequencerOutlinerColumnVisibility& ColumnVisibility : OutlinerColumnVisibilities)
	{
		if (EnumHasAnyFlags(ColumnVisibility.Column->GetLayout().Flags, EOutlinerColumnFlags::Hidden))
		{
			continue;
		}

		auto ToggleVisibility = [this, &ColumnVisibility]
		{
			ColumnVisibility.bIsColumnVisible = !ColumnVisibility.bIsColumnVisible;
			if(ColumnVisibility.bIsColumnVisible)
			{
				FName ColumnName = ColumnVisibility.Column->GetColumnName();
				FSequencerOutlinerColumnVisibility* AutoDisable = nullptr;

				// Auto disable mutually exclusive columns
				if (ColumnName == FCommonOutlinerNames::Nav)
				{
					AutoDisable = Algo::FindBy(this->OutlinerColumnVisibilities, FCommonOutlinerNames::KeyFrame, [](const FSequencerOutlinerColumnVisibility& In) { return In.Column->GetColumnName(); });
				}
				else if (ColumnName == FCommonOutlinerNames::KeyFrame)
				{
					AutoDisable = Algo::FindBy(this->OutlinerColumnVisibilities, FCommonOutlinerNames::Nav, [](const FSequencerOutlinerColumnVisibility& In) { return In.Column->GetColumnName(); });
				}

				if (AutoDisable)
				{
					AutoDisable->bIsColumnVisible = false;
				}
			}
			
			this->UpdateOutlinerViewColumns();
		};


		InMenuBuilder.AddMenuEntry(
			ColumnVisibility.Column->GetColumnLabel(),
			LOCTEXT("SetColumnVisibilityTooltip", "Enable or disable this outliner column"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(ToggleVisibility),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([&ColumnVisibility] { 
						// Capture by ref here since the array itself cannot be re-allocated while this menu is open
						return ColumnVisibility.bIsColumnVisible;
					})
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
	}
}

void SSequencer::FillPlayheadScrubbingLimitMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.AddMenuEntry(FSequencerCommands::Get().SetPlayheadScrubbingLimitNone);
	InMenuBuilder.AddMenuEntry(FSequencerCommands::Get().SetPlayheadScrubbingLimitPlaybackRange);
	InMenuBuilder.AddMenuEntry(FSequencerCommands::Get().SetPlayheadScrubbingLimitSelectionRange);
}

void SSequencer::FillTimeDisplayFormatMenu(UToolMenu* InMenu)
{
	USequencerToolMenuContext* ContextObject = InMenu->FindContext<USequencerToolMenuContext>();
	if (!ContextObject)
	{
		return;
	}

	TWeakPtr<FSequencer> WeakSequencer = StaticCastWeakPtr<FSequencer>(ContextObject->WeakSequencer);
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	bool bShouldDisplayDropFormat = FTimecode::UseDropFormatTimecode(Sequencer->GetFocusedDisplayRate());

	const UEnum* FrameNumberDisplayEnum = StaticEnum<EFrameNumberDisplayFormats>();
	check(FrameNumberDisplayEnum);

	if (USequencerSettings* Settings = Sequencer->GetSequencerSettings())
	{
		for (int32 Index = 0; Index < FrameNumberDisplayEnum->NumEnums() - 1; Index++)
		{
			if (!FrameNumberDisplayEnum->HasMetaData(TEXT("Hidden"), Index))
			{
				EFrameNumberDisplayFormats Value = (EFrameNumberDisplayFormats)FrameNumberDisplayEnum->GetValueByIndex(Index);

				// Don't show None Drop Frame Timecode when the format support drop format and the engine wants to use the drop format by default.
				if (Value == EFrameNumberDisplayFormats::NonDropFrameTimecode && bShouldDisplayDropFormat)
				{
					continue;
				}

				// Don't show Drop Frame Timecode when they're in a format that doesn't support it.
				if (Value == EFrameNumberDisplayFormats::DropFrameTimecode && !bShouldDisplayDropFormat)
				{
					continue;
				}

				FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
					FrameNumberDisplayEnum->GetNameByIndex(Index),
					FrameNumberDisplayEnum->GetDisplayNameTextByIndex(Index),
					FrameNumberDisplayEnum->GetToolTipTextByIndex(Index),
					FSlateIcon(),
					FToolUIActionChoice(
						FUIAction(
							FExecuteAction::CreateUObject(Settings, &USequencerSettings::SetTimeDisplayFormat, Value),
							FCanExecuteAction(),
							FIsActionChecked::CreateLambda([Settings, Value] { return Settings->GetTimeDisplayFormat() == Value; })
						)
					),
					EUserInterfaceActionType::RadioButton
				);
				InMenu->AddMenuEntry(NAME_None, Entry);
			}
		}
	}
}

TSharedRef<SWidget> SSequencer::MakePlaybackMenu()
{
	FMenuBuilder MenuBuilder( true, SequencerPtr.Pin()->GetCommandBindings() );

	// playback range options
	MenuBuilder.BeginSection("PlaybackThisSequence", LOCTEXT("PlaybackThisSequenceHeader", "Playback - This Sequence"));
	{
		// Menu entry for the start position
		auto OnStartChanged = [this](double NewValue){

			FFrameNumber ValueAsFrame = FFrameTime::FromDecimal(NewValue).GetFrame();
			FFrameNumber PlayStart = ValueAsFrame;
			FFrameNumber PlayEnd = UE::MovieScene::DiscreteExclusiveUpper(SequencerPtr.Pin()->GetPlaybackRange());
			if (PlayStart >= PlayEnd)
			{
				FFrameNumber Duration = PlayEnd - UE::MovieScene::DiscreteInclusiveLower(SequencerPtr.Pin()->GetPlaybackRange());
				PlayEnd = PlayStart + Duration;
			}

			SequencerPtr.Pin()->SetPlaybackRange(TRange<FFrameNumber>(PlayStart, PlayEnd));

			TRange<double> PlayRangeSeconds = SequencerPtr.Pin()->GetPlaybackRange() / SequencerPtr.Pin()->GetFocusedTickResolution();
			const double AdditionalRange = (PlayRangeSeconds.GetUpperBoundValue() - PlayRangeSeconds.GetLowerBoundValue()) * 0.1;

			TRange<double> NewClampRange = SequencerPtr.Pin()->GetClampRange();
			NewClampRange.SetLowerBoundValue(SequencerPtr.Pin()->GetPlaybackRange().GetLowerBoundValue() / SequencerPtr.Pin()->GetFocusedTickResolution() - AdditionalRange);
			if (SequencerPtr.Pin()->GetClampRange().GetLowerBoundValue() > NewClampRange.GetLowerBoundValue())
			{
				SequencerPtr.Pin()->SetClampRange(NewClampRange);
			}

			TRange<double> NewViewRange = SequencerPtr.Pin()->GetViewRange();
			NewViewRange.SetLowerBoundValue(SequencerPtr.Pin()->GetPlaybackRange().GetLowerBoundValue() / SequencerPtr.Pin()->GetFocusedTickResolution() - AdditionalRange);
			if (SequencerPtr.Pin()->GetViewRange().GetLowerBoundValue() > NewViewRange.GetLowerBoundValue())
			{
				SequencerPtr.Pin()->SetViewRange(NewViewRange);
			}
		};

		MenuBuilder.AddWidget(
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				[
					SNew(SSpacer)
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SSpinBox<double>)
						.TypeInterface(SequencerPtr.Pin().Get(), &FSequencer::GetNumericTypeInterface, UE::Sequencer::ENumericIntent::Position)
						.IsEnabled_Lambda([this]() {
							return !SequencerPtr.Pin()->IsPlaybackRangeLocked();
						})
						.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
						.OnValueCommitted_Lambda([=](double Value, ETextCommit::Type){ OnStartChanged(Value); })
						.OnValueChanged_Lambda([=](double Value) { OnStartChanged(Value); })
						.OnBeginSliderMovement(OnPlaybackRangeBeginDrag)
						.OnEndSliderMovement_Lambda([=, this](double Value){ OnStartChanged(Value); OnPlaybackRangeEndDrag.ExecuteIfBound(); })
						.MinValue(TOptional<double>())
						.MaxValue(TOptional<double>())
						.Value_Lambda([this]() -> double {
							return SequencerPtr.Pin()->GetPlaybackRange().GetLowerBoundValue().Value;
						})
						.Delta(this, &SSequencer::GetSpinboxDelta)
						.LinearDeltaSensitivity(25)
			],
			LOCTEXT("PlaybackStartLabel", "Start"));

		// Menu entry for the end position
		auto OnEndChanged = [this](double NewValue) {

			FFrameNumber ValueAsFrame = FFrameTime::FromDecimal(NewValue).GetFrame();
			FFrameNumber PlayStart = UE::MovieScene::DiscreteInclusiveLower(SequencerPtr.Pin()->GetPlaybackRange());
			FFrameNumber PlayEnd = ValueAsFrame;
			if (PlayEnd <= PlayStart)
			{
				FFrameNumber Duration = UE::MovieScene::DiscreteExclusiveUpper(SequencerPtr.Pin()->GetPlaybackRange()) - PlayStart;
				PlayStart = PlayEnd - Duration;
			}

			SequencerPtr.Pin()->SetPlaybackRange(TRange<FFrameNumber>(PlayStart, PlayEnd));

			TRange<double> PlayRangeSeconds = SequencerPtr.Pin()->GetPlaybackRange() / SequencerPtr.Pin()->GetFocusedTickResolution();
			const double AdditionalRange = (PlayRangeSeconds.GetUpperBoundValue() - PlayRangeSeconds.GetLowerBoundValue()) * 0.1;

			TRange<double> NewClampRange = SequencerPtr.Pin()->GetClampRange();
			NewClampRange.SetUpperBoundValue(SequencerPtr.Pin()->GetPlaybackRange().GetUpperBoundValue() / SequencerPtr.Pin()->GetFocusedTickResolution() + AdditionalRange);
			if (SequencerPtr.Pin()->GetClampRange().GetUpperBoundValue() < NewClampRange.GetUpperBoundValue())
			{
				SequencerPtr.Pin()->SetClampRange(NewClampRange);
			}

			TRange<double> NewViewRange = SequencerPtr.Pin()->GetViewRange();
			NewViewRange.SetUpperBoundValue(SequencerPtr.Pin()->GetPlaybackRange().GetUpperBoundValue() / SequencerPtr.Pin()->GetFocusedTickResolution() + AdditionalRange);
			if (SequencerPtr.Pin()->GetViewRange().GetUpperBoundValue() < NewViewRange.GetUpperBoundValue())
			{
				SequencerPtr.Pin()->SetViewRange(NewViewRange);
			}
		};

		MenuBuilder.AddWidget(
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				[
					SNew(SSpacer)
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SSpinBox<double>)
						.TypeInterface(SequencerPtr.Pin().Get(), &FSequencer::GetNumericTypeInterface, UE::Sequencer::ENumericIntent::Position)
						.IsEnabled_Lambda([this]() {
					 		return !SequencerPtr.Pin()->IsPlaybackRangeLocked();
						})
						.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
						.OnValueCommitted_Lambda([=](double Value, ETextCommit::Type){ OnEndChanged(Value); })
						.OnValueChanged_Lambda([=](double Value) { OnEndChanged(Value); })
						.OnBeginSliderMovement(OnPlaybackRangeBeginDrag)
						.OnEndSliderMovement_Lambda([=, this](double Value){ OnEndChanged(Value); OnPlaybackRangeEndDrag.ExecuteIfBound(); })
						.MinValue(TOptional<double>())
						.MaxValue(TOptional<double>())
						.Value_Lambda([this]() -> double {
					 		return SequencerPtr.Pin()->GetPlaybackRange().GetUpperBoundValue().Value;
						})
						.Delta(this, &SSequencer::GetSpinboxDelta)
						.LinearDeltaSensitivity(25)
				],
			LOCTEXT("PlaybackStartEnd", "End"));


		MenuBuilder.AddSubMenu(LOCTEXT("PlaybackSpeedHeader", "Playback Speed"), FText::GetEmpty(), FNewMenuDelegate::CreateRaw(this, &SSequencer::FillPlaybackSpeedMenu));

		MenuBuilder.AddMenuEntry( FSequencerCommands::Get().TogglePlaybackRangeLocked );

		if (SequencerPtr.Pin()->IsLevelEditorSequencer())
		{
			MenuBuilder.AddMenuEntry( FSequencerCommands::Get().ToggleCleanPlaybackMode );
			MenuBuilder.AddMenuEntry( FSequencerCommands::Get().ToggleRerunConstructionScripts );
		}

		MenuBuilder.AddMenuEntry( FSequencerCommands::Get().ToggleAsyncEvaluation );
		MenuBuilder.AddMenuEntry( FSequencerCommands::Get().ToggleDynamicWeighting );
		MenuBuilder.AddMenuEntry( FSequencerCommands::Get().ToggleLoopCuts );

		MenuBuilder.AddSubMenu(LOCTEXT("PlayheadScrubbingLimit", "Playhead Scrubbing Limit"), FText::GetEmpty(), FNewMenuDelegate::CreateRaw(this, &SSequencer::FillPlayheadScrubbingLimitMenu));
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection( "PlaybackAllSequences", LOCTEXT( "PlaybackRangeAllSequencesHeader", "Playback Range - All Sequences" ) );
	{
		if (SequencerPtr.Pin()->IsLevelEditorSequencer())
		{
			MenuBuilder.AddMenuEntry( FSequencerCommands::Get().ToggleEvaluateSubSequencesInIsolation );
			MenuBuilder.AddMenuEntry( FSequencerCommands::Get().ToggleResetPlayheadWhenNavigating );
		}

		if (!SequencerPtr.Pin()->IsLevelEditorSequencer())
		{
			MenuBuilder.AddMenuEntry(FSequencerCommands::Get().ToggleKeepPlaybackRangeInSectionBounds);
		}
		
		MenuBuilder.AddMenuEntry(FSequencerCommands::Get().ToggleLinkCurveEditorTimeRange);
	}
	MenuBuilder.EndSection();

	// Menu entry for the jump frame increment
	auto OnJumpFrameIncrementChanged = [this](double NewValue) {
		FFrameRate TickResolution = SequencerPtr.Pin()->GetFocusedTickResolution();
		FFrameRate DisplayRate = SequencerPtr.Pin()->GetFocusedDisplayRate();
		FFrameNumber JumpFrameIncrement = FFrameRate::TransformTime(FFrameTime::FromDecimal(NewValue), TickResolution, DisplayRate).CeilToFrame();						
		SequencerPtr.Pin()->GetSequencerSettings()->SetJumpFrameIncrement(JumpFrameIncrement);
	};

	MenuBuilder.AddWidget(
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
			[
				SNew(SSpacer)
			]
		+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSpinBox<double>)
					.TypeInterface(SequencerPtr.Pin().Get(), &FSequencer::GetNumericTypeInterface, UE::Sequencer::ENumericIntent::Duration)
					.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
					.OnValueCommitted_Lambda([=](double Value, ETextCommit::Type){ OnJumpFrameIncrementChanged(Value); })
					.OnValueChanged_Lambda([=](double Value) { OnJumpFrameIncrementChanged(Value); })
					.MinValue(TOptional<double>())
					.MaxValue(TOptional<double>())
					.Value_Lambda([this]() -> double {
						FFrameNumber JumpFrameIncrement = SequencerPtr.Pin()->GetSequencerSettings()->GetJumpFrameIncrement();
						FFrameRate TickResolution = SequencerPtr.Pin()->GetFocusedTickResolution();
						FFrameRate DisplayRate = SequencerPtr.Pin()->GetFocusedDisplayRate();
						int32 ConvertedValue = FFrameRate::TransformTime(JumpFrameIncrement, DisplayRate, TickResolution).CeilToFrame().Value;						
					 	return ConvertedValue;
					})
					.Delta(this, &SSequencer::GetSpinboxDelta)
					.LinearDeltaSensitivity(25)
			],
		LOCTEXT("JumpFrameIncrement", "Jump Frame Increment"));

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SSequencer::MakeRenderMovieMenu()
{
	FMenuBuilder MenuBuilder( false, SequencerPtr.Pin()->GetCommandBindings() );

	MenuBuilder.BeginSection( "RenderMovie", LOCTEXT( "RenderMovieMenuHeader", "Render Movie" ) );
	{
		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
		TArray<FString> MovieRendererNames = SequencerModule.GetMovieRendererNames();

		for (FString MovieRendererName : MovieRendererNames)
		{
			MenuBuilder.AddMenuEntry(
				FText::FromString(MovieRendererName),
				FText::FromString(MovieRendererName),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, MovieRendererName] { SequencerPtr.Pin()->GetSequencerSettings()->SetMovieRendererName(MovieRendererName); }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this, MovieRendererName]
					{ 
						return MovieRendererName == SequencerPtr.Pin()->GetMovieRendererName();
					})),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}		

		MenuBuilder.AddMenuEntry(
			LOCTEXT("RenderMovieLegacy", "Movie Scene Capture (Legacy)"),
			LOCTEXT("RenderMovieTooltip", "Movie Scene Capture (Legacy)"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this] { SequencerPtr.Pin()->GetSequencerSettings()->SetMovieRendererName(TEXT("MovieSceneCapture")); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]
				{
					return SequencerPtr.Pin()->GetMovieRendererName() == TEXT("MovieSceneCapture");
				})),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SSequencer::MakeSnapMenu()
{	
	static const FName MenuName("Sequencer.SnapOptions");
	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		UToolMenu* Toolbar = UToolMenus::Get()->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu);
		Toolbar->bSearchable = false;
		{
			FToolMenuSection& Section = Toolbar->AddSection( "KeyAndSectionsSnapping", LOCTEXT( "SnappingMenuKeyAndSectionsHeader", "Key and Sections Snapping" ) );
			Section.AddMenuEntry( FSequencerCommands::Get().ToggleSnapKeyTimesToElements );
			Section.AddMenuEntry( FSequencerCommands::Get().ToggleSnapSectionTimesToElements);
		}

		{
			FToolMenuSection& Section = Toolbar->AddSection( "PlayHeadSnapping", LOCTEXT( "SnappingMenuPlayHeadHeader", "Play Head Snapping" ) );
			Section.AddMenuEntry( FSequencerCommands::Get().ToggleSnapPlayTimeToKeys );
			Section.AddMenuEntry( FSequencerCommands::Get().ToggleSnapPlayTimeToSections );
			Section.AddMenuEntry( FSequencerCommands::Get().ToggleSnapPlayTimeToMarkers );
			Section.AddMenuEntry( FSequencerCommands::Get().ToggleSnapPlayTimeToPressedKey );
			Section.AddMenuEntry( FSequencerCommands::Get().ToggleSnapPlayTimeToDraggedKey );
		}

		{
			FToolMenuSection& Section = Toolbar->AddSection( "CurveSnapping", LOCTEXT( "SnappingMenuCurveHeader", "Curve Snapping" ) );
			Section.AddMenuEntry( FSequencerCommands::Get().ToggleSnapCurveValueToInterval );
		}
	}

	USequencerToolMenuContext* ContextObject = NewObject<USequencerToolMenuContext>();
	ContextObject->WeakSequencer = SequencerPtr;
	FToolMenuContext Context(SequencerPtr.Pin()->GetCommandBindings(), nullptr, ContextObject);

	return UToolMenus::Get()->GenerateWidget(MenuName, Context);
}

TSharedRef<SWidget> SSequencer::MakeTimeRange(const TSharedRef<SWidget>& InnerContent, bool bShowWorkingRange, bool bShowViewRange, bool bShowPlaybackRange)
{
	ISequencerWidgetsModule& SequencerWidgets = FModuleManager::Get().LoadModuleChecked<ISequencerWidgetsModule>( "SequencerWidgets" );

	EShowRange ShowRange = EShowRange::None;
	if (bShowWorkingRange)
	{
		ShowRange |= EShowRange::WorkingRange;
	}
	if (bShowViewRange)
	{
		ShowRange |= EShowRange::ViewRange;
	}
	if (bShowPlaybackRange)
	{
		ShowRange |= EShowRange::PlaybackRange;
	}

	FTimeRangeArgs Args(
		ShowRange,
		TimeSliderController.ToSharedRef(),
		EVisibility::Visible,
		MakeAttributeSP(SequencerPtr.Pin().Get(), &FSequencer::GetNumericTypeInterface, UE::Sequencer::ENumericIntent::Position),
		GetSequencerSettings()->GetPlaybackRangeStartColor(),
		GetSequencerSettings()->GetPlaybackRangeEndColor());
	return SequencerWidgets.CreateTimeRange(Args, InnerContent);
}

TSharedPtr<ITimeSlider> SSequencer::GetTopTimeSliderWidget() const
{
	return TopTimeSlider;
}

SSequencer::~SSequencer()
{
	using namespace UE::Sequencer;

	if (SimpleViewTimeline.IsValid())
	{
		SimpleViewTimeline->Shutdown();
		SimpleViewTimeline.Reset();
	}

	USelection::SelectionChangedEvent.RemoveAll(this);

	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	if(Sequencer && Sequencer->GetToolkitHost() && Sequencer->GetToolkitHost()->GetTabManager())
	{
		if (Sequencer->GetHostCapabilities().bSupportsCurveEditor)
		{
			TSharedPtr<FEditorViewModel> RootModel = Sequencer->GetViewModel();
			FCurveEditorExtension* CurveEditorExtension = RootModel->CastDynamicChecked<FCurveEditorExtension>();
			CurveEditorExtension->CloseCurveEditor();
		}
	}

	if (TSharedPtr<SWindow> Window = WeakExposedBindingsWindow.Pin())
	{
		Window->DestroyWindowImmediately();
	}

	if (TSharedPtr<SWindow> Window = WeakNodeGroupWindow.Pin())
	{
		Window->DestroyWindowImmediately();
		NodeGroupManager.Reset();
	}

	// Ensure the FilterBarWidget destructor is called
	FilterBarSplitterContainer->SetContent(SNullWidget::NullWidget);

	CloseSelectionDetailsTab();

	UnregisterSelectionDetailsTab();
}

void SSequencer::RegisterActiveTimerForPlayback()
{
	if (!bIsActiveTimerRegistered)
	{
		bIsActiveTimerRegistered = true;
		RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SSequencer::EnsureSlateTickDuringPlayback));
	}
}


EActiveTimerReturnType SSequencer::EnsureSlateTickDuringPlayback(double InCurrentTime, float InDeltaTime)
{
	if (SequencerPtr.IsValid())
	{
		auto PlaybackStatus = SequencerPtr.Pin()->GetPlaybackStatus();
		if (PlaybackStatus == EMovieScenePlayerStatus::Playing || PlaybackStatus == EMovieScenePlayerStatus::Scrubbing)
		{
			return EActiveTimerReturnType::Continue;
		}
	}

	bIsActiveTimerRegistered = false;
	return EActiveTimerReturnType::Stop;
}

void SSequencer::UpdateLayoutTree()
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	if ( Sequencer.IsValid() )
	{
		// Update the node tree
		Sequencer->GetNodeTree()->Update();

		// Existing track lanes are kept so they (and the keys, decoration backgrounds, and
		// section widgets they own) remain on screen. STableViewBase's incremental
		// generation pass reuses any rows whose models are still in the tree and only
		// retires rows whose models are gone, which in turn drops references to the
		// track lanes that should no longer exist.
		TreeView->Refresh();

		if (!NodePathToRename.IsEmpty())
		{
			TArray<TSharedPtr<FViewModel>> AllNodes;
			Sequencer->GetNodeTree()->GetAllNodes(AllNodes);
			for (TSharedPtr<FViewModel> Node : AllNodes)
			{
				const FString CurNodePath = IOutlinerExtension::GetPathName(Node);
				if (CurNodePath == NodePathToRename)
				{
					GEditor->GetTimerManager()->SetTimerForNextTick([Node]
					{
						if (IRenameableExtension* Rename = Node->CastThis<IRenameableExtension>())
						{
							Rename->OnRenameRequested().Broadcast();
						}
					});
					break;
				}
			}
			NodePathToRename.Empty();
		}

		// Isolate binding object guids after the tree view is refreshed and the new tracks are created
		if (!NewNodePathsToIsolate.IsEmpty())
		{
			for (const TViewModelPtr<IOutlinerExtension>& OutlinerItem : Sequencer->GetNodeTree()->GetRootNode()->GetDescendantsOfType<IOutlinerExtension>())
			{
				const FString ItemPath = IOutlinerExtension::GetPathName(OutlinerItem);
				if (NewNodePathsToIsolate.Contains(ItemPath))
				{
					Sequencer->GetFilterBar()->IsolateTracks({ OutlinerItem }, true);
					NewNodePathsToIsolate.Remove(ItemPath);
				}
			}
			NewNodePathsToIsolate.Empty();
		}

		// Local Mute/Solo state (transient, non dirtying, mute/solo evaluation)
		if (Sequencer->GetFocusedMovieSceneSequence())
		{
			TSharedPtr<FSharedViewModelData> SharedData    = Sequencer->GetViewModel()->GetRootModel()->GetSharedData();
			FOutlinerCacheExtension*         OutlinerCache = SharedData->CastThis<FOutlinerCacheExtension>();
			FDeactiveStateCacheExtension*    DeactiveState = SharedData->CastThis<FDeactiveStateCacheExtension>();
			FMuteStateCacheExtension*        MuteState     = SharedData->CastThis<FMuteStateCacheExtension>();
			FSoloStateCacheExtension*        SoloState     = SharedData->CastThis<FSoloStateCacheExtension>();

			check(OutlinerCache && DeactiveState && MuteState && SoloState);

			// Hack - we shouldn't really not just forcibly update these here, but currently this function is getting forcibly called before
			//        UpdateCachedFlags has a chance to naturally update itself in response to the signature change
			OutlinerCache->UpdateCachedFlags();

			const bool bAnySoloNodes = EnumHasAnyFlags(SoloState->GetRootFlags(), ECachedSoloState::Soloed | ECachedSoloState::PartiallySoloedChildren);

			bool bAnyChanged = false;

			for (TViewModelPtr<ITrackExtension> TrackNode : Sequencer->GetNodeTree()->GetRootNode()->GetDescendantsOfType<ITrackExtension>())
			{
				// Skip nodes that also implement ITrackRowExtension. These represent
				// individual rows within a shared track (e.g. Animation Mixer layers)
				// and will be handled by the per-row loop below. Applying track-level
				// SetLocalEvalDisabled here would disable the entire shared track
				// based on a single row's mute state.
				if (TrackNode.AsModel()->IsA<ITrackRowExtension>())
				{
					continue;
				}

				UMovieSceneTrack* const Track = TrackNode->GetTrack();
				if (!IsValid(Track))
				{
					continue;
				}

				const ECachedMuteState MuteFlags = MuteState->GetCachedFlags(TrackNode);
				const ECachedSoloState SoloFlags = SoloState->GetCachedFlags(TrackNode);

				const bool bIsMuted = EnumHasAnyFlags(MuteFlags, ECachedMuteState::Muted | ECachedMuteState::ImplicitlyMutedByParent);
				const bool bIsSoloed = EnumHasAnyFlags(SoloFlags, ECachedSoloState::Soloed | ECachedSoloState::ImplicitlySoloedByParent);

				const bool bLocalEvalDisabled = bIsSoloed ? false : (bIsMuted || (bAnySoloNodes && !bIsSoloed));

				const bool bCurrentLocalEvalDisabled = Track->IsLocalEvalDisabled();

				if (bLocalEvalDisabled != bCurrentLocalEvalDisabled)
				{
					Track->MarkAsChanged();
					Track->SetLocalEvalDisabled(bLocalEvalDisabled);

					bAnyChanged = true;
				}
			}

			for (TViewModelPtr<ITrackRowExtension> TrackRowNode : Sequencer->GetNodeTree()->GetRootNode()->GetDescendantsOfType<ITrackRowExtension>())
			{
				UMovieSceneTrack* const Track = TrackRowNode->GetParentTrack();
				if (!IsValid(Track))
				{
					continue;
				}

				const ECachedMuteState MuteFlags = MuteState->GetCachedFlags(TrackRowNode);
				const ECachedSoloState SoloFlags = SoloState->GetCachedFlags(TrackRowNode);

				const bool bIsMuted = EnumHasAnyFlags(MuteFlags, ECachedMuteState::Muted | ECachedMuteState::ImplicitlyMutedByParent);
				const bool bIsSoloed = EnumHasAnyFlags(SoloFlags, ECachedSoloState::Soloed | ECachedSoloState::ImplicitlySoloedByParent);

				const bool bLocalEvalDisabled = bIsSoloed ? false : (bIsMuted || (bAnySoloNodes && !bIsSoloed));

				const int32 TrackRowIndex = TrackRowNode->GetRowIndex();
				const bool bCurrentLocalRowEvalDisabled = Track->IsLocalRowEvalDisabled(TrackRowIndex);

				if (bLocalEvalDisabled != bCurrentLocalRowEvalDisabled
					|| (bIsSoloed && (bIsSoloed != bLocalEvalDisabled)))
				{
					Track->MarkAsChanged();
					if (!bLocalEvalDisabled)
					{
						Track->SetLocalEvalDisabled(false);
					}
					Track->SetLocalRowEvalDisabled(bLocalEvalDisabled, TrackRowIndex);

					bAnyChanged = true;
				}

				// If this row also owns a separate nested track (different from its parent track),
				// apply the same mute/solo state to that nested track's local eval flag. Without this,
				// the nested track's sections continue to evaluate when the row is collapsed, because
				// the section outliner models that normally inherit the mute state only exist when the
				// row is expanded.
				TViewModelPtr<ITrackExtension> TrackExtension = TrackRowNode.ImplicitCast();
				if (TrackExtension)
				{
					UMovieSceneTrack* NestedTrack = TrackExtension->GetTrack();
					if (IsValid(NestedTrack) && NestedTrack != Track)
					{
						if (bLocalEvalDisabled != NestedTrack->IsLocalEvalDisabled())
						{
							NestedTrack->MarkAsChanged();
							NestedTrack->SetLocalEvalDisabled(bLocalEvalDisabled);
							bAnyChanged = true;
						}
					}
				}
			}

			// Pass 3: Section owners that are not tracks or track rows (e.g. FSectionOutlinerModel, FDecorationOutlinerModel).
			// Writes only the section's own mute/solo state into bIsLocalEvalDisabled. Parent-inherited
			// disable (row deactivated, parent layer muted, etc.) is queried by UMovieSceneSection::IsLocalEvalDisabled
			// at read time so we don't strand a stale stored flag when the section's outliner model isn't in the tree.
			for (TViewModelPtr<ISectionOwnerExtension> SectionOwner : Sequencer->GetNodeTree()->GetRootNode()->GetDescendantsOfType<ISectionOwnerExtension>())
			{
				// Skip nodes already handled by the track or track row loops above
				if (SectionOwner.AsModel()->IsA<ITrackExtension>() || SectionOwner.AsModel()->IsA<ITrackRowExtension>())
				{
					continue;
				}

				const ECachedMuteState MuteFlags = MuteState->GetCachedFlags(SectionOwner);
				const ECachedSoloState SoloFlags = SoloState->GetCachedFlags(SectionOwner);

				const bool bIsOwnMuted = EnumHasAnyFlags(MuteFlags, ECachedMuteState::Muted);
				const bool bIsOwnSoloed = EnumHasAnyFlags(SoloFlags, ECachedSoloState::Soloed);

				const bool bLocalEvalDisabled = bIsOwnSoloed ? false : (bIsOwnMuted || (bAnySoloNodes && !bIsOwnSoloed));

				for (UMovieSceneSection* Section : SectionOwner->GetSections())
				{
					if (Section && bLocalEvalDisabled != Section->IsLocalEvalDisabled())
					{
						Section->SetLocalEvalDisabled(bLocalEvalDisabled);
						Section->MarkAsChanged();
						bAnyChanged = true;
					}
				}
			}

			if (bAnyChanged)
			{
				Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
			}
		}

		if (NodeGroupManager.IsValid())
		{
			NodeGroupManager->RefreshNodeGroups();
		}
		
		if (SequencerTreeFilterStatusBar.IsValid())
		{
			SequencerTreeFilterStatusBar->UpdateText();
		}
		
	}
}

void SSequencer::UpdateBreadcrumbs()
{
	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	BreadcrumbTrail->ClearCrumbs();

	for (FMovieSceneSequenceID SequenceID : Sequencer->GetSubSequenceHierarchy())
	{
		TAttribute<FText> CrumbNameAttribute;

		if (SequenceID == MovieSceneSequenceID::Root)
		{
			CrumbNameAttribute = MakeAttributeSP(this, &SSequencer::GetBreadcrumbTextForSequence, MakeWeakObjectPtr(SequencerPtr.Pin()->GetRootMovieSceneSequence()), true);
		}
		else
		{
			TWeakObjectPtr<UMovieSceneSubSection> SubSection = Sequencer->FindSubSection(SequenceID);
			CrumbNameAttribute = MakeAttributeSP(this, &SSequencer::GetBreadcrumbTextForSection, SubSection);
		}

		BreadcrumbTrail->PushCrumb( CrumbNameAttribute, FSequencerBreadcrumb( SequenceID, CrumbNameAttribute.Get()) );
	}
}

void SSequencer::ResetBreadcrumbs()
{
	BreadcrumbTrail->ClearCrumbs();

	TAttribute<FText> CrumbNameAttribute = MakeAttributeSP(this, &SSequencer::GetBreadcrumbTextForSequence, MakeWeakObjectPtr(SequencerPtr.Pin()->GetRootMovieSceneSequence()), true);
	BreadcrumbTrail->PushCrumb(CrumbNameAttribute, FSequencerBreadcrumb(MovieSceneSequenceID::Root, CrumbNameAttribute.Get()));
}

void SSequencer::PopBreadcrumb()
{
	BreadcrumbTrail->PopCrumb();
}

void SSequencer::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	// @todo sequencer: Add drop validity cue
}


void SSequencer::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	// @todo sequencer: Clear drop validity cue
}


FReply SSequencer::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	for (FOptionalOnDragDrop Delegate : OnReceivedDragOver)
	{
		if (Delegate.IsBound())
		{
			FReply DelegateReply = FReply::Unhandled();
			if (Delegate.Execute(MyGeometry, DragDropEvent, DelegateReply))
			{
				return DelegateReply;
			}
		}
	}

	bool bIsDragSupported = false;

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (Operation.IsValid() && (
		Operation->IsOfType<FAssetDragDropOp>() ||
		Operation->IsOfType<FClassDragDropOp>() ||
		Operation->IsOfType<FActorDragDropOp>() ||
		Operation->IsOfType<FFolderDragDropOp>() ) )
	{
		bIsDragSupported = true;
	}

	return bIsDragSupported ? FReply::Handled() : FReply::Unhandled();
}


FReply SSequencer::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();

	// Drop operation may not be supported
	if (!Sequencer->GetHostCapabilities().bSupportsDragAndDrop)
	{
		return FReply::Unhandled();
	}
	
	for (FOptionalOnDragDrop Delegate : OnReceivedDrop)
	{
		if (Delegate.IsBound())
		{
			FReply DelegateReply = FReply::Unhandled();
			if (Delegate.Execute(MyGeometry, DragDropEvent, DelegateReply))
			{
				return DelegateReply;
			}
		}
	}

	bool bWasDropHandled = false;

	// @todo sequencer: Get rid of hard-code assumptions about dealing with ACTORS at this level?

	// @todo sequencer: We may not want any actor-specific code here actually.  We need systems to be able to
	// register with sequencer to support dropping assets/classes/actors, or OTHER types!

	// @todo sequencer: Handle drag and drop from other FDragDropOperations, including unloaded classes/asset and external drags!

	// @todo sequencer: Consider allowing drops into the level viewport to add to the MovieScene as well.
	//		- Basically, when Sequencer is open it would take over drops into the level and auto-add puppets for these instead of regular actors
	//		- This would let people drag smoothly and precisely into the view to drop assets/classes into the scene

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

	if (Operation.IsValid() )
	{
		if ( Operation->IsOfType<FAssetDragDropOp>() )
		{
			TSharedPtr<FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);

			OnAssetsDropped( DragDropOp );
			bWasDropHandled = true;
		}
		else if( Operation->IsOfType<FClassDragDropOp>() )
		{
			TSharedPtr<FClassDragDropOp> DragDropOp = StaticCastSharedPtr<FClassDragDropOp>(Operation);

			OnClassesDropped( DragDropOp );
			bWasDropHandled = true;
		}
		else if( Operation->IsOfType<FActorDragDropOp>() )
		{
			TSharedPtr<FActorDragDropOp> DragDropOp = StaticCastSharedPtr<FActorDragDropOp>(Operation);

			OnActorsDropped( DragDropOp );
			bWasDropHandled = true;
		}
		else if (Operation->IsOfType<FFolderDragDropOp>())
		{
			TSharedPtr<FFolderDragDropOp> DragDropOp = StaticCastSharedPtr<FFolderDragDropOp>(Operation);

			OnFolderDropped(DragDropOp);
			bWasDropHandled = true;
		}
		else if (Operation->IsOfType<FCompositeDragDropOp>())
		{
			const TSharedPtr<FCompositeDragDropOp> CompositeOp = StaticCastSharedPtr<FCompositeDragDropOp>(Operation);
			if (const TSharedPtr<FActorDragDropOp> ActorDragDropOp = CompositeOp->GetSubOp<FActorDragDropOp>())
			{
				OnActorsDropped(ActorDragDropOp);
				bWasDropHandled = true;
			}
			if (const TSharedPtr<FFolderDragDropOp> FolderDragDropOp = CompositeOp->GetSubOp<FFolderDragDropOp>())
			{
				OnFolderDropped(FolderDragDropOp);
				bWasDropHandled = true;
			}
		}
	}

	return bWasDropHandled ? FReply::Handled() : FReply::Unhandled();
}

FReply SSequencer::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) 
{
	// A toolkit tab is active, so direct all command processing to it
	const TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	if (!Sequencer.IsValid())
	{
		return FReply::Unhandled();
	}

	// Give the independent timeline commands a chance to process before Sequencer
	if (SimpleViewTimeline.IsValid()
		&& SimpleViewTimeline->IsInSimpleView()
		&& SimpleViewTimeline->GetCommandList()->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	if (Sequencer->GetCommandBindings()->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SSequencer::OnFocusChanging( const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent )
{
	if (NewWidgetPath.ContainsWidget(this))
	{
		OnReceivedFocus.ExecuteIfBound();
	}
}

FReply SSequencer::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// A toolkit tab is active, so direct all command processing to it
	TSharedPtr<FSequencer> SequencerPin = SequencerPtr.Pin();
	if (SequencerPin.IsValid())
	{
		if (SequencerPin->GetCommandBindings()->ProcessCommandBindings(MouseEvent))
		{
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void SSequencer::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
	PendingFocus.SetPendingFocusIfNeeded(AsWeak());
}

void SSequencer::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseLeave(MouseEvent);
	PendingFocus.ResetPendingFocus();
}

void SSequencer::AddFromContentBrowser()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FAssetData> SelectedAssets;
	ContentBrowserModule.Get().GetSelectedAssets(SelectedAssets);

	TSharedRef<FAssetDragDropOp> DragDropOp = FAssetDragDropOp::New(SelectedAssets);
	OnAssetsDropped(DragDropOp.ToSharedPtr());
}

bool SSequencer::CanAddFromContentBrowser() const
{
	FSequencer& SequencerRef = *SequencerPtr.Pin();
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FAssetData> SelectedAssets;
	ContentBrowserModule.Get().GetSelectedAssets(SelectedAssets);
	for (const FAssetData& AssetData : SelectedAssets)
	{
		if (MovieSceneToolHelpers::IsValidAsset(SequencerRef.GetFocusedMovieSceneSequence(), AssetData))
		{
			return true;
		}
	}
	return false;
}

void SSequencer::OnAssetsDropped(TSharedPtr<FAssetDragDropOp> DragDropOp)
{
	using namespace UE::Sequencer;

	FSequencer& SequencerRef = *SequencerPtr.Pin();

	TArray< UObject* > DroppedObjects;
	bool bAllAssetsWereLoaded = true;
	bool bNeedsLoad = false;

	for (const FAssetData& AssetData : DragDropOp->GetAssets())
	{
		if (!AssetData.IsAssetLoaded())
		{
			bNeedsLoad = true;
			break;
		}
	}

	if (bNeedsLoad)
	{
		GWarn->BeginSlowTask(LOCTEXT("OnDrop_FullyLoadPackage", "Fully Loading Package For Drop"), true, false);
	}

	for (const FAssetData& AssetData : DragDropOp->GetAssets())
	{
		if (!MovieSceneToolHelpers::IsValidAsset(SequencerRef.GetFocusedMovieSceneSequence(), AssetData))
		{
			continue;
		}

		UObject* Object = AssetData.GetAsset();

		if ( Object != nullptr )
		{
			DroppedObjects.Add( Object );
		}
		else
		{
			bAllAssetsWereLoaded = false;
		}
	}

	if (bNeedsLoad)
	{
		GWarn->EndSlowTask();
	}

	FGuid TargetObjectGuid;
	// if exactly one object node is selected, we have a target object guid
	if (SequencerPtr.Pin()->GetViewModel()->GetSelection()->Outliner.Num() == 1)
	{
		for (TViewModelPtr<FObjectBindingModel> ObjectBindingNode : SequencerPtr.Pin()->GetViewModel()->GetSelection()->Outliner.Filter<FObjectBindingModel>())
		{
			TargetObjectGuid = ObjectBindingNode->GetObjectGuid();
		}
	}

	ESequencerDropResult DropResult = ESequencerDropResult::Unhandled;

	const FScopedTransaction Transaction(LOCTEXT("DropAssets", "Drop Assets"));

	// See if any callback wants to handle this drop.
	for (FOnAssetsDrop Delegate : OnAssetsDrop)
	{
		if (Delegate.IsBound())
		{
			DropResult = Delegate.Execute(DroppedObjects, *DragDropOp);
			if (DropResult != ESequencerDropResult::Unhandled)
			{
				break;
			}
		}
	}

	// If nobody took care of it, do the default behaviour.
	if (DropResult == ESequencerDropResult::Unhandled)
	{
		FMovieSceneTrackEditor::BeginKeying(SequencerPtr.Pin()->GetLocalTime().Time.FrameNumber);

		TArray<UMovieSceneFolder*> Folders;
		SequencerRef.GetSelectedFolders(Folders);

		for (TArray<UObject*>::TConstIterator CurObjectIter = DroppedObjects.CreateConstIterator(); CurObjectIter; ++CurObjectIter)
		{
			UObject* CurObject = *CurObjectIter;

			if (!SequencerRef.OnHandleAssetDropped(CurObject, TargetObjectGuid))
			{
				// Doesn't make sense to drop a level sequence asset into sequencer as a spawnable actor
				if (CurObject->IsA<ULevelSequence>())
				{
					UE_LOGF(LogSequencer, Warning, "Can't add '%ls' as a spawnable", *CurObject->GetName());
					continue;
				}

				bool bPreferenceReplaceable = FSlateApplication::Get().GetModifierKeys().IsAltDown();

				UE::Sequencer::FCreateBindingParams Params;
				Params.BindingNameOverride = CurObject->GetName();
				Params.bSpawnable = bPreferenceReplaceable ? false : true;
				Params.bReplaceable = true;
				Params.ActorFactory = DragDropOp->GetActorFactory();

				FGuid Guid = SequencerRef.CreateBinding(*CurObject, Params);
				if (Guid.IsValid())
				{
					DropResult = ESequencerDropResult::DropHandled;

					if (Folders.Num() > 0)
					{
						Folders[0]->AddChildObjectBinding(Guid);
					}
				}
			}
		}

		FMovieSceneTrackEditor::EndKeying();
	}

	if (DropResult == ESequencerDropResult::DropHandled)
	{
		// Update the sequencers view of the movie scene data when any object is added
		SequencerRef.NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemAdded );

		// Update the tree and synchronize selection
		UpdateLayoutTree();

		SequencerRef.SynchronizeSequencerSelectionWithExternalSelection();
	}
}


void SSequencer::OnClassesDropped(TSharedPtr<FClassDragDropOp> DragDropOp )
{
	const FScopedTransaction Transaction(LOCTEXT("DropClasses", "Drop Classes"));

	ESequencerDropResult DropResult = ESequencerDropResult::Unhandled;

	for (FOnClassesDrop Delegate : OnClassesDrop)
	{
		if (Delegate.IsBound())
		{
			DropResult = Delegate.Execute(DragDropOp->ClassesToDrop, *DragDropOp);
			if (DropResult != ESequencerDropResult::Unhandled)
			{
				break;
			}
		}
	}

	if (DropResult == ESequencerDropResult::Unhandled)
	{
		FSequencer& SequencerRef = *SequencerPtr.Pin();

		for (auto ClassIter = DragDropOp->ClassesToDrop.CreateConstIterator(); ClassIter; ++ClassIter)
		{
			UClass* Class = (*ClassIter).Get();
			if (Class != nullptr)
			{
				UObject* Object = Class->GetDefaultObject();

				UE::Sequencer::FCreateBindingParams Params;
				Params.bSpawnable = true;

				SequencerRef.CreateBinding(*Object, Params);
			}
		}
	}
}

void SSequencer::OnActorsDropped(TSharedPtr<FActorDragDropOp> DragDropOp )
{
	const FScopedTransaction Transaction(LOCTEXT("DropActors", "Drop Actors"));

	ESequencerDropResult DropResult = ESequencerDropResult::Unhandled;

	for (FOnActorsDrop Delegate : OnActorsDrop)
	{
		if (Delegate.IsBound())
		{
			DropResult = Delegate.Execute(DragDropOp->Actors, *DragDropOp);
			if (DropResult != ESequencerDropResult::Unhandled)
			{
				break;
			}
		}
	}

	if (DropResult == ESequencerDropResult::Unhandled)
	{
		SequencerPtr.Pin()->OnActorsDropped(DragDropOp->Actors);
	}
}

void SSequencer::OnFolderDropped(TSharedPtr<FFolderDragDropOp> DragDropOp )
{
	// Sequencer doesn't support dragging folder with a root object
	if (!FFolder::IsRootObjectPersistentLevel(DragDropOp->RootObject))
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("DropActors", "Drop Actors"));

	ESequencerDropResult DropResult = ESequencerDropResult::Unhandled;

	TArray<TWeakObjectPtr<AActor>> DraggedActors;

	// Find any actors in the global editor world that have any of the dragged paths.
	// WARNING: Actor iteration can be very slow, so this needs to be optimized
	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	UObject* PlaybackContext = Sequencer->GetPlaybackContext();
	UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
	if (World)
	{
		FActorFolders::GetWeakActorsFromFolders(*World, DragDropOp->Folders, DraggedActors);
	}
	
	for (FOnFoldersDrop Delegate : OnFoldersDrop)
	{
		if (Delegate.IsBound())
		{
			DropResult = Delegate.Execute(DragDropOp->Folders, *DragDropOp);
			if (DropResult != ESequencerDropResult::Unhandled)
			{
				break;
			}
		}
	}

	if (DropResult == ESequencerDropResult::Unhandled)
	{
		SequencerPtr.Pin()->OnActorsDropped(DraggedActors);
	}
}

void SSequencer::OnCrumbClicked(const FSequencerBreadcrumb& Item)
{
	if( SequencerPtr.Pin()->GetFocusedTemplateID() != Item.SequenceID )
	{
		SequencerPtr.Pin()->PopToSequenceInstance( Item.SequenceID );
	}
}


FText SSequencer::GetRootAnimationName() const
{
	return SequencerPtr.Pin()->GetRootMovieSceneSequence()->GetDisplayName();
}


TSharedPtr<UE::Sequencer::SOutlinerView> SSequencer::GetTreeView() const
{
	return TreeView;
}


TSharedPtr<UE::Sequencer::SOutlinerView> SSequencer::GetPinnedTreeView() const
{
	return PinnedTreeView;
}


void SSequencer::OnSaveMovieSceneClicked()
{
	SequencerPtr.Pin()->SaveCurrentMovieScene();
}

void SSequencer::OnSaveMovieSceneAsClicked()
{
	SequencerPtr.Pin()->SaveCurrentMovieSceneAs();
}

float SSequencer::GetPinnedAreaMaxHeight() const
{
	if (!MainSequencerArea.IsValid())
	{
		return 0.0f;
	}

	// Allow the pinned area to use up to 2/3rds of the sequencer area
	return MainSequencerArea->GetCachedGeometry().GetLocalSize().Y * 0.666f;
}

EVisibility SSequencer::GetPinnedAreaVisibility() const
{
	return PinnedTreeView->GetNumRootNodes() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SSequencer::GetBreadcrumbTextForSection(TWeakObjectPtr<UMovieSceneSubSection> SubSection) const
{
	UMovieSceneSubSection* SubSectionPtr = SubSection.Get();
	return SubSectionPtr ? GetBreadcrumbTextForSequence(SubSectionPtr->GetSequence(), SubSectionPtr->IsActive()) : FText();
}


FText SSequencer::GetBreadcrumbTextForSequence(TWeakObjectPtr<UMovieSceneSequence> Sequence, bool bIsActive) const
{
	UMovieSceneSequence* SequencePtr = Sequence.Get();

	bool bIsDirty = SequencePtr->GetMovieScene()->GetOuter()->GetPackage()->IsDirty();

	if (bIsActive)
	{
		if (bIsDirty)
		{
			return FText::Format(LOCTEXT("DirtySequenceBreadcrumbFormat", "{0}*"), SequencePtr->GetDisplayName());
		}
		else
		{
			return SequencePtr->GetDisplayName();
		}
	}
	else
	{
		if (bIsDirty)
		{
			return FText::Format(LOCTEXT("DirtyInactiveSequenceBreadcrumbFormat", "{0}* [{1}]"),
				SequencePtr->GetDisplayName(),
				LOCTEXT("InactiveSequenceBreadcrumb", "Inactive"));

		}
		else
		{
			return FText::Format(LOCTEXT("InactiveSequenceBreadcrumbFormat", "{0} [{1}]"),
				SequencePtr->GetDisplayName(),
				LOCTEXT("InactiveSequenceBreadcrumb", "Inactive"));
		}
	}
}

EVisibility SSequencer::GetBreadcrumbNavigateVisibility() const
{
	return CanNavigateBreadcrumbs() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SSequencer::GetBreadcrumbTrailVisibility() const
{
	return SequencerPtr.Pin()->IsLevelEditorSequencer() ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SSequencer::CanNavigateBreadcrumbs() const
{
	if (SequencerPtr.Pin()->IsLevelEditorSequencer())
	{
		UMovieSceneSequence* RootSequence = SequencerPtr.Pin()->GetRootMovieSceneSequence();
		UMovieScene* MovieScene = RootSequence ? RootSequence->GetMovieScene() : nullptr;
		if (RootSequence)
		{
			for (UMovieSceneTrack* Track : MovieScene->GetTracks())
			{
				if (Track && Track->IsA<UMovieSceneSubTrack>())
				{
					return true;
				}
			}
		}
	}

	return false;
}


EVisibility SSequencer::GetBottomTimeSliderVisibility() const
{
	return GetSequencerSettings()->GetShowRangeSlider() ? EVisibility::Hidden : EVisibility::Visible;
}


EVisibility SSequencer::GetTimeRangeVisibility() const
{
	return GetSequencerSettings()->GetShowRangeSlider() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SSequencer::GetInfoButtonVisibility() const
{
	return GetSequencerSettings()->GetShowInfoButton() ? EVisibility::SelfHitTestInvisible : EVisibility::Hidden;
}

EVisibility SSequencer::GetShowTickLines() const
{
	return GetSequencerSettings()->GetShowTickLines() ? EVisibility::SelfHitTestInvisible : EVisibility::Hidden;
}

EVisibility SSequencer::GetShowSequencerToolbar() const
{
	if (SimpleViewTimeline.IsValid() && IsSimpleView())
	{
		const USequencerSimpleViewSettings* const SimpleViewSettings = GetDefault<USequencerSimpleViewSettings>();
		const bool bShowToolbar = SimpleViewSettings->bShowToolBar;
		return bShowToolbar ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
	}

	return GetSequencerSettings()->GetShowSequencerToolbar()
		? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
}

EFrameNumberDisplayFormats SSequencer::GetTimeDisplayFormat() const
{
	return GetSequencerSettings()->GetTimeDisplayFormat();
}

void SSequencer::OnCurveEditorVisibilityChanged(bool bShouldBeVisible)
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();

	// Curve Editor may not be supported
	if (!Sequencer->GetHostCapabilities().bSupportsCurveEditor)
	{
		return;
	}

	TSharedPtr<FEditorViewModel> RootModel = Sequencer->GetViewModel();
	FCurveEditorExtension* CurveEditorExtension = RootModel->CastDynamicChecked<FCurveEditorExtension>();

	if (bShouldBeVisible)
	{
		CurveEditorExtension->OpenCurveEditor();
	}
	else
	{
		CurveEditorExtension->CloseCurveEditor();
	}
}


void SSequencer::OnTimeSnapIntervalChanged(float InInterval)
{
	// @todo: sequencer-timecode: Address dealing with different time intervals
	// TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	// if ( Sequencer.IsValid() )
	// {
	// 	UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	// 	if (!FMath::IsNearlyEqual(MovieScene->GetFixedFrameInterval(), InInterval))
	// 	{
	// 		FScopedTransaction SetFixedFrameIntervalTransaction( NSLOCTEXT( "Sequencer", "SetFixedFrameInterval", "Set scene fixed frame interval" ) );
	// 		MovieScene->Modify();
	// 		MovieScene->SetFixedFrameInterval( InInterval );

	// 		// Update the current time to the new interval
	// 		float NewTime = SequencerHelpers::SnapTimeToInterval(Sequencer->GetLocalTime(), InInterval);
	// 		Sequencer->SetLocalTime(NewTime);
	// 	}
	// }
}


FPaintPlaybackRangeArgs SSequencer::GetSectionPlaybackRangeArgs() const
{
	if (GetBottomTimeSliderVisibility() == EVisibility::Visible)
	{
		static FPaintPlaybackRangeArgs Args(FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_L"), FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_R"), 6.f);
		return Args;
	}
	else
	{
		static FPaintPlaybackRangeArgs Args(FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Bottom_L"), FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Bottom_R"), 6.f);
		return Args;
	}
}


UE::Sequencer::FVirtualTrackArea SSequencer::GetVirtualTrackArea(const UE::Sequencer::STrackAreaView* InTrackArea) const
{
	using namespace UE::Sequencer;

	const STrackAreaView* TargetTrackArea = TrackArea.Get();
	TSharedPtr<SOutlinerView> TargetTreeView = TreeView;
	
	if (InTrackArea != nullptr)
	{
		TargetTrackArea = InTrackArea;
		TargetTreeView = TargetTrackArea->GetOutliner().Pin();
	}

	TSharedPtr<FTrackAreaViewModel> TargetTrackAreaViewModel = TargetTrackArea->GetViewModel();

	return FVirtualTrackArea(*TargetTrackAreaViewModel, *TargetTreeView.Get(), TargetTrackArea->GetCachedGeometry());
}

FPasteContextMenuArgs SSequencer::GeneratePasteArgs(FFrameNumber PasteAtTime, TSharedPtr<FMovieSceneClipboard> Clipboard)
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	if (GetSequencerSettings()->GetForceWholeFrames())
	{
		FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
		FFrameRate DisplayRate    = Sequencer->GetFocusedDisplayRate();

		PasteAtTime = ConvertFrameTime(PasteAtTime, TickResolution, DisplayRate).RoundToFrame();
		PasteAtTime = ConvertFrameTime(PasteAtTime, DisplayRate, TickResolution).FrameNumber;
	}

	// Open a paste menu at the current mouse position
	FSlateApplication& Application = FSlateApplication::Get();
	FVector2D LocalMousePosition = TrackArea->GetCachedGeometry().AbsoluteToLocal(Application.GetCursorPos());

	FVirtualTrackArea VirtualTrackArea = GetVirtualTrackArea();

	// Paste into the currently selected sections, or hit test the mouse position as a last resort
	TArray<TViewModelPtr<IOutlinerExtension>> PasteIntoNodes;
	{
		TSharedPtr<FSequencerSelection> Selection = Sequencer->GetViewModel()->GetSelection();

		TSet<UMovieSceneSection*> Sections = Selection->GetSelectedSections();

		for (FKeyHandle Key : Selection->KeySelection)
		{
			TSharedPtr<FChannelModel> Channel = Selection->KeySelection.GetModelForKey(Key);
			UMovieSceneSection*       Section = Channel ? Channel->GetSection() : nullptr;
			if (Channel)
			{
				Sections.Add(Section);
			}
		}

		for (UMovieSceneSection* Section : Sections)
		{
			if (TSharedPtr<FSectionModel> Handle = Sequencer->GetNodeTree()->GetSectionModel(Section))
			{
				TViewModelPtr<IOutlinerExtension> TrackModel = Handle->GetParentSectionOwnerModel().ImplicitCast();
				if (TrackModel)
				{
					PasteIntoNodes.Add(TrackModel);
				}
			}
		}

		for (FViewModelPtr SelectedNode : Sequencer->GetViewModel()->GetSelection()->Outliner)
		{
			if (SelectedNode->IsA<FCategoryGroupModel>() || SelectedNode->IsA<ITrackExtension>() || SelectedNode->IsA<ITrackRowExtension>() || SelectedNode->IsA<FChannelGroupModel>())
			{
				TViewModelPtr<IOutlinerExtension> TrackModel = CastViewModelChecked<IOutlinerExtension>(SelectedNode);
				if (TrackModel)
				{
					PasteIntoNodes.Add(TrackModel);
				}
			}
		}
	}

	if (PasteIntoNodes.Num() == 0)
	{
		TSharedPtr<FViewModel> Node = VirtualTrackArea.HitTestNode(LocalMousePosition.Y);
		if (Node.IsValid())
		{
			PasteIntoNodes.Add(FViewModelPtr(Node).ImplicitCast());
		}
	}

	return FPasteContextMenuArgs::PasteInto(MoveTemp(PasteIntoNodes), PasteAtTime, Clipboard);
}

void SSequencer::OnPaste()
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	if (Sequencer->GetViewModel()->GetSelection()->Outliner.Num() == 0)
	{
		if (!OpenPasteMenu())
		{
			DoPaste();
		}
	}
	else
	{
		if (!DoPaste())
		{
			OpenPasteMenu();
		}
	}
}

bool SSequencer::CanPaste()
{
	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	// Attempts to deserialize the text into object bindings/tracks that Sequencer understands.
	if (Sequencer->CanPaste(TextToImport))
	{
		return true;
	}

	return SequencerPtr.Pin()->GetClipboardStack().Num() != 0;
}

bool SSequencer::DoPaste()
{
	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();

	return Sequencer->DoPaste();
}

bool SSequencer::OpenPasteMenu()
{
	TSharedPtr<FPasteContextMenu> ContextMenu;

	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	if (Sequencer->GetClipboardStack().Num() != 0)
	{
		FPasteContextMenuArgs Args = GeneratePasteArgs(Sequencer->GetLocalTime().Time.FrameNumber, Sequencer->GetClipboardStack().Last());
		ContextMenu = FPasteContextMenu::CreateMenu(SequencerPtr, Args);
	}

	if (!ContextMenu.IsValid() || !ContextMenu->IsValidPaste())
	{
		return false;
	}
	else if (ContextMenu->AutoPaste())
	{
		return true;
	}

	const bool bShouldCloseWindowAfterMenuSelection = true;
	TSharedPtr<FExtender> MenuExtender = MakeShared<FExtender>();
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, SequencerPtr.Pin()->GetCommandBindings(), MenuExtender);

	ContextMenu->PopulateMenu(MenuBuilder, MenuExtender);

	FWidgetPath Path;
	FSlateApplication::Get().FindPathToWidget(AsShared(), Path);
	
	FSlateApplication::Get().PushMenu(
		AsShared(),
		Path,
		MenuBuilder.MakeWidget(),
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);

	return true;
}

void SSequencer::PasteFromHistory()
{
	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	if (Sequencer->GetClipboardStack().Num() == 0)
	{
		return;
	}

	FPasteContextMenuArgs Args = GeneratePasteArgs(Sequencer->GetLocalTime().Time.FrameNumber);
	TSharedPtr<FPasteFromHistoryContextMenu> ContextMenu = FPasteFromHistoryContextMenu::CreateMenu(SequencerPtr, Args);

	if (ContextMenu.IsValid())
	{
		const bool bShouldCloseWindowAfterMenuSelection = true;
		TSharedPtr<FExtender> MenuExtender = MakeShared<FExtender>();
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, Sequencer->GetCommandBindings(), MenuExtender);

		ContextMenu->PopulateMenu(MenuBuilder, MenuExtender);

		FWidgetPath Path;
		FSlateApplication::Get().FindPathToWidget(AsShared(), Path);
		
		FSlateApplication::Get().PushMenu(
			AsShared(),
			Path,
			MenuBuilder.MakeWidget(),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
			);
	}
}

EVisibility SSequencer::GetDebugVisualizerVisibility() const
{
	return GetSequencerSettings()->ShouldShowDebugVisualization() ? EVisibility::Visible : EVisibility::Collapsed;
}

double SSequencer::GetSpinboxDelta() const
{
	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	return Sequencer->GetDisplayRateDeltaFrameCount();
}

bool SSequencer::IsReadOnly() const
{
	const TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	return Sequencer.IsValid() ? Sequencer->IsReadOnly() : false;
}

void SSequencer::SetReadOnly(const bool bInReadOnly)
{
	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();

	if (Sequencer->GetFocusedMovieSceneSequence())
	{
		const FText TransactionText = bInReadOnly
			? LOCTEXT("LockMovieScene", "Lock Movie Scene")
			: LOCTEXT("UnlockMovieScene", "Unlock Movie Scene");
		const FScopedTransaction Transaction(TransactionText);

		UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
		MovieScene->Modify();
		MovieScene->SetReadOnly(bInReadOnly);

		TArray<UMovieScene*> DescendantMovieScenes;
		MovieSceneHelpers::GetDescendantMovieScenes(Sequencer->GetFocusedMovieSceneSequence(), DescendantMovieScenes);

		for (UMovieScene* DescendantMovieScene : DescendantMovieScenes)
		{
			if (DescendantMovieScene && bInReadOnly != DescendantMovieScene->IsReadOnly())
			{
				DescendantMovieScene->Modify();
				DescendantMovieScene->SetReadOnly(bInReadOnly);
			}
		}

		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::Unknown);
	}
}

void SSequencer::SetPlayTimeClampedByWorkingRange(double Frame)
{
	if (SequencerPtr.IsValid())
	{
		TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
		// Some of our spin boxes need to use an unbounded min/max so that they can drag linearly instead of based on the current value.
		// We clamp the value here by the working range to emulate the behavior of the Cinematic Level Viewport
		FFrameRate   PlayRate = Sequencer->GetLocalTime().Rate;
		UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
		double StartInSeconds = MovieScene->GetEditorData().WorkStart;
		double EndInSeconds = MovieScene->GetEditorData().WorkEnd;

		Frame = FMath::Clamp(Frame, (double)(StartInSeconds*PlayRate).GetFrame().Value, (double)(EndInSeconds*PlayRate).GetFrame().Value);

		Sequencer->SetLocalTime(FFrameTime::FromDecimal(Frame));
	}
}

void SSequencer::SetPlayTime(double Frame)
{
	if (SequencerPtr.IsValid())
	{
		FFrameTime NewFrame = FFrameTime::FromDecimal(Frame);

		TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
		FFrameRate PlayRate = Sequencer->GetLocalTime().Rate;
		double FrameInSeconds = PlayRate.AsSeconds(NewFrame);

		TRange<double> NewWorkingRange = Sequencer->GetClampRange();
		if (FrameInSeconds < NewWorkingRange.GetLowerBoundValue())
		{
			NewWorkingRange.SetLowerBoundValue(FrameInSeconds);
			NewWorkingRange.SetLowerBoundValue(UE::MovieScene::ExpandRange(NewWorkingRange, NewWorkingRange.Size<double>() * 0.1f).GetLowerBoundValue());
		}
		if (FrameInSeconds > NewWorkingRange.GetUpperBoundValue())
		{
			NewWorkingRange.SetUpperBoundValue(FrameInSeconds);
			NewWorkingRange.SetUpperBoundValue(UE::MovieScene::ExpandRange(NewWorkingRange, NewWorkingRange.Size<double>() * 0.1f).GetUpperBoundValue());
		}

		TRange<double> NewViewRange = Sequencer->GetViewRange();
		if (FrameInSeconds < NewViewRange.GetLowerBoundValue())
		{
			NewViewRange.SetLowerBoundValue(FrameInSeconds);
			NewViewRange.SetLowerBoundValue(UE::MovieScene::ExpandRange(NewViewRange, NewViewRange.Size<double>() * 0.1f).GetLowerBoundValue());
		}
		if (FrameInSeconds > NewViewRange.GetUpperBoundValue())
		{
			NewViewRange.SetUpperBoundValue(FrameInSeconds);
			NewViewRange.SetUpperBoundValue(UE::MovieScene::ExpandRange(NewViewRange, NewViewRange.Size<double>() * 0.1f).GetUpperBoundValue());
		}

		Sequencer->SetClampRange(NewWorkingRange);
		
		Sequencer->SetViewRange(NewViewRange);
		
		Sequencer->SetLocalTime(NewFrame);

		// Refocus on the previously focused widget so that user can continue on after setting a time
		if (PlayTimeDisplay)
		{
			PlayTimeDisplay->Refocus();
		}
	}
}

void SSequencer::ApplySequencerCustomizations(const TArrayView<const FSequencerCustomizationInfo> Customizations)
{
	AddMenuExtenders.Reset();
	ToolbarExtenders.Reset();
	ActionsMenuExtenders.Reset();
	ViewMenuExtenders.Reset();

	OnReceivedDragOver.Reset();
	OnReceivedDrop.Reset();
	OnAssetsDrop.Reset();
	OnActorsDrop.Reset();
	OnClassesDrop.Reset();
	OnFoldersDrop.Reset();

	ApplySequencerCustomization(RootCustomization);
	for (const FSequencerCustomizationInfo& Info : Customizations)
	{
		ApplySequencerCustomization(Info);
	}

	ToolbarContainer->SetContent(MakeToolBar());
}

void SSequencer::ApplySequencerCustomization(const FSequencerCustomizationInfo& Customization)
{
	if (Customization.AddMenuExtender != nullptr)
	{
		AddMenuExtenders.Add(Customization.AddMenuExtender);
	}
	if (Customization.ToolbarExtender != nullptr)
	{
		ToolbarExtenders.Add(Customization.ToolbarExtender);
	}
	if (Customization.ActionsMenuExtender != nullptr)
	{
		ActionsMenuExtenders.Add(Customization.ActionsMenuExtender);
	}
	if (Customization.ViewMenuExtender != nullptr)
	{
		ViewMenuExtenders.Add(Customization.ViewMenuExtender);
	}

	if (Customization.OnReceivedDragOver.IsBound())
	{
		OnReceivedDragOver.Add(Customization.OnReceivedDragOver);
	}
	if (Customization.OnReceivedDrop.IsBound())
	{
		OnReceivedDrop.Add(Customization.OnReceivedDrop);
	}
	if (Customization.OnAssetsDrop.IsBound())
	{
		OnAssetsDrop.Add(Customization.OnAssetsDrop);
	}
	if (Customization.OnActorsDrop.IsBound())
	{
		OnActorsDrop.Add(Customization.OnActorsDrop);
	}
	if (Customization.OnClassesDrop.IsBound())
	{
		OnClassesDrop.Add(Customization.OnClassesDrop);
	}
	if (Customization.OnFoldersDrop.IsBound())
	{
		OnFoldersDrop.Add(Customization.OnFoldersDrop);
	}
}

USequencerSettings* SSequencer::GetSequencerSettings() const
{
	if (SequencerPtr.IsValid())
	{
		return SequencerPtr.Pin()->GetSequencerSettings();
	}
	return nullptr;
}


bool SSequencer::RegisterDrawer(FSidebarDrawerConfig&& InDrawerConfig)
{
	if (DetailsSidebar.IsValid())
	{
		return DetailsSidebar->RegisterDrawer(MoveTemp(InDrawerConfig));
	}
	return false;
}

bool SSequencer::UnregisterDrawer(const FName InDrawerId)
{
	if (DetailsSidebar.IsValid())
	{
		return DetailsSidebar->UnregisterDrawer(InDrawerId);
	}
	return false;
}

bool SSequencer::RegisterDrawerSection(const FName InDrawerId, const TSharedPtr<ISidebarDrawerContent>& InSection)
{
	if (DetailsSidebar.IsValid())
	{
		return DetailsSidebar->RegisterDrawerSection(InDrawerId, InSection);
	}
	return false;
}

bool SSequencer::UnregisterDrawerSection(const FName InDrawerId, const FName InSectionId)
{
	if (DetailsSidebar.IsValid())
	{
		return DetailsSidebar->UnregisterDrawerSection(InDrawerId, InSectionId);
	}
	return false;
}

void SSequencer::OnSidebarStateChanged(const FSidebarState& InNewState)
{
	if (USequencerSettings* const SequencerSettings = GetSequencerSettings())
	{
		SequencerSettings->SetSidebarState(InNewState);
	}
}

bool SSequencer::IsSidebarVisible() const
{
	if (USequencerSettings* const SequencerSettings = GetSequencerSettings())
	{
		return SequencerSettings->GetSidebarState().IsVisible();
	}
	return false;
}

void SSequencer::SetSidebarVisible(const bool bInVisible)
{
	USequencerSettings* const SequencerSettings = GetSequencerSettings();
	if (!SequencerSettings)
	{
		return;
	}

	FSidebarState SidebarState = SequencerSettings->GetSidebarState();
	SidebarState.SetVisible(bInVisible);
	SequencerSettings->SetSidebarState(SidebarState);

	RebuildMainContent();
}

void SSequencer::ToggleSidebarVisible()
{
	SetSidebarVisible(!IsSidebarVisible());
}

bool SSequencer::IsSelectionDetailsPoppedOut() const
{
	if (USequencerSettings* const SequencerSettings = GetSequencerSettings())
	{
		return SequencerSettings->IsSelectionDetailsPoppedOut();
	}
	return false;
}

void SSequencer::SetSelectionDetailsPoppedOut(const bool bInPoppedOut)
{
	const TSharedPtr<FTabManager> TabManager = GetTabManager();
	if (!TabManager.IsValid())
	{
		return;
	}

	USequencerSettings* const SequencerSettings = GetSequencerSettings();
	if (!SequencerSettings)
	{
		return;
	}

	SequencerSettings->SetSelectionDetailsPoppedOut(bInPoppedOut);

	if (bInPoppedOut)
	{
		RegisterSelectionDetailsTab();

		// Only proceed with unregistering the drawer if the tab was successfully created
		if (TabManager->TryInvokeTab(FSequencer::SelectionDetailsTabId).IsValid())
		{
			SequencerSettings->SetSelectionDetailsPoppedOut(true);

			if (DetailsSidebar.IsValid() && DetailsSidebar->IsDrawerDocked(FSequencer::SelectionDrawerId))
			{
				DetailsSidebar->SetDrawerDocked(FSequencer::SelectionDrawerId, false);
			}

			UnregisterDrawerSection(FSequencer::SelectionDrawerId, FSequencerSelectionDrawer::UniqueId);
		}
	}
	else
	{
		if (const TSharedPtr<SDockTab> ExistingTab = TabManager->FindExistingLiveTab(FSequencer::SelectionDetailsTabId))
		{
			ExistingTab->RequestCloseTab();
		}

		UnregisterSelectionDetailsTab();

		if (SelectionDetailsDrawer.IsValid())
		{
			RegisterDrawerSection(FSequencer::SelectionDrawerId, SelectionDetailsDrawer);
		}
	}
}

void SSequencer::ToggleSelectionDetailsPoppedOut()
{
	SetSelectionDetailsPoppedOut(!IsSelectionDetailsPoppedOut());
}

void SSequencer::ToggleSidebarSelectionDrawerOpen()
{
	if (!DetailsSidebar.IsValid())
	{
		return;
	}

	if (DetailsSidebar->IsDrawerDocked(FSequencer::SelectionDrawerId))
	{
		DetailsSidebar->UndockAllDrawers();
	}
	else if (DetailsSidebar->IsDrawerOpened(FSequencer::SelectionDrawerId))
	{
		DetailsSidebar->CloseAllDrawers();
	}
	else
	{
		DetailsSidebar->TryOpenDrawer(FSequencer::SelectionDrawerId);
	}
}

void SSequencer::ToggleSidebarDrawerDock()
{
	if (!DetailsSidebar.IsValid())
	{
		return;
	}

	const FName OpenedDrawerId = DetailsSidebar->GetOpenedDrawerId();
	if (!OpenedDrawerId.IsNone())
	{
		DetailsSidebar->SetDrawerDocked(OpenedDrawerId, true);
		DetailsSidebar->CloseAllDrawers();
	}
	else if (DetailsSidebar->HasDrawerDocked())
	{
		DetailsSidebar->UndockAllDrawers();
	}
}

void SSequencer::EnablePendingFocusOnHovering(const bool InEnabled)
{
	PendingFocus.Enable(InEnabled);
	EnableCurveEditorPendingFocusOnHovering(InEnabled);
}

void SSequencer::EnableCurveEditorPendingFocusOnHovering(const bool InEnabled) const
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	if (!Sequencer->GetHostCapabilities().bSupportsCurveEditor)
	{
		return;
	}

	const FCurveEditorExtension* CurveEditorExtension = Sequencer->GetViewModel()->CastDynamic<FCurveEditorExtension>();
	const TSharedPtr<FCurveEditor> CurveEditor = CurveEditorExtension ? CurveEditorExtension->GetCurveEditor() : nullptr;
	const TSharedPtr<SCurveEditorPanel> CurveEditorPanel = CurveEditor.IsValid() ? CurveEditor->GetPanel() : nullptr;
	if (!CurveEditorPanel)
	{
		return;
	}
	
	CurveEditorPanel->EnablePendingFocusOnHovering(InEnabled);
}

TSharedPtr<FSequencerFilterBar> SSequencer::GetFilterBar() const
{
	if (const TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin())
	{
		return Sequencer->GetFilterBar();
	}
	return nullptr;
}

void SSequencer::RebuildSearchAndFilterRow()
{
	using namespace UE::Sequencer;
	
	SearchAndFilterRow->ClearChildren();
	SearchAndFilterRow->AddSlot()
		.AutoHeight()
		[
			SNew(SScrollBox)
			.Orientation(Orient_Horizontal)
			.ScrollBarAlwaysVisible(false)
			.ScrollBarVisibility(EVisibility::Collapsed)
			+ SScrollBox::Slot()
			.HAlign(HAlign_Fill)
			.FillContentSize(1.f)
			[
				FilterRow.ToSharedRef()
			]
		];
	
	const TSharedPtr<FSequencer> SequencerPin = SequencerPtr.Pin();
	check(SequencerPin);

	SearchAndFilterRow->AddSlot()
	.AutoHeight()
	.Padding(0)
	[
		SNew(SFilterBarSwitcher)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("SequencerTrackFilters")))
		.Sequencer(SequencerPin)
		.FilterAreaConfigId(ConfigIds::FilterArea_Outliner)
		.LinkedFilterViewModel(SequencerPin->GetViewModelImpl()->GetLinkedFilterViewModelImpl())
	];
}

TSharedPtr<FTabManager> SSequencer::GetTabManager() const
{
	if (const TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin())
	{
		if (const TSharedPtr<IToolkitHost> ToolkitHost = Sequencer->GetToolkitHost())
		{
			return ToolkitHost->GetTabManager();
		}
	}
	return nullptr;
}

void SSequencer::RegisterSelectionDetailsTab()
{
	const TSharedPtr<FTabManager> TabManager = GetTabManager();
	if (!TabManager.IsValid() || TabManager->HasTabSpawner(FSequencer::SelectionDetailsTabId))
	{
		return;
	}

	TabManager->RegisterTabSpawner(FSequencer::SelectionDetailsTabId
		, FOnSpawnTab::CreateSP(this, &SSequencer::SpawnSelectionDetailsTab))
			.SetDisplayName(LOCTEXT("SequencerSelectionDetailsDisplayName", "Sequencer Details"))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Details")))
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCinematicsCategory());
}

void SSequencer::UnregisterSelectionDetailsTab()
{
	if (const TSharedPtr<FTabManager> TabManager = GetTabManager())
	{
		TabManager->UnregisterTabSpawner(FSequencer::SelectionDetailsTabId);
	}
}

TSharedRef<SDockTab> SSequencer::SpawnSelectionDetailsTab(const FSpawnTabArgs& InArgs)
{
	const TWeakPtr<SSequencer> WeakSequencerWidget = SharedThis(this);

	const TSharedRef<SDockTab> ToolTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.ShouldAutosize(false)
		.OnTabClosed_Lambda([WeakSequencerWidget](const TSharedRef<SDockTab> InDockTab)
			{
				const TSharedPtr<SSequencer> SequencerWidget = WeakSequencerWidget.Pin();
				if (!SequencerWidget.IsValid())
				{
					return;
				}

				if (USequencerSettings* const SequencerSettings = SequencerWidget->GetSequencerSettings())
				{
					SequencerSettings->SetSelectionDetailsPoppedOut(false);
				}

				SequencerWidget->UnregisterSelectionDetailsTab();

				if (ensure(SequencerWidget->SelectionDetailsDrawer.IsValid()))
				{
					SequencerWidget->RegisterDrawerSection(FSequencer::SelectionDrawerId, SequencerWidget->SelectionDetailsDrawer);
				}
			})
		[
			SelectionDetailsDrawer.IsValid() ? SelectionDetailsDrawer->CreateContentWidget() : SNullWidget::NullWidget
		];

	return ToolTab;
}

void SSequencer::CloseSelectionDetailsTab()
{
	if (const TSharedPtr<FTabManager> TabManager = GetTabManager())
	{
		if (const TSharedPtr<SDockTab> ExistingTab = TabManager->FindExistingLiveTab(FSequencer::SelectionDetailsTabId))
		{
			ExistingTab->RequestCloseTab();
		}
	}
}

FText SSequencer::GetSidebarSelectionDrawerToolTipText() const
{
	const TSharedRef<const FInputChord> DrawerOpenActiveChord = FSequencerCommands::Get().ToggleSidebarSelectionDrawerOpen->GetFirstValidChord();
	const TSharedRef<const FInputChord> DrawerDockActiveChord = FSequencerCommands::Get().ToggleSidebarDrawerDock->GetFirstValidChord();

	FText ToolTipText = LOCTEXT("SelectionDetailsPanelTooltip", "Open Sequencer selection details panel.");

	if (DrawerOpenActiveChord->IsValidChord() || DrawerDockActiveChord->IsValidChord())
	{
		ToolTipText = FText::Format(LOCTEXT("ExtendedSelectionDetailsPanelTooltip", "{0}\n"), ToolTipText);
	}

	if (DrawerOpenActiveChord->IsValidChord())
	{
		ToolTipText = FText::Format(LOCTEXT("ExtendedSelectionDrawerOpenDetailsPanelTooltip", "{0}\n"
			"{1} to toggle the drawer open or closed")
			, ToolTipText
			, DrawerOpenActiveChord->GetInputText(true));
	}

	if (DrawerDockActiveChord->IsValidChord())
	{
		ToolTipText = FText::Format(LOCTEXT("ExtendedSelectionDrawerDockDetailsPanelTooltip", "{0}\n"
			"{1} to toggle the drawer docked or undocked")
			, ToolTipText
			, DrawerDockActiveChord->GetInputText(true));
	}

	return ToolTipText;
}

FOptionalSize SSequencer::GetOutlinerWidth() const
{
	// Returns true if the geometry is valid and has non-zero dimensions
	auto GetCachedLocalSize = [this](FVector2D& OutSize) -> bool
		{
			if (OutlinerAndTrackAreaContent.IsValid())
			{
				OutSize = OutlinerAndTrackAreaContent->GetCachedGeometry().GetLocalSize();
				return OutSize.X > 0 || OutSize.Y > 0;
			}
			return false;
		};

	FVector2D LocalSize = FVector2D::ZeroVector;
	if (GetCachedLocalSize(LocalSize))
	{
		// TreeViewWidth is set to SequencerSettings->GetTreeViewWidth() on first construct.
		// If this is the old percentage value, we will need to convert it to a pixel width value.
		const float WorkingWidth = FMath::Max(0.f, TreeViewWidth);
		const bool bIsPercent = WorkingWidth > 0.f && WorkingWidth < 1.f;
		const float PixelWidth = (LocalSize.X > 0.f && bIsPercent) ? WorkingWidth * LocalSize.X : WorkingWidth;
		return FMath::Clamp<float>(PixelWidth, MinOutlinerWidth, LocalSize.X);
	}

	// No reliable geometry yet
	return FMath::Max<float>(TreeViewWidth, MinOutlinerWidth);
}

bool SSequencer::IsTransportControlsVisible() const
{
	const USequencerSettings* const SequencerSettings = GetSequencerSettings();
	return SequencerSettings && SequencerSettings->ShouldShowTransportControls();
}

void SSequencer::SetTransportControlsVisible(const bool bInVisible)
{
	USequencerSettings* const SequencerSettings = GetSequencerSettings();
	if (!SequencerSettings || SequencerSettings->ShouldShowTransportControls() == bInVisible)
	{
		return;
	}

	SequencerSettings->SetShowTransportControls(bInVisible);

	if (bInVisible)
	{
		const bool bShouldRestorePoppedOut = SequencerSettings->ShouldRestoreTransportControlsPoppedOut();

		// Restore the live state from the hidden persisted preference
		SequencerSettings->SetTransportControlsPoppedOut(bShouldRestorePoppedOut);

		if (TransportControlsPopoutTab.IsValid())
		{
			TransportControlsPopoutTab->Popout(bShouldRestorePoppedOut);
		}
	}
	else
	{
		// Hide the visible popout window if present, but do not erase the restore preference
		if (TransportControlsPopoutTab.IsValid())
		{
			TransportControlsPopoutTab->Popout(false);
		}

		// Live state becomes "not currently popped out" while hidden
		SequencerSettings->SetTransportControlsPoppedOut(false);
	}

	RebuildMainContent();
}

void SSequencer::ToggleTransportControlsVisible()
{
	SetTransportControlsVisible(!IsTransportControlsVisible());
}

bool SSequencer::IsTransportControlsPoppedOut() const
{
	return TransportControlsPopoutTab.IsValid() ? TransportControlsPopoutTab->IsPoppedOut() : false;
}

void SSequencer::SetTransportControlsPoppedOut(const bool bInPoppedOut)
{
	USequencerSettings* const SequencerSettings = GetSequencerSettings();
	if (!SequencerSettings)
	{
		return;
	}

	const bool bWasTransportControlsPoppedOut = IsTransportControlsPoppedOut();

	// Update both the live state and the hidden persisted restore preference
	SequencerSettings->SetTransportControlsPoppedOut(bInPoppedOut);
	SequencerSettings->SetRestoreTransportControlsPoppedOut(bInPoppedOut);

	if (SequencerSettings->ShouldShowTransportControls() && TransportControlsPopoutTab.IsValid())
	{
		TransportControlsPopoutTab->Popout(bInPoppedOut);
	}

	if (bInPoppedOut)
	{
		if (!bWasTransportControlsPoppedOut)
		{
			SetSelectionDetailsPoppedOut(true);
		}
	}
	else
	{
		if (bWasTransportControlsPoppedOut)
		{
			SetSelectionDetailsPoppedOut(false);
		}
	}

	RebuildMainContent();
}

void SSequencer::ToggleTransportControlsPoppedOut()
{
	if (const USequencerSettings* const SequencerSettings = GetSequencerSettings())
	{
		SetTransportControlsPoppedOut(!SequencerSettings->ShouldRestoreTransportControlsPoppedOut());
	}
}

EVisibility SSequencer::GetPopoutTransportControlsVisibility() const
{
	const TSharedPtr<ISequencer> Sequencer = SequencerPtr.Pin();
	if (!Sequencer.IsValid())
	{
		return EVisibility::Collapsed;
	}

	const USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!SequencerSettings)
	{
		return EVisibility::Visible;
	}

	const bool ShowTransportControls = SequencerSettings->ShouldShowTransportControls();

	return UE::Sequencer::IsPopoutTransportControlsVisible(Sequencer.ToSharedRef(), ShowTransportControls)
		? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedPtr<UE::Sequencer::SimpleView::FSimpleViewTimeline> SSequencer::GetSimpleViewTimeline() const
{
	return SimpleViewTimeline;
}

bool SSequencer::IsSimpleView() const
{
	return SimpleViewTimeline.IsValid() ? SimpleViewTimeline->IsInSimpleView() : false;
}

void SSequencer::EnableSimpleView(const bool bInEnable)
{
	if (SimpleViewTimeline.IsValid())
	{
		SimpleViewTimeline->EnableSimpleView(bInEnable);

		RebuildMainContent();
	}
}

void SSequencer::ToggleSimpleView()
{
	EnableSimpleView(!IsSimpleView());
}

void SSequencer::ToggleToolBarVisible()
{
	const TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	USequencerSettings* const SequencerSettings = Sequencer.IsValid() ? Sequencer->GetSequencerSettings() : nullptr;
	if (SequencerSettings)
	{
		SequencerSettings->SetShowSequencerToolbar(!SequencerSettings->GetShowSequencerToolbar());
	}
}

#undef LOCTEXT_NAMESPACE
