// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimatedRange.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "UObject/GCObject.h"
#include "Misc/NotifyHook.h"
#include "Widgets/SCompoundWidget.h"
#include "MovieSceneSequenceID.h"
#include "ITimeSlider.h"
#include "ISequencerModule.h"
#include "ISequencerWidget.h"
#include "ToolMenu.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Text/STextBlock.h"
#include "Sequencer.h"
#include "SequencerWidgetsDelegates.h"
#include "STemporarilyFocusedSpinBox.h"
#include "WidgetFocusUtils.h"

class FActorDragDropOp;
class FFolderDragDropOp;
class FAssetDragDropOp;
class FClassDragDropOp;
class FMovieSceneClipboard;
class FSequencerSelectionDrawer;
class FSequencerTimeSliderController;
class FSequencerTrackFilter;
class FSequencerViewOptionsMenu;
class ISidebarDrawerContent;
class SCurveEditorTree;
class SSequencerFilterBar;
class SSequencerTransformBox;
class SSequencerStretchBox;
class SCurveEditorPanel;
class SBox;
class SDockTab;
class SFilterSearchBox;
class SSidebar;
class SSidebarContainer;
class SWindow;
class USequencerSettings;
class SPopoutTabInlineContent;
class SSequencerGroupManager;
class SSequencerSearchBox;
class SSequencerTreeFilterStatusBar;
class SFilterSearchBox;
enum class ESequencerFilterChange : uint8;
struct FPaintPlaybackRangeArgs;
struct FSequencerCustomizationInfo;
struct FSidebarDrawerConfig;
struct FTimeSliderArgs;

namespace UE
{
namespace Sequencer
{
class SFilterBarSwitcher;
class SOutlinerView;
class SSequencerSearchAndFilterRow;
class STrackAreaView;
class FVirtualTrackArea;
class IOutlinerColumn;
struct FSequencerSelectionCurveFilter;

namespace SimpleView
{
	class FSimpleViewTimeline;
}

} // namespace Sequencer
} // namespace UE

namespace SequencerLayoutConstants
{
	/** The amount to indent child nodes of the layout tree */
	const float IndentAmount = 12.0f;

	/** Height of each folder node */
	const float FolderNodeHeight = 20.0f;

	/** Height of each object node */
	const float ObjectNodeHeight = 20.0f;

	/** Height of each section area if there are no sections (note: section areas may be larger than this if they have children. This is the height of a section area with no children or all children hidden) */
	const float SectionAreaDefaultHeight = 27.0f;

	/** Height of each key area */
	const float KeyAreaHeight = 15.0f;

	/** Height of each category node */
	const float CategoryNodeHeight = 15.0f;
}


/**
 * The kind of breadcrumbs that sequencer uses
 */
struct FSequencerBreadcrumb
{
	/** The movie scene this may point to */
	FMovieSceneSequenceID SequenceID;

	/** The display name of this breadcrumb */
	FText BreadcrumbName;

	FSequencerBreadcrumb(FMovieSceneSequenceIDRef InSequenceID, FText CrumbName)
		: SequenceID(InSequenceID)
		, BreadcrumbName(CrumbName)
	{ }
};


/**
 * Holds an outliner column and its visibility state
 */
struct FSequencerOutlinerColumnVisibility
{
	TSharedPtr<UE::Sequencer::IOutlinerColumn> Column;
	bool bIsColumnVisible = false;

	FSequencerOutlinerColumnVisibility(TSharedPtr<UE::Sequencer::IOutlinerColumn> InColumn);
	FSequencerOutlinerColumnVisibility(TSharedPtr<UE::Sequencer::IOutlinerColumn> InColumn, bool bInIsColumnVisible);

	bool operator==(const FSequencerOutlinerColumnVisibility& InOther) const;
};


/**
 * Main sequencer UI widget
 */
class SSequencer
	: public SCompoundWidget
	, public FGCObject
	, public FNotifyHook
	, public UE::Sequencer::ISequencerWidget
{
public:

	static constexpr float DefaultOutlinerWidth = 250.f;
	static constexpr float MinOutlinerWidth = 150.f;

	DECLARE_DELEGATE_OneParam( FOnToggleBoolOption, bool )
	SLATE_BEGIN_ARGS( SSequencer )
	{ }
		/** The current view range (seconds) */
		SLATE_ATTRIBUTE( FAnimatedRange, ViewRange )

		/** The current clamp range (seconds) */
		SLATE_ATTRIBUTE( FAnimatedRange, ClampRange )

		/** The playback range */
		SLATE_ATTRIBUTE( TRange<FFrameNumber>, PlaybackRange )

		/** The time bounds */
		SLATE_ATTRIBUTE(TRange<FFrameNumber>, TimeBounds)

		/** The selection range */
		SLATE_ATTRIBUTE( TRange<FFrameNumber>, SelectionRange)

		/** The Vertical Frames */
		SLATE_ATTRIBUTE(TSet<FFrameNumber>, VerticalFrames)

		/** The Marked Frames */
		SLATE_ATTRIBUTE(TArray<FMovieSceneMarkedFrame>, MarkedFrames)

		/** The Global Marked Frames */
		SLATE_ATTRIBUTE(TArray<FMovieSceneMarkedFrame>, GlobalMarkedFrames)

		/** The current sub sequence range */
		SLATE_ATTRIBUTE( TOptional<TRange<FFrameNumber>>, SubSequenceRange)

		/** Called to populate the playback speeds menu. */
		SLATE_EVENT(ISequencer::FOnGetPlaybackSpeeds, OnGetPlaybackSpeeds)

		/** The playback status */
		SLATE_ATTRIBUTE( EMovieScenePlayerStatus::Type, PlaybackStatus )

		/** Called when the user changes the playback range */
		SLATE_EVENT( FOnFrameRangeChanged, OnPlaybackRangeChanged )

		/** Called when the user has begun dragging the playback range */
		SLATE_EVENT( FSimpleDelegate, OnPlaybackRangeBeginDrag )

		/** Called when the user has finished dragging the playback range */
		SLATE_EVENT( FSimpleDelegate, OnPlaybackRangeEndDrag )

		/** Called when the user changes the selection range */
		SLATE_EVENT( FOnFrameRangeChanged, OnSelectionRangeChanged )

		/** Called when the user has begun dragging the selection range */
		SLATE_EVENT( FSimpleDelegate, OnSelectionRangeBeginDrag )

		/** Called when the user has finished dragging the selection range */
		SLATE_EVENT( FSimpleDelegate, OnSelectionRangeEndDrag )

		/** Called when the user has begun dragging a mark */
		SLATE_EVENT(FSimpleDelegate, OnMarkBeginDrag)

		/** Called when the user has finished dragging a mark */
		SLATE_EVENT(FSimpleDelegate, OnMarkEndDrag)

		/** Whether the playback range is locked */
		SLATE_ATTRIBUTE( bool, IsPlaybackRangeLocked )

		/** Called when the user toggles the play back range lock */
		SLATE_EVENT( FSimpleDelegate, OnTogglePlaybackRangeLocked )

		/** The current scrub position in (seconds) */
		SLATE_ATTRIBUTE( FFrameTime, ScrubPosition )

		/** The current scrub position text */
		SLATE_ATTRIBUTE( FString, ScrubPositionText )

		/** The parent sequence that the scrub position display text is relative to */
		SLATE_ATTRIBUTE( FMovieSceneSequenceID, ScrubPositionParent )

		/** Called when the scrub position parent sequence is changed */
		SLATE_EVENT( FOnScrubPositionParentChanged, OnScrubPositionParentChanged )

		/** Attribute for the parent sequence chain of the current sequence */
		SLATE_ATTRIBUTE( TArray<FMovieSceneSequenceID>, ScrubPositionParentChain )

		/** Called when the user changes the view range */
		SLATE_EVENT( FOnViewRangeChanged, OnViewRangeChanged )

		/** Called when the user sets a marked frame */
		SLATE_EVENT(FOnSetMarkedFrame, OnSetMarkedFrame)

		/** Called when the user adds a marked frame */
		SLATE_EVENT(FOnAddMarkedFrame, OnAddMarkedFrame)

		/** Called when all marked frames should be deleted */
		SLATE_EVENT( FSimpleDelegate, OnDeleteAllMarkedFrames)

		/** Whether marked frames are locked */
		SLATE_ATTRIBUTE( bool, AreMarkedFramesLocked )

		/** Called when the user toggles the marked frames lock */
		SLATE_EVENT( FSimpleDelegate, OnToggleMarkedFramesLocked )

		/** Called when the user changes the clamp range */
		SLATE_EVENT( FOnTimeRangeChanged, OnClampRangeChanged )

		/** Called to get the nearest key */
		SLATE_EVENT( FOnGetNearestKey, OnGetNearestKey )

		/** Called when the user has begun scrubbing */
		SLATE_EVENT( FSimpleDelegate, OnBeginScrubbing )

		/** Called when the user has finished scrubbing */
		SLATE_EVENT( FSimpleDelegate, OnEndScrubbing )

		/** Called when the user changes the scrub position */
		SLATE_EVENT( FOnScrubPositionChanged, OnScrubPositionChanged )

		/** Called when any widget contained within sequencer has received focus */
		SLATE_EVENT( FSimpleDelegate, OnReceivedFocus )

		/** Called when initializing tool menu context */
		SLATE_EVENT(FOnInitToolMenuContext, OnInitToolMenuContext)

		/** Called when something is dragged over the sequencer. */
		SLATE_EVENT( FOptionalOnDragDrop, OnReceivedDragOver )

		/** Called when something is dropped onto the sequencer. */
		SLATE_EVENT( FOptionalOnDragDrop, OnReceivedDrop )

		/** Called when an asset is dropped on the sequencer. Not called if OnReceivedDrop is bound and returned true. */
		SLATE_EVENT( FOnAssetsDrop, OnAssetsDrop )

		/** Called when a class is dropped on the sequencer. Not called if OnReceivedDrop is bound and returned true. */
		SLATE_EVENT( FOnClassesDrop, OnClassesDrop )
		
		/** Called when an actor is dropped on the sequencer. Not called if OnReceivedDrop is bound and returned true. */
		SLATE_EVENT( FOnActorsDrop, OnActorsDrop )

		/** Called when a folder is dropped onto the sequencer. Not called if OnReceivedDrop is bound and returned true. */
		SLATE_EVENT(FOnFoldersDrop, OnFoldersDrop)

		/** Extender to use for the add menu. */
		SLATE_ARGUMENT( TSharedPtr<FExtender>, AddMenuExtender )

		/** Extender to use for the toolbar. */
		SLATE_ARGUMENT( TSharedPtr<FExtender>, ToolbarExtender )

		/** Whether to display the playback range spin box in time range slider */
		SLATE_ARGUMENT( bool, ShowPlaybackRangeInTimeSlider )

	SLATE_END_ARGS()


	void Construct(const FArguments& InArgs, TSharedRef<FSequencer> InSequencer);

	void BindCommands(TSharedRef<FUICommandList> SequencerCommandBindings);
	
	~SSequencer();
	
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) { }
	virtual FString GetReferencerName() const override
	{
		return TEXT("SSequencer");
	}

	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

	/** Updates the layout node tree from movie scene data */
	void UpdateLayoutTree();

	/** Causes the widget to register an empty active timer that persists until Sequencer playback stops */
	void RegisterActiveTimerForPlayback();

	/** Updates the breadcrumbs from a change in the shot filter state. */
	void UpdateBreadcrumbs();
	void ResetBreadcrumbs();
	void PopBreadcrumb();

	/** Called when the save button is clicked */
	void OnSaveMovieSceneClicked();
	void OnSaveMovieSceneAsClicked();

	/** Called when the curve editor is shown or hidden */
	void OnCurveEditorVisibilityChanged(bool bShouldBeVisible);

	/** Access the tree view for this sequencer */
	TSharedPtr<UE::Sequencer::SOutlinerView> GetTreeView() const;

	/** Access the pinned tree view for this sequencer */
	TSharedPtr<UE::Sequencer::SOutlinerView> GetPinnedTreeView() const;

	/** 
	 * Generate a helper structure that can be used to transform between phsyical space and virtual space in the track area
	 *
	 * @param InTrackArea	(optional) The track area to generate helper structure for, if not specified the main track area will be used.
	 */
	UE::Sequencer::FVirtualTrackArea GetVirtualTrackArea(const UE::Sequencer::STrackAreaView* InTrackArea = nullptr) const;

	/** Access this widget's track area widget */
	TSharedPtr<UE::Sequencer::STrackAreaView> GetTrackAreaWidget() const { return TrackArea; }

	void OpenTickResolutionOptions();

	/** Sets the play time for the sequence but clamped by the working range. This is useful for cases where we can't clamp via the UI control. */
	void SetPlayTimeClampedByWorkingRange(double Frame);

	/** Sets the play time for the sequence. Will extend the working range if out of bounds. */
	void SetPlayTime(double Frame);

public:

	// FNotifyHook overrides

	void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged);

protected:

	// SWidget interface

	// @todo Sequencer Basic drag and drop support. Doesn't belong here most likely.
	virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	virtual void OnFocusChanging( const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent ) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

private:

	static constexpr float CommonPadding = 3.f;

	TSharedRef<SWidget> ConstructToolBar();
	TSharedRef<SWidget> ConstructSidebarContent();
	TSharedRef<SWidget> ConstructFilterBarContent();
	TSharedRef<SSplitter> ConstructOutlinerAndTrackArea(const FArguments& InArgs, const FTimeSliderArgs& InTimeSliderArgs);
	TSharedRef<SWidget> ConstructMainTreeView(const TSharedRef<FSequencer>& InSequencer);
	TSharedRef<SWidget> ConstructTransportPanel(const TSharedRef<ISequencer>& InSequencer);
	TSharedRef<SWidget> ConstructTransportControls(const TSharedRef<ISequencer>& InSequencer);

	/** Can be called anytime after Construct() to rebuild most of the main content (except for the toolbar).
	 * If the sidebar visibility, sidebar layout orientation, or host capabilities for simple view change, this function must be called. */
	void RebuildMainContent();

	/** Can be called anytime after Construct() to rebuild most of the main content */
	void RebuildFilterBarContent();
	void RebuildSearchAndFilterRow();

	/** Initializes outliner column list from settings and SequencerCore */
	void InitializeOutlinerColumns();

	/** Handles key selection changes. */
	void HandleKeySelectionChanged();

	/** Handles changes to the selected outliner nodes. */
	void HandleOutlinerNodeSelectionChanged();

	/** Empty active timer to ensure Slate ticks during Sequencer playback */
	EActiveTimerReturnType EnsureSlateTickDuringPlayback(double InCurrentTime, float InDeltaTime);	

	TWeakPtr<FSequencer> GetSequencer() { return SequencerPtr; }

	/** Builds the base UToolMenu for Sequencer's toolbar. */
	static void PopulateToolBar(UToolMenu* InMenu);

	/** Registers the tool menu for Sequencer.MainToolBar.ViewOptions. */
	static void RegisterViewOptionsToolMenu();

	/** Makes the toolbar. */
	TSharedRef<SWidget> MakeToolBar();

	/** Makes the add menu for the toolbar. */
	TSharedRef<SWidget> MakeAddMenu();

	/** Makes the actions menu for the toolbar. */
	TSharedRef<SWidget> MakeActionsMenu();

	/** Makes the view menu for the toolbar. */
	TSharedRef<SWidget> MakeViewMenu(const FToolMenuContext& InContext);
	
	/** Populates the view options menu with the default sequencer entries. */
	void PopulateViewOptionMenu(UToolMenu* ToolMenu);

	/** Makes the plabacky menu for the toolbar. */
	TSharedRef<SWidget> MakePlaybackMenu();

	/** Makes the render movie menu for the toolbar. */
	TSharedRef<SWidget> MakeRenderMovieMenu();

	/** Makes the snapping menu for the toolbar. */
	TSharedRef<SWidget> MakeSnapMenu();

	void OpenTaggedBindingManager();

	/** Makes the advanced menu for the toolbar. */
	void FillAdvancedMenu(FMenuBuilder& InMenuBuilder);

	/** Makes the playback speed menu for the toolbar. */
	void FillPlaybackSpeedMenu(FMenuBuilder& InMenuBuilder);

	/** Makes the view density menu for the toolbar. */
	void FillViewDensityMenu(FMenuBuilder& InMenuBuilder);

	/** Makes the column visibility menu for the toolbar. */
	void FillColumnVisibilityMenu(FMenuBuilder& InMenuBuilder);

	/** Makes the playhead scrubbing limit menu for the toolbar. */
	void FillPlayheadScrubbingLimitMenu(FMenuBuilder& InMenuBuilder);

	/** Return the current sequencer settings */ 
	USequencerSettings* GetSequencerSettings() const;

public:
	/** Makes the time display format menu for the toolbar and the play rate menu. */
	static void FillTimeDisplayFormatMenu(UToolMenu* InMenu);

	void OpenNodeGroupsManager();

	TSharedPtr<SSequencerGroupManager> GetNodeGroupsManager() const { return NodeGroupManager; }

public:	
	/** Makes a time range widget with the specified inner content */
	TSharedRef<SWidget> MakeTimeRange(const TSharedRef<SWidget>& InnerContent, bool bShowWorkingRange, bool bShowViewRange, bool bShowPlaybackRange);

	/** Gets the top time sliders widget. */
	TSharedPtr<ITimeSlider> GetTopTimeSliderWidget() const;

	bool IsReadOnly() const;
	void SetReadOnly(const bool bInReadOnly);

private:

	TRange<FFrameNumber> GetViewSpacePlaybackRange(TAttribute<TRange<FFrameNumber>> Range) const;
	void OnViewSpacePlaybackRangeChanged(TRange<FFrameNumber> NewRange, FOnFrameRangeChanged OnPlaybackRangeChanged);

	/**
	 * Called when any outliner column's visibily is modified.
	 * Updates SequencerSettings and visible outliner columns in Outliner View.
	 */
	void UpdateOutlinerViewColumns();

	/**
	* Called when the time snap interval changes.
	*
	* @param InInterval	The new time snap interval
	*/
	void OnTimeSnapIntervalChanged(float InInterval);

	/**
	* @return The value of the current value snap interval.
	*/
	float OnGetValueSnapInterval() const;

	/**
	* Called when the value snap interval changes.
	*
	* @param InInterval	The new value snap interval
	*/
	void OnValueSnapIntervalChanged( float InInterval );

	void AddFromContentBrowser();
	bool CanAddFromContentBrowser() const;

	/**
	 * Called when one or more assets are dropped into the widget
	 *
	 * @param	DragDropOp	Information about the asset(s) that were dropped
	 */
	void OnAssetsDropped(TSharedPtr<FAssetDragDropOp> DragDropOp);
	
	/**
	 * Called when one or more classes are dropped into the widget
	 *
	 * @param	DragDropOp	Information about the class(es) that were dropped
	 */
	void OnClassesDropped(TSharedPtr<FClassDragDropOp> DragDropOp);
		
	/**
	 * Called when one or more actors are dropped into the widget
	 *
	 * @param	DragDropOp	Information about the actor(s) that was dropped
	 */
	void OnActorsDropped(TSharedPtr<FActorDragDropOp> DragDropOp);

	/**
	 * Called when one or more folders are dropped into the widget
	 *
	 * @param	DragDropOp	Information about the objects(s) that was dropped
	 */
	void OnFolderDropped(TSharedPtr<FFolderDragDropOp> DragDropOp);

	/** Called when a breadcrumb is clicked on in the sequencer */
	void OnCrumbClicked(const FSequencerBreadcrumb& Item);

	/** Gets the root movie scene name */
	FText GetRootAnimationName() const;

	/** Get the maximum height the pinned track area should be allowed to be.. */
	float GetPinnedAreaMaxHeight() const;
	
	/** Gets whether or not the Pinned track area should be visible. */
	EVisibility GetPinnedAreaVisibility() const;

	FText GetBreadcrumbTextForSection(TWeakObjectPtr<UMovieSceneSubSection> SubSection) const;
	FText GetBreadcrumbTextForSequence(TWeakObjectPtr<UMovieSceneSequence> Sequence, bool bIsActive) const;

	/** @return Visibility of the breadcrumb navigation widgets */
	EVisibility GetBreadcrumbNavigateVisibility() const;

	/** Gets whether or not the breadcrumb trail should be visible. */
	EVisibility GetBreadcrumbTrailVisibility() const;

public:
	/** Return whether there are breadcrumbs to navigate. */
	bool CanNavigateBreadcrumbs() const;

private:
	/** Gets whether or not the bottom time slider should be visible. */
	EVisibility GetBottomTimeSliderVisibility() const;

	/** Gets whether or not the time range should be visible. */
	EVisibility GetTimeRangeVisibility() const;

	/** Gets whether the info button in the playback controls should be visible. */
	EVisibility GetInfoButtonVisibility() const;

	/** Gets whether the tick lines should be drawn. */
	EVisibility GetShowTickLines() const;

	/** Gets whether the sequencer toolbar should be displayed */
	EVisibility GetShowSequencerToolbar() const;

	/** What is the preferred display format for time values. */
	EFrameNumberDisplayFormats GetTimeDisplayFormat() const;

	/** Gets paint options for painting the playback range on sequencer */
	FPaintPlaybackRangeArgs GetSectionPlaybackRangeArgs() const;

	EVisibility GetDebugVisualizerVisibility() const;

	/** Controls how fast Spinboxes change values. */
	double GetSpinboxDelta() const;

	/** Returns whether or not the Curve Editor is enabled. Allows us to bind to the Slate Enabled attribute. */
	bool GetIsCurveEditorEnabled() const { return !IsReadOnly(); }

public:
	/** On Paste Command */
	void OnPaste();
	bool CanPaste();

	/**
	 * Handle Track Paste
	 * @return Whether the paste event was handled
	 */
	bool DoPaste();

	/** Open the paste menu */
	bool OpenPasteMenu();
	
	/** Open the paste from history menu */
	void PasteFromHistory();

	/** Generate a paste menu args structure */
	struct FPasteContextMenuArgs GeneratePasteArgs(FFrameNumber PasteAtTime, TSharedPtr<FMovieSceneClipboard> Clipboard = nullptr);

	/** This adds the specified path to the selection set to be restored the next time the tree view is refreshed. */
	void AddAdditionalPathToSelectionSet(const FString& Path) { AdditionalSelectionsToAdd.Add(Path); }

	/** Adds a node path to be isolated after the tree view is refreshed and the new tracks are created. */
	void AddNewNodePathsToIsolate(const TSet<FString>& InPaths) { NewNodePathsToIsolate.Append(InPaths); }

	/** Request to rename the given node path. */
	void RequestRenameNode(const FString& Path) { NodePathToRename = Path; }

	/** Applies dynamic sequencer customizations to this editor. */
	void ApplySequencerCustomizations(const TArrayView<const FSequencerCustomizationInfo> Customizations);

	/**
	 * Registers and displays a new drawer in the sidebar.
	 * 
	 * @param InDrawerConfig Configuration info for the new drawer
	 * 
	 * @return True if the new drawer registration was successful.
	 */
	bool RegisterDrawer(FSidebarDrawerConfig&& InDrawerConfig);

	/**
	 * Unregisters and removes a drawer from the sidebar.
	 *
	 * @param InDrawerId Unique drawer Id to unregister
	 * 
	 * @return True if the drawer removal was successful.
	 */
	bool UnregisterDrawer(const FName InDrawerId);

	/**
	 * Registers and displays a new drawer section in the sidebar.
	 * 
	 * @param InDrawerId Unique drawer Id to register
	 * @param InSection Drawer content interface for the section
	 * 
	 * @return True if the new drawer section registration was successful.
	 */
	bool RegisterDrawerSection(const FName InDrawerId, const TSharedPtr<ISidebarDrawerContent>& InSection);

	/**
	 * Unregisters and removes a drawer section from the sidebar.
	 * 
	 * @param InDrawerId Unique drawer Id that contains the section to unregister
	 * @param InSectionId Unique drawer section Id to unregister
	 * 
	 * @return True if the drawer removal was successful.
	 */
	bool UnregisterDrawerSection(const FName InDrawerId, const FName InSectionId);

	/** @return True if the sidebar is being displayed. */
	bool IsSidebarVisible() const;

	/** Set the visibility of the sidebar */
	void SetSidebarVisible(const bool bInVisible);

	/** Toggle the visibility of the sidebar. */
	void ToggleSidebarVisible();

	/** @return True if the selection details are popped out of the Sequencer tab to its own tab window. */
	bool IsSelectionDetailsPoppedOut() const;

	/** Set the popped out state of the selection details in the Sequencer tab. */
	void SetSelectionDetailsPoppedOut(const bool bInPoppedOut);

	/** Toggle the popped out state of the selection details in the Sequencer tab. */
	void ToggleSelectionDetailsPoppedOut();

	/** Toggles the sidebar "Selection" drawer open or closed. */
	void ToggleSidebarSelectionDrawerOpen();

	/** Undocks the docked sidebar drawer if docked or docks the sidebar drawer if there is one open and no currently docked drawer. */
	void ToggleSidebarDrawerDock();

	/** Enable/disable pending focus in sequencer */
	virtual void EnablePendingFocusOnHovering(const bool InEnabled) override;
	
	TSharedPtr<FSequencerFilterBar> GetFilterBar() const;
	
	bool IsTransportControlsVisible() const;
	void SetTransportControlsVisible(const bool bInVisible);
	void ToggleTransportControlsVisible();

	bool IsTransportControlsPoppedOut() const;
	void SetTransportControlsPoppedOut(const bool bInPoppedOut);
	void ToggleTransportControlsPoppedOut();

	TSharedPtr<FTabManager> GetTabManager() const;

	TSharedPtr<UE::Sequencer::SimpleView::FSimpleViewTimeline> GetSimpleViewTimeline() const;
	bool IsSimpleView() const;
	void EnableSimpleView(const bool bInEnable);
	void ToggleSimpleView();

	void ToggleToolBarVisible();

private:

	/** Pending focus handler */
	FPendingWidgetFocus PendingFocus;

	/** Enable/disable pending focus in the curve editor */
	void EnableCurveEditorPendingFocusOnHovering(const bool InEnabled) const;
	
	/** Applies a single customization. */
	void ApplySequencerCustomization(const FSequencerCustomizationInfo& Customization);

	void OnSidebarStateChanged(const FSidebarState& InNewState);

	void RegisterSelectionDetailsTab();
	void UnregisterSelectionDetailsTab();

	TSharedRef<SDockTab> SpawnSelectionDetailsTab(const FSpawnTabArgs& InArgs);
	void CloseSelectionDetailsTab();

	FText GetSidebarSelectionDrawerToolTipText() const;

	FOptionalSize GetOutlinerWidth() const;

	EVisibility GetPopoutTransportControlsVisibility() const;

	/** This is the main container widget for the Sequencer (minus the toolbar). This is the parent of
	 * the GridPanel widget below and is used to rebuild content when the sidebar is added/removed. */
	TSharedPtr<SBox> MainContentContainer;

	/** Splitter that holds most of the content and contains MainSequencerArea widget below.
	 * Built once at Sequencer initialize and used when rebuilding the main content container when the sidebar is added/removed. */
	TSharedPtr<SSplitter> OutlinerAndTrackAreaContent;

	/** Contains the outliner tree and track area. Used by GetPinnedAreaMaxHeight().
	 * Built once at Sequencer initialize and used when rebuilding the main content container when the sidebar is added/removed. */
	TSharedPtr<SVerticalBox> MainSequencerArea;

	/** The splitter container that holds the filter bar (if visible). */
	TSharedPtr<SBox> FilterBarSplitterContainer;

	/** Transform box widget. */
	TSharedPtr<SSequencerTransformBox> TransformBox;

	/** Stretch box widget. */
	TSharedPtr<SSequencerStretchBox> StretchBox;

	/** Filter Status Bar */
	TSharedPtr<SSequencerTreeFilterStatusBar> SequencerTreeFilterStatusBar;

	/** Section area widget */
	TSharedPtr<UE::Sequencer::STrackAreaView> TrackArea;

	/** Section area widget for pinned tracks*/
	TSharedPtr<UE::Sequencer::STrackAreaView> PinnedTrackArea;

	/** The breadcrumb trail widget for this sequencer */
	TSharedPtr<SBreadcrumbTrail<FSequencerBreadcrumb>> BreadcrumbTrail;

	/** The current playback time display. */
	TSharedPtr<STemporarilyFocusedSpinBox<double>> PlayTimeDisplay;

	/** The current loop display for when editing a looping sub-sequence. */
	TSharedPtr<STextBlock> LoopIndexDisplay;

	/** The sequencer tree view responsible for the outliner and track areas */
	TSharedPtr<UE::Sequencer::SOutlinerView> TreeView;

	/** The sequencer tree view for pinned tracks */
	TSharedPtr<UE::Sequencer::SOutlinerView> PinnedTreeView;

	TSharedPtr<SScrollBar> ScrollBar;
	TSharedPtr<SScrollBar> PinnedAreaScrollBar;

	/** Dropdown for selecting breadcrumbs */
	TSharedPtr<class SComboButton> BreadcrumbPickerButton;

	/** The main sequencer interface */
	TWeakPtr<FSequencer> SequencerPtr;

	/** The top time slider widget */
	TSharedPtr<ITimeSlider> TopTimeSlider;

	/** The bottom time slider widget */
	TSharedPtr<ITimeSlider> BottomTimeSlider;

	TSharedPtr<ITimeSlider> BottomTimeRange;

	/** Container for the toolbar, so that we can re-create it as needed. */
	TSharedPtr<SBox> ToolbarContainer;

	/** Exposing these slots so the equivalent track areas can match their height */
	SVerticalBox::FSlot* OutlinerSearchAndFilterSlot = nullptr;
	SVerticalBox::FSlot* OutlinerPinnedSlot = nullptr;

	/** Width of the outliner tree. Mainly used while sizing so we don't need to SaveConfig() and query settings, which causes a performance hit */
	mutable float TreeViewWidth = 0.f;

	/** List of registered outliner columns with their visibility states */
	TArray<FSequencerOutlinerColumnVisibility> OutlinerColumnVisibilities;

	/** Whether the active timer is currently registered */
	bool bIsActiveTimerRegistered;

	/** Default initialized in the view params to a lambda that gives us the standard speeds */
	ISequencer::FOnGetPlaybackSpeeds OnGetPlaybackSpeeds;
	
	/** Extender to use for the 'add' menu */
	TArray<TSharedPtr<FExtender>> AddMenuExtenders;

	/** Extender to use for the toolbar */
	TArray<TSharedPtr<FExtender>> ToolbarExtenders;

	/** Extender to use for the actions menu */
	TArray<TSharedPtr<FExtender>> ActionsMenuExtenders;

	/** Extender to use for the view menu */
	TArray<TSharedPtr<FExtender>> ViewMenuExtenders;

	/** Time slider controller for this sequencer */
	TSharedPtr<FSequencerTimeSliderController> TimeSliderController;

	/** Called when the user has begun dragging the selection selection range */
	FSimpleDelegate OnSelectionRangeBeginDrag;

	/** Called when the user has finished dragging the selection selection range */
	FSimpleDelegate OnSelectionRangeEndDrag;
	
	/** Called when the user has begun dragging the playback range */
	FSimpleDelegate OnPlaybackRangeBeginDrag;

	/** Called when the user has finished dragging the playback range */
	FSimpleDelegate OnPlaybackRangeEndDrag;

	/** Called when the user has begun dragging a mark */
	FSimpleDelegate OnMarkBeginDrag;

	/** Called when the user has finished dragging a mark */
	FSimpleDelegate OnMarkEndDrag;

	/** Called when any widget contained within sequencer has received focus */
	FSimpleDelegate OnReceivedFocus;

	/** Called when initializing tool menu context */
	FOnInitToolMenuContext OnInitToolMenuContext;

	/** Called when something is dragged over the sequencer. */
	TArray<FOptionalOnDragDrop> OnReceivedDragOver;

	/** Called when something is dropped onto the sequencer. */
	TArray<FOptionalOnDragDrop> OnReceivedDrop;

	/** Called when an asset is dropped on the sequencer. */
	TArray<FOnAssetsDrop> OnAssetsDrop;

	/** Called when a class is dropped on the sequencer. */
	TArray<FOnClassesDrop> OnClassesDrop;
	
	/** Called when an actor is dropped on the sequencer. */
	TArray<FOnActorsDrop> OnActorsDrop;

	/** Called when a folder is dropped on the sequencer. */
	TArray<FOnFoldersDrop> OnFoldersDrop;

	/** Stores the callbacks and extenders provided to the constructor. */
	FSequencerCustomizationInfo RootCustomization;

	/** Cached clamp and view range for unlinking the curve editor time range */
	TRange<double> CachedClampRange;
	TRange<double> CachedViewRange;

	/**
	 * A list of additional paths to add to the selection set when it is restored after rebuilding the tree.
	 * This can be used to highlight nodes that may not exist until the rebuild. Cleared after the tree is rebuilt
	 * and the selection list is restored.
	*/
	TArray<FString> AdditionalSelectionsToAdd;

	FString NodePathToRename;

	/** List of node paths that should be isolated on next tree update */
	TSet<FString> NewNodePathsToIsolate;

	TWeakPtr<SWindow> WeakTickResolutionOptionsWindow;

	TWeakPtr<SWindow> WeakExposedBindingsWindow;

	TWeakPtr<SWindow> WeakNodeGroupWindow;

	TSharedPtr<SSequencerGroupManager> NodeGroupManager;

	TSharedPtr<SSidebarContainer> SidebarContainer;
	TSharedPtr<SSidebar> DetailsSidebar;

	TSharedPtr<FSequencerSelectionDrawer> SelectionDetailsDrawer;

	/** Contains the filters widgets plus the filter bar. */
	TSharedPtr<SVerticalBox> SearchAndFilterRow;
	/** The widgets displayed above the tree view. */
	TSharedPtr<UE::Sequencer::SSequencerSearchAndFilterRow> FilterRow;

	/** Widget that manages popped out state (tab window registration) of the transport controls */
	TSharedPtr<SPopoutTabInlineContent> TransportControlsPopoutTab;

	/** Toolable timeline used for the Sequencer simple view */
	TSharedPtr<UE::Sequencer::SimpleView::FSimpleViewTimeline> SimpleViewTimeline;
};
