// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerDebugVisualizer.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "Styling/AppStyle.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Layout/ArrangedChildren.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "Sequencer.h"
#include "TimeToPixel.h"

#define LOCTEXT_NAMESPACE "SSequencerDebugVisualizer"

void SSequencerDebugVisualizer::Construct(const FArguments& InArgs, TSharedRef<FSequencer> InSequencer)
{
	AddSlot()
		.AutoHeight()
		[
			SNew(SSequencerEvaluationTemplateDebugVisualizer, InSequencer)
				.ViewRange(InArgs._ViewRange)
		];
	AddSlot()
		.AutoHeight()
		[
			SNew(SSequencerEntityComponentSystemDebugVisualizer, InSequencer)
				.ViewRange(InArgs._ViewRange)
		];
}

/** Evaluation template debug visualizer */

void SSequencerEvaluationTemplateDebugVisualizer::Construct(const FArguments& InArgs, TSharedRef<FSequencer> InSequencer)
{
	WeakSequencer = InSequencer;
	SetClipping(EWidgetClipping::ClipToBounds);
	ViewRange = InArgs._ViewRange;
	Refresh();
}

const FMovieSceneEvaluationField* SSequencerEvaluationTemplateDebugVisualizer::GetEvaluationField() const
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return nullptr;
	}

	UMovieSceneSequence* ActiveSequence = Sequencer->GetFocusedMovieSceneSequence();
	UMovieSceneCompiledDataManager* CompiledDataManager = Sequencer->GetEvaluationTemplate().GetCompiledDataManager();
	if (ActiveSequence == nullptr || CompiledDataManager == nullptr)
	{
		return nullptr;
	}

	const FMovieSceneCompiledDataID FocusedSequenceDataID = CompiledDataManager->GetDataID(ActiveSequence);
	const FMovieSceneEvaluationField* EvaluationField = CompiledDataManager->FindTrackTemplateField(FocusedSequenceDataID);
	return EvaluationField;
}

void SSequencerEvaluationTemplateDebugVisualizer::Refresh()
{
	Children.Empty();

	const FMovieSceneEvaluationField* EvaluationField = GetEvaluationField();
	if (EvaluationField == nullptr)
	{
		return;
	}

	CachedSignature = EvaluationField->GetSignature();

	TArray<int32> SegmentComplexity;
	SegmentComplexity.Reserve(EvaluationField->Size());

	// Heatmap complexity
	float AverageComplexity = 0;
	int32 MaxComplexity = 0;

	for (int32 Index = 0; Index < EvaluationField->Size(); ++Index)
	{
		const FMovieSceneEvaluationGroup& Group = EvaluationField->GetGroup(Index);

		int32 Complexity = 0;
		for (const FMovieSceneEvaluationGroupLUTIndex& LUTIndex : Group.LUTIndices)
		{
			// more groups is more complex
			Complexity += 1;
			// Add total init and eval complexity
			Complexity += LUTIndex.NumInitPtrs + LUTIndex.NumEvalPtrs;
		}

		SegmentComplexity.Add(Complexity);
		MaxComplexity = FMath::Max(MaxComplexity, Complexity);
		AverageComplexity += Complexity;
	}

	AverageComplexity /= SegmentComplexity.Num();

	static const FSlateBrush* SectionBackgroundBrush = FAppStyle::GetBrush("Sequencer.Section.Background");
	static const FSlateBrush* SectionBackgroundTintBrush = FAppStyle::GetBrush("Sequencer.Section.BackgroundTint");

	UMovieSceneSequence* ActiveSequence = WeakSequencer.Pin()->GetFocusedMovieSceneSequence();
	const FFrameRate SequenceResolution = ActiveSequence->GetMovieScene()->GetTickResolution();

	for (int32 Index = 0; Index < EvaluationField->Size(); ++Index)
	{
		TRange<FFrameNumber> Range = EvaluationField->GetRange(Index);

		const float Complexity = SegmentComplexity[Index];
 	
		float Lerp = FMath::Clamp((Complexity - AverageComplexity) / (MaxComplexity - AverageComplexity), 0.f, 1.f) * 0.5f +
			FMath::Clamp(Complexity / AverageComplexity, 0.f, 1.f) * 0.5f;

		// Blend from blue (240deg) to red (0deg)
		FLinearColor ComplexityColor = FLinearColor(FMath::Lerp(240.f, 0.f, FMath::Clamp(Lerp, 0.f, 1.f)), 1.f, 1.f, 0.5f).HSVToLinearRGB();

		Children.Add(
			SNew(SSequencerEvaluationTemplateDebugSlot, Index)
			.Visibility(this, &SSequencerEvaluationTemplateDebugVisualizer::GetSegmentVisibility, Range / SequenceResolution)
			.ToolTip(
				SNew(SToolTip)
				[
					GetTooltipForSegment(Index)
				]
			)
			[
				SNew(SBorder)
				.BorderImage(SectionBackgroundBrush)
				.Padding(FMargin(1.f))
				[
					SNew(SBorder)
					.BorderImage(SectionBackgroundTintBrush)
					.BorderBackgroundColor(ComplexityColor)
					.ForegroundColor(FLinearColor::Black)
					[
						SNew(STextBlock)
						.Text(FText::AsNumber(Index))
					]
				]
			]
		);
	}
}

FVector2D SSequencerEvaluationTemplateDebugVisualizer::ComputeDesiredSize(float) const
{
	// Note: X Size is not used
	return FVector2D(100, 20.f);
}

FGeometry SSequencerEvaluationTemplateDebugVisualizer::GetSegmentGeometry(const FGeometry& AllottedGeometry, const SSequencerEvaluationTemplateDebugSlot& Slot, const FTimeToPixel& TimeToPixelConverter) const
{
	const FMovieSceneEvaluationField* EvaluationField = GetEvaluationField();
	if (EvaluationField == nullptr || EvaluationField->GetSignature() != CachedSignature)
	{
		return AllottedGeometry.MakeChild(FSlateRenderTransform());
	}

	TRange<FFrameNumber> SegmentRange = EvaluationField->GetRange(Slot.GetSegmentIndex());

	float PixelStartX = SegmentRange.GetLowerBound().IsOpen() ? 0.f : TimeToPixelConverter.FrameToPixel(UE::MovieScene::DiscreteInclusiveLower(SegmentRange));
	float PixelEndX   = SegmentRange.GetUpperBound().IsOpen() ? AllottedGeometry.GetLocalSize().X : TimeToPixelConverter.FrameToPixel(UE::MovieScene::DiscreteExclusiveUpper(SegmentRange));

	const float MinSectionWidth = 0.f;
	float SectionLength = FMath::Max(MinSectionWidth, PixelEndX - PixelStartX);

	return AllottedGeometry.MakeChild(
		FVector2D(SectionLength, FMath::Max(Slot.GetDesiredSize().Y, 20.f)),
		FSlateLayoutTransform(FVector2D(PixelStartX, 0))
		);
}

EVisibility SSequencerEvaluationTemplateDebugVisualizer::GetSegmentVisibility(TRange<double> Range) const
{
	return ViewRange.Get().Overlaps(Range) ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<SWidget> SSequencerEvaluationTemplateDebugVisualizer::GetTooltipForSegment(int32 SegmentIndex) const
{
	const FMovieSceneEvaluationField* EvaluationField = GetEvaluationField();
	if (EvaluationField == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	const FMovieSceneEvaluationGroup& Group = EvaluationField->GetGroup(SegmentIndex);

	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);

	int32 NumInitEntities = 0;
	for (int32 Index = 0; Index < Group.LUTIndices.Num(); ++Index)
	{
		VerticalBox->AddSlot()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("EvalGroupFormat", "Evaluation Group {0}:"), Index))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(FText::Format(
					LOCTEXT("EvalTrackFormat", "{0} initialization steps, {1} evaluation steps"),
					FText::AsNumber(Group.LUTIndices[Index].NumInitPtrs),
					FText::AsNumber(Group.LUTIndices[Index].NumEvalPtrs)
				))
			]
		];
	}

	return VerticalBox;
}

void SSequencerEvaluationTemplateDebugVisualizer::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SPanel::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	const FMovieSceneEvaluationField* EvaluationField = GetEvaluationField();
 	if (EvaluationField == nullptr)
 	{
 		Children.Empty();
 	}
 	else if (EvaluationField->GetSignature() != CachedSignature)
 	{
 		Refresh();
 	}
}

void SSequencerEvaluationTemplateDebugVisualizer::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
 	TSharedPtr<FSequencer>     Sequencer      = WeakSequencer.Pin();
 	const UMovieSceneSequence* ActiveSequence = Sequencer.IsValid() ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;

 	if (!ActiveSequence)
 	{
 		return;
 	}

 	FTimeToPixel TimeToPixelConverter = FTimeToPixel(AllottedGeometry, ViewRange.Get(), ActiveSequence->GetMovieScene()->GetTickResolution());

 	for (int32 WidgetIndex = 0; WidgetIndex < Children.Num(); ++WidgetIndex)
 	{
 		const TSharedRef<SSequencerEvaluationTemplateDebugSlot>& Child = Children[WidgetIndex];

 		EVisibility WidgetVisibility = Child->GetVisibility();
 		if( ArrangedChildren.Accepts( WidgetVisibility ) )
 		{
 			FGeometry SegmentGeometry = GetSegmentGeometry(AllottedGeometry, *Child, TimeToPixelConverter);
 			if (SegmentGeometry.GetLocalSize().X >= 1)
 			{
 				ArrangedChildren.AddWidget( 
 					WidgetVisibility, 
 					AllottedGeometry.MakeChild(Child, FVector2D(SegmentGeometry.Position), SegmentGeometry.GetLocalSize())
 					);
 			}
 		}
 	}
}

/** Entity Component System debug visualizer */

static FLinearColor GetComponentColor(int32 ComponentBitIndex)
{
	static TArray<FLinearColor> NiceColors;

	if (NiceColors.Num() == 0)
	{
		const int32 MaxComponentTypes = 512;
		const float OneOverGoldenRatio = 2.f / (1.f + FMath::Sqrt(5.f));

		NiceColors.SetNum(MaxComponentTypes);
		float CurHue = 0.f;
		for (int32 Idx = 0; Idx < MaxComponentTypes; ++Idx)
		{
			CurHue = FMath::Fmod(CurHue + OneOverGoldenRatio, 1.f);
			FLinearColor HSVColor(CurHue * 360.f, 0.8f, 1.f);
			NiceColors[Idx] = HSVColor.HSVToLinearRGB();
		}
	}

	ComponentBitIndex = ComponentBitIndex % NiceColors.Num();
	return NiceColors[ComponentBitIndex];
}

class SSequencerDebugComponentSlot : public SBorder
{
	SLATE_BEGIN_ARGS(SSequencerDebugComponentSlot){}
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const int32 InComponentBitIndex, const FText& InComponentName)
	{
		static const FSlateBrush* SectionBackgroundBrush = FAppStyle::GetBrush("Sequencer.Section.Background");

		ComponentBitIndex = InComponentBitIndex;

		SetVAlign(EVerticalAlignment::VAlign_Center);
		SetHAlign(EHorizontalAlignment::HAlign_Center);
		SetBorderImage(SectionBackgroundBrush);
		SetBorderBackgroundColor(GetComponentColor(InComponentBitIndex));
		SetColorAndOpacity(GetComponentColor(InComponentBitIndex));

		ChildSlot
		[
			SNew(STextBlock)
				.Margin(FMargin(2.f, 4.f))
				.Text(InComponentName)
		];
	}

	int32 GetComponentBitIndex() const { return ComponentBitIndex; }

private:
	int32 ComponentBitIndex;
};

void SSequencerEntityComponentSystemDebugSlot::Construct(const FArguments& InArgs, TWeakPtr<FSequencer> InWeakSequencer, UMovieSceneSection* InSection)
{
	static const FSlateBrush* SectionBackgroundBrush = FAppStyle::GetBrush("Sequencer.Section.Background");

	WeakSequencer = InWeakSequencer;
	Section = InSection;

	SetClipping(EWidgetClipping::ClipToBounds);
	SetBorderImage(SectionBackgroundBrush);

	TSharedRef<SWrapBox> Container = SNew(SWrapBox)
		.UseAllottedSize(true)
		+SWrapBox::Slot()
		[
			SNew(SBorder)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.HAlign(EHorizontalAlignment::HAlign_Left)
				[
					SNew(STextBlock)
						.Text(this, &SSequencerEntityComponentSystemDebugSlot::GetEntityIDText)
				]
		];
	ChildSlot.AttachWidget(Container);

	Refresh();
}

void SSequencerEntityComponentSystemDebugSlot::Refresh()
{
	using namespace UE::MovieScene;

	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	TSharedRef<SWrapBox> Container = StaticCastSharedRef<SWrapBox>(ChildSlot.GetWidget());
	FChildren* ContainerChildren = Container->GetChildren();

	CachedEntityID = Sequencer->GetEvaluationTemplate().FindEntityFromOwner(Section, 0, Sequencer->GetFocusedTemplateID());

	UMovieSceneEntitySystemLinker* Linker = Sequencer->GetEvaluationTemplate().GetEntitySystemLinker();
	if (CachedEntityID.IsValid())
	{
		// Component widgets start at index 1 (index 0 is the "entity ID" widget).
		TMap<int32, TSharedRef<SWidget>> PreviousComponentWidgets;
		for (int32 Idx = 1; Idx < ContainerChildren->Num(); ++Idx)
		{
			TSharedRef<SSequencerDebugComponentSlot> Child = StaticCastSharedRef<SSequencerDebugComponentSlot>(ContainerChildren->GetChildAt(Idx));
			PreviousComponentWidgets.Add(Child->GetComponentBitIndex(), Child);
		}

		FEntityInfo EntityInfo = Linker->EntityManager.GetEntity(CachedEntityID);
		for (const FComponentHeader& ComponentHeader : EntityInfo.Data.Allocation->GetComponentHeaders())
		{
			const FComponentTypeInfo& ComponentTypeInfo = Linker->GetComponents()->GetComponentTypeChecked(ComponentHeader.ComponentType);
			const int32 ComponentBitIndex = ComponentHeader.ComponentType.BitIndex();

			if (!PreviousComponentWidgets.Contains(ComponentBitIndex))
			{
#if UE_MOVIESCENE_ENTITY_DEBUG
				// Component was added.
				Container->AddSlot()
					[
						SNew(SSequencerDebugComponentSlot, ComponentBitIndex, 
								FText::FromString(ComponentTypeInfo.DebugInfo->DebugName))
					];
#endif
			}
			else
			{
				// Component is still there.
				PreviousComponentWidgets.Remove(ComponentBitIndex);
			}
		}

		// Remove components that are not on the entity anymore.
		for (auto It : PreviousComponentWidgets)
		{
			Container->RemoveSlot(It.Value);
		}
	}
	else
	{
		// Remove all components... leave the "entity ID" widget, though.
		while (ContainerChildren->Num() > 1)
		{
			Container->RemoveSlot(ContainerChildren->GetChildAt(1));
		}
	}
}

FText SSequencerEntityComponentSystemDebugSlot::GetEntityIDText() const
{
	if (CachedEntityID.IsValid())
	{
		return FText::Format(LOCTEXT("DebugEntityID", "Entity ID:{0}"), CachedEntityID.AsIndex());
	}
	else
	{
		return FText(LOCTEXT("DebugEntityID_None", "(No EntityID)"));
	}
}

void SSequencerEntityComponentSystemDebugVisualizer::Construct(const FArguments& InArgs, TSharedRef<FSequencer> InSequencer)
{
	WeakSequencer = InSequencer;

	SetClipping(EWidgetClipping::ClipToBounds);

	ViewRange = InArgs._ViewRange;

	Refresh();
}

void SSequencerEntityComponentSystemDebugVisualizer::Refresh()
{
	if (!DoRefresh())
	{
		Children.Empty();
	}
}

bool SSequencerEntityComponentSystemDebugVisualizer::DoRefresh()
{
	using namespace UE::MovieScene;
	using namespace UE::Sequencer;

 	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
 	if (!Sequencer.IsValid())
 	{
 		return false;
 	}

 	UMovieSceneSequence* ActiveSequence = Sequencer->GetFocusedMovieSceneSequence();
 	if (!ActiveSequence)
 	{
 		return false;
 	}

	const FMovieSceneRootEvaluationTemplateInstance& RootEvalInstance = Sequencer->GetEvaluationTemplate();
	UMovieSceneCompiledDataManager* CompiledDataManager = RootEvalInstance.GetCompiledDataManager();
	if (!CompiledDataManager)
	{
		return false;
	}

	FMovieSceneCompiledDataID CompiledDataID = CompiledDataManager->GetDataID(ActiveSequence);
	UMovieScene* ActiveMovieScene = ActiveSequence->GetMovieScene();
	if (!ActiveMovieScene)
	{
		return false;
	}

	CachedSelectionSerialNumber = Sequencer->GetViewModel()->GetSelection()->GetSerialNumber();
	CachedSignature = CompiledDataManager->GetEntryRef(CompiledDataID).CompiledSignature;

 	const FFrameRate SequenceResolution = ActiveMovieScene->GetTickResolution();

	TMap<const UMovieSceneSection*, TSharedRef<SSequencerEntityComponentSystemDebugSlot>> SectionToWidget;
 	for (int32 WidgetIndex = 0; WidgetIndex < Children.Num(); ++WidgetIndex)
	{
 		const TSharedRef<SSequencerEntityComponentSystemDebugSlot>& Child = Children[WidgetIndex];
		SectionToWidget.Add(Child->GetSection(), Child);
	}

	for (TViewModelPtr<FSectionModel> SectionModel : Sequencer->GetViewModel()->GetSelection()->TrackArea.Filter<FSectionModel>())
	{
		UMovieSceneSection* Section = SectionModel->GetSection();
		if (Section != nullptr)
		{
			TSharedRef<SSequencerEntityComponentSystemDebugSlot>* ExistingChild = SectionToWidget.Find(Section);
			if (ExistingChild == nullptr)
			{
				// Newly selected section, add it to the debug visualization.
				const TRange<FFrameNumber> Range = Section->GetRange();

				Children.Add(
					SNew(SSequencerEntityComponentSystemDebugSlot, WeakSequencer, Section)
						.Visibility(this, &SSequencerEntityComponentSystemDebugVisualizer::GetSegmentVisibility, Range / SequenceResolution)
					);
			}
			else
			{
				// We already have this section in the debug visualization.
				SectionToWidget.Remove(Section);
				ExistingChild->Get().Refresh();
			}
		}
	}

	// Remove sections that we shouldn't show anymore.
	for (auto It : SectionToWidget)
	{
		Children.Remove(It.Value);
	}

	return true;
}

FVector2D SSequencerEntityComponentSystemDebugVisualizer::ComputeDesiredSize(float) const
{
	// Note: X Size is not used
	float Height = 0.f;
 	for (int32 WidgetIndex = 0; WidgetIndex < Children.Num(); ++WidgetIndex)
 	{
 		const TSharedRef<SSequencerEntityComponentSystemDebugSlot>& Child = Children[WidgetIndex];
		Height = FMath::Max(Height, Child->GetDesiredSize().Y);
	}
	return FVector2D(100, Height);
}

FGeometry SSequencerEntityComponentSystemDebugVisualizer::GetSegmentGeometry(const FGeometry& AllottedGeometry, const SSequencerEntityComponentSystemDebugSlot& Slot, const FTimeToPixel& TimeToPixelConverter) const
{
	const TRange<FFrameNumber> SegmentRange = Slot.GetSection()->GetRange();

	float PixelStartX = SegmentRange.GetLowerBound().IsOpen() ? 0.f : TimeToPixelConverter.FrameToPixel(UE::MovieScene::DiscreteInclusiveLower(SegmentRange));
	float PixelEndX   = SegmentRange.GetUpperBound().IsOpen() ? AllottedGeometry.GetLocalSize().X : TimeToPixelConverter.FrameToPixel(UE::MovieScene::DiscreteExclusiveUpper(SegmentRange));

	const float MinSectionWidth = 0.f;
	float SectionLength = FMath::Max(MinSectionWidth, PixelEndX - PixelStartX);

	return AllottedGeometry.MakeChild(
		FVector2D(SectionLength, FMath::Max(Slot.GetDesiredSize().Y, 20.f)),
		FSlateLayoutTransform(FVector2D(PixelStartX, 0))
		);
}

EVisibility SSequencerEntityComponentSystemDebugVisualizer::GetSegmentVisibility(TRange<double> Range) const
{
	return ViewRange.Get().Overlaps(Range) ? EVisibility::Visible : EVisibility::Collapsed;
}

void SSequencerEntityComponentSystemDebugVisualizer::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SPanel::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	bool bSignatureChanged = false;
	bool bSelectionChanged = false;

 	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
 	if (Sequencer.IsValid())
	{
		bSelectionChanged = CachedSelectionSerialNumber != Sequencer->GetViewModel()->GetSelection()->GetSerialNumber();

		UMovieSceneSequence* ActiveSequence = Sequencer->GetFocusedMovieSceneSequence();
		if (ActiveSequence != nullptr)
		{
			const FMovieSceneRootEvaluationTemplateInstance& RootEvalInstance = Sequencer->GetEvaluationTemplate();
			UMovieSceneCompiledDataManager* CompiledDataManager = RootEvalInstance.GetCompiledDataManager();
			if (CompiledDataManager != nullptr)
			{
				FMovieSceneCompiledDataID CompiledDataID = CompiledDataManager->GetDataID(ActiveSequence);
				if (CompiledDataManager->GetEntryRef(CompiledDataID).CompiledSignature != CachedSignature)
				{
					bSignatureChanged = true;
				}
			}
		}
	}

	if (bSignatureChanged || bSelectionChanged)
	{
		Refresh();
	}
}

void SSequencerEntityComponentSystemDebugVisualizer::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
 	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
 	const UMovieSceneSequence* ActiveSequence = Sequencer.IsValid() ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
 	if (!ActiveSequence)
 	{
 		return;
 	}

 	const FTimeToPixel TimeToPixelConverter = FTimeToPixel(
			AllottedGeometry, 
			ViewRange.Get(), 
			ActiveSequence->GetMovieScene()->GetTickResolution());

 	for (int32 WidgetIndex = 0; WidgetIndex < Children.Num(); ++WidgetIndex)
 	{
 		const TSharedRef<SSequencerEntityComponentSystemDebugSlot>& Child = Children[WidgetIndex];

 		EVisibility WidgetVisibility = Child->GetVisibility();
 		if(ArrangedChildren.Accepts(WidgetVisibility))
 		{
 			FGeometry SegmentGeometry = GetSegmentGeometry(AllottedGeometry, *Child, TimeToPixelConverter);
 			if (SegmentGeometry.GetLocalSize().X >= 1)
 			{
 				ArrangedChildren.AddWidget( 
 					WidgetVisibility, 
 					AllottedGeometry.MakeChild(Child, FVector2D(SegmentGeometry.Position), SegmentGeometry.GetLocalSize())
 					);
 			}
 		}
 	}
}

#undef LOCTEXT_NAMESPACE
