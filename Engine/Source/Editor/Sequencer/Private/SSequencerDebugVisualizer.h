// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Layout/Geometry.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "MovieSceneSequenceID.h"
#include "Sequencer.h"

class FArrangedChildren;
class SSequencerDebugSlot;
struct FTimeToPixel;
struct FMovieSceneEvaluationField;

namespace UE
{
namespace MovieScene
{
	class FEntityManager;
	struct FMovieSceneEntityID;
}
}

class SSequencerDebugVisualizer : public SVerticalBox
{
	SLATE_BEGIN_ARGS(SSequencerDebugVisualizer){}
		SLATE_ATTRIBUTE(TRange<double>, ViewRange)
	SLATE_END_ARGS()

	SSequencerDebugVisualizer() {}

	void Construct(const FArguments& InArgs, TSharedRef<FSequencer> InSequencer);
};

class SSequencerEvaluationTemplateDebugSlot : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SSequencerEvaluationTemplateDebugSlot){}
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, int32 InSegmentIndex)
	{
		SegmentIndex = InSegmentIndex;

		ChildSlot
		[
			InArgs._Content.Widget
		];
	}

	int32 GetSegmentIndex() const { return SegmentIndex; }

private:
	int32 SegmentIndex;
};

/**
 * Debug visualizer for the sequencer evaluation templates.
 */
class SSequencerEvaluationTemplateDebugVisualizer : public SPanel
{
public:
	SLATE_BEGIN_ARGS(SSequencerEvaluationTemplateDebugVisualizer){}
		SLATE_ATTRIBUTE(TRange<double>, ViewRange)
	SLATE_END_ARGS()

	SSequencerEvaluationTemplateDebugVisualizer()
		: Children(this)
	{}

	void Construct(const FArguments& InArgs, TSharedRef<FSequencer> InSequencer);

protected:
	/** SPanel Interface */
	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual FChildren* GetChildren() override { return &Children; }

protected:
 	void RequestRefresh();
 	void Refresh();
 	FGeometry GetSegmentGeometry(const FGeometry& AllottedGeometry, const SSequencerEvaluationTemplateDebugSlot& Slot, const FTimeToPixel& TimeToPixelConverter) const;
 	EVisibility GetSegmentVisibility(TRange<double> Range) const;
 	TSharedRef<SWidget> GetTooltipForSegment(int32 SegmentIndex) const;
 	void OnSequenceActivated(FMovieSceneSequenceIDRef);
 	const FMovieSceneEvaluationField* GetEvaluationField() const;

private:
 	/** The current view range */
 	TAttribute<TRange<double>> ViewRange;
	/** All the widgets in the panel */
	TSlotlessChildren<SSequencerEvaluationTemplateDebugSlot> Children;

	TWeakPtr<FSequencer> WeakSequencer;
 	FGuid CachedSignature;
};

class SSequencerEntityComponentSystemDebugSlot : public SBorder
{
	SLATE_BEGIN_ARGS(SSequencerEntityComponentSystemDebugSlot){}
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FSequencer> WeakSequencer, UMovieSceneSection* InSection);

	void Refresh();
	const UMovieSceneSection* GetSection() const { return Section; }

private:
	FText GetEntityIDText() const;

	UMovieSceneSection* Section = nullptr;

	UE::MovieScene::FMovieSceneEntityID CachedEntityID;

	TWeakPtr<FSequencer> WeakSequencer;
};

/**
 * Debug visualizer for the sequencer ECS.
 */
class SSequencerEntityComponentSystemDebugVisualizer : public SPanel
{
public:
	SLATE_BEGIN_ARGS(SSequencerEntityComponentSystemDebugVisualizer){}
		SLATE_ATTRIBUTE(TRange<double>, ViewRange)
	SLATE_END_ARGS()

	SSequencerEntityComponentSystemDebugVisualizer()
		: Children(this)
	{}

	void Construct(const FArguments& InArgs, TSharedRef<FSequencer> InSequencer);

protected:
	/** SPanel Interface */
	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual FChildren* GetChildren() override { return &Children; }

 	FGeometry GetSegmentGeometry(const FGeometry& AllottedGeometry, const SSequencerEntityComponentSystemDebugSlot& Slot, const FTimeToPixel& TimeToPixelConverter) const;
 	EVisibility GetSegmentVisibility(TRange<double> Range) const;

private:
 	void Refresh();
	bool DoRefresh();

 	/** The current view range */
 	TAttribute<TRange<double>> ViewRange;

	/** All the widgets in the panel */
	TSlotlessChildren<SSequencerEntityComponentSystemDebugSlot> Children;

	/** Parent sequencer */
 	TWeakPtr<FSequencer> WeakSequencer;

	/** Last known signature for the sequence */
 	FGuid CachedSignature;

	/** Last known selection */
	uint32 CachedSelectionSerialNumber;
};
