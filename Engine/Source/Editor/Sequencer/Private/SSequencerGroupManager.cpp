// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerGroupManager.h"

#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/IGroupableExtension.h"
#include "MVVM/Selection/Selection.h"
#include "Sequencer.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "SequencerOutlinerItemDragDropOp.h"
#include "SequencerUtilities.h"
#include "SlateOptMacros.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "EditorFontGlyphs.h"
#include "ScopedTransaction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"


#define LOCTEXT_NAMESPACE "SSequencerGroupManager"

class SSequencerGroupNodeRow;

struct FSequencerNodeGroupTreeNode
{
	enum Type
	{
		/* Base Node Type */
		BaseNode,
		/* Group */
		GroupNode,
		/* Item Node*/
		ItemNode
	};

	FSequencerNodeGroupTreeNode(const FText& InDisplayText)
		: DisplayText(InDisplayText)
	{
	}
	virtual ~FSequencerNodeGroupTreeNode() {}

	virtual Type GetType() const { return BaseNode; }

	FText DisplayText;

	TArray<TSharedPtr<FSequencerNodeGroupTreeNode>> Children;
};

struct FSequencerGroupItemNode : public FSequencerNodeGroupTreeNode
{
	FSequencerGroupItemNode(const FText& InDisplayText, const FString& InPath, TSharedPtr<FSequencerNodeGroupNode> InGroup)
		: FSequencerNodeGroupTreeNode(InDisplayText), Path(InPath), Group(InGroup)
	{
	}
	virtual ~FSequencerGroupItemNode() override {}

	virtual Type GetType() const override { return ItemNode; }

	FString Path;
	TSharedPtr<FSequencerNodeGroupNode> Group;
};

struct FSequencerNodeGroupNode : public FSequencerNodeGroupTreeNode
{
	FSequencerNodeGroupNode(const FText& InDisplayText, UMovieSceneNodeGroup* InGroup, TWeakPtr<SSequencerGroupManager> InGroupManager)
		: FSequencerNodeGroupTreeNode(InDisplayText), WeakGroupManager(InGroupManager), Group(InGroup)
	{
	}

	virtual ~FSequencerNodeGroupNode() override {}

	virtual Type GetType() const override { return GroupNode; }

	FReply OnEnableFilterClicked()
	{
		if (IsValid(Group))
		{
			Group->SetEnableFilter(!Group->GetEnableFilter());
		}
		return FReply::Handled();
	}

	bool IsFilterEnabled() const
	{
		return Group->GetEnableFilter();
	}

	bool VerifyNodeTextChanged(const FText& NewLabel, FText& OutErrorMessage)
	{
		return !NewLabel.IsEmptyOrWhitespace();
	}

	void HandleNodeLabelTextCommitted(const FText& NewLabel, ETextCommit::Type CommitType)
	{
		TSharedPtr<SSequencerGroupManager> GroupManager = WeakGroupManager.Pin();
		UMovieScene* MovieScene = GroupManager ? GroupManager->GetMovieScene() : nullptr;
		if (MovieScene)
		{
			const FScopedTransaction Transaction(LOCTEXT("RenameGroupTransaction", "Rename Group"));

			Group->SetName(FName(*FText::TrimPrecedingAndTrailing(NewLabel).ToString()));
		}
	}

	void OnRenameRequested()
	{
		if (InlineEditableTextBlock)
		{
			InlineEditableTextBlock->EnterEditingMode();
		}
	}

	TWeakPtr<SSequencerGroupManager> WeakGroupManager;
	UMovieSceneNodeGroup* Group;
	TSharedPtr<SInlineEditableTextBlock> InlineEditableTextBlock;
};

class SSequencerGroupNodeRow : public STableRow<TSharedPtr<FSequencerNodeGroupTreeNode>>
{
	SLATE_BEGIN_ARGS(SSequencerGroupNodeRow) {}
	SLATE_END_ARGS()

public:

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TWeakPtr<FSequencerNodeGroupTreeNode> InWeakSequencerGroupTreeNode, TWeakPtr<SSequencerGroupManager> InWeakSequencerGroupManager)
	{
		WeakSequencerGroupManager = InWeakSequencerGroupManager;
		WeakSequencerGroupTreeNode = InWeakSequencerGroupTreeNode;

		STableRow<TSharedPtr<FSequencerNodeGroupTreeNode>>::ConstructInternal(STableRow::FArguments()
			.Padding(5.f)
			.OnCanAcceptDrop(this, &SSequencerGroupNodeRow::OnCanAcceptDrop)
			.OnAcceptDrop(this, &SSequencerGroupNodeRow::OnAcceptDrop)
			, InOwnerTableView);

		TSharedPtr<FSequencerNodeGroupTreeNode> SequencerGroupTreeNode = WeakSequencerGroupTreeNode.Pin();

		if (!SequencerGroupTreeNode)
		{
			return;
		}

		TSharedPtr<SSequencerGroupManager> SequencerGroupManager = WeakSequencerGroupManager.Pin();

		const FSlateBrush* IconBrush = SequencerGroupManager ? SequencerGroupManager->GetIconBrush(SequencerGroupTreeNode) : nullptr;

		if(SequencerGroupTreeNode)
		{
			if(SequencerGroupTreeNode->GetType() == FSequencerNodeGroupTreeNode::Type::ItemNode)
			{
				TSharedPtr<FSequencerGroupItemNode> SequencerGroupItemNode = StaticCastSharedPtr<FSequencerGroupItemNode>(SequencerGroupTreeNode);
				this->ChildSlot
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SExpanderArrow, SharedThis(this))
					]

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(FMargin(5.f, 0.f, 5.f, 0.f))
					.AutoWidth()
					[
						SNew(SOverlay)
						+ SOverlay::Slot()
						[
							SNew(SImage)
								.Image(IconBrush ? IconBrush : FCoreStyle::Get().GetDefaultBrush())
								.ColorAndOpacity(IconBrush ? FLinearColor::White : FLinearColor::Transparent)
						]
					]

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(SequencerGroupTreeNode->DisplayText)
						.ToolTipText(FText::FromString(SequencerGroupItemNode->Path))
					]
				];
			}
			else if (SequencerGroupTreeNode->GetType() == FSequencerNodeGroupTreeNode::Type::GroupNode)
			{
				TSharedPtr<FSequencerNodeGroupNode> NodeGroupNode = StaticCastSharedPtr<FSequencerNodeGroupNode>(SequencerGroupTreeNode);

				this->ChildSlot
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SExpanderArrow, SharedThis(this))
					]

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(FMargin(5.f, 0.f, 5.f, 0.f))
					.AutoWidth()
					[
						SNew(SOverlay)
						+ SOverlay::Slot()
						[
							SNew(SButton)
							.OnClicked(FOnClicked::CreateSP(NodeGroupNode.ToSharedRef(), &FSequencerNodeGroupNode::OnEnableFilterClicked))
							.ButtonStyle( FAppStyle::Get(), "NoBorder" )
							.Content()
							[
								SNew(STextBlock)
								.TextStyle(FAppStyle::Get(), "GenericFilters.TextStyle")
								.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
								.Text(FEditorFontGlyphs::Filter)
								.ColorAndOpacity(NodeGroupNode->Group->GetEnableFilter() ? FLinearColor::White : FLinearColor(0.66f, 0.66f, 0.66f, 0.66f))
							]
						]
					]

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SAssignNew(NodeGroupNode->InlineEditableTextBlock, SInlineEditableTextBlock)
						.OnVerifyTextChanged(NodeGroupNode.ToSharedRef(), &FSequencerNodeGroupNode::VerifyNodeTextChanged)
						.OnTextCommitted(NodeGroupNode.ToSharedRef(), &FSequencerNodeGroupNode::HandleNodeLabelTextCommitted)
						.Text(NodeGroupNode->DisplayText)
						.Clipping(EWidgetClipping::ClipToBounds)
					]
				];

			}
		}
	}

private:

	TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<FSequencerNodeGroupTreeNode> SequencerGroupTreeNode)
	{
		using namespace UE::Sequencer;

		TSharedPtr<FSequencerOutlinerDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSequencerOutlinerDragDropOp>();
		if (DragDropOp.IsValid())
		{
			DragDropOp->ResetToDefaultToolTip();
			
			TOptional<EItemDropZone> AllowedDropZone;
			if (SequencerGroupTreeNode->GetType() == FSequencerNodeGroupTreeNode::Type::GroupNode)
			{
				AllowedDropZone = EItemDropZone::OntoItem;
				DragDropOp->CurrentHoverText = FText::Format(LOCTEXT("DragDropAddItemsFormat", "Add {0} item(s)"), FText::AsNumber(DragDropOp->GetDraggedViewModels().Num()));
			}

			if (AllowedDropZone.IsSet() == false)
			{
				DragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
			}
			return AllowedDropZone;
		}
		return TOptional<EItemDropZone>();
	}

	FReply OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<FSequencerNodeGroupTreeNode> SequencerGroupTreeNode)
	{
		using namespace UE::Sequencer;

		TSharedPtr<FSequencerOutlinerDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSequencerOutlinerDragDropOp>();
		if (DragDropOp.IsValid())
		{
			TSharedPtr<SSequencerGroupManager> SequencerGroupManager = WeakSequencerGroupManager.Pin();
			TSharedPtr<FSequencer> Sequencer = SequencerGroupManager ? SequencerGroupManager->GetSequencer() : nullptr;
			if (Sequencer.IsValid())
			{
				if (SequencerGroupTreeNode->GetType() == FSequencerNodeGroupTreeNode::Type::GroupNode)
				{
					TSharedPtr<FSequencerNodeGroupNode> NodeGroupNode = StaticCastSharedPtr<FSequencerNodeGroupNode>(SequencerGroupTreeNode);
					Sequencer->AddNodesToExistingNodeGroup(DragDropOp->GetDraggedViewModels(), NodeGroupNode->Group);

					return FReply::Handled();
				}
			}
		}
		return FReply::Unhandled();
	}

	TWeakPtr<SSequencerGroupManager> WeakSequencerGroupManager;
	TWeakPtr<FSequencerNodeGroupTreeNode> WeakSequencerGroupTreeNode;
};

UMovieScene* SSequencerGroupManager::GetMovieScene() const
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	UMovieSceneSequence* Sequence = Sequencer ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;

	return MovieScene;
}

void SSequencerGroupManager::Construct(const FArguments& InArgs, TWeakPtr<FSequencer> InWeakSequencer)
{
	WeakSequencer = InWeakSequencer;

	UMovieScene* MovieScene = GetMovieScene();

	if (!ensure(MovieScene))
	{
		return;
	}

	TWeakPtr<SSequencerGroupManager> WeakTabManager = SharedThis(this);
	auto HandleGenerateRow = [WeakTabManager](TSharedPtr<FSequencerNodeGroupTreeNode> InNode, const TSharedRef<STableViewBase>& InOwnerTableView) -> TSharedRef<ITableRow>
	{
		return SNew(SSequencerGroupNodeRow, InOwnerTableView, InNode, WeakTabManager);
	};

	auto HandleGetChildren = [](TSharedPtr<FSequencerNodeGroupTreeNode> InParent, TArray<TSharedPtr<FSequencerNodeGroupTreeNode>>& OutChildren)
	{
		OutChildren.Append(InParent->Children);
	};

	TreeView = SNew(STreeView<TSharedPtr<FSequencerNodeGroupTreeNode>>)
		.OnGenerateRow_Lambda(HandleGenerateRow)
		.OnGetChildren_Lambda(HandleGetChildren)
		.TreeItemsSource(&NodeGroupsTree)
		.OnSelectionChanged(this, &SSequencerGroupManager::HandleTreeSelectionChanged)
		.OnContextMenuOpening(this, &SSequencerGroupManager::OnContextMenuOpening);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			[
				SNew(SScrollBorder, TreeView.ToSharedRef())
				[
					TreeView.ToSharedRef()
				]
			]
		]
	];

	UpdateTree();
}

SSequencerGroupManager::~SSequencerGroupManager()
{

}

void SSequencerGroupManager::UpdateTree()
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin(); 
	UMovieScene* MovieScene = GetMovieScene();

	if (!ensure(Sequencer) || !ensure(MovieScene))
	{
		return;
	}

	TSharedRef<FSequencerNodeTree> NodeTree = Sequencer->GetNodeTree();

	TArray<UMovieSceneNodeGroup*> ExpandedSets;
	TSet<TSharedPtr<FSequencerNodeGroupTreeNode>> ExpandedTreeItems;
	TreeView->GetExpandedItems(ExpandedTreeItems);
	for(TSharedPtr<FSequencerNodeGroupTreeNode> ExpandedTreeItem : ExpandedTreeItems)
	{
		if (ExpandedTreeItem->GetType() == FSequencerNodeGroupTreeNode::Type::GroupNode)
		{
			TSharedPtr<FSequencerNodeGroupNode> NodeGroupNode = StaticCastSharedPtr<FSequencerNodeGroupNode>(ExpandedTreeItem);
			ExpandedSets.Add(NodeGroupNode->Group);
		}
	}

	NodeGroupsTree.Empty();
	AllNodeGroupItems.Empty();

	for (UMovieSceneNodeGroup* NodeGroup : MovieScene->GetNodeGroups())
	{
		TSharedPtr<FSequencerNodeGroupNode> SequencerGroupNode = MakeShared<FSequencerNodeGroupNode>(FText::FromName(NodeGroup->GetName()), NodeGroup, SharedThis(this));
		NodeGroupsTree.Add(SequencerGroupNode);
		for (const FString& NodePath : NodeGroup->GetNodes())
		{
			TViewModelPtr<IOutlinerExtension> Node = NodeTree->GetNodeAtPath(NodePath);
			if (Node)
			{
				TSharedPtr<FSequencerGroupItemNode> SequencerGroupItemNode = MakeShared<FSequencerGroupItemNode>(Node->GetLabel(), NodePath, SequencerGroupNode);
				SequencerGroupNode->Children.Add(SequencerGroupItemNode);
				AllNodeGroupItems.Add(NodePath);
			}
		}

		SequencerGroupNode->Children.Sort([](const TSharedPtr<FSequencerNodeGroupTreeNode>& A, const TSharedPtr<FSequencerNodeGroupTreeNode>& B) {
			return A->DisplayText.CompareTo(B->DisplayText) < 0;
		});
	}

	NodeGroupsTree.Sort([](const TSharedPtr<FSequencerNodeGroupTreeNode>& A, const TSharedPtr<FSequencerNodeGroupTreeNode>& B) {
		return A->DisplayText.CompareTo(B->DisplayText) < 0;
	});

	TreeView->SetTreeItemsSource(&NodeGroupsTree);

	for (TSharedPtr<FSequencerNodeGroupTreeNode> NodeGroupTreeNode : NodeGroupsTree)
	{
		if (NodeGroupTreeNode->GetType() == FSequencerNodeGroupTreeNode::Type::GroupNode)
		{
			TSharedPtr<FSequencerNodeGroupNode> NodeGroupNode = StaticCastSharedPtr<FSequencerNodeGroupNode>(NodeGroupTreeNode);

			if (ExpandedSets.Contains(NodeGroupNode->Group))
			{
				TreeView->SetItemExpansion(NodeGroupTreeNode,true);
			}
		}
	}

	TreeView->RequestTreeRefresh();

	bNodeGroupsDirty = false;
}

void SSequencerGroupManager::HandleTreeSelectionChanged(TSharedPtr<FSequencerNodeGroupTreeNode> InSelectedNode, ESelectInfo::Type SelectionType)
{
	SelectSelectedItemsInSequencer();
}

void SSequencerGroupManager::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bNodeGroupsDirty)
	{
		UpdateTree();
	}

	if (RequestedRenameNodeGroup && !TreeView->IsPendingRefresh())
	{
		for (const TSharedPtr<FSequencerNodeGroupTreeNode>& Node : NodeGroupsTree)
		{
			if (Node->GetType() == FSequencerNodeGroupTreeNode::Type::GroupNode)
			{
				TSharedPtr<FSequencerNodeGroupNode> SequencerGroupItemNode = StaticCastSharedPtr<FSequencerNodeGroupNode>(Node);
				if (SequencerGroupItemNode->Group == RequestedRenameNodeGroup)
				{
					SequencerGroupItemNode->OnRenameRequested();
					break;
				}
			}
		}

		RequestedRenameNodeGroup = nullptr;
	}
}

const FSlateBrush* SSequencerGroupManager::GetIconBrush(TSharedPtr<FSequencerNodeGroupTreeNode> NodeGroupTreeNode) const
{
	using namespace UE::Sequencer;

	if (NodeGroupTreeNode->GetType() == FSequencerNodeGroupTreeNode::Type::ItemNode)
	{
		TSharedPtr<FSequencerGroupItemNode> SequencerGroupItemNode = StaticCastSharedPtr<FSequencerGroupItemNode>(NodeGroupTreeNode);
		TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
		if (!Sequencer)
		{
			return nullptr;
		}

		TSharedRef<FSequencerNodeTree> NodeTree = Sequencer->GetNodeTree();

		// @todo_sequencer_mvvm: This is literally walking the entire tree to find a node by its path.
		//                       Worse still, it is doing so every frame, for every group node :/
		TViewModelPtr<IOutlinerExtension> OutlinerNode = NodeTree->GetNodeAtPath(SequencerGroupItemNode->Path);
		if (OutlinerNode)
		{
			return OutlinerNode->GetIconBrush();
		}
	}

	return nullptr;
}

void SSequencerGroupManager::SelectItemsInGroup(FSequencerNodeGroupNode* Node)
{
	TreeView->ClearSelection();

	for (TSharedPtr<FSequencerNodeGroupTreeNode> ChildNode : Node->Children)
	{
		TreeView->SetItemSelection(ChildNode, true);
	}
}

void SSequencerGroupManager::RequestDeleteNodeGroup(FSequencerNodeGroupNode * NodeGroupNode)
{
	UMovieScene* MovieScene = GetMovieScene();
	if (!ensure(MovieScene) || !ensure(NodeGroupNode))
	{
		return;
	}

	if (MovieScene->IsReadOnly())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("DeleteGroupTransaction", "Delete Group"));

	MovieScene->GetNodeGroups().RemoveNodeGroup(NodeGroupNode->Group);
}

void SSequencerGroupManager::RemoveSelectedItemsFromNodeGroup()
{
	UMovieScene* MovieScene = GetMovieScene();

	if (!ensure(MovieScene))
	{
		return;
	}

	if (MovieScene->IsReadOnly())
	{
		return;
	}

	TArray<TPair<UMovieSceneNodeGroup*,FString>> ItemsToRemove;
	TArray<TSharedPtr<FSequencerNodeGroupTreeNode>> SelectedNodes = TreeView->GetSelectedItems();
	for (const TSharedPtr<FSequencerNodeGroupTreeNode>& Node : SelectedNodes)
	{
		if (Node->GetType() == FSequencerNodeGroupTreeNode::Type::ItemNode)
		{
			TSharedPtr<FSequencerGroupItemNode> ItemNode = StaticCastSharedPtr<FSequencerGroupItemNode>(Node);
			ItemsToRemove.Add(TPair<UMovieSceneNodeGroup*, FString>(ItemNode->Group->Group,ItemNode->Path));
		}
	}

	if (ItemsToRemove.Num() < 1)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("RemoveItemFromGroupTransaction", "Remove Items From Group"));

	for (const TPair<UMovieSceneNodeGroup*, FString>& Item : ItemsToRemove)
	{
		Item.Key->RemoveNode(Item.Value);
	}
	
	RefreshNodeGroups();
}

void SSequencerGroupManager::CreateNodeGroup()
{
	UMovieScene* MovieScene = GetMovieScene();

	if (!ensure(MovieScene))
	{
		return;
	}

	if (MovieScene->IsReadOnly())
	{
		return;
	}

	TArray<FName> ExistingGroupNames;
	for (const UMovieSceneNodeGroup* NodeGroup : MovieScene->GetNodeGroups())
	{
		ExistingGroupNames.Add(NodeGroup->GetName());
	}

	const FScopedTransaction Transaction(LOCTEXT("CreateNewGroupTransaction", "Create New Group"));

	MovieScene->Modify();

	UMovieSceneNodeGroup* NewNodeGroup = NewObject<UMovieSceneNodeGroup>(&MovieScene->GetNodeGroups(), NAME_None, RF_Transactional);
	NewNodeGroup->SetName(FSequencerUtilities::GetUniqueName(FName("Group"), ExistingGroupNames));

	TSet<FString> SelectedNodePaths;
	GetSelectedItemsNodePaths(SelectedNodePaths);

	for (const FString& NodeToAdd : SelectedNodePaths)
	{
		NewNodeGroup->AddNode(NodeToAdd);
	}

	MovieScene->GetNodeGroups().AddNodeGroup(NewNodeGroup);

	RequestRenameNodeGroup(NewNodeGroup);
}

void SSequencerGroupManager::GetSelectedItemsNodePaths(TSet<FString>& OutSelectedNodePaths) const
{
	TArray<TSharedPtr<FSequencerNodeGroupTreeNode>> SelectedNodes = TreeView->GetSelectedItems();
	for (const TSharedPtr<FSequencerNodeGroupTreeNode>& Node : SelectedNodes)
	{
		if (Node->GetType() == FSequencerNodeGroupTreeNode::Type::ItemNode)
		{
			TSharedPtr<FSequencerGroupItemNode> ItemNode = StaticCastSharedPtr<FSequencerGroupItemNode>(Node);
			OutSelectedNodePaths.Add(ItemNode->Path);
		}
	}
}

void SSequencerGroupManager::SelectSelectedItemsInSequencer()
{
	if (bSynchronizingSelection)
	{
		return;
	}

	// When selection changes in the group manager tree, select the corresponding Sequencer items first
	{
		TGuardValue<bool> Guard(bSynchronizingSelection, true);

		TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
		if (!ensure(Sequencer))
		{
			return;
		}
	
		TSet<FString> SelectedNodePaths;
		GetSelectedItemsNodePaths(SelectedNodePaths);
	
		Sequencer->SelectNodesByPath(SelectedNodePaths);
	}
}

void SSequencerGroupManager::SelectItemsSelectedInSequencer()
{
	using namespace UE::Sequencer;

	if (bSynchronizingSelection)
	{
		return;
	}

	TGuardValue<bool> Guard(bSynchronizingSelection, true);

	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!ensure(Sequencer))
	{
		return;
	}

	TStringBuilder<128> TempString;

	// Build a list of the nodepaths that we want to consider for selection
	TSet<FString> NodesPathsToSelect;
	for (FViewModelPtr Model : Sequencer->GetViewModel()->GetSelection()->Outliner)
	{
		TViewModelPtr<IGroupableExtension> Groupable = Model->FindAncestorOfType<IGroupableExtension>(true);
		if (Groupable)
		{
			TempString.Reset();
			Groupable->GetIdentifierForGrouping(TempString);

			for (const FString& NodeGroupPath : AllNodeGroupItems)
			{
				// AllNodeGroupItems path is the full path (including folder) 
				if (NodeGroupPath.Contains(TempString.ToString()))
				{
					NodesPathsToSelect.Add(NodeGroupPath);
					break;
				}
			}
		}
	}

	TreeView->ClearSelection();

	// Build a list of the treenodes which match a nodepath we want to select
	for (const TSharedPtr<FSequencerNodeGroupTreeNode>& Node : NodeGroupsTree)
	{
		if (Node->GetType() == FSequencerNodeGroupTreeNode::Type::ItemNode)
		{
			TSharedPtr<FSequencerGroupItemNode> ItemNode = StaticCastSharedPtr<FSequencerGroupItemNode>(Node);
			if (NodesPathsToSelect.Contains(ItemNode->Path))
			{
				TreeView->SetItemSelection(Node, true);
			}
		}
		else if (Node->GetType() == FSequencerNodeGroupTreeNode::Type::GroupNode)
		{
			for (TSharedPtr<FSequencerNodeGroupTreeNode> ChildNode : Node->Children)
			{
				// Note: Currently, children of a set can only be item nodes, but that may change in the future.
				if (ChildNode->GetType() == FSequencerNodeGroupTreeNode::Type::ItemNode)
				{
					TSharedPtr<FSequencerGroupItemNode> ItemNode = StaticCastSharedPtr<FSequencerGroupItemNode>(ChildNode);
					if (NodesPathsToSelect.Contains(ItemNode->Path))
					{
						TreeView->SetItemSelection(ChildNode, true);
					}
				}
			}
		}
	}	
}

TSharedPtr<SWidget> SSequencerGroupManager::OnContextMenuOpening()
{
	TArray<TSharedPtr<FSequencerNodeGroupTreeNode>> SelectedNodes = TreeView->GetSelectedItems();

	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("CreateNodeGroup", "Create Group"),
		LOCTEXT("CreateNodeGroupTooltip", "Create a new group and add any selected items to it"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SSequencerGroupManager::CreateNodeGroup)));

	UMovieScene* MovieScene = GetMovieScene();
	bool bIsReadOnly = MovieScene? MovieScene->IsReadOnly() : true;

	for (const TSharedPtr<FSequencerNodeGroupTreeNode>& Node : SelectedNodes)
	{
		if (Node->GetType() == FSequencerNodeGroupTreeNode::Type::GroupNode)
		{
			TSharedPtr<FSequencerNodeGroupNode> NodeGroupNode = StaticCastSharedPtr<FSequencerNodeGroupNode>(Node);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("NodeGroupToggleFilter", "Toggle Filter"),
				LOCTEXT("NodeGroupToggleFilterTooltip", "Toggle whether this group should be used to filter items"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([NodeGroupNode]() { NodeGroupNode->OnEnableFilterClicked(); }),
					FCanExecuteAction::CreateLambda([bIsReadOnly]() { return !bIsReadOnly; }),
					FIsActionChecked::CreateLambda([NodeGroupNode]() { return NodeGroupNode->IsFilterEnabled(); })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("SelectItemsInGroup", "Select Items in Group"),
				LOCTEXT("SelectItemsInGroupTooltip", "Select items in group"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SSequencerGroupManager::SelectItemsInGroup, NodeGroupNode.Get())));

			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("RenameNodeGroupFormat", "Rename {0}"), NodeGroupNode->DisplayText),
				FText(),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([NodeGroupNode]() { NodeGroupNode->OnRenameRequested(); })));

			MenuBuilder.AddMenuEntry(
				LOCTEXT("DeleteNodeGroup", "Delete Group"),
				FText(),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SSequencerGroupManager::RequestDeleteNodeGroup, NodeGroupNode.Get())));

			break;
		}
	}

	bool bAnyItemSelected = false;
	for (const TSharedPtr<FSequencerNodeGroupTreeNode>& Node : SelectedNodes)
	{
		if (Node->GetType() == FSequencerNodeGroupTreeNode::Type::ItemNode)
		{
			bAnyItemSelected = true;
			break;
		}
	}

	if (bAnyItemSelected)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("RemoveItemsFromNodeGropu", "Remove Items From Group"),
			FText(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SSequencerGroupManager::RemoveSelectedItemsFromNodeGroup)));
	}

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
