// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class FSequencer;
class FMenuBuilder;
class UMovieScene;
class UMovieSceneNodeGroup;

struct FSequencerNodeGroupTreeNode;
struct FSequencerNodeGroupNode;

class SSequencerGroupManager : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSequencerGroupManager) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FSequencer> InWeakSequencer);
	
	~SSequencerGroupManager();

	TSharedPtr<FSequencer> GetSequencer() const
	{
		return WeakSequencer.Pin();
	}

	const FSlateBrush* GetIconBrush(TSharedPtr<FSequencerNodeGroupTreeNode> NodeGroupTreeNode) const;

	void RefreshNodeGroups() { bNodeGroupsDirty = true; }

	UMovieScene* GetMovieScene() const;

private:

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	void UpdateTree();
	void HandleTreeSelectionChanged(TSharedPtr<FSequencerNodeGroupTreeNode> InSelectedNode, ESelectInfo::Type SelectionType);
	void SelectItemsInGroup(FSequencerNodeGroupNode* NodeGroupNode);
	void RequestDeleteNodeGroup(FSequencerNodeGroupNode* NodeGroupNode);
	void RemoveSelectedItemsFromNodeGroup();
	void CreateNodeGroup();
	
	void GetSelectedItemsNodePaths(TSet<FString>& OutSelectedNodePaths) const;

public:
	void SelectSelectedItemsInSequencer();
	void SelectItemsSelectedInSequencer();

	/** Request entering rename editing for a selection set, after setup and any refresh have completed */
	void RequestRenameNodeGroup(UMovieSceneNodeGroup* NodeGroup) { RequestedRenameNodeGroup = NodeGroup; }

private:
	TSharedPtr<SWidget> OnContextMenuOpening();
	
private:

	/** The sequencer UI instance that is currently open */
	TWeakPtr<FSequencer> WeakSequencer;
	
	TSharedPtr<STreeView<TSharedPtr<FSequencerNodeGroupTreeNode>>> TreeView;

	TArray<TSharedPtr<FSequencerNodeGroupTreeNode>> NodeGroupsTree;

	/** A collection of NodePaths for all items which are in all node groups, for quick processing */
	TSet<FString> AllNodeGroupItems;
	
	/** If not null, enter rename editing for this set after any setup and refresh have completed */
	UMovieSceneNodeGroup* RequestedRenameNodeGroup;

	bool bNodeGroupsDirty;
	bool bSynchronizingSelection;

};