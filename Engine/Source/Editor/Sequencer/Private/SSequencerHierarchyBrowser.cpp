// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerHierarchyBrowser.h"

#include "Sequencer.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Sections/MovieSceneSubSection.h"

#include "SlateOptMacros.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "EditorFontGlyphs.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "SSequencerHierarchyBrowser"

struct FSequencerHierarchyNode
{
	FSequencerHierarchyNode(const FText& InLabel, UMovieSceneSubSection* InSubSection)
		: Label(InLabel)
		, SubSection(InSubSection) {}

	virtual ~FSequencerHierarchyNode() {}

	FText Label;
	TWeakObjectPtr<UMovieSceneSubSection> SubSection;

	TArray<TSharedPtr<FSequencerHierarchyNode>> Children;
};

class SSequencerHierarchyNodeRow : public STableRow<TSharedPtr<FSequencerHierarchyNode>>
{
	SLATE_BEGIN_ARGS(SSequencerHierarchyNodeRow) {}
	SLATE_END_ARGS()

public:

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TWeakPtr<FSequencerHierarchyNode> InWeakSequencerHierarchyNode, TWeakPtr<SSequencerHierarchyBrowser> InWeakSequencerHierarchyBrowser)
	{
		WeakSequencerHierarchyBrowser = InWeakSequencerHierarchyBrowser;
		WeakSequencerHierarchyNode = InWeakSequencerHierarchyNode;

		STableRow<TSharedPtr<FSequencerHierarchyNode>>::ConstructInternal(STableRow::FArguments()
			.Padding(5.f)
			, InOwnerTableView);

		TSharedPtr<FSequencerHierarchyNode> SequencerHierarchyNode = WeakSequencerHierarchyNode.Pin();

		if (!SequencerHierarchyNode)
		{
			return;
		}

		TSharedPtr<SSequencerHierarchyBrowser> SequencerHierarchyBrowser = WeakSequencerHierarchyBrowser.Pin();

		this->ChildSlot
		[
			SNew(SHorizontalBox)
						
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image_Lambda([SequencerHierarchyNode] 
						{
							if (!SequencerHierarchyNode->SubSection.IsValid())
							{
								return FAppStyle::GetNoBrush();
							}
							else if (SequencerHierarchyNode->SubSection.Get()->IsA<UMovieSceneCinematicShotSection>())
							{
								return FAppStyle::GetBrush("Sequencer.Tracks.CinematicShot");
							}
							else
							{
								return FAppStyle::GetBrush("Sequencer.Tracks.Sub"); 
							}
						})
				]
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(5.f, 5.f, 5.f, 5.f))
			[
				SNew(STextBlock)
				.Text(SequencerHierarchyNode->Label)
			]
		];
	}

private:

	TWeakPtr<SSequencerHierarchyBrowser> WeakSequencerHierarchyBrowser;
	TWeakPtr<FSequencerHierarchyNode> WeakSequencerHierarchyNode;
};

UMovieScene* SSequencerHierarchyBrowser::GetMovieScene() const
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	UMovieSceneSequence* Sequence = Sequencer ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;

	return MovieScene;
}

void SSequencerHierarchyBrowser::Construct(const FArguments& InArgs, TWeakPtr<FSequencer> InWeakSequencer)
{
	WeakSequencer = InWeakSequencer;

	UMovieScene* MovieScene = GetMovieScene();

	if (!ensure(MovieScene))
	{
		return;
	}

	TWeakPtr<SSequencerHierarchyBrowser> WeakTabManager = SharedThis(this);
	auto HandleGenerateRow = [WeakTabManager](TSharedPtr<FSequencerHierarchyNode> InNode, const TSharedRef<STableViewBase>& InOwnerTableView) -> TSharedRef<ITableRow>
	{
		return SNew(SSequencerHierarchyNodeRow, InOwnerTableView, InNode, WeakTabManager);
	};

	auto HandleGetChildren = [](TSharedPtr<FSequencerHierarchyNode> InParent, TArray<TSharedPtr<FSequencerHierarchyNode>>& OutChildren)
	{
		OutChildren.Append(InParent->Children);
	};

	FDisplayMetrics DisplayMetrics;
	FSlateApplication::Get().GetCachedDisplayMetrics(DisplayMetrics);

	TreeView = SNew(STreeView<TSharedPtr<FSequencerHierarchyNode>>)
		.OnGenerateRow_Lambda(HandleGenerateRow)
		.OnGetChildren_Lambda(HandleGetChildren)
		.TreeItemsSource(&NodeGroupsTree)
		.OnSelectionChanged(this, &SSequencerHierarchyBrowser::HandleTreeSelectionChanged);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.MaxHeight(DisplayMetrics.PrimaryDisplayHeight * 0.5)
			[
				SNew(SScrollBorder, TreeView.ToSharedRef())
				[
					TreeView.ToSharedRef()
				]
			]
		]
	];

	UpdateTree();
}

SSequencerHierarchyBrowser::~SSequencerHierarchyBrowser()
{
}

void SSequencerHierarchyBrowser::AddChildren(TSharedRef<FSequencerHierarchyNode> ParentNode, UMovieSceneSequence* Sequence)
{
	UMovieScene* MovieScene = Sequence->GetMovieScene();

	for (UMovieSceneTrack* Track : MovieScene->GetTracks())
	{
		if (UMovieSceneCinematicShotTrack* CinematicShotTrack = Cast<UMovieSceneCinematicShotTrack>(Track))
		{
			TArray<UMovieSceneCinematicShotSection*> ShotSections;
			for (UMovieSceneSection* Section : CinematicShotTrack->GetAllSections())
			{
				if (UMovieSceneCinematicShotSection* ShotSection = Cast<UMovieSceneCinematicShotSection>(Section))
				{
					if (ShotSection->GetSequence())
					{
						ShotSections.Add(ShotSection);
					}
				}
			}
			ShotSections.Sort([](const UMovieSceneCinematicShotSection& A, const UMovieSceneCinematicShotSection& B) { return A.GetShotDisplayName().Compare(B.GetShotDisplayName()) < 0; });

			for (UMovieSceneCinematicShotSection* ShotSection : ShotSections)
			{
				TSharedPtr<FSequencerHierarchyNode> ChildNode = MakeShared<FSequencerHierarchyNode>(FText::FromString(ShotSection->GetShotDisplayName()), ShotSection);
				ParentNode->Children.Add(ChildNode);

				AddChildren(ChildNode.ToSharedRef(), ShotSection->GetSequence());
			}
		}
		else if (UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track))
		{
			TArray<UMovieSceneSubSection*> SubSections;
			for (UMovieSceneSection* Section : SubTrack->GetAllSections())
			{
				if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
				{
					if (SubSection->GetSequence())
					{
						SubSections.Add(SubSection);
					}
				}
			}
			SubSections.Sort([](const UMovieSceneSubSection& A, const UMovieSceneSubSection& B) { return A.GetSequence()->GetDisplayName().CompareTo(B.GetSequence()->GetDisplayName()) < 0; });

			for (UMovieSceneSubSection* SubSection : SubSections)
			{
				TSharedPtr<FSequencerHierarchyNode> ChildNode = MakeShared<FSequencerHierarchyNode>(SubSection->GetSequence()->GetDisplayName(), SubSection);
				ParentNode->Children.Add(ChildNode);

				AddChildren(ChildNode.ToSharedRef(), SubSection->GetSequence());
			}
		}
	}
}

void SSequencerHierarchyBrowser::UpdateTree()
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	UMovieSceneSequence* Sequence = Sequencer ? Sequencer->GetRootMovieSceneSequence() : nullptr;

	if (!ensure(Sequencer) || !ensure(Sequence))
	{
		return;
	}

	TSharedRef<FSequencerNodeTree> NodeTree = Sequencer->GetNodeTree();

	NodeGroupsTree.Empty();

	// Make nodes from this parent sequence
	TSharedPtr<FSequencerHierarchyNode> ParentNode = MakeShared<FSequencerHierarchyNode>(Sequence->GetDisplayName(), nullptr);
	NodeGroupsTree.Add(ParentNode);

	AddChildren(ParentNode.ToSharedRef(), Sequence);

	NodeGroupsTree.Sort([](const TSharedPtr<FSequencerHierarchyNode>& A, const TSharedPtr<FSequencerHierarchyNode>& B) {
		return A->Label.CompareTo(B->Label) < 0;
		});

	TreeView->SetTreeItemsSource(&NodeGroupsTree);

	TreeView->SetItemExpansion(ParentNode, true);
}

void SSequencerHierarchyBrowser::HandleTreeSelectionChanged(TSharedPtr<FSequencerHierarchyNode> InSelectedNode, ESelectInfo::Type SelectionType)
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!ensure(Sequencer))
	{
		return;
	}

	if (!InSelectedNode.IsValid())
	{
		return;
	}

	if (!InSelectedNode->SubSection.IsValid())
	{
		Sequencer->PopToSequenceInstance(MovieSceneSequenceID::Root);
	}
	else
	{
		Sequencer->FocusSequenceInstance(*InSelectedNode->SubSection.Get());
	}

	FSlateApplication::Get().FindWidgetWindow(AsShared())->RequestDestroyWindow();
}

#undef LOCTEXT_NAMESPACE
