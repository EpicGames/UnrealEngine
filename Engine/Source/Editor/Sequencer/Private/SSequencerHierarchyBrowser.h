// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class FSequencer;
class UMovieScene;
class UMovieSceneSequence;

struct FSequencerHierarchyNode;

class SSequencerHierarchyBrowser : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSequencerHierarchyBrowser) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FSequencer> InWeakSequencer);

	~SSequencerHierarchyBrowser();

	TSharedPtr<FSequencer> GetSequencer() const
	{
		return WeakSequencer.Pin();
	}

	UMovieScene* GetMovieScene() const;

private:

	void AddChildren(TSharedRef<FSequencerHierarchyNode> ParentNode, UMovieSceneSequence* Sequence);
	void UpdateTree();
	void HandleTreeSelectionChanged(TSharedPtr<FSequencerHierarchyNode> InSelectedNode, ESelectInfo::Type SelectionType);

private:

	/** The sequencer UI instance that is currently open */
	TWeakPtr<FSequencer> WeakSequencer;

	TSharedPtr<STreeView<TSharedPtr<FSequencerHierarchyNode>>> TreeView;

	TArray<TSharedPtr<FSequencerHierarchyNode>> NodeGroupsTree;
};