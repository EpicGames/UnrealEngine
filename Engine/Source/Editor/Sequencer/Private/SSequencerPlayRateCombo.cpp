// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerPlayRateCombo.h"
#include "Sequencer.h"
#include "SSequencer.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "MovieSceneClock.h"
#include "MovieSceneCommonHelpers.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "ScopedTransaction.h"
#include "Widgets/SFrameRateEntryBox.h"
#include "EditorFontGlyphs.h"
#include "Evaluation/IMovieSceneCustomClockSource.h"
#include "SceneOutlinerModule.h"
#include "SceneOutlinerPublicTypes.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Misc/Timecode.h"
#include "Styling/ToolBarStyle.h"
#include "ActorTreeItem.h"
#include "SequencerToolMenuContext.h"

#define LOCTEXT_NAMESPACE "SSequencerPlayRateCombo"

void SSequencerPlayRateCombo::Construct(const FArguments& InArgs, TWeakPtr<FSequencer> InWeakSequencer, TWeakPtr<SSequencer> InWeakSequencerWidget)
{
	WeakSequencer = InWeakSequencer;
	WeakSequencerWidget = InWeakSequencerWidget;

	const FToolBarStyle& SequencerToolBarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>(InArgs._StyleName);

	SetToolTipText(MakeAttributeSP(this, &SSequencerPlayRateCombo::GetToolTipText));

	ChildSlot
	.VAlign(VAlign_Fill)
	[
		SNew(SComboButton)
		.ContentPadding(FMargin(2.f, 1.0f))
		.VAlign(VAlign_Fill)
		.ComboButtonStyle(&SequencerToolBarStyle.ComboButtonStyle)
		.OnGetMenuContent(this, &SSequencerPlayRateCombo::OnCreateMenu)
		.ButtonContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SSequencerPlayRateCombo::GetFrameRateText)
				.TextStyle(&SequencerToolBarStyle.LabelStyle)
			]

			+ SHorizontalBox::Slot()
			.Padding(3.f, 0.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Visibility(this, &SSequencerPlayRateCombo::GetFrameLockedVisibility)
				.TextStyle(&SequencerToolBarStyle.LabelStyle)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(FEditorFontGlyphs::Lock)
			]

			+ SHorizontalBox::Slot()
			.Padding(3.f, 0.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.Visibility(this, &SSequencerPlayRateCombo::GetClockSourceVisibility)
				[
					SNew(SImage)
					.Image(this, &SSequencerPlayRateCombo::GetClockSourceImage)
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, 3.f, 0.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ToolTipText(this, &SSequencerPlayRateCombo::GetFrameRateIsMultipleOfErrorDescription)
				.Visibility(this, &SSequencerPlayRateCombo::GetFrameRateIsMultipleOfErrorVisibility)
				.TextStyle(&SequencerToolBarStyle.LabelStyle)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(FEditorFontGlyphs::Exclamation_Triangle)
			]

			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, 3.f, 0.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ToolTipText(this, &SSequencerPlayRateCombo::GetFrameRateMismatchErrorDescription)
				.Visibility(this, &SSequencerPlayRateCombo::GetFrameRateMismatchErrorVisibility)
				.TextStyle(&SequencerToolBarStyle.LabelStyle)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(FEditorFontGlyphs::Exclamation_Triangle)
			]
		]
	];
}

EVisibility SSequencerPlayRateCombo::GetFrameLockedVisibility() const
{
	TSharedPtr<FSequencer> Sequencer  = WeakSequencer.Pin();
	UMovieSceneSequence*   Sequence   = Sequencer.IsValid() ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	return Sequence && Sequence->GetMovieScene()->GetEvaluationType() == EMovieSceneEvaluationType::FrameLocked ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
}

EVisibility SSequencerPlayRateCombo::GetFrameRateIsMultipleOfErrorVisibility() const
{
	TSharedPtr<FSequencer> Sequencer  = WeakSequencer.Pin();
	if (!Sequencer.IsValid() || Sequencer->GetFocusedDisplayRate().IsMultipleOf(Sequencer->GetFocusedTickResolution()))
	{
		return EVisibility::Collapsed;
	}
	return EVisibility::Visible;
}

EVisibility SSequencerPlayRateCombo::GetFrameRateMismatchErrorVisibility() const
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		FFrameRate DisplayRate = Sequencer->GetRootDisplayRate();

		const FMovieSceneRootEvaluationTemplateInstance& Template = Sequencer->GetEvaluationTemplate();
		const FMovieSceneSequenceHierarchy* Hierarchy = Template.GetCompiledDataManager()->FindHierarchy(Template.GetCompiledDataID());

		if (!Hierarchy)
		{
			return EVisibility::Collapsed;
		}

		for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
		{
			UMovieSceneSequence* SubSequence = Pair.Value.GetSequence();
			UMovieScene*         MovieScene  = SubSequence ? SubSequence->GetMovieScene() : nullptr;

			if (MovieScene && MovieScene->GetDisplayRate() != DisplayRate)
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

FText SSequencerPlayRateCombo::GetFrameRateIsMultipleOfErrorDescription() const
{
	TSharedPtr<FSequencer> Sequencer  = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		FFrameRate DisplayRate = Sequencer->GetFocusedDisplayRate();
		FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();

		const FCommonFrameRateInfo* DisplayRateInfo     = FCommonFrameRates::Find(DisplayRate);
		const FCommonFrameRateInfo* TickResolutionInfo  = FCommonFrameRates::Find(TickResolution);

		FText DisplayRateText     = DisplayRateInfo     ? DisplayRateInfo->DisplayName    : FText::Format(LOCTEXT("DisplayRateFormat", "{0} fps"), DisplayRate.AsDecimal());
		FText TickResolutionText  = TickResolutionInfo  ? TickResolutionInfo->DisplayName : FText::Format(LOCTEXT("TickResolutionFormat", "{0} ticks every second"), TickResolution.AsDecimal());

		return FText::Format(LOCTEXT("FrameRateIsMultipleOfErrorDescription", "The current display rate of {0} is incompatible with this sequence's tick resolution of {1} ticks per second."), DisplayRateText, TickResolutionText);
	}

	return FText();
}

FText SSequencerPlayRateCombo::GetFrameRateMismatchErrorDescription() const
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		FFrameRate DisplayRate = Sequencer->GetRootDisplayRate();

		const FMovieSceneRootEvaluationTemplateInstance& Template = Sequencer->GetEvaluationTemplate();
		const FMovieSceneSequenceHierarchy* Hierarchy = Template.GetCompiledDataManager()->FindHierarchy(Template.GetCompiledDataID());

		if (!Hierarchy)
		{
			return FText();
		}

		TArray<FText> SubSequenceDisplayRates;

		for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
		{
			UMovieSceneSequence* SubSequence = Pair.Value.GetSequence();
			UMovieScene*         MovieScene  = SubSequence ? SubSequence->GetMovieScene() : nullptr;

			if (MovieScene && MovieScene->GetDisplayRate() != DisplayRate)
			{
				FFrameRate SubDisplayRate = MovieScene->GetDisplayRate();

				const FCommonFrameRateInfo* SubDisplayRateInfo = FCommonFrameRates::Find(SubDisplayRate);

				FText SubDisplayRateText = SubDisplayRateInfo ? SubDisplayRateInfo->DisplayName : FText::Format(LOCTEXT("SubDisplayRateFormat", "{0} fps"), SubDisplayRate.AsDecimal());

				FText SubSequenceDescription = FText::Format(LOCTEXT("SubSequenceFrameRateMismatchDescription", "\t{0} is at {1}"), SubSequence->GetDisplayName(), SubDisplayRateText);
				SubSequenceDisplayRates.Add(SubSequenceDescription);
			}
		}

		if (SubSequenceDisplayRates.Num() != 0)
		{
			const FCommonFrameRateInfo* DisplayRateInfo = FCommonFrameRates::Find(DisplayRate);
			FText DisplayRateText = DisplayRateInfo ? DisplayRateInfo->DisplayName : FText::Format(LOCTEXT("DisplayRateFormat", "{0} fps"), DisplayRate.AsDecimal());		
		
			FText Description = FText::Format(LOCTEXT("FrameRateMismatchDescription", "Mismatch in display rate: {0} is at {1}"), Sequencer->GetRootMovieSceneSequence()->GetDisplayName(), DisplayRateText);
			SubSequenceDisplayRates.Insert(Description, 0);

			return FText::Join(FText::FromString(TEXT("\n")), SubSequenceDisplayRates);
		}
	}

	return FText();
}

bool SSequencerPlayRateCombo::GetIsSequenceReadOnly() const
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	return !Sequencer.IsValid() || Sequencer->IsReadOnly();
}

TSharedRef<SWidget> SSequencerPlayRateCombo::OnCreateMenu()
{
	TSharedPtr<FSequencer> Sequencer       = WeakSequencer.Pin();
	TSharedPtr<SSequencer> SequencerWidget = WeakSequencerWidget.Pin();
	if (!Sequencer.IsValid() || !SequencerWidget.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	FMenuBuilder MenuBuilder(true, nullptr);

	FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();

	TArray<FCommonFrameRateInfo> CompatibleRates;
	for (const FCommonFrameRateInfo& Info : FCommonFrameRates::GetAll())
	{
		if (Info.FrameRate.IsMultipleOf(TickResolution))
		{
			CompatibleRates.Add(Info);
		}
	}

	CompatibleRates.Sort(
		[=](const FCommonFrameRateInfo& A, const FCommonFrameRateInfo& B)
		{
			return A.FrameRate.AsDecimal() < B.FrameRate.AsDecimal();
		}
	);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("RecommendedRates", "Sequence Display Rate"));
	{
		for (const FCommonFrameRateInfo& Info : CompatibleRates)
		{
			AddMenuEntry(MenuBuilder, Info);
		}

		MenuBuilder.AddWidget(
			SNew(SBox)
			.HAlign(HAlign_Right)
			.MaxDesiredWidth(100.f)
			[
				SNew(SFrameRateEntryBox)
				.Value(this, &SSequencerPlayRateCombo::GetDisplayRate)
				.OnValueChanged(this, &SSequencerPlayRateCombo::SetDisplayRate)
				.IsEnabled_Lambda([this] { return !GetIsSequenceReadOnly(); })
			],
			LOCTEXT("CustomFramerateDisplayLabel", "Custom")
		);

		if (CompatibleRates.Num() != FCommonFrameRates::GetAll().Num())
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("IncompatibleRates", "Incompatible Rates"),
				FText::Format(LOCTEXT("IncompatibleRates_Description", "Choose from a list of display rates that are incompatible with a resolution of {0} ticks per second"), TickResolution.AsDecimal()),
				FNewMenuDelegate::CreateSP(this, &SSequencerPlayRateCombo::PopulateIncompatibleRatesMenu)
				);
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.AddMenuSeparator();

	MenuBuilder.AddSubMenu(
		LOCTEXT("ShowTimesAs", "Show Time As"),
		LOCTEXT("ShowTimesAs_Description", "Change how to display times in Sequencer"),
		FNewMenuDelegate::CreateLambda([this](FMenuBuilder& InMenuBuilder) {
			if (WeakSequencerWidget.IsValid())
			{
				if (!UToolMenus::Get()->IsMenuRegistered("Sequencer.TimeDisplayMenu"))
				{
					UToolMenu* Toolbar = UToolMenus::Get()->RegisterMenu("Sequencer.TimeDisplayMenu", NAME_None, EMultiBoxType::Menu);
					Toolbar->AddDynamicSection("DefaultFormats", FNewToolMenuDelegate::CreateStatic(&SSequencer::FillTimeDisplayFormatMenu));
				}

				USequencerToolMenuContext* ContextObject = NewObject<USequencerToolMenuContext>();
				ContextObject->WeakSequencer = WeakSequencer;

				InMenuBuilder.AddWidget(UToolMenus::Get()->GenerateWidget("Sequencer.TimeDisplayMenu", FToolMenuContext(ContextObject)), FText(), true, false);
			}
		})
	);

	MenuBuilder.AddSubMenu(
		LOCTEXT("ClockSource", "Clock Source"),
		LOCTEXT("ClockSource_Description", "Change which clock should be used when playing back this sequence"),
		FNewMenuDelegate::CreateSP(this, &SSequencerPlayRateCombo::PopulateClockSourceMenu)
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("LockPlayback", "Lock to Display Rate at Runtime"),
		LOCTEXT("LockPlayback_Description", "When enabled, causes all runtime evaluation and the engine FPS to be locked to the current display frame rate"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SSequencerPlayRateCombo::OnToggleFrameLocked),
			FCanExecuteAction::CreateLambda([this] { return !GetIsSequenceReadOnly(); } ),
			FGetActionCheckState::CreateSP(this, &SSequencerPlayRateCombo::OnGetFrameLockedCheckState)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AdvancedOptions", "Advanced Options"),
		LOCTEXT("AdvancedOptions_Description", "Open advanced time-related properties for this sequence"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(SequencerWidget.Get(), &SSequencer::OpenTickResolutionOptions)
		)
	);

	return MenuBuilder.MakeWidget();
}

void SSequencerPlayRateCombo::PopulateIncompatibleRatesMenu(FMenuBuilder& MenuBuilder)
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();

		TArray<FCommonFrameRateInfo> IncompatibleRates;
		for (const FCommonFrameRateInfo& Info : FCommonFrameRates::GetAll())
		{
			if (!Info.FrameRate.IsMultipleOf(TickResolution))
			{
				IncompatibleRates.Add(Info);
			}
		}

		IncompatibleRates.Sort(
			[=](const FCommonFrameRateInfo& A, const FCommonFrameRateInfo& B)
			{
				return A.FrameRate.AsDecimal() < B.FrameRate.AsDecimal();
			}
		);

		for (const FCommonFrameRateInfo& Info : IncompatibleRates)
		{
			AddMenuEntry(MenuBuilder, Info);
		}
	}
}

void SSequencerPlayRateCombo::PopulateClockSourceMenu(FMenuBuilder& MenuBuilder)
{
	static const FName MenuName("Sequencer.PlayRateCombo.ClockSource");
	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		UToolMenu* ClockSourceMenu = UToolMenus::Get()->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu);
		
		ClockSourceMenu->AddDynamicSection("StandardClocks", FNewToolMenuDelegate::CreateStatic([](UToolMenu* InMenu){
			USequencerClockSourceMenuContext* ContextObject = InMenu->FindContext<USequencerClockSourceMenuContext>();
			TSharedPtr<SSequencerPlayRateCombo> Combo = ContextObject ? ContextObject->WeakSequencerCombo.Pin() : nullptr;
			TSharedPtr<ISequencer>     Sequencer     = Combo ? Combo->WeakSequencer.Pin() : nullptr;
			UMovieSceneSequence*	   RootSequence  = Sequencer.IsValid() ? Sequencer->GetRootMovieSceneSequence() : nullptr;

			const UEnum* ClockSourceEnum = StaticEnum<EUpdateClockSource>();

			check(ClockSourceEnum);
			if (RootSequence)
			{
				const bool IsFocusedOnRootSequence = Sequencer->GetRootMovieSceneSequence() == Sequencer->GetFocusedMovieSceneSequence();

				for (int32 Index = 0; Index < ClockSourceEnum->NumEnums() - 1; Index++)
				{
					if (!ClockSourceEnum->HasMetaData(TEXT("Hidden"), Index))
					{
						EUpdateClockSource Value = (EUpdateClockSource)ClockSourceEnum->GetValueByIndex(Index);

						if (Value == EUpdateClockSource::Custom)
						{
							FToolMenuEntry Item = FToolMenuEntry::InitSubMenu(NAME_None,
								ClockSourceEnum->GetDisplayNameTextByIndex(Index),
								ClockSourceEnum->GetToolTipTextByIndex(Index),
								FNewToolMenuChoice(FNewMenuDelegate::CreateSP(Combo.Get(), &SSequencerPlayRateCombo::PopulateCustomClockSourceMenu)),
								FToolUIActionChoice(FUIAction(
									FExecuteAction::CreateSP(Combo.Get(), &SSequencerPlayRateCombo::SetClockSource, Value),
									FCanExecuteAction::CreateLambda([Combo, IsFocusedOnRootSequence]{ return !Combo->GetIsSequenceReadOnly() && IsFocusedOnRootSequence; }),
									FIsActionChecked::CreateLambda([=]{ return RootSequence->GetMovieScene()->GetClockSource() == Value; })
								)),
								EUserInterfaceActionType::RadioButton);
							
							InMenu->AddMenuEntry("StandardClocks", Item);
						}
						else
						{
							FToolMenuEntry Item = FToolMenuEntry::InitMenuEntry(
								NAME_None,
								ClockSourceEnum->GetDisplayNameTextByIndex(Index),
								ClockSourceEnum->GetToolTipTextByIndex(Index),
								FSlateIcon(),
								FToolUIActionChoice(FUIAction(
									FExecuteAction::CreateSP(Combo.Get(), &SSequencerPlayRateCombo::SetClockSource, Value),
									FCanExecuteAction::CreateLambda([Combo, IsFocusedOnRootSequence]{ return !Combo->GetIsSequenceReadOnly() && IsFocusedOnRootSequence; }),
									FIsActionChecked::CreateLambda([=]{ return RootSequence->GetMovieScene()->GetClockSource() == Value; })
									)),
								EUserInterfaceActionType::RadioButton
								);
							InMenu->AddMenuEntry("StandardClocks", Item);
						}
					}
				}
				FToolMenuEntry Sep = FToolMenuEntry::InitSeparator("ExtensionClockTypes");
				InMenu->AddMenuEntry("StandardClocks", Sep);
			}
		}));
	}
	USequencerClockSourceMenuContext* ContextObject = NewObject<USequencerClockSourceMenuContext>();
	ContextObject->WeakSequencer = WeakSequencer;
	ContextObject->WeakSequencerCombo = StaticCastWeakPtr<SSequencerPlayRateCombo>(AsWeak()); 
	FToolMenuContext MenuContext(ContextObject);
	MenuBuilder.AddWidget(UToolMenus::Get()->GenerateWidget(MenuName, MenuContext), FText(), true, false);
}

void SSequencerPlayRateCombo::PopulateCustomClockSourceMenu(FMenuBuilder& MenuBuilder)
{
	auto ActorClockSourceMenu = [this](FMenuBuilder& ActorMenuBuilder)
	{
		auto IsActorValid = [](const AActor* Actor)
		{
			if (!Actor)
			{
				return false;
			}

			if (Actor->GetClass()->ImplementsInterface(UMovieSceneCustomClockSource::StaticClass()))
			{
				return true;
			}

			TArray<UActorComponent*> ComponentsWithClockSource = Actor->GetComponentsByInterface(UMovieSceneCustomClockSource::StaticClass());
			if (ComponentsWithClockSource.Num() > 0)
			{
				return true;
			}

			return false;
		};

		// Set up a menu entry to assign an actor to the object binding node
		FSceneOutlinerInitializationOptions InitOptions;

		// We hide the header row to keep the UI compact.
		InitOptions.bShowHeaderRow = false;
		InitOptions.bShowSearchBox = true;
		InitOptions.bShowCreateNewFolder = false;
		InitOptions.bFocusSearchBoxWhenOpened = true;
		// Only want the actor label column
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0));

		// Only display actors that are not possessed already
		InitOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateLambda( IsActorValid ) );

		TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
		const float WidthOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
		const float HeightOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

		// actor selector to allow the user to choose an actor
		FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
		ActorMenuBuilder.AddWidget(
			SNew(SBox)
			.WidthOverride(WidthOverride)
			.HeightOverride(HeightOverride)
			[
				SceneOutlinerModule.CreateActorPicker(InitOptions, FOnActorPicked::CreateLambda([this](AActor* In){ this->SetCustomClockSource(In); }))
			],
			FText(),
			true /*bNoIndent*/
		);
	};

	auto AssetClockSourceMenu = [this](FMenuBuilder& ActorMenuBuilder)
	{
		FAssetPickerConfig AssetPickerConfig;
		AssetPickerConfig.Filter.ClassPaths.Add(UMovieSceneCustomClockSource::StaticClass()->GetClassPathName());
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda([this](const FAssetData& In){ this->SetCustomClockSource(In.GetAsset()); });
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.bAllowDragging = false;

		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		ActorMenuBuilder.AddWidget(
			SNew(SBox)
			.WidthOverride(300)
			.HeightOverride(300)
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			],
			FText(),
			true /*bNoIndent*/
		);
	};

	MenuBuilder.AddSubMenu(LOCTEXT("ActorClockSource", "Actors"), FText(), FNewMenuDelegate::CreateLambda(ActorClockSourceMenu));
	MenuBuilder.AddSubMenu(LOCTEXT("AssetClockSource", "Assets"), FText(), FNewMenuDelegate::CreateLambda(AssetClockSourceMenu));
}

void SSequencerPlayRateCombo::AddMenuEntry(FMenuBuilder& MenuBuilder, const FCommonFrameRateInfo& Info)
{
	MenuBuilder.AddMenuEntry(
		Info.DisplayName,
		Info.Description,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SSequencerPlayRateCombo::SetDisplayRate, Info.FrameRate),
			FCanExecuteAction::CreateLambda([this]{ return !GetIsSequenceReadOnly(); }),
			FIsActionChecked::CreateSP(this, &SSequencerPlayRateCombo::IsSameDisplayRate, Info.FrameRate)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);
}

void SSequencerPlayRateCombo::SetClockSource(EUpdateClockSource NewClockSource)
{
	TSharedPtr<FSequencer> Sequencer    = WeakSequencer.Pin();
	UMovieSceneSequence*   RootSequence = Sequencer.IsValid() ? Sequencer->GetRootMovieSceneSequence() : nullptr;
	if (RootSequence)
	{
		UMovieScene* MovieScene = RootSequence->GetMovieScene();

		if (MovieScene->IsReadOnly())
		{
			return;
		}

		FScopedTransaction ScopedTransaction(LOCTEXT("SetClockSource", "Set Clock Source"));

		MovieScene->Modify();
		MovieScene->SetClockSource(NewClockSource);

		Sequencer->ResetTimeController();
	}
}

void SSequencerPlayRateCombo::SetCustomClockSource(UObject* InClockSource)
{
	TSharedPtr<FSequencer> Sequencer    = WeakSequencer.Pin();
	UMovieSceneSequence*   RootSequence = Sequencer.IsValid() ? Sequencer->GetRootMovieSceneSequence() : nullptr;
	if (RootSequence && InClockSource)
	{
		UMovieScene* MovieScene = RootSequence->GetMovieScene();

		if (MovieScene->IsReadOnly())
		{
			return;
		}

		FScopedTransaction ScopedTransaction(LOCTEXT("SetClockSource", "Set Clock Source"));

		MovieScene->Modify();

		UMovieSceneExternalClock* External = MovieScene->GetCustomClockAs<UMovieSceneExternalClock>();
		if (External == nullptr)
		{
			External = NewObject<UMovieSceneExternalClock>(MovieScene);
			MovieScene->SetCustomClock(External);
		}

		External->Modify();
		External->CustomClockSourcePath = InClockSource;

		Sequencer->ResetTimeController();
	}
}

EVisibility SSequencerPlayRateCombo::GetClockSourceVisibility() const
{
	TSharedPtr<FSequencer> Sequencer    = WeakSequencer.Pin();
	UMovieSceneSequence*   RootSequence = Sequencer.IsValid() ? Sequencer->GetRootMovieSceneSequence() : nullptr;
	if (RootSequence)
	{
		UMovieScene* MovieScene = RootSequence->GetMovieScene();

		if (MovieScene && MovieScene->GetClockSource() != EUpdateClockSource::Tick)
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Hidden;
}

const FSlateBrush* SSequencerPlayRateCombo::GetClockSourceImage() const
{
	TSharedPtr<FSequencer> Sequencer    = WeakSequencer.Pin();
	UMovieSceneSequence*   RootSequence = Sequencer.IsValid() ? Sequencer->GetRootMovieSceneSequence() : nullptr;
	if (RootSequence)
	{
		UMovieScene* MovieScene = RootSequence->GetMovieScene();

		if (MovieScene)
		{
			switch (MovieScene->GetClockSource())
			{
				case EUpdateClockSource::Tick:
					return nullptr;
				case EUpdateClockSource::Platform:
					return FAppStyle::GetBrush("Sequencer.ClockSource.Platform");
				case EUpdateClockSource::Audio:
					return FAppStyle::GetBrush("Sequencer.ClockSource.Audio");
				case EUpdateClockSource::RelativeTimecode:
					return FAppStyle::GetBrush("Sequencer.ClockSource.RelativeTimecode");
				case EUpdateClockSource::Timecode:
					return FAppStyle::GetBrush("Sequencer.ClockSource.Timecode");
				case EUpdateClockSource::PlayEveryFrame:
					return FAppStyle::GetBrush("Sequencer.ClockSource.PlayEveryFrame");
				case EUpdateClockSource::Custom:
					return FAppStyle::GetBrush("Sequencer.ClockSource.Custom");
				default:
					return nullptr;
			}
		}
	}
	return nullptr;
}


void SSequencerPlayRateCombo::SetDisplayRate(FFrameRate InFrameRate)
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	UMovieSceneSequence* FocusedSequence = Sequencer.IsValid() ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	if (FocusedSequence)
	{
		UMovieScene* MovieScene = FocusedSequence->GetMovieScene();
		if (MovieScene->IsReadOnly())
		{
			return;
		}

		FScopedTransaction ScopedTransaction(FText::Format(LOCTEXT("SetDisplayRate", "Set Display Rate to {0}"), InFrameRate.ToPrettyText()));

		MovieScene->Modify();
		MovieScene->SetDisplayRate(InFrameRate);

		TArray<UMovieScene*> DescendantMovieScenes;
		MovieSceneHelpers::GetDescendantMovieScenes(FocusedSequence, DescendantMovieScenes);

		for (UMovieScene* DescendantMovieScene : DescendantMovieScenes)
		{
			if (DescendantMovieScene && InFrameRate != DescendantMovieScene->GetDisplayRate())
			{
				if (!DescendantMovieScene->IsReadOnly())
				{
					DescendantMovieScene->Modify();
					DescendantMovieScene->SetDisplayRate(InFrameRate);
				}
			}
		}
	}

	// Snap the local time to the new display rate
	Sequencer->SetLocalTime(Sequencer->GetLocalTime().Time, ESnapTimeMode::STM_Interval);
}

FFrameRate SSequencerPlayRateCombo::GetDisplayRate() const
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	return Sequencer.IsValid() ? Sequencer->GetFocusedDisplayRate() : FFrameRate();
}

bool SSequencerPlayRateCombo::IsSameDisplayRate(FFrameRate InFrameRate) const
{
	return GetDisplayRate() == InFrameRate;
}

FText SSequencerPlayRateCombo::GetFrameRateText() const
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();

	return Sequencer.IsValid() ? Sequencer->GetFocusedDisplayRate().ToPrettyText() : FText();
}

FText SSequencerPlayRateCombo::GetToolTipText() const
{
	TSharedPtr<FSequencer> Sequencer         = WeakSequencer.Pin();
	UMovieSceneSequence*   FocusedSequence   = Sequencer.IsValid() ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	UMovieScene*           FocusedMovieScene = FocusedSequence ? FocusedSequence->GetMovieScene() : nullptr;

	if (FocusedMovieScene)
	{
		FFrameRate DisplayRate     = FocusedMovieScene->GetDisplayRate();
		FFrameRate TickResolution  = FocusedMovieScene->GetTickResolution();

		const FCommonFrameRateInfo* DisplayRateInfo    = FCommonFrameRates::Find(DisplayRate);
		const FCommonFrameRateInfo* TickResolutionInfo = FCommonFrameRates::Find(TickResolution);

		FText DisplayRateText     = DisplayRateInfo     ? DisplayRateInfo->DisplayName    : FText::Format(LOCTEXT("DisplayRateFormat", "{0} fps"), DisplayRate.AsDecimal());
		FText TickResolutionText  = TickResolutionInfo  ? TickResolutionInfo->DisplayName : FText::Format(LOCTEXT("TickResolutionFormat", "{0} ticks every second"), TickResolution.AsDecimal());

		return FocusedMovieScene->GetEvaluationType() == EMovieSceneEvaluationType::FrameLocked
			? FText::Format(LOCTEXT("ToolTip_Format_FrameLocked", "This sequence is locked at runtime to {0} and uses an underlying tick resolution of {1}."), DisplayRateText, TickResolutionText)
			: FText::Format(LOCTEXT("ToolTip_Format", "This sequence is being presented as {0} and uses an underlying tick resolution of {1}."), DisplayRateText, TickResolutionText);
	}

	return FText();
}


void SSequencerPlayRateCombo::OnToggleFrameLocked()
{
	TSharedPtr<FSequencer> Sequencer         = WeakSequencer.Pin();
	UMovieSceneSequence*   FocusedSequence   = Sequencer.IsValid() ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	UMovieScene*           FocusedMovieScene = FocusedSequence ? FocusedSequence->GetMovieScene() : nullptr;

	if (FocusedMovieScene)
	{
		if (FocusedMovieScene->IsReadOnly())
		{
			return;
		}

		EMovieSceneEvaluationType NewType = FocusedMovieScene->GetEvaluationType() == EMovieSceneEvaluationType::WithSubFrames
			? EMovieSceneEvaluationType::FrameLocked
			: EMovieSceneEvaluationType::WithSubFrames;

		FScopedTransaction ScopedTransaction(NewType == EMovieSceneEvaluationType::FrameLocked
			? LOCTEXT("FrameLockedTransaction",   "Lock to Display Rate at Runtime")
			: LOCTEXT("WithSubFramesTransaction", "Unlock to runtime frame rate"));

		FocusedMovieScene->Modify();
		FocusedMovieScene->SetEvaluationType(NewType);
	}
}

ECheckBoxState SSequencerPlayRateCombo::OnGetFrameLockedCheckState() const
{
	TSharedPtr<FSequencer> Sequencer         = WeakSequencer.Pin();
	UMovieSceneSequence*   FocusedSequence   = Sequencer.IsValid() ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	UMovieScene*           FocusedMovieScene = FocusedSequence ? FocusedSequence->GetMovieScene() : nullptr;

	return FocusedMovieScene && FocusedMovieScene->GetEvaluationType() == EMovieSceneEvaluationType::FrameLocked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

#undef LOCTEXT_NAMESPACE