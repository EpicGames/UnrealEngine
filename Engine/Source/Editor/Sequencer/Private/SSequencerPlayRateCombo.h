// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Containers/ArrayView.h"
#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/AppStyle.h"
#include "MovieSceneFwd.h"
#include "CommonFrameRates.h"

class SSequencer;
class FSequencer;
class FMenuBuilder;

class SSequencerPlayRateCombo : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSequencerPlayRateCombo)
		: _StyleSet(&FAppStyle::Get())
		, _BlockLocation(EMultiBlockLocation::None)
		, _StyleName("SlimToolBar")
	{}

		SLATE_ARGUMENT(const ISlateStyle*, StyleSet)

		SLATE_ARGUMENT(EMultiBlockLocation::Type, BlockLocation)

		SLATE_ARGUMENT(FName, StyleName)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FSequencer> Sequencer, TWeakPtr<SSequencer> InWeakSequencerWidget);

private:

	FText GetFrameRateText() const;
	FText GetToolTipText() const;

	TSharedRef<SWidget> OnCreateMenu();
	void PopulateIncompatibleRatesMenu(FMenuBuilder& MenuBuilder);
	void PopulateClockSourceMenu(FMenuBuilder& MenuBuilder);
	void PopulateCustomClockSourceMenu(FMenuBuilder& MenuBuilder);
	void AddMenuEntry(FMenuBuilder& MenuBuilder, const FCommonFrameRateInfo& Info);

	void SetCustomClockSource(UObject* Object);

	void OnToggleFrameLocked();
	ECheckBoxState OnGetFrameLockedCheckState() const;

	void SetDisplayRate(FFrameRate InFrameRate);
	FFrameRate GetDisplayRate() const;
	bool IsSameDisplayRate(FFrameRate InFrameRate) const;

	void SetClockSource(EUpdateClockSource NewClockSource);

	FText GetFrameRateIsMultipleOfErrorDescription() const;
	FText GetFrameRateMismatchErrorDescription() const;

	EVisibility GetFrameLockedVisibility() const;
	EVisibility GetFrameRateIsMultipleOfErrorVisibility() const;
	EVisibility GetFrameRateMismatchErrorVisibility() const;

	EVisibility GetClockSourceVisibility() const;
	const FSlateBrush* GetClockSourceImage() const;

	bool GetIsSequenceReadOnly() const;

	/** Sequencer pointer */
	TWeakPtr<FSequencer> WeakSequencer;
	TWeakPtr<SSequencer> WeakSequencerWidget;
};