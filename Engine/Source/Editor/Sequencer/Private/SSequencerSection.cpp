// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerSection.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/EditorSharedViewModelData.h"
#include "MVVM/Views/ITrackAreaHotspot.h"
#include "MVVM/Views/STrackAreaView.h"
#include "MVVM/Views/SCompoundTrackLaneView.h"
#include "MVVM/Views/STrackLane.h"
#include "MVVM/Views/SChannelView.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Selection/Selection.h"
#include "AnimatedRange.h"
#include "Rendering/DrawElements.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "SequencerSelectionPreview.h"
#include "SequencerSettings.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "Sequencer.h"
#include "SequencerCommands.h"
#include "SequencerSectionPainter.h"
#include "MovieSceneSequence.h"
#include "ISequencerEditTool.h"
#include "ISequencerSection.h"
#include "SequencerHotspots.h"
#include "Widgets/SOverlay.h"
#include "MovieScene.h"
#include "Fonts/FontCache.h"
#include "Fonts/FontMeasure.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Framework/Application/SlateApplication.h"
#include "MovieSceneTimeHelpers.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "Sections/MovieSceneSubSection.h"
#include "Generators/MovieSceneEasingFunction.h"
#include "IKeyArea.h"
#include "Widgets/SWeakWidget.h"
#include "Algo/Transform.h"
#include "Tracks/IMovieSceneSectionsToKey.h"
#include "Misc/InputState.h"

namespace UE
{
namespace Sequencer
{

double SSequencerSection::SectionSelectionThrobEndTime = 0;
double SSequencerSection::KeySelectionThrobEndTime = 0;


/** A point on an easing curve used for rendering */
struct FEasingCurvePoint
{
	FEasingCurvePoint(FVector2D InLocation, const FLinearColor& InPointColor) : Location(InLocation), Color(InPointColor) {}

	/** The location of the point (x=time, y=easing value [0-1]) */
	FVector2D Location;
	/** The color of the point */
	FLinearColor Color;
};

static bool IsSectionToKey(const UMovieSceneTrack* Track, UMovieSceneSection* SectionObject)
{
	if (const IMovieSceneSectionsToKey* MultipleSectionsToKey = Cast<IMovieSceneSectionsToKey>(Track))
	{
		TArray<TWeakObjectPtr<UMovieSceneSection>> SectionsToKey = MultipleSectionsToKey->GetSectionsToKey();
		return SectionsToKey.Contains(SectionObject);
	}
	return (Track->GetSectionToKey() == SectionObject);
}

struct FSequencerSectionPainterImpl : FSequencerSectionPainter
{
	FSequencerSectionPainterImpl(FSequencer& InSequencer, TSharedPtr<FTrackAreaViewModel> InTrackAreaViewModel, TSharedPtr<FSectionModel> InSection, FSlateWindowElementList& _OutDrawElements, const FGeometry& InSectionGeometry, const SSequencerSection& InSectionWidget)
		: FSequencerSectionPainter(_OutDrawElements, InSectionGeometry, InSection)
		, Sequencer(InSequencer)
		, SectionWidget(InSectionWidget)
		, TimeToPixelConverter(*InSectionWidget.GetTimeToPixel())
		, TrackAreaViewModel(InTrackAreaViewModel)
		, bClipRectEnabled(false)
	{
		CalculateSelectionColor();

		const ISequencerEditTool* EditTool = TrackAreaViewModel->GetEditTool();
		Hotspot = EditTool ? EditTool->GetDragHotspot() : nullptr;
		if (!Hotspot)
		{
			Hotspot = TrackAreaViewModel->GetHotspot();
		}
	}

	FLinearColor GetFinalTintColor(const FLinearColor& Tint) const
	{
		FLinearColor FinalTint = STrackAreaView::BlendDefaultTrackColor(Tint);
		if (bIsHighlighted && SectionModel->GetRange() != TRange<FFrameNumber>::All())
		{
			float Lum = FinalTint.GetLuminance() * 0.2f;
			FinalTint = FinalTint + FLinearColor(Lum, Lum, Lum, 0.f);
		}

		FinalTint.A *= GhostAlpha;

		return FinalTint;
	}

	virtual int32 PaintSectionBackground(const FLinearColor& Tint) override
	{
		using namespace UE::MovieScene;

		const USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings();
		check(SequencerSettings);

		TRange<FFrameNumber> SectionRange = SectionModel->GetRange();

		const ESlateDrawEffect DrawEffects = bParentEnabled
			? ESlateDrawEffect::None
			: ESlateDrawEffect::DisabledEffect;

		static const FSlateBrush* CollapsedSectionBackgroundBrush = FAppStyle::GetBrush("Sequencer.Section.Background_Collapsed");
		static const FSlateBrush* SectionHeaderBackgroundBrush = FAppStyle::GetBrush("Sequencer.Section.Background_Header");
		static const FSlateBrush* SectionContentsBackgroundBrush = FAppStyle::GetBrush("Sequencer.Section.Background_Contents");

		static const FSlateBrush* CollapsedSelectedSectionOverlay = FAppStyle::GetBrush("Sequencer.Section.CollapsedSelectedSectionOverlay");
		static const FSlateBrush* SectionHeaderSelectedSectionOverlay = FAppStyle::GetBrush("Sequencer.Section.SectionHeaderSelectedSectionOverlay");

		FLinearColor FinalTint = GetFinalTintColor(Tint);

		// Offset lower bounds and size for infinte sections so we don't draw the rounded border on the visible area
		const float InfiniteLowerOffset = SectionRange.GetLowerBound().IsClosed() ? 0.f : 100.f;
		const float InfiniteSizeOffset  = InfiniteLowerOffset + (SectionRange.GetUpperBound().IsClosed() ? 0.f : 100.f);

		FGeometry ExpandedSectionGeometry = SectionGeometry.MakeChild(
			SectionGeometry.GetLocalSize() + FVector2D(InfiniteSizeOffset, 0.f),
			FSlateLayoutTransform(FVector2D(-InfiniteLowerOffset, 0.f))
			);

		if (SequencerSettings->ShouldShowPrePostRoll())
		{
			TOptional<FSlateClippingState> PreviousClipState = DrawElements.GetClippingState();
			if (PreviousClipState.IsSet())
			{
				DrawElements.PopClip();
			}

			static const FSlateBrush* PreRollBrush = FAppStyle::GetBrush("Sequencer.Section.PreRoll");
			float BrushHeight = 16.f, BrushWidth = 10.f;

			if (SectionRange.GetLowerBound().IsClosed())
			{
				FFrameNumber SectionStartTime = DiscreteInclusiveLower(SectionRange);
				FFrameNumber PreRollStartTime = SectionStartTime - SectionModel->GetPreRollFrames();

				const float PreRollPx = TimeToPixelConverter.FrameToPixel(SectionStartTime) - TimeToPixelConverter.FrameToPixel(PreRollStartTime);
				if (PreRollPx > 0)
				{
					const float RoundedPreRollPx = (int(PreRollPx / BrushWidth)+1) * BrushWidth;

					// Round up to the nearest BrushWidth size
					FGeometry PreRollArea = SectionGeometry.MakeChild(
						FVector2D(RoundedPreRollPx, BrushHeight),
						FSlateLayoutTransform(FVector2D(-PreRollPx, (SectionGeometry.GetLocalSize().Y - BrushHeight)*.5f))
						);

					FSlateDrawElement::MakeBox(
						DrawElements,
						LayerId,
						PreRollArea.ToPaintGeometry(),
						PreRollBrush,
						DrawEffects
					);
				}
			}

			if (SectionRange.GetUpperBound().IsClosed())
			{
				FFrameNumber SectionEndTime  = DiscreteExclusiveUpper(SectionRange.GetUpperBound());
				FFrameNumber PostRollEndTime = SectionEndTime + SectionModel->GetPostRollFrames();

				const float PostRollPx = TimeToPixelConverter.FrameToPixel(PostRollEndTime) - TimeToPixelConverter.FrameToPixel(SectionEndTime);
				if (PostRollPx > 0)
				{
					const float RoundedPostRollPx = (int(PostRollPx / BrushWidth)+1) * BrushWidth;
					const float Difference = RoundedPostRollPx - PostRollPx;

					// Slate border brushes tile UVs along +ve X, so we round the arrows to a multiple of the brush width, and offset, to ensure we don't have a partial tile visible at the end
					FGeometry PostRollArea = SectionGeometry.MakeChild(
						FVector2D(RoundedPostRollPx, BrushHeight),
						FSlateLayoutTransform(FVector2D(SectionGeometry.GetLocalSize().X - Difference, (SectionGeometry.GetLocalSize().Y - BrushHeight)*.5f))
						);

					FSlateDrawElement::MakeBox(
						DrawElements,
						LayerId,
						PostRollArea.ToPaintGeometry(),
						PreRollBrush,
						DrawEffects
					);
				}
			}

			if (PreviousClipState.IsSet())
			{
				DrawElements.GetClippingManager().PushClippingState(PreviousClipState.GetValue());
			}
		}

		// If this section has any children, we draw the section header on the top row, and a dimmed fill for all children

		TViewModelPtr<IOutlinerExtension> Outliner = SectionModel->FindAncestorOfType<IOutlinerExtension>();
		const bool bHasChildren = Outliner && Outliner.AsModel()->GetChildren(EViewModelListType::Outliner);
		const bool bIsExpanded  = Outliner && Outliner->IsExpanded();

		const FLinearColor BlendedTint = BlendColor(Tint).CopyWithNewOpacity(SequencerSettings->GetSectionColorAlpha());

		if (!bHasChildren || !bIsExpanded)
		{
			FSlateDrawElement::MakeBox(
				DrawElements,
				LayerId,
				ExpandedSectionGeometry.ToPaintGeometry(),
				CollapsedSectionBackgroundBrush,
				DrawEffects,
				BlendedTint
			);

			// Draw the selection hash
			if (SelectionColor.IsSet())
			{
				FSlateDrawElement::MakeBox(
					DrawElements,
					++LayerId,
					ExpandedSectionGeometry.ToPaintGeometry(ExpandedSectionGeometry.GetLocalSize() - FVector2f(2.f,2.f), FSlateLayoutTransform(FVector2f(1.f, 1.f))),
					CollapsedSelectedSectionOverlay,
					DrawEffects,
					SelectionColor.GetValue().CopyWithNewOpacity(0.8f)
				);
			}
		}
		else
		{
			FGeometry LocalHeaderGeometry = ExpandedSectionGeometry.MakeChild(
				FVector2D(ExpandedSectionGeometry.GetLocalSize().X, HeaderGeometry.GetLocalSize().Y),
				FSlateLayoutTransform()
			);
			const float HeaderHeight = LocalHeaderGeometry.GetLocalSize().Y;
			FGeometry ContentsGeometry = ExpandedSectionGeometry.MakeChild(
				FVector2D(ExpandedSectionGeometry.GetLocalSize().X, ExpandedSectionGeometry.GetLocalSize().Y - HeaderHeight),
				FSlateLayoutTransform(FVector2D(0.f, HeaderHeight)) 
			);

			FSlateDrawElement::MakeBox(
				DrawElements,
				LayerId,
				LocalHeaderGeometry.ToPaintGeometry(),
				SectionHeaderBackgroundBrush,
				DrawEffects,
				BlendedTint
			);

			FLinearColor FillTint = BlendedTint.LinearRGBToHSV();
			FillTint.G *= .5f;
			FillTint.B = FMath::Max(.03f, FillTint.B*.1f);
			// Draw contents background at base LayerId (not incremented) so child track lanes
			// that paint at the same layer can draw their content on top
			FSlateDrawElement::MakeBox(
				DrawElements,
				LayerId,
				ContentsGeometry.ToPaintGeometry(),
				SectionContentsBackgroundBrush,
				DrawEffects,
				FillTint.HSVToLinearRGB()
			);

			// Draw the selection hash
			if (SelectionColor.IsSet())
			{
				FSlateDrawElement::MakeBox(
					DrawElements,
					++LayerId,
					LocalHeaderGeometry.ToPaintGeometry(LocalHeaderGeometry.GetLocalSize() - FVector2f(2.f, 2.f), FSlateLayoutTransform(FVector2f(1.f, 1.f))),
					SectionHeaderSelectedSectionOverlay,
					DrawEffects,
					SelectionColor.GetValue().CopyWithNewOpacity(0.8f)
				);

				FLinearColor FillSelectionColor = SelectionColor.GetValue().LinearRGBToHSV();
				FillSelectionColor.G *= .5f;
				FillSelectionColor.B = FMath::Max(.03f, FillSelectionColor.B * .1f);
				FSlateDrawElement::MakeBox(
					DrawElements,
					++LayerId,
					ContentsGeometry.ToPaintGeometry(ContentsGeometry.GetLocalSize() - FVector2f(2.f, 2.f), FSlateLayoutTransform(FVector2f(1.f, 1.f))),
					SectionHeaderSelectedSectionOverlay,
					DrawEffects,
					FillSelectionColor.HSVToLinearRGB().CopyWithNewOpacity(0.8f)
				);
			}
		}

		if (!ensure(!bClipRectEnabled))
		{
			FSlateClippingZone ClippingZone(SectionClippingRect);
			DrawElements.PushClip(ClippingZone);

			bClipRectEnabled = true;
		}

		++LayerId;

		// Draw underlapping sections
		DrawOverlaps(FinalTint);

		// Draw empty space
		DrawEmptySpace();

		// Draw the blend type text
		DrawBlendType();

		// Draw the locked / key border
		DrawBorder(FinalTint);

		// Draw easing curves
		DrawEasing(FinalTint);

		return LayerId;
	}

	virtual const FTimeToPixel& GetTimeConverter() const
	{
		return TimeToPixelConverter;
	}

	void CalculateSelectionColor()
	{
		FSequencerSelection& Selection = *Sequencer.GetViewModel()->GetSelection();
		FSequencerSelectionPreview& SelectionPreview = Sequencer.GetSelectionPreview();

		ESelectionPreviewState SelectionPreviewState = SelectionPreview.GetSelectionState(SectionWidget.WeakSectionModel);

		if (SelectionPreviewState == ESelectionPreviewState::NotSelected)
		{
			// Explicitly not selected in the preview selection
			return;
		}
		
		if (SelectionPreviewState == ESelectionPreviewState::Undefined && !Selection.TrackArea.IsSelected(SectionModel))
		{
			// No preview selection for this section, and it's not selected
			return;
		}

		SelectionColor = FAppStyle::GetSlateColor(SequencerSectionConstants::SelectionColorName).GetColor(FWidgetStyle());

		// Use a muted selection color for selection previews
		if (SelectionPreviewState == ESelectionPreviewState::Selected)
		{
			SelectionColor.GetValue() = SelectionColor.GetValue().LinearRGBToHSV();
			SelectionColor.GetValue().R += 0.1f; // +10% hue
			SelectionColor.GetValue().G = 0.6f; // 60% saturation

			SelectionColor = SelectionColor.GetValue().HSVToLinearRGB();
		}

		SelectionColor->A *= GhostAlpha;
	}

	void DrawBlendType()
	{
		// Draw the blend type text if necessary
		UMovieSceneSection* SectionObject = SectionModel->GetSection();
		UMovieSceneTrack* Track = GetTrack();
		if (!Track || Track->GetSupportedBlendTypes().Num() <= 1 || !SectionObject->GetBlendType().IsValid() || !bIsHighlighted || SectionObject->GetBlendType().Get() == EMovieSceneBlendType::Absolute)
		{
			return;
		}

		TSharedPtr<STrackLane> TrackLane = SectionWidget.WeakOwningTrackLane.Pin();
		if (!TrackLane)
		{
			return;
		}

		const float LaneHeight = TrackLane->GetDesiredSize().Y;

		TSharedRef<FSlateFontCache> FontCache = FSlateApplication::Get().GetRenderer()->GetFontCache();

		UEnum* Enum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/MovieScene.EMovieSceneBlendType"), EFindObjectFlags::ExactClass);
		FText DisplayText = Enum->GetDisplayNameTextByValue((int64)SectionObject->GetBlendType().Get());

		FSlateFontInfo FontInfo = FAppStyle::GetFontStyle("Sequencer.Section.BackgroundText");
		FontInfo.Size = 24;

		auto GetFontHeight = [&]
		{
			return FontCache->GetMaxCharacterHeight(FontInfo, 1.f) + FontCache->GetBaseline(FontInfo, 1.f);
		};
		while( GetFontHeight() > SectionGeometry.Size.Y && FontInfo.Size > 11 )
		{
			FontInfo.Size = FMath::Max(FMath::FloorToInt(FontInfo.Size - 6.f), 11);
		}

		const float FontHeight = GetFontHeight();

		// Offset more to the right of the lower bound since there's a handle there
		FVector2D TextOffset = SectionModel->GetRange().HasLowerBound() ? FVector2D(8.f, LaneHeight - FontHeight - 4.f) : FVector2D(1.f, LaneHeight - FontHeight - 4.f);
		FVector2D TextPosition = SectionGeometry.AbsoluteToLocal(SectionClippingRect.GetTopLeft()) + TextOffset;

		FSlateDrawElement::MakeText(
			DrawElements,
			LayerId,
			SectionGeometry.MakeChild(
				FVector2D(SectionGeometry.Size.X, FontHeight),
				FSlateLayoutTransform(TextPosition)
			).ToPaintGeometry(),
			DisplayText,
			FontInfo,
			bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			FLinearColor(1.f,1.f,1.f,.2f)
		);
	}

	float GetEaseHighlightAmount(UMovieSceneSection* InSection, float EaseInInterp, float EaseOutInterp) const
	{
		using namespace UE::Sequencer;

		float EaseInScale = 0.f, EaseOutScale = 0.f;
		if (TSharedPtr<FSectionEasingHandleHotspot> EasingHandleHotspot = HotspotCast<FSectionEasingHandleHotspot>(Hotspot))
		{
			if (EasingHandleHotspot->GetSection() == InSection)
			{
				if (EasingHandleHotspot->HandleType == ESequencerEasingType::In)
				{
					EaseInScale = 1.f;
				}
				else
				{
					EaseOutScale = 1.f;
				}
			}
		}
		else if (TSharedPtr<FSectionEasingAreaHotspot> EasingAreaHotspot = HotspotCast<FSectionEasingAreaHotspot>(Hotspot))
		{
			for (const FEasingAreaHandle& Easing : EasingAreaHotspot->Easings)
			{
				TSharedPtr<FSectionModel> EasingSectionModel = Easing.WeakSectionModel.Pin();
				if (EasingSectionModel && EasingSectionModel->GetSection() == InSection)
				{
					if (Easing.EasingType == ESequencerEasingType::In)
					{
						EaseInScale = 1.f;
					}
					else
					{
						EaseOutScale = 1.f;
					}
				}
			}
		}
		else
		{
			return 0.f;
		}

		const float TotalScale = EaseInScale + EaseOutScale;
		return TotalScale > 0.f ? EaseInInterp * (EaseInScale/TotalScale) + ((1.f-EaseOutInterp) * (EaseOutScale/TotalScale)) : 0.f;
	}

	FEasingCurvePoint MakeCurvePoint(UMovieSceneSection* InSection, FFrameTime Time, const FLinearColor& FinalTint, const FLinearColor& EaseSelectionColor) const
	{
		TOptional<float> EaseInValue, EaseOutValue;
		float EaseInInterp = 0.f, EaseOutInterp = 1.f;
		InSection->EvaluateEasing(Time, EaseInValue, EaseOutValue, &EaseInInterp, &EaseOutInterp);

		return FEasingCurvePoint(
			FVector2D(Time / TimeToPixelConverter.GetTickResolution(), EaseInValue.Get(1.f) * EaseOutValue.Get(1.f)),
			FMath::Lerp(FinalTint, EaseSelectionColor, GetEaseHighlightAmount(InSection, EaseInInterp, EaseOutInterp))
		);
	}

	/** Adds intermediate control points for the specified section's easing up to a given threshold */
	void RefineCurvePoints(UMovieSceneSection* SectionObject, const FLinearColor& FinalTint, const FLinearColor& EaseSelectionColor, TArray<FEasingCurvePoint>& InOutPoints)
	{
		static float GradientThreshold = .05f;
		static float ValueThreshold = .05f;

		float MinTimeSize = FMath::Max(0.0001, TimeToPixelConverter.PixelToSeconds(2.5) - TimeToPixelConverter.PixelToSeconds(0));

		for (int32 Index = 0; Index < InOutPoints.Num() - 1; ++Index)
		{
			const FEasingCurvePoint& Lower = InOutPoints[Index];
			const FEasingCurvePoint& Upper = InOutPoints[Index + 1];

			if ((Upper.Location.X - Lower.Location.X)*.5f > MinTimeSize)
			{
				FVector2D::FReal      NewPointTime  = (Upper.Location.X + Lower.Location.X)*.5f;
				FFrameTime FrameTime     = NewPointTime * TimeToPixelConverter.GetTickResolution();
				float      NewPointValue = SectionObject->EvaluateEasing(FrameTime);

				// Check that the gradient is changing significantly
				FVector2D::FReal LinearValue = (Upper.Location.Y + Lower.Location.Y) * .5f;
				FVector2D::FReal PointGradient = NewPointValue - SectionObject->EvaluateEasing(FMath::Lerp(Lower.Location.X, NewPointTime, 0.9f) * TimeToPixelConverter.GetTickResolution());
				FVector2D::FReal OuterGradient = Upper.Location.Y - Lower.Location.Y;
				if (!FMath::IsNearlyEqual(OuterGradient, PointGradient, GradientThreshold) ||
					!FMath::IsNearlyEqual(LinearValue, NewPointValue, ValueThreshold))
				{
					// Add the point
					InOutPoints.Insert(MakeCurvePoint(SectionObject, FrameTime, FinalTint, EaseSelectionColor), Index+1);
					--Index;
				}
			}
		}
	}

	void DrawEasingForSegment(const FOverlappingSections& Segment, const FGeometry& InnerSectionGeometry, const FLinearColor& FinalTint)
	{
		// @todo: sequencer-timecode: Test that start offset is not required here
		const float RangeStartPixel = TimeToPixelConverter.FrameToPixel(UE::MovieScene::DiscreteInclusiveLower(Segment.Range));
		const float RangeEndPixel = TimeToPixelConverter.FrameToPixel(UE::MovieScene::DiscreteExclusiveUpper(Segment.Range));
		const float RangeSizePixel = RangeEndPixel - RangeStartPixel;

		TViewModelPtr<IGeometryExtension> Geometry = SectionModel->FindAncestorOfType<IOutlinerExtension>().ImplicitCast();
		const float EasingHeight = (Geometry ? Geometry->GetVirtualGeometry().GetHeight() : InnerSectionGeometry.Size.Y) - 2.f;

		FGeometry RangeGeometry = InnerSectionGeometry.MakeChild(FVector2D(RangeSizePixel, EasingHeight), FSlateLayoutTransform(FVector2D(RangeStartPixel, 1.f)));
		if (!FSlateRect::DoRectanglesIntersect(RangeGeometry.GetLayoutBoundingRect(), ParentClippingRect))
		{
			return;
		}

		UMovieSceneTrack* Track = GetTrack();
		if (!Track)
		{
			return;
		}

		const FSlateBrush* MyBrush = FAppStyle::Get().GetBrush("Sequencer.Timeline.EaseInOut");

		FSlateResourceHandle ResourceHandle = FSlateApplication::Get().GetRenderer()->GetResourceHandle(*MyBrush);
		const FSlateShaderResourceProxy* ResourceProxy = ResourceHandle.GetResourceProxy();

		FVector2f AtlasOffset = ResourceProxy ? ResourceProxy->StartUV : FVector2f(0.f, 0.f);
		FVector2f AtlasUVSize = ResourceProxy ? ResourceProxy->SizeUV : FVector2f(1.f, 1.f);

		FSlateRenderTransform RenderTransform;

		const FVector2f Pos = FVector2f(RangeGeometry.GetAbsolutePosition());	// LWC_TODO: Precision loss
		const FVector2D Size = RangeGeometry.GetLocalSize();

		FLinearColor EaseSelectionColor = FAppStyle::GetSlateColor(SequencerSectionConstants::SelectionColorName).GetColor(FWidgetStyle());

		FColor FillColor(0,0,0,51);

		TArray<FEasingCurvePoint> CurvePoints;

		// Segment.Impls are already sorted bottom to top
		for (int32 CurveIndex = 0; CurveIndex < Segment.Sections.Num(); ++CurveIndex)
		{
			UMovieSceneSection* CurveSection = Segment.Sections[CurveIndex].Pin()->GetSection();

			// Make the points for the curve
			CurvePoints.Reset(20);
			{
				CurvePoints.Add(MakeCurvePoint(CurveSection, Segment.Range.GetLowerBoundValue(), FinalTint, EaseSelectionColor));
				CurvePoints.Add(MakeCurvePoint(CurveSection, Segment.Range.GetUpperBoundValue(), FinalTint, EaseSelectionColor));

				// Refine the control points
				int32 LastNumPoints;
				do
				{
					LastNumPoints = CurvePoints.Num();
					RefineCurvePoints(CurveSection, FinalTint, EaseSelectionColor, CurvePoints);
				} while(LastNumPoints != CurvePoints.Num());
			}

			TArray<SlateIndex> Indices;
			TArray<FSlateVertex> Verts;
			TArray<FVector2D> BorderPoints;
			TArray<FLinearColor> BorderPointColors;

			Indices.Reserve(CurvePoints.Num()*6);
			Verts.Reserve(CurvePoints.Num()*2);

			const FVector2f SizeAsFloatVec = FVector2f(Size);	// LWC_TODO: Precision loss

			for (const FEasingCurvePoint& Point : CurvePoints)
			{
				float SegmentStartTime = UE::MovieScene::DiscreteInclusiveLower(Segment.Range) / TimeToPixelConverter.GetTickResolution();
				float U = (Point.Location.X - SegmentStartTime) / ( FFrameNumber(UE::MovieScene::DiscreteSize(Segment.Range)) / TimeToPixelConverter.GetTickResolution() );

				// Add verts top->bottom
				FVector2f UV(U, 0.f);
				Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, (Pos + UV*SizeAsFloatVec*RangeGeometry.Scale), AtlasOffset + UV*AtlasUVSize, FillColor));	// LWC_TODO: Precision loss

				UV.Y = 1.f - Point.Location.Y;
				BorderPoints.Add(FVector2D(UV)*Size);
				BorderPointColors.Add(Point.Color);
				Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, (Pos + UV*SizeAsFloatVec*RangeGeometry.Scale), AtlasOffset + FVector2f(UV.X, 0.5f)*AtlasUVSize, FillColor));	// LWC_TODO: Precision loss

				if (Verts.Num() >= 4)
				{
					int32 Index0 = Verts.Num()-4, Index1 = Verts.Num()-3, Index2 = Verts.Num()-2, Index3 = Verts.Num()-1;
					Indices.Add(Index0);
					Indices.Add(Index1);
					Indices.Add(Index2);

					Indices.Add(Index1);
					Indices.Add(Index2);
					Indices.Add(Index3);
				}
			}

			if (Indices.Num())
			{
				FSlateDrawElement::MakeCustomVerts(
					DrawElements,
					LayerId,
					ResourceHandle,
					Verts,
					Indices,
					nullptr,
					0,
					0, ESlateDrawEffect::PreMultipliedAlpha);

				const ESlateDrawEffect DrawEffects = bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
				FSlateDrawElement::MakeLines(
					DrawElements,
					LayerId + 1,
					RangeGeometry.ToPaintGeometry(),
					BorderPoints,
					BorderPointColors,
					DrawEffects | ESlateDrawEffect::PreMultipliedAlpha,
					FLinearColor::White,
					true);
			}
		}

		++LayerId;
	}

	void DrawEasing(const FLinearColor& FinalTint)
	{
		if (!SectionModel->GetSection()->GetBlendType().IsValid())
		{
			return;
		}

		// Compute easing geometry by insetting from the current section geometry by 1px
		FGeometry InnerSectionGeometry = SectionGeometry.MakeChild(SectionGeometry.Size - FVector2D(2.f, 2.f), FSlateLayoutTransform(FVector2D(1.f, 1.f)));
		for (const FOverlappingSections& Segment : SectionWidget.UnderlappingEasingSegments)
		{
			DrawEasingForSegment(Segment, InnerSectionGeometry, FinalTint);
		}

		++LayerId;
	}

	void DrawOverlaps(const FLinearColor& FinalTint)
	{
		using namespace UE::Sequencer;

		FGeometry InnerSectionGeometry = SectionGeometry.MakeChild(SectionGeometry.Size - FVector2D(2.f, 2.f), FSlateLayoutTransform(FVector2D(1.f, 1.f)));

		UMovieSceneTrack* Track = GetTrack();
		if (!Track)
		{
			return;
		}

		static const FSlateBrush* PinCusionBrush = FAppStyle::GetBrush("Sequencer.Section.PinCusion");
		static const FSlateBrush* OverlapBorderBrush = FAppStyle::GetBrush("Sequencer.Section.OverlapBorder");

		const ESlateDrawEffect DrawEffects = bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

		const float StartTimePixel = SectionModel->GetSection()->HasStartFrame() ? TimeToPixelConverter.FrameToPixel(SectionModel->GetSection()->GetInclusiveStartFrame()) : 0.f;

		for (int32 SegmentIndex = 0; SegmentIndex < SectionWidget.UnderlappingSegments.Num(); ++SegmentIndex)
		{
			const FOverlappingSections& Segment = SectionWidget.UnderlappingSegments[SegmentIndex];

			const float RangeStartPixel	= Segment.Range.GetLowerBound().IsOpen() ? 0.f							: TimeToPixelConverter.FrameToPixel(UE::MovieScene::DiscreteInclusiveLower(Segment.Range));
			const float RangeEndPixel	= Segment.Range.GetUpperBound().IsOpen() ? InnerSectionGeometry.Size.X	: TimeToPixelConverter.FrameToPixel(UE::MovieScene::DiscreteExclusiveUpper(Segment.Range));
			const float RangeSizePixel	= RangeEndPixel - RangeStartPixel;

			FGeometry RangeGeometry = InnerSectionGeometry.MakeChild(FVector2D(RangeSizePixel, InnerSectionGeometry.Size.Y), FSlateLayoutTransform(FVector2D(RangeStartPixel - StartTimePixel, 0.f)));
			if (!FSlateRect::DoRectanglesIntersect(RangeGeometry.GetLayoutBoundingRect(), ParentClippingRect))
			{
				continue;
			}

			const FOverlappingSections* NextSegment = SegmentIndex < SectionWidget.UnderlappingSegments.Num() - 1 ? &SectionWidget.UnderlappingSegments[SegmentIndex+1] : nullptr;
			const bool bDrawRightMostBound = !NextSegment || !Segment.Range.Adjoins(NextSegment->Range);

			FSlateDrawElement::MakeBox(
				DrawElements,
				LayerId,
				RangeGeometry.ToPaintGeometry(),
				PinCusionBrush,
				DrawEffects,
				FinalTint
			);

			FPaintGeometry PaintGeometry = bDrawRightMostBound ? RangeGeometry.ToPaintGeometry() : RangeGeometry.ToPaintGeometry(FVector2D(RangeGeometry.Size) + FVector2D(10.f, 0.f), FSlateLayoutTransform(FVector2D::ZeroVector));
			FSlateDrawElement::MakeBox(
				DrawElements,
				LayerId,
				PaintGeometry,
				OverlapBorderBrush,
				DrawEffects,
				FLinearColor(1.f,1.f,1.f,.3f)
			);
		}

		++LayerId;
	}

	void DrawEmptySpace()
	{
		using namespace UE::Sequencer;

		TSharedPtr<STrackLane> OwningTrackLane = SectionWidget.WeakOwningTrackLane.Pin();
		if (!OwningTrackLane)
		{
			return;
		}

		// Find all our child track lanes
		TArray<TSharedPtr<STrackLane>> EmptyChildLanes;
		Algo::Transform(OwningTrackLane->GetChildLanes(), EmptyChildLanes, [](const TWeakPtr<STrackLane>& In){ return In.Pin(); });

		// Remove invalid track lanes and lanes that have their own section widgets
		// (sections handle their own background painting). Lanes with only channel/key
		// area widgets still need background from this parent section.
		for (int32 Index = EmptyChildLanes.Num()-1; Index >= 0; --Index)
		{
			if (EmptyChildLanes[Index])
			{
				bool bHasOwnSection = false;
				for (const auto& Pair : EmptyChildLanes[Index]->GetAllWidgets())
				{
					if (TSharedPtr<FViewModel> Model = Pair.Key.Pin())
					{
						if (Model->IsA<FSectionModel>())
						{
							bHasOwnSection = true;
							break;
						}
					}
				}
				if (bHasOwnSection)
				{
					EmptyChildLanes.RemoveAtSwap(Index, EAllowShrinking::No);
				}
			}
			else
			{
				EmptyChildLanes.RemoveAtSwap(Index, EAllowShrinking::No);
			}
		}

		if (EmptyChildLanes.IsEmpty())
		{
			return;
		}

		// Paint section contents background for remaining track lanes (e.g., decoration channels)
		// Use the same brush and color computation as the section contents area
		const ESlateDrawEffect DrawEffects = bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
		static const FSlateBrush* ContentsBrush = FAppStyle::GetBrush("Sequencer.Section.Background_Contents");
		if (!ContentsBrush)
		{
			return;
		}

		// Compute the fill tint the same way PaintSectionBackground does for expanded section contents
		const USequencerSettings* const SequencerSettings = Sequencer.GetSequencerSettings();
		if (!SequencerSettings)
		{
			return;
		}
		UMovieSceneSection* SectionObject = SectionModel->GetSection();
		UMovieSceneTrack* OuterTrack = SectionObject ? SectionObject->GetTypedOuter<UMovieSceneTrack>() : nullptr;
		const FLinearColor TrackColor = OuterTrack ? OuterTrack->GetColorTint() : FLinearColor::White;
		const FLinearColor BlendedTint = BlendColor(TrackColor).CopyWithNewOpacity(SequencerSettings->GetSectionColorAlpha());
		FLinearColor FillTint = BlendedTint.LinearRGBToHSV();
		FillTint.G *= .5f;
		FillTint.B = FMath::Max(.03f, FillTint.B * .1f);
		const FLinearColor FinalFillColor = FillTint.HSVToLinearRGB();

		for (TSharedPtr<STrackLane> EmptyTrackLane : EmptyChildLanes)
		{
			const float LaneTop = EmptyTrackLane->GetVerticalPosition() - OwningTrackLane->GetVerticalPosition();
			const FVector2D LaneSize(SectionGeometry.GetLocalSize().X, EmptyTrackLane->GetOutlinerItem()->GetOutlinerSizing().Height);

			FSlateDrawElement::MakeBox(
				DrawElements,
				LayerId,
				SectionGeometry.MakeChild(
					LaneSize,
					FSlateLayoutTransform(FVector2D(0.f, LaneTop))
				).ToPaintGeometry(),
				ContentsBrush,
				DrawEffects,
				FinalFillColor
			);
		}

		++LayerId;
	}

	void DrawBorder(const FLinearColor& FinalTint)
	{
		UMovieSceneSection* SectionObject = SectionModel->GetSection();
		UMovieSceneTrack* Track = SectionObject->GetTypedOuter<UMovieSceneTrack>();

		// Show the locked border if it's locked or read only
		const bool bLocked = SectionObject->IsLocked() || SectionObject->IsReadOnly();

		// Only show section to key border if we have more than one section
		const bool bIsSectionToKey = Track && Track->GetAllSections().Num() > 1 && IsSectionToKey(Track,SectionObject);

		const ESlateDrawEffect DrawEffects = bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

		if (bLocked)
		{
			static const FName SelectionBorder("Sequencer.Section.LockedBorder");

			FSlateDrawElement::MakeBox(
				DrawElements,
				LayerId,
				SectionGeometry.ToPaintGeometry(),
				FAppStyle::GetBrush(SelectionBorder),
				DrawEffects,
				FStyleColors::Error.GetColor(FWidgetStyle())
			);
			++LayerId;
		}
		else if (bIsSectionToKey)
		{
			static const FName SelectionBorder("Sequencer.Section.LockedBorder");

			FSlateDrawElement::MakeBox(
				DrawElements,
				LayerId,
				SectionGeometry.ToPaintGeometry(),
				FAppStyle::GetBrush(SelectionBorder),
				DrawEffects,
				FStyleColors::Success.GetColor(FWidgetStyle())
			);
			++LayerId;
		}
	}

	TOptional<FLinearColor> SelectionColor;
	FSequencer& Sequencer;
	const SSequencerSection& SectionWidget;
	FTimeToPixel TimeToPixelConverter;
	TSharedPtr<FTrackAreaViewModel> TrackAreaViewModel;
	TSharedPtr<ITrackAreaHotspot> Hotspot;

	/** The clipping rectangle of the parent widget */
	FSlateRect ParentClippingRect;

	bool bClipRectEnabled;
};

void SSequencerSection::Construct( const FArguments& InArgs, TSharedPtr<FSequencer> InSequencer, TSharedPtr<FSectionModel> InSectionModel, TSharedPtr<STrackLane> OwningTrackLane)
{
	using namespace UE::MovieScene;

	Sequencer = InSequencer;
	WeakSectionModel = InSectionModel;
	WeakOwningTrackLane = OwningTrackLane;
	SectionInterface = InSectionModel->GetSectionInterface();
	HandleOffsetPx = 0.f;
	TimeToPixel = MakeShared<FTimeToPixel>(100.f, TRange<double>(0.0, 1.0), FFrameRate());

	SetEnabled(MakeAttributeSP(this, &SSequencerSection::IsEnabled));
	SetToolTipText(MakeAttributeSP(this, &SSequencerSection::GetToolTipText));

	UMovieSceneSection* Section = InSectionModel->GetSection();
	UMovieSceneTrack*   Track   = Section ? Section->GetTypedOuter<UMovieSceneTrack>() : nullptr;
	if (ensure(Track))
	{
		Track->EventHandlers.Link(TrackModifiedBinding, this);
	}

	UpdateUnderlappingSegments();

	TSharedRef<SOverlay> Overlay =

		SNew(SOverlay)

		+ SOverlay::Slot(FCreateSectionViewWidgetParams::CompoundTrackLaneViewOrder)
		[
			SAssignNew(ChildLaneWidgets, SCompoundTrackLaneView, SharedThis(this))
		]

		+ SOverlay::Slot(FCreateSectionViewWidgetParams::ChannelViewOrder)
		[
			SNew(SChannelView, InSectionModel, OwningTrackLane->GetTrackAreaView())
			.KeyBarColor(this, &SSequencerSection::GetTopLevelKeyBarColor)
			.Visibility(this, &SSequencerSection::GetTopLevelChannelGroupVisibility)
		];

	FCreateSectionViewWidgetParams Params{
		Overlay,
		SharedThis(this),
		OwningTrackLane.ToSharedRef(),
		OwningTrackLane->GetTrackAreaView().ToSharedRef(),
		InSectionModel.ToSharedRef()
	};
	SectionInterface->CreateViewWidgets(Params);

	ChildSlot
	.Padding(MakeAttributeSP(this, &SSequencerSection::GetHandleOffsetPadding))
	[
		Overlay
	];
}

SSequencerSection::~SSequencerSection()
{
	TSharedPtr<FTrackAreaViewModel> TrackAreaViewModel = GetTrackAreaViewModel();
	if (TrackAreaViewModel.IsValid())
	{
		TrackAreaViewModel->SetHotspot(nullptr);
	}
}

EVisibility SSequencerSection::GetTopLevelChannelGroupVisibility() const
{
	using namespace UE::MovieScene;

	TSharedPtr<FSectionModel> SectionModel = WeakSectionModel.Pin();
	TViewModelPtr<IOutlinerExtension> Outliner = SectionModel ? SectionModel->FindAncestorOfType<IOutlinerExtension>() : nullptr;

	if (!SectionModel || !Outliner || Outliner->IsExpanded() || !SectionModel->GetDescendantsOfType<FChannelModel>())
	{
		return EVisibility::Collapsed;
	}

	// Allow sections to opt out of showing the key area when collapsed
	if (SectionInterface && !SectionInterface->ShouldShowKeyAreaWhenCollapsed())
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

FLinearColor SSequencerSection::GetTopLevelKeyBarColor() const
{
	TSharedPtr<FSectionModel> SectionModel = WeakSectionModel.Pin();
	TSharedPtr<ITrackExtension> Track = SectionModel ? SectionModel->FindAncestorOfType<ITrackExtension>() : nullptr;
	UMovieSceneTrack* TrackObject = Track ? Track->GetTrack() : nullptr;

	FLinearColor Tint = FLinearColor::White;
	if (TrackObject)
	{
		Tint = FSequencerSectionPainter::BlendColor(TrackObject->GetColorTint()).CopyWithNewOpacity(1.f).LinearRGBToHSV();
		// These values relate to those that are draw in the expanded section fill in PaintSectionBackground
		// Except the Saturation is 75% rather than 50%, and the value is 25% rather than 10%
		//Tint.G *= .75f;
		Tint.B = FMath::Max(.03f, Tint.B*.25f);
		Tint = Tint.HSVToLinearRGB();
	}

	return Tint;
}

FMargin SSequencerSection::GetHandleOffsetPadding() const
{
	return FMargin(HandleOffsetPx, 0.0);
}

FText SSequencerSection::GetToolTipText() const
{
	const UMovieSceneSection* SectionObject = SectionInterface->GetSectionObject();
	const UMovieScene* MovieScene = SectionObject ? SectionObject->GetTypedOuter<UMovieScene>() : nullptr;

	// Optional section specific content to add to tooltip
	FText SectionToolTipContent = SectionInterface->GetSectionToolTip();

	FText SectionTitleText = SectionInterface->GetSectionTitle();
	if (!SectionTitleText.IsEmpty())
	{
		SectionTitleText = FText::Format(FText::FromString(TEXT("{0}\n")), SectionTitleText);
	}

	// If the objects are valid and the section is not unbounded, add frame information to the tooltip
	if (SectionObject && MovieScene && SectionObject->HasStartFrame() && SectionObject->HasEndFrame())
	{
		FFrameRate TickResolution = MovieScene->GetTickResolution();
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();

		int32 StartFrame = ConvertFrameTime(SectionObject->GetInclusiveStartFrame(), MovieScene->GetTickResolution(), MovieScene->GetDisplayRate()).RoundToFrame().Value;
		int32 EndFrame = ConvertFrameTime(SectionObject->GetExclusiveEndFrame(), MovieScene->GetTickResolution(), MovieScene->GetDisplayRate()).RoundToFrame().Value;
	
		FText SectionToolTip;
		if (SectionToolTipContent.IsEmpty())
		{
			SectionToolTip = FText::Format(NSLOCTEXT("SequencerSection", "TooltipFormat", "{0}{1} - {2} ({3} frames)"), SectionTitleText,
				StartFrame,
				EndFrame,
				EndFrame - StartFrame);
		}
		else
		{
			SectionToolTip = FText::Format(NSLOCTEXT("SequencerSection", "TooltipFormatWithSectionContent", "{0}{1} - {2} ({3} frames)\n{4}"), SectionTitleText,
				StartFrame,
				EndFrame,
				EndFrame - StartFrame,
				SectionToolTipContent);
		}
	
		if (SectionObject->Easing.EaseIn.GetObject() && SectionObject->Easing.GetEaseInDuration() > 0)
		{
			int32 EaseInFrames = ConvertFrameTime(SectionObject->Easing.GetEaseInDuration(), TickResolution, DisplayRate).RoundToFrame().Value;
			FText EaseInText = FText::Format(NSLOCTEXT("SequencerSection", "EaseInFormat", "Ease In: {0} ({1} frames)"), SectionObject->Easing.EaseIn->GetDisplayName(), EaseInFrames);
			SectionToolTip = FText::Join(FText::FromString("\n"), SectionToolTip, EaseInText);
		}

		if (SectionObject->Easing.EaseOut.GetObject() && SectionObject->Easing.GetEaseOutDuration() > 0)
		{
			int32 EaseOutFrames = ConvertFrameTime(SectionObject->Easing.GetEaseOutDuration(), TickResolution, DisplayRate).RoundToFrame().Value;
			FText EaseOutText = FText::Format(NSLOCTEXT("SequencerSection", "EaseOutFormat", "Ease Out: {0} ({1} frames)"), SectionObject->Easing.EaseOut->GetDisplayName(), EaseOutFrames);
			SectionToolTip = FText::Join(FText::FromString("\n"), SectionToolTip, EaseOutText);
		}
		
		return SectionToolTip;
	}
	else
	{
		if (SectionToolTipContent.IsEmpty())
		{
			return SectionInterface->GetSectionTitle();
		}
		else
		{
			return FText::Format(NSLOCTEXT("SequencerSection", "TooltipSectionContentFormat", "{0}{1}"), SectionTitleText, SectionToolTipContent);
		}
	}
}

bool SSequencerSection::IsEnabled() const
{
	return !SectionInterface->IsReadOnly();
}

FVector2D SSequencerSection::ComputeDesiredSize(float) const
{
	return FVector2D(0.f, 0.f);
}

void SSequencerSection::ReportParentGeometry(const FGeometry& InParentGeometry)
{
	ParentGeometry = InParentGeometry;
}

FTrackLaneScreenAlignment SSequencerSection::GetAlignment(const ITrackLaneWidgetSpace& InScreenSpace, const FGeometry& InParentGeometry) const
{
	using namespace UE::MovieScene;
	TSharedPtr<FSectionModel> SectionModel = WeakSectionModel.Pin();
	if (!SectionModel)
	{
		return FTrackLaneScreenAlignment();
	}

	FTrackLaneVirtualAlignment VirtualAlignment = SectionModel->ArrangeVirtualTrackLaneView();
	FTrackLaneScreenAlignment  ScreenAlignment  = VirtualAlignment.ToScreen(InScreenSpace.GetScreenSpace(VirtualAlignment.ViewSpaceID), InParentGeometry);

	if (TOptional<FFrameNumber> FiniteLength = VirtualAlignment.GetFiniteLength())
	{
		constexpr float MinSectionWidth = 1.f;

		const float FinalSectionWidth = FMath::Max(MinSectionWidth + SectionInterface->GetSectionGripSize() * 2.f, ScreenAlignment.WidthPx);
		const float GripOffset        = (FinalSectionWidth - ScreenAlignment.WidthPx) / 2.f;

		ScreenAlignment.LeftPosPx -= GripOffset;
		ScreenAlignment.WidthPx    = FMath::Max(FinalSectionWidth, MinSectionWidth + SectionInterface->GetSectionGripSize() * 2.f);
	}

	return ScreenAlignment;
}

int32 SSequencerSection::GetOverlapPriority() const
{
	if (TSharedPtr<FSectionModel> SectionModel = WeakSectionModel.Pin())
	{
		if (UMovieSceneSection* Section = SectionModel->GetSection())
		{
			return Section->GetOverlapPriority();
		}
	}
	return 0;
}

bool SSequencerSection::CheckForEasingHandleInteraction( const FPointerEvent& MouseEvent, const FGeometry& AllottedGeometry )
{
	UMovieSceneSection* ThisSection = SectionInterface->GetSectionObject();
	if (!ThisSection)
	{
		return false;
	}

	UMovieSceneTrack* Track = ThisSection->GetTypedOuter<UMovieSceneTrack>();
	if (!Track || Track->GetSupportedBlendTypes().Num() == 0)
	{
		return false;
	}

	FMovieSceneSupportsEasingParams SupportsEasingParams(ThisSection);
	EMovieSceneTrackEasingSupportFlags EasingFlags = Track->SupportsEasing(SupportsEasingParams);
	if (!EnumHasAnyFlags(EasingFlags, EMovieSceneTrackEasingSupportFlags::ManualEasing))
	{
		return false;
	}

	FGeometry SectionGeometry = MakeHandlesGeometry(AllottedGeometry);

	const double MousePositionX = SectionGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).X;

	// Easing handle is a square in the top corner of the grip handle.
	const double GripSizePx = SectionInterface->GetSectionGripSize();
	const double GripHalfSizePx = GripSizePx / 2.f;

	// Now test individual easing handles if we're at the correct vertical position
	float LocalMouseY = SectionGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).Y;
	if (LocalMouseY < 0.f || LocalMouseY > GripSizePx)
	{
		return false;
	}

	// Gather all underlapping sections
	TArray<TSharedPtr<FSectionModel>> AllUnderlappingSections;
	if (TSharedPtr<FSectionModel> SectionModel = WeakSectionModel.Pin())
	{
		AllUnderlappingSections.Add(SectionModel);
	}
	for (const TSharedPtr<FSectionModel>& Section : GetUnderlappingSectionModels())
	{
		AllUnderlappingSections.AddUnique(Section);
	}

	TSharedPtr<FTrackAreaViewModel> TrackAreaViewModel = GetTrackAreaViewModel();
	for (const TSharedPtr<FSectionModel>& SectionModel : AllUnderlappingSections)
	{
		UMovieSceneSection* EasingSectionObj = SectionModel->GetSection();

		if (EasingSectionObj->HasStartFrame() && EnumHasAllFlags(EasingFlags, EMovieSceneTrackEasingSupportFlags::ManualEaseIn))
		{
			// Compute the ease-in handle screen position.
			const TRange<FFrameNumber> EaseInRange = EasingSectionObj->GetEaseInRange();
			const FFrameNumber HandleTime = EaseInRange.IsEmpty() ? EasingSectionObj->GetInclusiveStartFrame() : EaseInRange.GetUpperBoundValue();
			const double HandlePosition = TimeToPixel->FrameToPixel(HandleTime) - HandleOffsetPx;

			if (FMath::IsNearlyEqual(MousePositionX, HandlePosition + GripHalfSizePx, GripHalfSizePx))
			{
				TrackAreaViewModel->SetHotspot(MakeShared<FSectionEasingHandleHotspot>(ESequencerEasingType::In, SectionModel, Sequencer));
				return true;
			}
		}

		if (EasingSectionObj->HasEndFrame() && EnumHasAllFlags(EasingFlags, EMovieSceneTrackEasingSupportFlags::ManualEaseOut))
		{
			// Compute the ease-out handle screen position.
			TRange<FFrameNumber> EaseOutRange = EasingSectionObj->GetEaseOutRange();
			const FFrameNumber HandleTime = EaseOutRange.IsEmpty() ? EasingSectionObj->GetExclusiveEndFrame() : EaseOutRange.GetLowerBoundValue();
			const double HandlePosition = TimeToPixel->FrameToPixel(HandleTime) + HandleOffsetPx;

			if (FMath::IsNearlyEqual(MousePositionX, HandlePosition - GripHalfSizePx, GripHalfSizePx))
			{
				TrackAreaViewModel->SetHotspot(MakeShared<FSectionEasingHandleHotspot>(ESequencerEasingType::Out, SectionModel, Sequencer));
				return true;
			}
		}
	}

	return false;
}


bool SSequencerSection::CheckForEdgeInteraction( const FPointerEvent& MouseEvent, const FGeometry& AllottedGeometry )
{
	UMovieSceneSection* ThisSection = SectionInterface->GetSectionObject();
	if (!ThisSection)
	{
		return false;
	}

	TSharedPtr<FTrackAreaViewModel> TrackAreaViewModel = GetTrackAreaViewModel();
	const ISequencerEditTool* EditTool = TrackAreaViewModel->GetEditTool();
	TSharedPtr<ITrackAreaHotspot> Hotspot = EditTool ? EditTool->GetDragHotspot() : nullptr;
	if (!Hotspot)
	{
		Hotspot = TrackAreaViewModel->GetHotspot();
	}

	if (Hotspot && !Hotspot->IsA<FSectionHotspotBase>())
	{
		return false;
	}

	TArray<TSharedPtr<FSectionModel>> AllUnderlappingSections;
	if (TSharedPtr<FSectionModel> SectionModel = WeakSectionModel.Pin())
	{
		AllUnderlappingSections.Add(SectionModel);
	}
	for (const TSharedPtr<FSectionModel>& Section : GetUnderlappingSectionModels())
	{
		AllUnderlappingSections.AddUnique(Section);
	}

	FGeometry SectionGeometry = MakeHandlesGeometry(AllottedGeometry);

	for (const TSharedPtr<FSectionModel>& UnderlappingSection : AllUnderlappingSections)
	{
		UMovieSceneSection*           UnderlappingSectionObj       = UnderlappingSection->GetSection();
		TSharedPtr<ISequencerSection> UnderlappingSectionInterface = UnderlappingSection->GetSectionInterface();
		if (!UnderlappingSectionInterface->SectionIsResizable())
		{
			continue;
		}

		const float ThisHandleOffset = UnderlappingSectionObj == ThisSection ? HandleOffsetPx : 0.f;
		const FVector2D GripSize( UnderlappingSectionInterface->GetSectionGripSize(), SectionGeometry.Size.Y );

		if (UnderlappingSectionObj->HasStartFrame())
		{
			// Make areas to the left and right of the geometry.  We will use these areas to determine if someone dragged the left or right edge of a section
			FGeometry SectionRectLeft = SectionGeometry.MakeChild(
				GripSize,
				FSlateLayoutTransform(FVector2D( TimeToPixel->FrameToPixel(UnderlappingSectionObj->GetInclusiveStartFrame()) - ThisHandleOffset, 0.f ))
			);

			if( SectionRectLeft.IsUnderLocation( MouseEvent.GetScreenSpacePosition() ) )
			{
				TrackAreaViewModel->SetHotspot(MakeShareable( new FSectionResizeHotspot(FSectionResizeHotspot::Left, UnderlappingSection, Sequencer)) );
				return true;
			}
		}

		if (UnderlappingSectionObj->HasEndFrame())
		{
			FGeometry SectionRectRight = SectionGeometry.MakeChild(
				GripSize,
				FSlateLayoutTransform(FVector2D( TimeToPixel->FrameToPixel(UnderlappingSectionObj->GetExclusiveEndFrame()) - UnderlappingSectionInterface->GetSectionGripSize() + ThisHandleOffset, 0 ))
			);

			if( SectionRectRight.IsUnderLocation( MouseEvent.GetScreenSpacePosition() ) )
			{
				TrackAreaViewModel->SetHotspot(MakeShareable( new FSectionResizeHotspot(FSectionResizeHotspot::Right, UnderlappingSection, Sequencer)) );
				return true;
			}
		}
	}
	return false;
}

bool SSequencerSection::CheckForEasingAreaInteraction( const FPointerEvent& MouseEvent, const FGeometry& AllottedGeometry )
{
	TSharedPtr<FSectionModel> ThisSectionModel = WeakSectionModel.Pin();
	if (!ThisSectionModel)
	{
		return false;
	}

	FVector2D LocalMousePos = AllottedGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	TViewModelPtr<IGeometryExtension> HeaderGeometry = ThisSectionModel->FindAncestorOfType<IOutlinerExtension>().ImplicitCast();
	const float EasingHeight = HeaderGeometry ? HeaderGeometry->GetVirtualGeometry().GetHeight() : AllottedGeometry.GetLocalSize().Y;

	if (LocalMousePos.Y < 0 || LocalMousePos.Y > EasingHeight)
	{
		return false;
	}

	UMovieSceneSection* ThisSection = SectionInterface->GetSectionObject();
	const FFrameNumber MouseTime = TimeToPixel->PixelToFrame(LocalMousePos.X).FrameNumber;

	// First off, set the hotspot to an easing area if necessary
	TSharedPtr<FTrackAreaViewModel> TrackAreaViewModel = GetTrackAreaViewModel();
	for (const FOverlappingSections& Segment : UnderlappingEasingSegments)
	{
		if (!Segment.Range.Contains(MouseTime))
		{
			continue;
		}

		TArray<FEasingAreaHandle> EasingAreas;
		for (const TWeakPtr<FSectionModel>& SectionModel : Segment.Sections)
		{
			UMovieSceneSection* Section = SectionModel.Pin()->GetSection();
			if (Section->GetEaseInRange().Contains(MouseTime))
			{
				EasingAreas.Add(FEasingAreaHandle{ SectionModel, ESequencerEasingType::In });
			}
			if (Section->GetEaseOutRange().Contains(MouseTime))
			{
				EasingAreas.Add(FEasingAreaHandle{ SectionModel, ESequencerEasingType::Out });
			}
		}

		if (EasingAreas.Num())
		{
			TrackAreaViewModel->SetHotspot(MakeShared<FSectionEasingAreaHotspot>(EasingAreas, WeakSectionModel, Sequencer));
			return true;
		}
	}
	return false;
}

bool SSequencerSection::CheckForSectionInteraction( const FPointerEvent& MouseEvent, const FGeometry& AllottedGeometry )
{
	FGeometry SectionGeometry = MakeSectionGeometryWithoutHandles(AllottedGeometry);
	if (SectionGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition()))
	{
		TSharedPtr<FTrackAreaViewModel> TrackAreaViewModel = GetTrackAreaViewModel();
		TrackAreaViewModel->SetHotspot( MakeShareable( new FSectionHotspot(WeakSectionModel, Sequencer)) );
		return true;
	}

	return false;
}

FSequencer& SSequencerSection::GetSequencer() const
{
	return *Sequencer.Pin().Get();
}

TSharedPtr<STrackAreaView> SSequencerSection::GetTrackAreaView() const
{
	if (TSharedPtr<STrackLane> OwningTrackLane = WeakOwningTrackLane.Pin())
	{
		return OwningTrackLane->GetTrackAreaView();
	}
	return nullptr;
}

TSharedPtr<FTrackAreaViewModel> SSequencerSection::GetTrackAreaViewModel() const
{
	if (TSharedPtr<STrackAreaView> TrackAreaView = GetTrackAreaView())
	{
		return TrackAreaView->GetViewModel();
	}
	return nullptr;
}

void DrawFrameTimeHint(FSequencerSectionPainter& InPainter, const FFrameTime& CurrentTime, const FFrameTime& FrameTime, const TSharedPtr<INumericTypeInterface<double>> FrameNumberInterface, int32 LayerId)
{
	FString FrameTimeString;
	if (FrameNumberInterface)
	{
		FrameTimeString = FrameNumberInterface->ToString(FrameTime.AsDecimal());
	}
	else
	{
		FrameTimeString = FString::FromInt(FrameTime.GetFrame().Value);
	}

	const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);
	const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	FVector2D TextSize = FontMeasureService->Measure(FrameTimeString, SmallLayoutFont);

	const float PixelX = InPainter.GetTimeConverter().FrameToPixel(CurrentTime);

	// Flip the text position if getting near the end of the view range
	static const float TextOffsetPx = 10.f;
	bool  bDrawLeft = (InPainter.SectionGeometry.Size.X - PixelX) < (TextSize.X + 22.f) - TextOffsetPx;
	float TextPosition = bDrawLeft ? PixelX - TextSize.X - TextOffsetPx : PixelX + TextOffsetPx;
	//handle mirrored labels
	const float MajorTickHeight = 9.0f;
	FVector2D TextOffset(TextPosition, InPainter.SectionGeometry.Size.Y - (MajorTickHeight + TextSize.Y));

	const FLinearColor DrawColor = FAppStyle::GetSlateColor("SelectionColor").GetColor(FWidgetStyle()).CopyWithNewOpacity(InPainter.GhostAlpha);
	const FVector2D BoxPadding = FVector2D(4.0f, 2.0f);
	// draw time string

	FSlateDrawElement::MakeBox(
		InPainter.DrawElements,
		LayerId,
		InPainter.SectionGeometry.ToPaintGeometry(TextSize + 2.0f * BoxPadding, FSlateLayoutTransform(TextOffset - BoxPadding)),
		FAppStyle::GetBrush("WhiteBrush"),
		ESlateDrawEffect::None,
		FLinearColor::Black.CopyWithNewOpacity(0.5f * InPainter.GhostAlpha)
	);

	ESlateDrawEffect DrawEffects = InPainter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	FSlateDrawElement::MakeText(
		InPainter.DrawElements,
		LayerId,
		InPainter.SectionGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextOffset)),
		FrameTimeString,
		SmallLayoutFont,
		DrawEffects,
		DrawColor
	);
}

int32 SSequencerSection::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	TSharedPtr<FSectionModel> SectionModel = WeakSectionModel.Pin();
	UMovieSceneSection* SectionObject = SectionModel ? SectionModel->GetSection() : nullptr;
	if (!SectionModel || !SectionObject)
	{
		return LayerId;
	}

	TSharedPtr<FTrackAreaViewModel> TrackAreaViewModel = GetTrackAreaViewModel();
	const ISequencerEditTool* EditTool = TrackAreaViewModel->GetEditTool();
	TSharedPtr<ITrackAreaHotspot> Hotspot = EditTool ? EditTool->GetDragHotspot() : nullptr;
	if (!Hotspot)
	{
		Hotspot = TrackAreaViewModel->GetHotspot();
	}

	UMovieSceneTrack* Track = SectionObject->GetTypedOuter<UMovieSceneTrack>();

	FGuid BindingID;
	if (TSharedPtr<IObjectBindingExtension> ObjectBindingExtension = SectionModel->FindAncestorOfType<IObjectBindingExtension>())
	{
		BindingID = ObjectBindingExtension->GetObjectGuid();
	}
	const UMovieSceneCondition* Condition = MovieSceneHelpers::GetSequenceCondition(Track, SectionObject);

	const bool bConditionPasses = !Condition || MovieSceneHelpers::EvaluateSequenceCondition(BindingID, GetSequencer().GetFocusedTemplateID(), Condition, Track, GetSequencer().GetSharedPlaybackState());
	const bool bTrackDisabled = Track && (Track->IsEvalDisabled() || Track->IsRowEvalDisabled(SectionObject->GetRowIndex()));
	const bool bSectionLocalDisabled = SectionObject->IsLocalEvalDisabled();
	const bool bEnabled = bParentEnabled && SectionObject->IsActive() && bConditionPasses && !(bTrackDisabled) && !bSectionLocalDisabled;

	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	FGeometry SectionGeometry = MakeHandlesGeometry(AllottedGeometry);

	FSequencerSectionPainterImpl Painter(GetSequencer(), TrackAreaViewModel, SectionModel, OutDrawElements, SectionGeometry, *this);

	FGeometry PaintSpaceParentGeometry = ParentGeometry;
	PaintSpaceParentGeometry.AppendTransform(FSlateLayoutTransform(Inverse(Args.GetWindowToDesktopTransform())));

	Painter.ParentClippingRect = PaintSpaceParentGeometry.GetLayoutBoundingRect();

	// Clip vertically
	Painter.ParentClippingRect.Top = FMath::Max(Painter.ParentClippingRect.Top, MyCullingRect.Top);
	Painter.ParentClippingRect.Bottom = FMath::Min(Painter.ParentClippingRect.Bottom, MyCullingRect.Bottom);

	Painter.SectionClippingRect = Painter.SectionGeometry.GetLayoutBoundingRect().InsetBy(FMargin(1.f)).IntersectionWith(Painter.ParentClippingRect);

	Painter.LayerId = LayerId;
	Painter.bParentEnabled = bEnabled;
	Painter.bIsHighlighted = IsSectionHighlighted(SectionObject, Hotspot);
	if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(SectionObject))
	{
		if (( SubSection->GetNetworkMask() & GetSequencer().GetEvaluationTemplate().GetEmulatedNetworkMask() ) == EMovieSceneServerClientMask::None)
		{
			Painter.GhostAlpha = .3f;
		}
	}

	Painter.bIsSelected = GetSequencer().GetSelection().TrackArea.IsSelected(SectionModel);

	// Ask the interface to draw the section
	LayerId = SectionInterface->OnPaintSection(Painter);

	FLinearColor SelectionColor = FAppStyle::GetSlateColor(SequencerSectionConstants::SelectionColorName).GetColor(FWidgetStyle());

	LayerId = SCompoundWidget::OnPaint( Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bEnabled );

	// We paint section handles after channels so they draw on top, but we allow some leeway for channels to paint on a higher layer than they
	// reported. This enables us to still draw and interact with keys that overlap section handles, even if we drew the handles chronologically later
	DrawSectionHandles(AllottedGeometry, OutDrawElements, LayerId, DrawEffects, SelectionColor, Hotspot);

	Painter.LayerId = (++LayerId);
	PaintEasingHandles( Painter, SelectionColor, Hotspot );

	// Artificially increase the layer now to ensure that we are now painting above keys now
	LayerId += 10;

	// --------------------------------------------
	// Draw section selection tint
	const float SectionThrobValue = GetSectionSelectionThrobValue();
	if (SectionThrobValue != 0.f && GetSequencer().GetSelection().TrackArea.IsSelected(SectionModel))
	{
		static const FName BackgroundTrackTintBrushName("Sequencer.Section.BackgroundTint");

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId++,
			AllottedGeometry.ToPaintGeometry(),
			FAppStyle::GetBrush(BackgroundTrackTintBrushName),
			DrawEffects,
			SelectionColor.CopyWithNewOpacity(SectionThrobValue)
		);
	}

	LayerId = Painter.LayerId;

	// Section name with drop shadow
	FText SectionTitle = SectionInterface->GetSectionTitle();
	FMargin ContentPadding = SectionInterface->GetContentPadding();

	const int32 EaseInAmount = SectionObject->Easing.GetEaseInDuration();
	if (EaseInAmount > 0)
	{
		ContentPadding.Left += Painter.GetTimeConverter().FrameToPixel(EaseInAmount) - Painter.GetTimeConverter().FrameToPixel(0);
	}

	if (!SectionTitle.IsEmpty())
	{
		TViewModelPtr<IGeometryExtension> Geometry = SectionModel->FindAncestorOfType<IOutlinerExtension>().ImplicitCast();
		const float AllocatedFontHeight = (Geometry ? Geometry->GetVirtualGeometry().GetHeight() : SectionGeometry.Size.Y) - 4.f; // 2px minimum padding

		// Align the section title within the actual track area geometry, excluding any expanded channels
		FSlateClippingZone ClippingZone(Painter.SectionClippingRect);
		OutDrawElements.PushClip(ClippingZone);

		FVector2D TopLeft = SectionGeometry.AbsoluteToLocal(Painter.SectionClippingRect.GetTopLeft()) + FVector2D(1.f, -1.f);
		FSlateFontInfo FontInfo = FAppStyle::GetFontStyle("NormalFont");

		TSharedRef<FSlateFontCache> FontCache = FSlateApplication::Get().GetRenderer()->GetFontCache();

		auto GetFontHeight = [&]
		{
			return FontCache->GetMaxCharacterHeight(FontInfo, 1.f) + FontCache->GetBaseline(FontInfo, 1.f);
		};
		while (GetFontHeight() > AllocatedFontHeight && FontInfo.Size > 7.f)
		{
			FontInfo.Size = FMath::Max(FMath::FloorToInt(FontInfo.Size - 6.f), 7.f);
		}

		float TitlePosition = ContentPadding.Top;
		if (FontInfo.Size + ContentPadding.Top + ContentPadding.Bottom > AllocatedFontHeight)
		{
			TitlePosition = (AllocatedFontHeight - FontInfo.Size) * 0.5f;
		}

		// Drop shadow
		FSlateDrawElement::MakeText(
			OutDrawElements,
			LayerId,
			SectionGeometry.MakeChild(
				FVector2D(SectionGeometry.Size.X, GetFontHeight()),
				FSlateLayoutTransform(TopLeft + FVector2D(ContentPadding.Left, TitlePosition) + FVector2D(1.f, 1.f))
			).ToPaintGeometry(),
			SectionTitle,
			FontInfo,
			DrawEffects,
			FLinearColor(0, 0, 0, .5f * Painter.GhostAlpha)
		);

		FSlateDrawElement::MakeText(
			OutDrawElements,
			LayerId,
			SectionGeometry.MakeChild(
				FVector2D(SectionGeometry.Size.X, GetFontHeight()),
				FSlateLayoutTransform(TopLeft + FVector2D(ContentPadding.Left, TitlePosition))
			).ToPaintGeometry(),
			SectionTitle,
			FontInfo,
			DrawEffects,
			FColor(192, 192, 192, static_cast<uint8>(Painter.GhostAlpha * 255))
		);

		OutDrawElements.PopClip();
	}

	++LayerId;

	TOptional<FFrameTime> SectionTime = SectionInterface->GetSectionTime(Painter);
	if (SectionTime.IsSet())
	{
		TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface = GetSequencer().GetNumericTypeInterface(ENumericIntent::Position);
		DrawFrameTimeHint(Painter, GetSequencer().GetLocalTime().Time, SectionTime.GetValue(), NumericTypeInterface, LayerId);
	}

	++LayerId;

	if (Painter.bClipRectEnabled)
	{
		OutDrawElements.PopClip();
	}

	return LayerId;
}

void SSequencerSection::PaintEasingHandles( FSequencerSectionPainter& InPainter, FLinearColor SelectionColor, TSharedPtr<ITrackAreaHotspot> Hotspot ) const
{
	if (!SectionInterface->GetSectionObject()->GetBlendType().IsValid())
	{
		return;
	}

	if (Hotspot && !Hotspot->IsA<FSectionHotspotBase>())
	{
		return;
	}

	TSharedPtr<FSectionModel>              SectionModel = WeakSectionModel.Pin();
	TSharedPtr<FSharedViewModelData>       SharedModelData = SectionModel ? SectionModel->GetSharedData() : nullptr;
	TSharedPtr<FEditorSharedViewModelData> EditorSharedModelData = SharedModelData ? SharedModelData->CastThisShared<FEditorSharedViewModelData>() : nullptr;
	if (!SectionModel || !EditorSharedModelData)
	{
		return;
	}

	TArray<TSharedPtr<FSectionModel>> AllUnderlappingSections;
	if (IsSectionHighlighted(SectionInterface->GetSectionObject(), Hotspot))
	{
		if (SectionModel)
		{
			AllUnderlappingSections.Add(SectionModel);
		}
	}

	for (const TSharedPtr<FSectionModel>& Section : GetUnderlappingSectionModels())
	{
		UMovieSceneSection* SectionObject = Section->GetSection();
		if (IsSectionHighlighted(SectionObject, Hotspot))
		{
			AllUnderlappingSections.AddUnique(Section);
		}
	}

	FTimeToPixel TimeToPixelConverter = InPainter.GetTimeConverter();
	for (const TSharedPtr<FSectionModel>& UnderlappingSectionModel : AllUnderlappingSections)
	{
		UMovieSceneSection* UnderlappingSectionObj = UnderlappingSectionModel->GetSection();
		if (UnderlappingSectionObj->GetRange() == TRange<FFrameNumber>::All())
		{
			continue;
		}

		bool bDrawThisSectionsHandles = true;
		bool bLeftHandleActive = false;
		bool bRightHandleActive = false;

		// Get the hovered/selected state for the section handles from the hotspot
		if (Hotspot)
		{
			if (const FSectionEasingHandleHotspot* EasingHotspot = Hotspot->CastThis<FSectionEasingHandleHotspot>())
			{
				bDrawThisSectionsHandles = (EasingHotspot->WeakSectionModel.Pin() == UnderlappingSectionModel);
				bLeftHandleActive = EasingHotspot->HandleType == ESequencerEasingType::In;
				bRightHandleActive = EasingHotspot->HandleType == ESequencerEasingType::Out;
			}
			else if (const FSectionEasingAreaHotspot* EasingAreaHotspot = Hotspot->CastThis<FSectionEasingAreaHotspot>())
			{
				for (const FEasingAreaHandle& Easing : EasingAreaHotspot->Easings)
				{
					TSharedPtr<FSectionModel> EasingSectionModel = Easing.WeakSectionModel.Pin();
					if (EasingSectionModel && EasingSectionModel->GetSection() == UnderlappingSectionObj)
					{
						if (Easing.EasingType == ESequencerEasingType::In)
						{
							bLeftHandleActive = true;
						}
						else
						{
							bRightHandleActive = true;
						}

						if (bLeftHandleActive && bRightHandleActive)
						{
							break;
						}
					}
				}
			}
		}

		const UMovieSceneTrack* Track = UnderlappingSectionObj->GetTypedOuter<UMovieSceneTrack>();
		FMovieSceneSupportsEasingParams SupportsEasingParams(UnderlappingSectionObj);
		EMovieSceneTrackEasingSupportFlags EasingSupportFlags = Track->SupportsEasing(SupportsEasingParams);

		if (!bDrawThisSectionsHandles || !EnumHasAnyFlags(EasingSupportFlags, EMovieSceneTrackEasingSupportFlags::ManualEasing))
		{
			continue;
		}

		// We only draw the corner radius for the easing handle when it's at the same place as the grip. Otherwise we
		// draw a straight triangle.
		const float HandleCornerRadius = 2.f;
		const float MinHandleSize = 8.f;

		const ESlateDrawEffect DrawEffects = InPainter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
		const bool bIsSectionToKey = Track->GetAllSections().Num() > 1 && IsSectionToKey(Track, UnderlappingSectionObj);

		// If this is the section to key, we draw the easing handle in the same bright green color as the border
		// outline. We also make the handle a bit bigger, otherwise that border being drawn on top makes it look
		// smaller.
		// If there isn't any "section to key" border, we just draw the handle white.
		const FLinearColor InactiveHandleColor = bIsSectionToKey ? FStyleColors::Success.GetColor(FWidgetStyle()) : FStyleColors::AccentYellow.GetColor(FWidgetStyle());
		const float HalfSectionHeight = SectionInterface->GetSectionHeight(EditorSharedModelData->GetEditor()->GetViewDensity()) / 2.f;
		const float HandleSize = FMath::Max(MinHandleSize, FMath::Min(HalfSectionHeight, SectionInterface->GetSectionGripSize())) + (bIsSectionToKey ? 3.f : 0.f);

		const FSlateBrush* HandleBrush = FAppStyle::GetBrush("Sequencer.Section.EasingHandle");
		FSlateResourceHandle HandleResource = FSlateApplication::Get().GetRenderer()->GetResourceHandle(*HandleBrush);

		const FSlateRenderTransform& HandleRenderTransform = InPainter.SectionGeometry.GetAccumulatedRenderTransform();

		if (UnderlappingSectionObj->HasStartFrame() && EnumHasAllFlags(EasingSupportFlags, EMovieSceneTrackEasingSupportFlags::ManualEaseIn))
		{
			TRange<FFrameNumber> EaseInRange = UnderlappingSectionObj->GetEaseInRange();
			const bool bHasEaseIn = !EaseInRange.IsEmpty();
			FFrameNumber HandleFrame = bHasEaseIn ? UE::MovieScene::DiscreteExclusiveUpper(EaseInRange) : UnderlappingSectionObj->GetInclusiveStartFrame();
			const float HandlePos(TimeToPixel->FrameToPixel(HandleFrame) - HandleOffsetPx);
			FColor HandleColor = (bLeftHandleActive ? SelectionColor : InactiveHandleColor).ToFColorSRGB();

			TArray<FSlateVertex> Verts;
			Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(HandleRenderTransform, FVector2f(HandlePos, 0.f), FVector2f(0.f, 0.f), HandleColor));
			Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(HandleRenderTransform, FVector2f(HandlePos, HandleCornerRadius), FVector2f(0.f, 0.1f), HandleColor));
			Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(HandleRenderTransform, FVector2f(HandlePos + HandleCornerRadius, 0.f), FVector2f(0.1f, 0.f), HandleColor));
			Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(HandleRenderTransform, FVector2f(HandlePos + HandleSize, 0.f), FVector2f(1.f, 0.f), HandleColor));
			Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(HandleRenderTransform, FVector2f(HandlePos, HandleSize), FVector2f(0.f, 1.f), HandleColor));

			TArray<SlateIndex> Indices;
			if (bHasEaseIn)
			{
				Indices.Add(0); Indices.Add(3); Indices.Add(4);
			}
			else
			{
				Indices.Add(1); Indices.Add(2); Indices.Add(3);
				Indices.Add(1); Indices.Add(3); Indices.Add(4);
			}

			FSlateDrawElement::MakeCustomVerts(
				InPainter.DrawElements,
				InPainter.LayerId,
				HandleResource,
				Verts,
				Indices,
				nullptr,
				0,
				0,
				DrawEffects | ESlateDrawEffect::PreMultipliedAlpha
			);
		}

		if (UnderlappingSectionObj->HasEndFrame() && EnumHasAllFlags(EasingSupportFlags, EMovieSceneTrackEasingSupportFlags::ManualEaseOut))
		{
			TRange<FFrameNumber> EaseOutRange = UnderlappingSectionObj->GetEaseOutRange();
			const bool bHasEaseOut = !EaseOutRange.IsEmpty();
			FFrameNumber HandleFrame = bHasEaseOut ? UE::MovieScene::DiscreteInclusiveLower(EaseOutRange) : UnderlappingSectionObj->GetExclusiveEndFrame();
			const float HandlePos(TimeToPixel->FrameToPixel(HandleFrame) + HandleOffsetPx);
			const FColor HandleColor = (bRightHandleActive ? SelectionColor : InactiveHandleColor).ToFColorSRGB();

			TArray<FSlateVertex> Verts;
			Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(HandleRenderTransform, FVector2f(HandlePos, 0.f), FVector2f(1.f, 0.f), HandleColor));
			Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(HandleRenderTransform, FVector2f(HandlePos - HandleCornerRadius, 0.f), FVector2f(0.9f, 0.f), HandleColor));
			Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(HandleRenderTransform, FVector2f(HandlePos, HandleCornerRadius), FVector2f(1.f, 0.1f), HandleColor));
			Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(HandleRenderTransform, FVector2f(HandlePos, HandleSize), FVector2f(1.f, 1.f), HandleColor));
			Verts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(HandleRenderTransform, FVector2f(HandlePos - HandleSize, 0.f), FVector2f(0.f, 0.f), HandleColor));

			TArray<SlateIndex> Indices;
			if (bHasEaseOut)
			{
				Indices.Add(0); Indices.Add(3); Indices.Add(4);
			}
			else
			{
				Indices.Add(4); Indices.Add(1); Indices.Add(2);
				Indices.Add(4); Indices.Add(2); Indices.Add(3);
			}

			FSlateDrawElement::MakeCustomVerts(
				InPainter.DrawElements,
				InPainter.LayerId,
				HandleResource,
				Verts,
				Indices,
				nullptr,
				0,
				0,
				DrawEffects | ESlateDrawEffect::PreMultipliedAlpha
			);
		}
	}
}


void SSequencerSection::DrawSectionHandles( const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, ESlateDrawEffect DrawEffects, FLinearColor SelectionColor, TSharedPtr<ITrackAreaHotspot> Hotspot ) const
{
	if (CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport))
	{
		return;
	}

	UMovieSceneSection* ThisSection = SectionInterface->GetSectionObject();
	if (!ThisSection)
	{
		return;
	}

	if (Hotspot && !Hotspot->IsA<FSectionHotspotBase>())
	{
		return;
	}

	TOptional<FSlateClippingState> PreviousClipState = OutDrawElements.GetClippingState();
	if (PreviousClipState.IsSet())
	{
		OutDrawElements.PopClip();
	}

	OutDrawElements.PushClip(FSlateClippingZone(AllottedGeometry.GetLayoutBoundingRect()));

	TArray<TSharedPtr<FSectionModel>> AllUnderlappingSections;
	if (IsSectionHighlighted(SectionInterface->GetSectionObject(), Hotspot))
	{
		if (TSharedPtr<FSectionModel> SectionModel = WeakSectionModel.Pin())
		{
			AllUnderlappingSections.Add(SectionModel);
		}
	}

	for (const TSharedPtr<FSectionModel>& Section : GetUnderlappingSectionModels())
	{
		UMovieSceneSection* SectionObject = Section->GetSection();
		if (IsSectionHighlighted(SectionObject, Hotspot))
		{
			AllUnderlappingSections.AddUnique(Section);
		}
	}

	FGeometry SectionGeometry = MakeHandlesGeometry(AllottedGeometry);

	for (TSharedPtr<FSectionModel> SectionModel : AllUnderlappingSections)
	{
		UMovieSceneSection*           UnderlappingSectionObj = SectionModel->GetSection();
		TSharedPtr<ISequencerSection> UnderlappingSection    = SectionModel->GetSectionInterface();
		if (!UnderlappingSection->SectionIsResizable() || UnderlappingSectionObj->GetRange() == TRange<FFrameNumber>::All())
		{
			continue;
		}

		bool bDrawThisSectionsHandles = (UnderlappingSectionObj == ThisSection && HandleOffsetPx != 0) || IsSectionHighlighted(UnderlappingSectionObj, Hotspot);
		bool bLeftHandleActive = false;
		bool bRightHandleActive = false;

		// Get the hovered/selected state for the section handles from the hotspot
		if (TSharedPtr<FSectionResizeHotspot> ResizeHotspot = HotspotCast<FSectionResizeHotspot>(Hotspot))
		{
			if (ResizeHotspot->WeakSectionModel.Pin() == SectionModel)
			{
				bDrawThisSectionsHandles = true;
				bLeftHandleActive = ResizeHotspot->HandleType == FSectionResizeHotspot::Left;
				bRightHandleActive = ResizeHotspot->HandleType == FSectionResizeHotspot::Right;
			}
			else
			{
				bDrawThisSectionsHandles = false;
			}
		}

		if (!bDrawThisSectionsHandles)
		{
			continue;
		}

		const float ThisHandleOffset = UnderlappingSectionObj == ThisSection ? HandleOffsetPx : 0.f;
		FVector2D GripSize( UnderlappingSection->GetSectionGripSize(), UnderlappingSection->GetSectionGripHeight(AllottedGeometry.Size.Y) );

		float Opacity = 0.85f;
		if (ThisHandleOffset != 0)
		{
			Opacity = FMath::Clamp(Opacity + ThisHandleOffset / GripSize.X * (1.f-Opacity), Opacity, 1.f);
		}

		const FSlateBrush* LeftGripBrush = FAppStyle::GetBrush("Sequencer.Section.GripLeft");
		const FSlateBrush* RightGripBrush = FAppStyle::GetBrush("Sequencer.Section.GripRight");

		// Left Grip
		if (UnderlappingSectionObj->HasStartFrame())
		{
			FGeometry SectionRectLeft = SectionGeometry.MakeChild(
				GripSize,
				FSlateLayoutTransform(FVector2D( TimeToPixel->FrameToPixel(UnderlappingSectionObj->GetInclusiveStartFrame()) - ThisHandleOffset, 0.f ))
			);
			FSlateDrawElement::MakeBox
			(
				OutDrawElements,
				LayerId,
				SectionRectLeft.ToPaintGeometry(),
				LeftGripBrush,
				DrawEffects,
				(bLeftHandleActive ? SelectionColor : LeftGripBrush->GetTint(FWidgetStyle())).CopyWithNewOpacity(Opacity)
			);
		}

		// Right Grip
		if (UnderlappingSectionObj->HasEndFrame())
		{
			FGeometry SectionRectRight = SectionGeometry.MakeChild(
				GripSize,
				FSlateLayoutTransform(FVector2D( TimeToPixel->FrameToPixel(UnderlappingSectionObj->GetExclusiveEndFrame()) - UnderlappingSection->GetSectionGripSize() + ThisHandleOffset, 0 ))
			);
			FSlateDrawElement::MakeBox
			(
				OutDrawElements,
				LayerId,
				SectionRectRight.ToPaintGeometry(),
				RightGripBrush,
				DrawEffects,
				(bRightHandleActive ? SelectionColor : RightGripBrush->GetTint(FWidgetStyle())).CopyWithNewOpacity(Opacity)
			);
		}
	}

	OutDrawElements.PopClip();
	if (PreviousClipState.IsSet())
	{
		OutDrawElements.GetClippingManager().PushClippingState(PreviousClipState.GetValue());
	}
}

void SSequencerSection::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	UMovieSceneSection*       Section      = SectionInterface->GetSectionObject();
	TSharedPtr<FSequencer>    SequencerPtr = Sequencer.Pin();
	TSharedPtr<FSectionModel> SectionModel = WeakSectionModel.Pin();
	UMovieScene*              MovieScene   = Section ? Section->GetTypedOuter<UMovieScene>() : nullptr;

	if (!Section || !SequencerPtr || !SectionModel || !MovieScene)
	{
		return;
	}

	if (GetVisibility() == EVisibility::Visible)
	{
		if (Section->HasStartFrame() && Section->HasEndFrame())
		{
			constexpr float MinSectionWidth = 1.f;

			FTimeToPixel TimeToPixelConverter(ParentGeometry, GetSequencer().GetViewRange(), MovieScene->GetTickResolution());
			const FFrameNumber SectionLength = UE::MovieScene::DiscreteSize(Section->ComputeEffectiveRange());
			const float SectionWidth = TimeToPixelConverter.FrameDeltaToPixel(SectionLength);

			const float GripSize = SectionInterface->GetSectionGripSize();
			const float FinalSectionWidth = FMath::Max(MinSectionWidth + GripSize * 2.f, SectionWidth);
			HandleOffsetPx = FMath::RoundToFloat((FinalSectionWidth - SectionWidth) * 0.5f);
		}
		else
		{
			HandleOffsetPx = 0;
		}

		FGeometry SectionGeometry = MakeSectionGeometryWithoutHandles(AllottedGeometry);

		FTrackLaneVirtualAlignment VirtualAlignment = SectionModel->ArrangeVirtualTrackLaneView();
		*TimeToPixel = *WeakOwningTrackLane.Pin()->GetTrackAreaView()->GetTimeToPixel(VirtualAlignment.ViewSpaceID);
		if (Section->HasStartFrame())
		{
			*TimeToPixel = TimeToPixel->RelativeTo(Section->GetInclusiveStartFrame());
		}
		SectionInterface->Tick(SectionGeometry, ParentGeometry, InCurrentTime, InDeltaTime);
	}
	else
	{
		FTrackLaneVirtualAlignment VirtualAlignment = SectionModel->ArrangeVirtualTrackLaneView();
		*TimeToPixel = *WeakOwningTrackLane.Pin()->GetTrackAreaView()->GetTimeToPixel(VirtualAlignment.ViewSpaceID);
		if (Section->HasStartFrame())
		{
			*TimeToPixel = TimeToPixel->RelativeTo(Section->GetInclusiveStartFrame());
		}
	}
}

void SSequencerSection::AddChildView(TSharedPtr<ITrackLaneWidget> ChildWidget, TWeakPtr<STrackLane> InWeakOwningLane)
{
	ChildLaneWidgets->AddWeakWidget(ChildWidget, InWeakOwningLane);
}

void SSequencerSection::OnModifiedIndirectly(UMovieSceneSignedObject* Object)
{
	if (Object->IsA<UMovieSceneSection>())
	{
		UpdateUnderlappingSegments();
	}
}

TSharedRef<FTimeToPixel> SSequencerSection::GetTimeToPixel() const
{
	return TimeToPixel.ToSharedRef();
}

FTimeToPixel SSequencerSection::GetScreenSpace(const FGuid& ViewSpaceID) const
{
	using namespace UE::MovieScene;

	TSharedPtr<STrackLane>     TrackLane = WeakOwningTrackLane.Pin();
	TSharedPtr<STrackAreaView> TrackArea = TrackLane ? TrackLane->GetTrackAreaView() : nullptr;
	if (TrackArea)
	{
		TSharedPtr<FTimeToPixel> Space = TrackArea->GetTimeToPixel(ViewSpaceID);

		TSharedPtr<FSectionModel> Section = WeakSectionModel.Pin();
		if (Section)
		{
			TRange<FFrameNumber> Range = Section->GetRange();
			if (Range.GetLowerBound().IsClosed())
			{
				return Space->RelativeTo(DiscreteInclusiveLower(Range));
			}
		}

		return *Space;
	}

	return *TimeToPixel;
}


FReply SSequencerSection::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Unhandled();
}

FGeometry SSequencerSection::MakeSectionGeometryWithoutHandles(const FGeometry& AllottedGeometry) const
{
	const FVector2f LocalSize = AllottedGeometry.GetLocalSize();

	FVector2f SectionSizeWithoutHandles = LocalSize - FVector2f(HandleOffsetPx * 2.f, 0.0f);
	SectionSizeWithoutHandles.X = FMath::Max(1.f, SectionSizeWithoutHandles.X);

	return AllottedGeometry.MakeChild(
		SectionSizeWithoutHandles,
		FSlateLayoutTransform(FVector2D(HandleOffsetPx, 0))
	);
}

FGeometry SSequencerSection::MakeHandlesGeometry(const FGeometry& AllottedGeometry) const
{
	const FVector2f LocalSize = AllottedGeometry.GetLocalSize();

	FVector2f SectionSizeWithoutHandles = LocalSize - FVector2f(HandleOffsetPx * 2.f, 0.f);
	SectionSizeWithoutHandles.X = FMath::Max(1.f, SectionSizeWithoutHandles.X);
	SectionSizeWithoutHandles.Y = FMath::Max(1.f, SectionInterface->GetSectionGripHeight(SectionSizeWithoutHandles.Y));

	return AllottedGeometry.MakeChild(
		SectionSizeWithoutHandles,
		FSlateLayoutTransform(FVector2D(HandleOffsetPx, 0))
	);
}

void SSequencerSection::UpdateUnderlappingSegments()
{
	TSharedPtr<FSectionModel> SectionModel = WeakSectionModel.Pin();
	if (SectionModel)
	{
		UnderlappingSegments = SectionModel->GetUnderlappingSections();
		UnderlappingEasingSegments = SectionModel->GetEasingSegments();
	}
	else
	{
		UnderlappingSegments.Reset();
		UnderlappingEasingSegments.Reset();
	}
}

TArray<TSharedPtr<FSectionModel>> SSequencerSection::GetUnderlappingSectionModels() const
{
	TArray<TSharedPtr<FSectionModel>> ValidSectionModels;
	for (const FOverlappingSections& Segment : UnderlappingSegments)
	{
		for (const TWeakPtr<FSectionModel>& WeakSection : Segment.Sections)
		{
			if (TSharedPtr<FSectionModel> Section = WeakSection.Pin())
			{
				ValidSectionModels.AddUnique(Section);
			}
		}
	}
	return ValidSectionModels;
}

FReply SSequencerSection::OnMouseButtonDoubleClick( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	TSharedPtr<FSectionModel> SectionModel = WeakSectionModel.Pin();
	if (!SectionModel)
	{
		return FReply::Handled();
	}

	if( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		FReply Reply = SectionInterface->OnSectionDoubleClicked( MyGeometry, MouseEvent );
		if (!Reply.IsEventHandled())
		{
			// Find the object binding this node is underneath
			FGuid ObjectBinding;
			if (TSharedPtr<IObjectBindingExtension> ObjectBindingExtension = SectionModel->FindAncestorOfType<IObjectBindingExtension>())
			{
				ObjectBinding = ObjectBindingExtension->GetObjectGuid();
			}

			Reply = SectionInterface->OnSectionDoubleClicked(MyGeometry, MouseEvent, ObjectBinding);
		}

		if (Reply.IsEventHandled())
		{
			return Reply;
		}

		GetSequencer().ZoomToFit();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}


FReply SSequencerSection::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport))
	{
		if (TSharedPtr<FTrackAreaViewModel> TrackAreaViewModel = GetTrackAreaViewModel())
		{
			TrackAreaViewModel->SetHotspot(nullptr);
		}
		return FReply::Unhandled();
	}

	if (MouseEvent.GetCursorDelta().Size() > 0)
	{
		if (CheckForEasingHandleInteraction(MouseEvent, MyGeometry) ||
			CheckForEdgeInteraction(MouseEvent, MyGeometry) ||
			CheckForEasingAreaInteraction(MouseEvent, MyGeometry) ||
			CheckForSectionInteraction(MouseEvent, MyGeometry))
		{
			// Do nothing: we always want to return FReply::Unhandled because we're just setting up
			// hotspots here. STrackAreaView will actually handle the mouse movement event, including
			// calling it on the active hotspot, doing panning, and more.
		}
	}
	return FReply::Unhandled();
}

FReply SSequencerSection::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Unhandled();
}

void SSequencerSection::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	SCompoundWidget::OnMouseLeave( MouseEvent );

	TSharedPtr<FTrackAreaViewModel> TrackAreaViewModel = GetTrackAreaViewModel();
	if (TrackAreaViewModel)
	{
		TrackAreaViewModel->SetHotspot(nullptr);
	}
}

static float SectionThrobDurationSeconds = 1.f;
void SSequencerSection::ThrobSectionSelection(int32 ThrobCount)
{
	SectionSelectionThrobEndTime = FPlatformTime::Seconds() + ThrobCount*SectionThrobDurationSeconds;
}

static float KeyThrobDurationSeconds = .5f;
void SSequencerSection::ThrobKeySelection(int32 ThrobCount)
{
	KeySelectionThrobEndTime = FPlatformTime::Seconds() + ThrobCount*KeyThrobDurationSeconds;
}

float EvaluateThrob(float Alpha)
{
	return .5f - FMath::Cos(FMath::Pow(Alpha, 0.5f) * 2 * PI) * .5f;
}

float SSequencerSection::GetSectionSelectionThrobValue()
{
	double CurrentTime = FPlatformTime::Seconds();

	if (SectionSelectionThrobEndTime > CurrentTime)
	{
		float Difference = SectionSelectionThrobEndTime - CurrentTime;
		return EvaluateThrob(1.f - FMath::Fmod(Difference, SectionThrobDurationSeconds));
	}

	return 0.f;
}

float SSequencerSection::GetKeySelectionThrobValue()
{
	double CurrentTime = FPlatformTime::Seconds();

	if (KeySelectionThrobEndTime > CurrentTime)
	{
		float Difference = KeySelectionThrobEndTime - CurrentTime;
		return EvaluateThrob(1.f - FMath::Fmod(Difference, KeyThrobDurationSeconds));
	}

	return 0.f;
}

bool SSequencerSection::IsSectionHighlighted(UMovieSceneSection* InSection, TSharedPtr<ITrackAreaHotspot> Hotspot)
{
	if (!Hotspot)
	{
		return false;
	}

	if (FKeyHotspot* KeyHotspot = Hotspot->CastThis<FKeyHotspot>())
	{
		for (const FSequencerSelectedKey& Key : KeyHotspot->Keys)
		{
			if (Key.Section == InSection)
			{
				return true;
			}
		}
	}
	else if (FSectionEasingAreaHotspot* EasingAreaHotspot = Hotspot->CastThis<FSectionEasingAreaHotspot>())
	{
		return EasingAreaHotspot->Contains(InSection);
	}
	else if (FSectionHotspotBase* SectionHotspot = Hotspot->CastThis<FSectionHotspotBase>())
	{
		if (TSharedPtr<FSectionModel> SectionModel = SectionHotspot->WeakSectionModel.Pin())
		{
			return SectionModel->GetSection() == InSection;
		}
	}

	return false;
}

} // namespace Sequencer
} // namespace UE
