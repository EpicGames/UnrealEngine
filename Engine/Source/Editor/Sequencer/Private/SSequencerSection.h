// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Geometry.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Rendering/RenderingCommon.h"

#include "ISequencerSection.h"
#include "EventHandlers/ISignedObjectEventHandler.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/Extensions/ITrackLaneExtension.h"

class FPaintArgs;
class FSequencer;
class FSequencerSectionPainter;
class FSlateWindowElementList;

namespace UE
{
namespace Sequencer
{

class SCompoundTrackLaneView;
class STrackAreaView;
class FSectionModel;
class FTrackAreaViewModel;
struct ITrackAreaHotspot;

class SSequencerSection : public SCompoundWidget, public ITrackLaneWidget, public UE::Sequencer::ISectionView, public UE::Sequencer::ITrackLaneWidgetSpace, public UE::MovieScene::ISignedObjectEventHandler
{
public:
	SLATE_BEGIN_ARGS( SSequencerSection )
	{}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, TSharedPtr<FSequencer> Sequencer, TSharedPtr<FSectionModel> InSectionModel, TSharedPtr<STrackLane> InOwningTrackLane);
	~SSequencerSection();

	/**
	 * Get the section interface
	 */
	TSharedPtr<ISequencerSection> GetSectionInterface() const { return SectionInterface; }

private:

	/**
	 * Checks for mouse interaction with the left and right edge of the section
	 *
	 * @param MousePosition		The current screen space position of the mouse
	 * @param SectionGeometry	The geometry of the section
	 */
	bool CheckForEdgeInteraction( const FPointerEvent& MousePosition, const FGeometry& SectionGeometry );

	/**
	 * Checks for mouse interaction with the ease in/out handles of the section
	 *
	 * @param MousePosition		The current screen space position of the mouse
	 * @param SectionGeometry	The geometry of the section
	 */
	bool CheckForEasingHandleInteraction( const FPointerEvent& MousePosition, const FGeometry& SectionGeometry );

	/**
	 * Checks for mouse interaction with the ease in/out area of the section
	 *
	 * @param MousePosition		The current screen space position of the mouse
	 * @param SectionGeometry	The geometry of the section
	 */
	bool CheckForEasingAreaInteraction( const FPointerEvent& MousePosition, const FGeometry& SectionGeometry );

	/**
	 * Checks for mouse interaction with the left/right grip handles of the section
	 *
	 * @param MousePosition		The current screen space position of the mouse
	 * @param SectionGeometry	The geometry of the section
	 */
	bool CheckForSectionInteraction( const FPointerEvent& MousePosition, const FGeometry& SectionGeometry );

	/*~ SWidget interface */
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseButtonDoubleClick( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;

	/*~ ITrackLaneWidget interface */
	virtual FTrackLaneScreenAlignment GetAlignment(const ITrackLaneWidgetSpace& ScreenSpace, const FGeometry& InParentGeometry) const override;
	virtual int32 GetOverlapPriority() const override;
	virtual void ReportParentGeometry(const FGeometry& InParentGeometry) override;
	virtual TSharedRef<const SWidget> AsWidget() const override { return AsShared(); }
	virtual bool AcceptsChildren() const override { return true; }
	virtual void AddChildView(TSharedPtr<ITrackLaneWidget> ChildWidget, TWeakPtr<STrackLane> InWeakOwningLane) override;

	/*~ ISignedObjectEventHandler interface */
	virtual void OnModifiedIndirectly(UMovieSceneSignedObject* Object) override;

	/*~ ISectionView interface */
	virtual TSharedRef<FTimeToPixel> GetTimeToPixel() const override;

	/*~ ITrackLaneWidgetSpace interface */
	virtual FTimeToPixel GetScreenSpace(const FGuid& ViewSpaceID = FGuid()) const override;

	/**
	 * Paint the easing handles for this section
	 */
	void PaintEasingHandles( FSequencerSectionPainter& InPainter, FLinearColor SelectionColor, TSharedPtr<ITrackAreaHotspot> Hotspot ) const;

	/**
	 * Draw the section resize handles.
	 */
	void DrawSectionHandles( const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, ESlateDrawEffect DrawEffects, FLinearColor SelectionColor, TSharedPtr<ITrackAreaHotspot> Hotspot ) const;

	/** @return the sequencer interface */
	FSequencer& GetSequencer() const;

	/** @return the track area view that this section belongs to */
	TSharedPtr<STrackAreaView> GetTrackAreaView() const;

	/** @return the track area view-model that this section belongs to */
	TSharedPtr<FTrackAreaViewModel> GetTrackAreaViewModel() const;

	/**
	 * Ensure that the cached array of underlapping sections is up to date
	 */
	void UpdateUnderlappingSegments();

	/** @return the underlapping section models */
	TArray<TSharedPtr<FSectionModel>> GetUnderlappingSectionModels() const;

	/**
	 * Retrieve the tooltip text for this section
	 */
	FText GetToolTipText() const;

	/**
	 * Check whether this section widget is enabled or not
	 */
	bool IsEnabled() const;

	/**
	 * Gets the visibility of this section's top-level channel
	 */
	EVisibility GetTopLevelChannelGroupVisibility() const;

	/**
	 * Gets the color of the top-level key bar
	 */
	FLinearColor GetTopLevelKeyBarColor() const;

	/**
	 * Get the padding offset around the actual section's geometry
	 */
	FMargin GetHandleOffsetPadding() const;

	/** 
	 * Creates geometry for a section without space for the handles
	 */
	FGeometry MakeSectionGeometryWithoutHandles(const FGeometry& AllottedGeometry) const;

	/** 
	 * Creates geometry for the section handles
	 */
	FGeometry MakeHandlesGeometry(const FGeometry& AllottedGeometry) const;

public:

	/** Indicate that the current section selection should throb the specified number of times. A single throb takes 0.2s. */
	static void ThrobSectionSelection(int32 ThrobCount = 1);

	/** Indicate that the current key selection should throb the specified number of times. A single throb takes 0.2s. */
	static void ThrobKeySelection(int32 ThrobCount = 1);

	/** Get a value between 0 and 1 that indicates the amount of throb-scale to apply to the currently selected keys */
	static float GetKeySelectionThrobValue();

	/** Get a value between 0 and 1 that indicates the amount of throb-scale to apply to the currently selected sections */
	static float GetSectionSelectionThrobValue();

	/** Check to see whether the specified section is highlighted */
	static bool IsSectionHighlighted(UMovieSceneSection* InSection, TSharedPtr<ITrackAreaHotspot> Hotspot);

private:

	/** The parent sequencer */
	TWeakPtr<FSequencer> Sequencer;
	/** Interface to section data */
	TSharedPtr<ISequencerSection> SectionInterface;
	/** Section model */
	TWeakPtr<FSectionModel> WeakSectionModel;
	/** Widget container for child lanes */
	TSharedPtr<SCompoundTrackLaneView> ChildLaneWidgets;
	/** The track lane that this widget is on */
	TWeakPtr<STrackLane> WeakOwningTrackLane;
	/** Shared time to local pixel space for this section */
	TSharedPtr<FTimeToPixel> TimeToPixel;
	/** Cached parent geometry to pass down to any section interfaces that need it during tick */
	FGeometry ParentGeometry;
	/** The end time for a throbbing animation for selected sections */
	static double SectionSelectionThrobEndTime;
	/** The end time for a throbbing animation for selected keys */
	static double KeySelectionThrobEndTime;
	/** Handle offset amount in pixels */
	float HandleOffsetPx;
	/** Array of segments that define other sections that reside below this one */
	TArray<FOverlappingSections> UnderlappingSegments;
	/** Array of segments that define other sections that reside below this one */
	TArray<FOverlappingSections> UnderlappingEasingSegments;

	MovieScene::TNonIntrusiveEventHandler<MovieScene::ISignedObjectEventHandler> TrackModifiedBinding;

	friend struct FSequencerSectionPainterImpl;
};

} // namespace Sequencer
} // namespace UE

