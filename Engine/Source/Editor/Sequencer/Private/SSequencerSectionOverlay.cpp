// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerSectionOverlay.h"

int32 SSequencerSectionOverlay::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	FPaintViewAreaArgs PaintArgs;
	PaintArgs.bDisplayTickLines = bDisplayTickLines.Get();
	PaintArgs.bDisplayScrubPosition = bDisplayScrubPosition.Get();
	PaintArgs.bDisplayMarkedFrames = bDisplayMarkedFrames.Get();

	if (PaintPlaybackRangeArgs.IsSet())
	{
		PaintArgs.PlaybackRangeArgs = PaintPlaybackRangeArgs.Get();
	}

	TimeSliderController->OnPaintViewArea( AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, ShouldBeEnabled( bParentEnabled ), PaintArgs );

	return SCompoundWidget::OnPaint( Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled  );
}
