// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "SequencerTimeSliderController.h"

class FPaintArgs;
class FSlateWindowElementList;

/**
 * An overlay that displays global information in the section area
 */
class SSequencerSectionOverlay : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SSequencerSectionOverlay )
		: _DisplayTickLines( true )
		, _DisplayScrubPosition( false )
		, _DisplayMarkedFrames( false )
	{}
		SLATE_ATTRIBUTE( bool, DisplayTickLines )
		SLATE_ATTRIBUTE( bool, DisplayScrubPosition )
		SLATE_ATTRIBUTE( bool, DisplayMarkedFrames )
		SLATE_ATTRIBUTE( FPaintPlaybackRangeArgs, PaintPlaybackRangeArgs )

	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, TSharedRef<FSequencerTimeSliderController> InTimeSliderController )
	{
		bDisplayScrubPosition = InArgs._DisplayScrubPosition;
		bDisplayTickLines = InArgs._DisplayTickLines;
		bDisplayMarkedFrames = InArgs._DisplayMarkedFrames;
		PaintPlaybackRangeArgs = InArgs._PaintPlaybackRangeArgs;
		TimeSliderController = InTimeSliderController;
	}

private:
	/** SWidget Interface */
	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;

private:
	/** Controller for manipulating time */
	TSharedPtr<FSequencerTimeSliderController> TimeSliderController;
	/** Whether or not to display the scrub position */
	TAttribute<bool> bDisplayScrubPosition;
	/** Whether or not to display tick lines */
	TAttribute<bool> bDisplayTickLines;
	/** Whether or not to display marked frames */
	TAttribute<bool> bDisplayMarkedFrames;
	/** User-supplied options for drawing playback range */
	TAttribute<FPaintPlaybackRangeArgs> PaintPlaybackRangeArgs;

};
