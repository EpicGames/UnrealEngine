// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerShotFilterOverlay.h"
#include "Rendering/DrawElements.h"
#include "Styling/AppStyle.h"
#include "TimeToPixel.h"

/* SSequencerShotFilterOverlay interface
 *****************************************************************************/

void SSequencerShotFilterOverlay::Construct(const FArguments& InArgs, TSharedRef<FSequencer> InSequencer)
{
	ViewRange = InArgs._ViewRange;
	Sequencer = InSequencer;
}


/* SWidget interface
 *****************************************************************************/

int32 SSequencerShotFilterOverlay::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	float Alpha = Sequencer.Pin()->GetOverlayFadeCurve();

	if (Alpha > 0.f)
	{
		FTimeToPixel TimeToPixelConverter(AllottedGeometry, ViewRange.Get(), Sequencer.Pin()->GetFocusedTickResolution());
		
		TRange<float> TimeBounds = TRange<float>(
			TimeToPixelConverter.PixelToSeconds(0),
			TimeToPixelConverter.PixelToSeconds(AllottedGeometry.GetLocalSize().X)
		);

		TArray< TRange<float> > OverlayRanges = ComputeOverlayRanges(TimeBounds, CachedFilteredRanges);

		for (int32 i = 0; i < OverlayRanges.Num(); ++i)
		{
			float LowerBound = TimeToPixelConverter.SecondsToPixel(OverlayRanges[i].GetLowerBoundValue());
			float UpperBound = TimeToPixelConverter.SecondsToPixel(OverlayRanges[i].GetUpperBoundValue());

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(FVector2f(UpperBound - LowerBound, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2f(LowerBound, 0.f))),
				FAppStyle::GetBrush("Sequencer.ShotFilter"),
				ESlateDrawEffect::None,
				FLinearColor(1.f, 1.f, 1.f, Alpha)
			);
		}
	}

	return LayerId;
}


FVector2D SSequencerShotFilterOverlay::ComputeDesiredSize(float) const
{
	return FVector2D(100, 100);
}


/* SSequencerShotFilterOverlay implementation
 *****************************************************************************/

TArray<TRange<float>> SSequencerShotFilterOverlay::ComputeOverlayRanges(TRange<float> TimeBounds, TArray< TRange<float> > RangesToSubtract) const
{
	TArray<TRange<float>> FilteredRanges;
	FilteredRanges.Add(TimeBounds);

	// @todo Sequencer Optimize - This is O(n^2)
	// However, n is likely to stay very low, and the average case is likely O(n)
	// Also, the swapping of TArrays in this loop could use some heavy optimization as well

	for (int32 i = 0; i < RangesToSubtract.Num(); ++i)
	{
		TRange<float>& CurrentRange = RangesToSubtract[i];

		// ignore ranges that don't overlap with the time bounds
		if (CurrentRange.Overlaps(TimeBounds))
		{
			TArray<TRange<float>> NewFilteredRanges;

			for (int32 j = 0; j < FilteredRanges.Num(); ++j)
			{
				TArray<TRange<float>> SubtractedRanges = TRange<float>::Difference(FilteredRanges[j], CurrentRange);
				NewFilteredRanges.Append(SubtractedRanges);
			}

			FilteredRanges = NewFilteredRanges;
		}
	}

	return FilteredRanges;
}
