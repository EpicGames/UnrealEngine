// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"
#include "Sequencer.h"

class FPaintArgs;
class FSlateWindowElementList;

/**
 * The shot filter overlay displays the overlay needed to filter out widgets
 * based on which shots are actively in use.
 *
 * @todo sequencer: this class is currently not used and may no longer be needed
 */
class SSequencerShotFilterOverlay
	: public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SSequencerShotFilterOverlay) { }
		SLATE_ATTRIBUTE(TRange<double>, ViewRange)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FSequencer> InSequencer);
	
public:

	// SWidget interface

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FVector2D ComputeDesiredSize(float) const override;

protected:

	/**
	 * Given a range of time bounds, find what ranges still should be
	 * filtered based on shot filters
	 */
	TArray<TRange<float>> ComputeOverlayRanges(TRange<float> TimeBounds, TArray< TRange<float>> RangesToSubtract) const;

private:

	/** The current minimum view range */
	TAttribute<TRange<double>> ViewRange;

	/** The main sequencer interface */
	TWeakPtr<FSequencer> Sequencer;

	/** Cached set of ranges that are being filtering currently */
	TArray<TRange<float>> CachedFilteredRanges;
};
