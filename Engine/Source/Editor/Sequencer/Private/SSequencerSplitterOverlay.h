// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Margin.h"
#include "Widgets/SOverlay.h"
#include "Layout/ArrangedChildren.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"

/** Splitter used on the sequencer as an overlay. Input is disabled on all areas except the draggable positions */
class SSequencerSplitterOverlay : public SOverlay
{
public:
	typedef SSplitter::FArguments FArguments;

	void Construct( const FArguments& InArgs )
	{
		SetVisibility(EVisibility::SelfHitTestInvisible);

		Splitter = SArgumentNew(InArgs, SSplitter);
		Splitter->SetVisibility(EVisibility::HitTestInvisible);
		AddSlot()
		[
			Splitter.ToSharedRef()
		];

		for (int32 Index = 0; Index < Splitter->GetChildren()->Num() - 1; ++Index)
		{
			AddSlot()
			.Padding(TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(this, &SSequencerSplitterOverlay::GetSplitterHandlePadding, Index)))
			[
				SNew(SBox)
				.Visibility(EVisibility::Visible)
			];
		}
	}

	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override
	{
		FArrangedChildren SplitterChildren(ArrangedChildren.GetFilter());
		Splitter->ArrangeChildren(AllottedGeometry, SplitterChildren);

		SlotPadding.Reset();

		for (int32 Index = 0; Index < SplitterChildren.Num() - 1; ++Index)
		{
			const auto& ThisGeometry = SplitterChildren[Index].Geometry;
			const auto& NextGeometry = SplitterChildren[Index + 1].Geometry;

			if (Splitter->GetOrientation() == EOrientation::Orient_Horizontal)
			{
				SlotPadding.Add(FMargin(
					ThisGeometry.Position.X + ThisGeometry.GetLocalSize().X,
					0,
					AllottedGeometry.Size.X - NextGeometry.Position.X,
					0)
				);
			}
			else
			{
				SlotPadding.Add(FMargin(
					0,
					ThisGeometry.Position.Y + ThisGeometry.GetLocalSize().Y,
					0,
					AllottedGeometry.Size.Y - NextGeometry.Position.Y)
				);
			}
		}

		SOverlay::OnArrangeChildren(AllottedGeometry, ArrangedChildren);
	}

	FMargin GetSplitterHandlePadding(int32 Index) const
	{
		if (SlotPadding.IsValidIndex(Index))
		{
			return SlotPadding[Index];
		}

		return 0.f;
	}
	
	virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override
	{
		return Splitter->OnCursorQuery(MyGeometry, CursorEvent);
	}

	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		FReply Reply = Splitter->OnMouseButtonDown(MyGeometry, MouseEvent);
		if (Reply.GetMouseCaptor().IsValid())
		{
			// Set us to be the mouse captor so we can forward events properly
			Reply.CaptureMouse( SharedThis(this) );
			SetVisibility(EVisibility::Visible);
		}
		return Reply;
	}
	
	virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override
	{
		SetVisibility(EVisibility::SelfHitTestInvisible);
		SOverlay::OnMouseCaptureLost(CaptureLostEvent);
	}

	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		FReply Reply = Splitter->OnMouseButtonUp(MyGeometry, MouseEvent);
		if (Reply.ShouldReleaseMouse())
		{
			SetVisibility(EVisibility::SelfHitTestInvisible);
		}
		return Reply;
	}
	virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		return Splitter->OnMouseMove(MyGeometry, MouseEvent);
	}
	virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override
	{
		return Splitter->OnMouseLeave(MouseEvent);
	}

	TSharedPtr<SSplitter> Splitter;
	mutable TArray<FMargin> SlotPadding;
};
