// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerStretchBox.h"
#include "Sequencer.h"
#include "SequencerSettings.h"
#include "SequencerCommonHelpers.h"
#include "Styling/AppStyle.h"
#include "EditorFontGlyphs.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "Sequencer"

void SSequencerStretchBox::Construct(const FArguments& InArgs, const TSharedRef<FSequencer>& InSequencer, USequencerSettings& InSettings, const TSharedRef<INumericTypeInterface<double>>& InNumericTypeInterface)
{
	SequencerPtr = InSequencer;
	Settings = &InSettings;
	NumericTypeInterface = InNumericTypeInterface;
	
	CachedStretchTime = FFrameNumber(0);
	CachedShrinkTime = FFrameNumber(0);

	const FDockTabStyle* GenericTabStyle = &FCoreStyle::Get().GetWidgetStyle<FDockTabStyle>("Docking.Tab");
	const FButtonStyle* const CloseButtonStyle = &GenericTabStyle->CloseButtonStyle;

	ChildSlot
	[
		SAssignNew(Border, SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
			.Padding(6.0f)
			.Visibility(EVisibility::Collapsed)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
							.ToolTipText(LOCTEXT("StretchForward_Tooltip", "Stretch time from the current time forwards"))
							.OnClicked(this, &SSequencerStretchBox::OnStretchButtonClicked)
							[
								SNew(STextBlock)
								.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
								.Text(FEditorFontGlyphs::Angle_Right)
							]
					]

				+ SHorizontalBox::Slot()
					.Padding(6.0f, 0.0f, 0.0f, 0.0f)
					.AutoWidth()
					[
						SAssignNew(StretchEntryBox, SNumericEntryBox<double>)
						.MinDesiredValueWidth(32.0f)
						.TypeInterface(NumericTypeInterface)
						.ToolTipText(LOCTEXT("StretchDelta_Tooltip", "The amount of time to stretch forward in front of the current time"))
						.OnValueCommitted(this, &SSequencerStretchBox::OnStretchCommitted)
						.OnValueChanged(this, &SSequencerStretchBox::OnStretchChanged)
						.Value_Lambda([this](){ return CachedStretchTime.Value; })
					]

				+ SHorizontalBox::Slot()
					.Padding(6.0f, 0.0f, 0.0f, 0.0f)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
							.ToolTipText(LOCTEXT("ShrinkForward_Tooltip", "Shrink time in front of the current time"))
							.OnClicked(this, &SSequencerStretchBox::OnShrinkButtonClicked)
							[
								SNew(STextBlock)
								.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
								.Text(FEditorFontGlyphs::Angle_Left)
							]
					]

				+ SHorizontalBox::Slot()
					.Padding(6.0f, 0.0f, 0.0f, 0.0f)
					.AutoWidth()
					[
						SAssignNew(ShrinkEntryBox, SNumericEntryBox<double>)
						.MinDesiredValueWidth(32.0f)
						.TypeInterface(NumericTypeInterface)
						.ToolTipText(LOCTEXT("ShrinkDelta_Tooltip", "The amount of time to shrink in front of the current time"))
						.OnValueCommitted(this, &SSequencerStretchBox::OnShrinkCommitted)
						.OnValueChanged(this, &SSequencerStretchBox::OnShrinkChanged)
						.Value_Lambda([this]() { return CachedShrinkTime.Value; })
					]

				+ SHorizontalBox::Slot()
				.Padding(3.0f, 0.0f, 0.0f, 0.0f)
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle( CloseButtonStyle )
					.OnClicked( this, &SSequencerStretchBox::OnCloseButtonClicked )
					.ContentPadding( 0 )
					[
						SNew(SSpacer)
						.Size( CloseButtonStyle->Normal.ImageSize )
					]
				]
			]
	];
}


void SSequencerStretchBox::ToggleVisibility()
{
	FSlateApplication& SlateApplication = FSlateApplication::Get();

	if (Border->GetVisibility() == EVisibility::Visible)
	{
		if (LastFocusedWidget.IsValid())
		{
			SlateApplication.SetAllUserFocus(LastFocusedWidget.Pin(), EFocusCause::Navigation);
		}
		
		Border->SetVisibility(EVisibility::Collapsed);
	}
	else
	{
		Border->SetVisibility(EVisibility::Visible);
		LastFocusedWidget = SlateApplication.GetUserFocusedWidget(0);
		SlateApplication.SetAllUserFocus(StretchEntryBox, EFocusCause::Navigation);
	}
}

void SSequencerStretchBox::OnStretchCommitted(double Value, ETextCommit::Type CommitType)
{
	OnStretchChanged(Value);
}

void SSequencerStretchBox::OnStretchChanged(double Value)
{
	CachedStretchTime = FFrameTime::FromDecimal(Value).GetFrame();
}

void SSequencerStretchBox::OnShrinkCommitted(double Value, ETextCommit::Type CommitType)
{
	OnShrinkChanged(Value);
}

void SSequencerStretchBox::OnShrinkChanged(double Value)
{
	CachedShrinkTime = FFrameTime::FromDecimal(Value).GetFrame();
}

FReply SSequencerStretchBox::OnStretchButtonClicked()
{
	if (CachedStretchTime != 0)
	{
		TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();

		Sequencer->StretchTime(CachedStretchTime);
	}

	return FReply::Handled();
}

FReply SSequencerStretchBox::OnShrinkButtonClicked()
{
	if (CachedShrinkTime != 0)
	{
		TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
		
		Sequencer->ShrinkTime(CachedShrinkTime);
	}

	return FReply::Handled();
}

FReply SSequencerStretchBox::OnCloseButtonClicked()
{
	ToggleVisibility();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
