// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Misc/QualifiedFrameTime.h"

class FSequencer;
class USequencerSettings;

class SSequencerStretchBox
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSequencerStretchBox) { }
	SLATE_END_ARGS()

	/** Construct the widget. */
	void Construct(const FArguments& InArgs, const TSharedRef<FSequencer>& InSequencer, USequencerSettings& InSettings, const TSharedRef<INumericTypeInterface<double>>& InNumericTypeInterface);

	/** Toggle the widget's visibility. */
	void ToggleVisibility();

private:

	/** Callback for transform operations. */
	FReply OnStretchButtonClicked();
	FReply OnShrinkButtonClicked();
	void OnStretchCommitted(double Value, ETextCommit::Type CommitType);
	void OnStretchChanged(double Value);
	void OnShrinkCommitted(double Value, ETextCommit::Type CommitType);
	void OnShrinkChanged(double Value);

	/** Callback for when the close button is clicked. */
	FReply OnCloseButtonClicked();

private:

	/** The border widget. */
	TSharedPtr<SWidget> Border;

	/** The stretch box widget. */
	TSharedPtr<SNumericEntryBox<double>> StretchEntryBox;

	/** The shrink entry box widget. */
	TSharedPtr<SNumericEntryBox<double>> ShrinkEntryBox;

	/** The widget that focused prior to this transform box. */
	TWeakPtr<SWidget> LastFocusedWidget;

	/** Numeric type interface used for converting parsing and generating strings from numbers. */
	TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface;

	/** The main sequencer interface. */
	TWeakPtr<FSequencer> SequencerPtr;

	/** Cached settings provided to the sequencer itself on creation. */
	USequencerSettings* Settings;

	/** Cached stretch time. */
	FFrameNumber CachedStretchTime;

	/** Cached shrink time. */
	FFrameNumber CachedShrinkTime;
};
