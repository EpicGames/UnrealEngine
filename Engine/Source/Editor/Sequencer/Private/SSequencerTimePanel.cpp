// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerTimePanel.h"
#include "Widgets/SFrameRatePicker.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SButton.h"
#include "SPrimaryButton.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "ScopedTransaction.h"
#include "Sequencer.h"
#include "Styling/AppStyle.h"
#include "MovieSceneToolHelpers.h"
#include "MovieSceneSequenceVisitor.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "SSequencerTimePanel"

void SSequencerTimePanel::Construct(const FArguments& InArgs, TWeakPtr<FSequencer> InSequencer)
{
	WeakSequencer = InSequencer;

	TArray<FCommonFrameRateInfo> TickResolutionRates;
	{
		TArrayView<const FCommonFrameRateInfo> CommonRates = FCommonFrameRates::GetAll();
		TickResolutionRates.Append(CommonRates.GetData(), CommonRates.Num());

		TickResolutionRates.Add(FCommonFrameRateInfo{ FFrameRate(1000, 1),   LOCTEXT("1000_Name",   "1000 fps (ms precision)"),                       LOCTEXT("1000_Description",   "Allows placement of sequence keys and sections with millisecond precision") });
		TickResolutionRates.Add(FCommonFrameRateInfo{ FFrameRate(24000, 1),  LOCTEXT("24000_Name",  "24000 fps (all integer rates + 23.976)"),        LOCTEXT("24000_Description",  "A very high framerate that allows frame-accurate evaluation of all common integer frame rates as well as NTSC 24.") });
		TickResolutionRates.Add(FCommonFrameRateInfo{ FFrameRate(60000, 1),  LOCTEXT("60000_Name",  "60000 fps (all integer rates + 29.97 & 59.94)"), LOCTEXT("60000_Description",  "A very high framerate that allows frame-accurate evaluation of all common integer frame rates as well as NTSC 30 and 60.") });
		TickResolutionRates.Add(FCommonFrameRateInfo{ FFrameRate(120000, 1), LOCTEXT("120000_Name", "120000 fps (all common rates)"),                 LOCTEXT("120000_Description", "A very high framerate that allows frame-accurate evaluation of all common integer and NTSC frame rates.") });
	}

	FText Description = LOCTEXT("Description", "Sequences stores section start times and keys at points in time called 'ticks'.\n\nThe size of a single tick is defined per-sequence; it is recommended that you choose a tick-interval that fits into your desired display rate or content frame rates. Increasing the resolution will reduce the total supported time range.");

	static float VerticalGridPadding = 15.f;
	static float HorizontalGridPadding = 10.f;

	static FMargin Col1Padding(0.f, 0.f, HorizontalGridPadding, VerticalGridPadding);
	static FMargin Col2Padding(HorizontalGridPadding, 0.f, 0.f, VerticalGridPadding);
	bIsRecursive = true;

	static FLinearColor WarningColor(FColor(0xffbbbb44));
	ChildSlot
	[
		SNew(SScrollBox)
		+ SScrollBox::Slot()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.Padding(16.f)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Text(Description)
				]

				+SVerticalBox::Slot()
				.FillHeight(1.0f)
				.Padding(0.0f, 10.0f)
				[
					SNew(SBorder)
					.Padding(FMargin(10.0f, 10.0f))
					.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
					[
						SNew(SGridPanel)
						.FillColumn(1, 1.f)

						+ SGridPanel::Slot(0, 0)
						.Padding(Col1Padding)
						.HAlign(HAlign_Right)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("NewTickInterval", "Desired Tick Interval"))
						]
						+ SGridPanel::Slot(1, 0)
						.HAlign(HAlign_Left)
						.Padding(Col2Padding)
						[
							SNew(SFrameRatePicker)
							.RecommendedText(LOCTEXT("CompatibleWithDisplayRate", "Compatible with this sequence"))
							.NotRecommendedText(LOCTEXT("NotCompatibleWithDisplayRate", "Other"))
							.NotRecommendedToolTip(LOCTEXT("NotCompatibleWithDisplayRate_Tip", "All other preset frame rates that are not compatible with the current display and tick rate"))
							.IsPresetRecommended(this, &SSequencerTimePanel::IsRecommendedResolution)
							.PresetValues(MoveTemp(TickResolutionRates))
							.Value(this, &SSequencerTimePanel::GetCurrentTickResolution)
							.OnValueChanged(this, &SSequencerTimePanel::OnSetTickResolution)
						]

						+ SGridPanel::Slot(0, 1)
						.HAlign(HAlign_Right)
						.Padding(FMargin(0, 0, HorizontalGridPadding, 0))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ApplyRecursively", "Apply Recursively"))
							.ToolTipText(LOCTEXT("ApplyRecursively_Tooltip", "If true then the tick rate change will propagate to all child sub-sequences of the current sequence. The UI warnings only apply to the top level sequence."))
						]
						+ SGridPanel::Slot(1, 1)
						.HAlign(HAlign_Left)
						.Padding(FMargin(HorizontalGridPadding, 0, 0, 0))
						[
							SNew(SCheckBox)
							.IsChecked(this, &SSequencerTimePanel::GetIsRecursive)
							.OnCheckStateChanged(this, &SSequencerTimePanel::OnSetIsRecursive)
						]

						+ SGridPanel::Slot(0, 2)
						.ColumnSpan(2)
						.Padding(FMargin(0.f, VerticalGridPadding, 0.f, VerticalGridPadding))
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Top)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("NewTickInterval_Tip", "Sequence will have the following properties if applied:"))
						]

						+ SGridPanel::Slot(0, 3)
						.Padding(Col1Padding)
						.HAlign(HAlign_Right)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ResultingRange", "Time Range"))
						]
						+ SGridPanel::Slot(1, 3)
						.Padding(Col2Padding)
						[
							SNew(STextBlock)
							.Text(this, &SSequencerTimePanel::GetSupportedTimeRange)
						]

						+ SGridPanel::Slot(0, 4)
						.Padding(Col1Padding)
						.HAlign(HAlign_Right)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("SupportedFrameRates", "Supported Rates"))
						]
						+ SGridPanel::Slot(1, 4)
						.Padding(Col2Padding)
						[
							SAssignNew(CommonFrameRates, SVerticalBox)
						]
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(5.f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)
					.Visibility(this, &SSequencerTimePanel::GetWarningVisibility)

					+ SHorizontalBox::Slot()
					.Padding(FMargin(0.f, 0.f, 7.f, 0.f))
					.AutoWidth()
					[
						SNew(STextBlock)
						.ColorAndOpacity(WarningColor)
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
						.Text(FText::FromString(FString(TEXT("\xf071"))) /*fa-exclamation-triangle*/)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.ColorAndOpacity(WarningColor)
						.Text(LOCTEXT("ApplyWarning", "Applying these settings may result in changes to key positions or section boundaries."))
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				.Padding(0.0f, 3.0f)
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Bottom)
					.Padding(8, 0)
					[
						SNew(SPrimaryButton)
						.Text(LOCTEXT("ApplyButtonText", "Apply"))
						.OnClicked(this, &SSequencerTimePanel::Apply)
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Bottom)
					[
						SNew(SButton)
						.Text(LOCTEXT("CancelButtonText", "Cancel"))
						.OnClicked(this, &SSequencerTimePanel::Close)
					]
				]
			]
		]
	];

	UpdateCommonFrameRates();
}

FReply SSequencerTimePanel::Close()
{
	CurrentTickResolution.Reset();
	TSharedRef<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow( AsShared() ).ToSharedRef();
	FSlateApplication::Get().RequestDestroyWindow( ParentWindow );
	return FReply::Handled();
}

FReply SSequencerTimePanel::Apply()
{
	UMovieSceneSequence* Sequence = GetFocusedSequence();
	UMovieScene* MovieScene       = Sequence ? Sequence->GetMovieScene() : nullptr;

	if (MovieScene)
	{
		FFrameRate Src = MovieScene->GetTickResolution();
		FFrameRate Dst = GetCurrentTickResolution();
		bool bRecursive = GetIsRecursive() == ECheckBoxState::Checked;

		FScopedTransaction ScopedTransaction(FText::Format(LOCTEXT("MigrateFrameTimes", "Convert sequence tick interval from {0} to {1}"), Src.ToPrettyText(), Dst.ToPrettyText()));

		// We quickly iterate through the sequence hierarchy to check for any readonly sequences. If we find a read-only sequence we're going to just warn the user
		// that we automatically unlocked, edited, and re-locked the sequence (as there should be no data change apparent to the user). We're doing this here to avoid
		// firing off the warning for every sequence, though we will print in the output log which ones were unlocked.
		{
			struct FSequenceReadOnlyVisitor : UE::MovieScene::ISequenceVisitor
			{
				virtual void VisitSubSequence(UMovieSceneSequence* InSequence, const FGuid&, const UE::MovieScene::FSubSequenceSpace& LocalSpace) 
				{
					if (InSequence->GetMovieScene()->IsReadOnly())
					{
						ReadOnlyMovieScenes.Add(InSequence->GetMovieScene());
					}
				}
				TArray<UMovieScene*> ReadOnlyMovieScenes;
			};

			UE::MovieScene::FSequenceVisitParams Params;
			Params.bVisitRootTracks = true;
			Params.bVisitSubSequences = true;
			FSequenceReadOnlyVisitor ReadOnlyVisitor;
			VisitSequence(Sequence, Params, ReadOnlyVisitor);

			for (UMovieScene* ReadOnlyMovieScene : ReadOnlyVisitor.ReadOnlyMovieScenes)
			{
				UE_LOGF(LogMovieScene, Log, "The following sequence was read-only but had its tick resolution adjusted automatically. No action is required. %ls", *ReadOnlyMovieScene->GetPathName());
			}

			if (ReadOnlyVisitor.ReadOnlyMovieScenes.Num() > 0)
			{
				FNotificationInfo Info(LOCTEXT("ReadOnlyScenesEdited", "Edited read-only sequence, see output log for details. No action is required."));
				Info.bFireAndForget = true;
				Info.FadeOutDuration = 0.5f;
				Info.ExpireDuration = 5.0f;

				TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
				NotificationPtr->SetCompletionState(SNotificationItem::CS_Success);
			}

		}

		UE::MovieScene::TimeHelpers::MigrateFrameTimes(Src, Dst, MovieScene, bRecursive);
	}

	return Close();
}

EVisibility SSequencerTimePanel::GetWarningVisibility() const
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	return Sequencer.IsValid() && Sequencer->GetFocusedTickResolution().IsMultipleOf(GetCurrentTickResolution()) ? EVisibility::Collapsed : EVisibility::Visible;
}

void SSequencerTimePanel::UpdateCommonFrameRates()
{
	TArray<FCommonFrameRateInfo> CompatibleRates;
	for (const FCommonFrameRateInfo& Info : FCommonFrameRates::GetAll())
	{
		if (Info.FrameRate.IsMultipleOf(GetCurrentTickResolution()))
		{
			CompatibleRates.Add(Info);
		}
	}

	CompatibleRates.Sort(
		[=](const FCommonFrameRateInfo& A, const FCommonFrameRateInfo& B)
		{
			return A.FrameRate.AsDecimal() < B.FrameRate.AsDecimal();
		}
	);

	CommonFrameRates->ClearChildren();
	for (const FCommonFrameRateInfo& FrameRate : CompatibleRates)
	{
		CommonFrameRates->AddSlot()
		[
			SNew(STextBlock)
			.Text(FrameRate.DisplayName)
		];
	}
}

FText SSequencerTimePanel::GetSupportedTimeRange() const
{
	double FrameRate = GetCurrentTickResolution().AsDecimal();

	int64 TotalMaxSeconds = static_cast<int64>(TNumericLimits<int32>::Max() / FrameRate);

	int64 Days    =  (TotalMaxSeconds        ) / 86400;
	int64 Hours   =  (TotalMaxSeconds % 86400) / 3600;
	int64 Minutes =  (TotalMaxSeconds % 3600 ) / 60;
	int64 Seconds =  (TotalMaxSeconds % 60   );

	FString String = Days > 0
		? FString::Printf(TEXT("+/- %02" INT64_FMT "d %02" INT64_FMT "h %02" INT64_FMT "m %02" INT64_FMT "s"), Days, Hours, Minutes, Seconds)
		: FString::Printf(TEXT("+/- %02" INT64_FMT "h %02" INT64_FMT "m %02" INT64_FMT "s"), Hours, Minutes, Seconds);

	return FText::FromString(String);
}

UMovieSceneSequence* SSequencerTimePanel::GetFocusedSequence() const
{
	TSharedPtr<FSequencer> Sequencer  = WeakSequencer.Pin();
	return Sequencer.IsValid() ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
}

bool SSequencerTimePanel::IsRecommendedResolution(FFrameRate InFrameRate) const
{
	UMovieSceneSequence* Sequence = GetFocusedSequence();
	return !Sequence || (InFrameRate.IsFactorOf(Sequence->GetMovieScene()->GetDisplayRate()) && InFrameRate.IsFactorOf(Sequence->GetMovieScene()->GetTickResolution()));
}

FFrameRate SSequencerTimePanel::GetCurrentTickResolution() const
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	return CurrentTickResolution.Get(Sequencer.IsValid() ? Sequencer->GetFocusedTickResolution() : FFrameRate(24000, 1));
}

void SSequencerTimePanel::OnSetTickResolution(FFrameRate InTickResolution)
{
	CurrentTickResolution = InTickResolution;
	UpdateCommonFrameRates();
}


ECheckBoxState SSequencerTimePanel::GetIsRecursive() const
{
	return bIsRecursive ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SSequencerTimePanel::OnSetIsRecursive(ECheckBoxState InCheckBoxState)
{
	bIsRecursive = InCheckBoxState == ECheckBoxState::Checked;
}


#undef LOCTEXT_NAMESPACE