// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "MovieSceneFwd.h"
#include "FrameNumberDisplayFormat.h"
#include "Misc/FrameRate.h"

class FSequencer;
class UMovieScene;
class UMovieSceneTrack;
class UMovieSceneSection;
class UMovieSceneSequence;

class SSequencerTimePanel : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSequencerTimePanel){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FSequencer> Sequencer);

private:

	FReply Close();
	FReply Apply();

	EVisibility GetWarningVisibility() const;

	FFrameRate GetCurrentTickResolution() const;
	bool IsRecommendedResolution(FFrameRate InFrameRate) const;

	FText GetSupportedTimeRange() const;

	void OnSetTickResolution(FFrameRate InTickResolution);

	void UpdateCommonFrameRates();
	
	ECheckBoxState GetIsRecursive() const;
	void OnSetIsRecursive(ECheckBoxState InCheckBoxState);


	UMovieSceneSequence* GetFocusedSequence() const;

	static void MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate, UMovieScene* MovieScene);
	static void MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate, UMovieSceneTrack* Track);
	static void MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate, UMovieSceneSection* Section);

	TOptional<FFrameRate> CurrentTickResolution;
	TSharedPtr<SVerticalBox> CommonFrameRates;
	TWeakPtr<FSequencer> WeakSequencer;
	bool bIsRecursive;
};