// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerTransformBox.h"

#include "Misc/FrameRate.h"
#include "Sequencer.h"
#include "SequencerSettings.h"
#include "SequencerCommonHelpers.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "Sequencer"

void SSequencerTransformBox::Construct(const FArguments& InArgs, const TSharedRef<FSequencer>& InSequencer, USequencerSettings& InSettings, const TSharedRef<INumericTypeInterface<double>>& InNumericTypeInterface)
{
	SequencerPtr = InSequencer;
	Settings = &InSettings;
	NumericTypeInterface = InNumericTypeInterface;
	
	ScaleFactor = 1.f;

	const FDockTabStyle* GenericTabStyle = &FCoreStyle::Get().GetWidgetStyle<FDockTabStyle>("Docking.Tab");
	const FButtonStyle* const CloseButtonStyle = &GenericTabStyle->CloseButtonStyle;

	ChildSlot
	[
		SAssignNew(Border, SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
			.Padding(6.0f)
			.Visibility(EVisibility::Collapsed)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
							.Text(LOCTEXT("PlusLabel", "+"))
							.OnClicked(this, &SSequencerTransformBox::OnPlusButtonClicked)
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
							.Text(LOCTEXT("MinusLabel", "-"))
							.OnClicked(this, &SSequencerTransformBox::OnMinusButtonClicked)
					]

				+ SHorizontalBox::Slot()
					.Padding(6.0f, 0.0f, 0.0f, 0.0f)
					.AutoWidth()
					[
						SAssignNew(OffsetEntryBox, SNumericEntryBox<double>)
						.MinDesiredValueWidth(32.0f)
						.TypeInterface(NumericTypeInterface)
						.ToolTipText(LOCTEXT("TransformDelta_Tooltip", "The amount to offset the selected keys/sections by"))
						.OnValueCommitted(this, &SSequencerTransformBox::OnDeltaCommitted)
						.OnValueChanged(this, &SSequencerTransformBox::OnDeltaChanged)
						.Value_Lambda([this](){ return DeltaTime.IsSet() ? DeltaTime.GetValue().Value : 0.0; })
					]

				+ SHorizontalBox::Slot()
					.Padding(6.0f, 0.0f, 0.0f, 0.0f)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
							.Text(LOCTEXT("MultiplyLabel", "*"))
							.OnClicked(this, &SSequencerTransformBox::OnMultiplyButtonClicked)
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
							.Text(LOCTEXT("DivideLabel", "/"))
							.OnClicked(this, &SSequencerTransformBox::OnDivideButtonClicked)
					]

				+ SHorizontalBox::Slot()
					.Padding(6.0f, 0.0f, 0.0f, 0.0f)
					.AutoWidth()
					[
						 SAssignNew(ScaleEntryBox, SNumericEntryBox<float>)
						 	.MinDesiredValueWidth(32.0f)
						 	.ToolTipText(LOCTEXT("TransformScale_Tooltip", "The amount to scale the selected keys/section by (about the local time)"))
						 	.OnValueCommitted(this, &SSequencerTransformBox::OnScaleCommitted)
							.OnValueChanged(this, &SSequencerTransformBox::OnScaleChanged)
							.Value_Lambda([this](){ return ScaleFactor; })
					]

				+ SHorizontalBox::Slot()
				.Padding(3.0f, 0.0f, 0.0f, 0.0f)
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle( CloseButtonStyle )
					.OnClicked( this, &SSequencerTransformBox::OnCloseButtonClicked )
					.ContentPadding( 0 )
					[
						SNew(SSpacer)
						.Size( CloseButtonStyle->Normal.ImageSize )
					]
				]
			]
	];
}


void SSequencerTransformBox::ToggleVisibility()
{
	FSlateApplication& SlateApplication = FSlateApplication::Get();

	if (Border->GetVisibility() == EVisibility::Visible)
	{
		if (LastFocusedWidget.IsValid())
		{
			SlateApplication.SetAllUserFocus(LastFocusedWidget.Pin(), EFocusCause::Navigation);
		}
		
		Border->SetVisibility(EVisibility::Collapsed);
	}
	else
	{
		if (!DeltaTime.IsSet())
		{
			InitializeDeltaTime();
		}

		Border->SetVisibility(EVisibility::Visible);
		LastFocusedWidget = SlateApplication.GetUserFocusedWidget(0);
		SlateApplication.SetAllUserFocus(OffsetEntryBox, EFocusCause::Navigation);
	}
}

void SSequencerTransformBox::OnDeltaCommitted(double Value, ETextCommit::Type CommitType)
{
	OnDeltaChanged(Value);
}

void SSequencerTransformBox::OnDeltaChanged(double Value)
{
	DeltaTime = FFrameTime::FromDecimal(Value).GetFrame();
}

void SSequencerTransformBox::OnScaleCommitted(float Value, ETextCommit::Type CommitType)
{
	OnScaleChanged(Value);
}

void SSequencerTransformBox::OnScaleChanged(float Value)
{
	ScaleFactor = Value;
}

FReply SSequencerTransformBox::OnPlusButtonClicked()
{
	if (!DeltaTime.IsSet())
	{
		InitializeDeltaTime();
	}

	if (DeltaTime.GetValue() != 0)
	{
		TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();

		Sequencer->TransformSelection(DeltaTime.GetValue(), 1.f);
	}

	return FReply::Handled();
}

FReply SSequencerTransformBox::OnMinusButtonClicked()
{
	if (!DeltaTime.IsSet())
	{
		InitializeDeltaTime();
	}

	if (DeltaTime.GetValue() != 0)
	{
		TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
	
		Sequencer->TransformSelection(-DeltaTime.GetValue(), 1.f);
	}

	return FReply::Handled();
}

FReply SSequencerTransformBox::OnMultiplyButtonClicked()
{
	if (ScaleFactor != 0.f && ScaleFactor != 1.f)
	{
		TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();
		
		Sequencer->TransformSelection(0, ScaleFactor);
	}

	return FReply::Handled();
}

FReply SSequencerTransformBox::OnDivideButtonClicked()
{
	if (ScaleFactor != 0.f && ScaleFactor != 1.f)
	{
		TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();

		Sequencer->TransformSelection(0, 1.f/ScaleFactor);
	}
	return FReply::Handled();
}

FReply SSequencerTransformBox::OnCloseButtonClicked()
{
	ToggleVisibility();

	return FReply::Handled();
}

void SSequencerTransformBox::InitializeDeltaTime()
{
	TSharedPtr<FSequencer> Sequencer = SequencerPtr.Pin();

	// Default to 1 frame
	const FFrameRate DisplayRate = Sequencer->GetFocusedDisplayRate();
	const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
	DeltaTime = FFrameRate::TransformTime(FFrameTime(1), DisplayRate, TickResolution).FrameNumber;
}

#undef LOCTEXT_NAMESPACE
