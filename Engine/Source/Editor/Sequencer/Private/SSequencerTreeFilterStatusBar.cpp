// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerTreeFilterStatusBar.h"
#include "Sequencer.h"

#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"

#include "Algo/Count.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SSequencerTreeFilterStatusBar"

void SSequencerTreeFilterStatusBar::Construct(const FArguments& InArgs, TSharedPtr<FSequencer> InSequencer)
{
	WeakSequencer = InSequencer;

	ChildSlot
	.Padding(FMargin(5.f, 0.f))
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(3.f, 0.f, 0.f, 0.f))
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(FMargin(3.0f, 3.0f))
			[
				SNew(SHyperlink)
				.Visibility(this, &SSequencerTreeFilterStatusBar::GetVisibilityFromFilter)
				.Text(LOCTEXT("ClearFilters", "clear"))
				.OnNavigate(this, &SSequencerTreeFilterStatusBar::ClearFilters)
			]
		]
	
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(FMargin(3.0f, 3.0f))
			.Visibility(EVisibility::HitTestInvisible)
			[
				SAssignNew(TextBlock, STextBlock)
			]
		]
	];
}

void SSequencerTreeFilterStatusBar::ClearFilters()
{
	if (const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin())
	{
		Sequencer->ClearFilters();
	}
}

EVisibility SSequencerTreeFilterStatusBar::GetVisibilityFromFilter() const
{
	if (const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin())
	{
		return Sequencer->GetFilterInterface()->HasAnyFilterActive() ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return EVisibility::Collapsed;
}

void SSequencerTreeFilterStatusBar::UpdateText()
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	FText NewText;
	FLinearColor NewColor = FLinearColor::White;

	const TSharedRef<FSequencerNodeTree> NodeTree = Sequencer->GetNodeTree();
	const FOutlinerSelection& SelectedOutlinerItems = Sequencer->GetViewModel()->GetSelection()->Outliner;

	FFormatNamedArguments NamedArgs;
	NamedArgs.Add("Total", NodeTree->GetTotalDisplayNodeCount());

	const bool bHasSelection = SelectedOutlinerItems.Num() != 0;
	const bool bHasFilter = Sequencer->GetFilterInterface()->HasAnyFilterActive();
	const int32 NumFiltered = NodeTree->GetFilteredDisplayNodeCount();

	if (bHasSelection)
	{
		NamedArgs.Add("NumSelected", SelectedOutlinerItems.Num());
	}

	if (bHasFilter)
	{
		NamedArgs.Add("NumMatched", NumFiltered);
	}

	if (bHasFilter)
	{
		if (NumFiltered == 0)
		{
			// Red = no matched
			NewColor = FLinearColor( 1.0f, 0.4f, 0.4f );
		}
		else
		{
			// Green = matched filter
			NewColor = FLinearColor( 0.4f, 1.0f, 0.4f );
		}

		if (bHasSelection)
		{
			NewText = FText::Format(LOCTEXT("FilteredStatus_WithSelection", "Showing {NumMatched} of {Total} items ({NumSelected} selected)"), NamedArgs);
		}
		else 
		{
			NewText = FText::Format(LOCTEXT("FilteredStatus_NoSelection", "Showing {NumMatched} of {Total} items"), NamedArgs);
		}
	}
	else if (bHasSelection)
	{
		NewText = FText::Format(LOCTEXT("UnfilteredStatus_WithSelection", "{Total} items ({NumSelected} selected)"), NamedArgs);
	}
	else
	{
		NewText = FText::Format(LOCTEXT("UnfilteredStatus_NoSelection", "{Total} items"), NamedArgs);
	}

	TextBlock->SetColorAndOpacity(NewColor);
	TextBlock->SetText(NewText);
}

static double SequencerOpacityDurationSeconds = 2.0;

void SSequencerTreeFilterStatusBar::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (GetVisibility() == EVisibility::Visible && OpacityThrobEndTime != FLT_MAX)
	{
		if (IsHovered())
		{
			// Make sure the status bar stays visible and keep it from fading until OnUnhovered
			ShowStatusBar();
			FadeOutStatusBar();
		}
		else
		{
			double CurrentTime = FPlatformTime::Seconds();

			float Opacity = 0.f;
			if (OpacityThrobEndTime > CurrentTime)
			{
				double Difference = OpacityThrobEndTime - CurrentTime;
				Opacity = Difference / SequencerOpacityDurationSeconds;
			}

			if (Opacity > 0.f)
			{
				FLinearColor BorderColor = GetColorAndOpacity();
				BorderColor.A = Opacity;
				SetColorAndOpacity(BorderColor);
			}
			else
			{
				SetVisibility(EVisibility::Hidden);
			}
		}
	}

	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SSequencerTreeFilterStatusBar::ShowStatusBar()
{
	OpacityThrobEndTime = FLT_MAX;
	FLinearColor BorderColor = GetColorAndOpacity();
	BorderColor.A = 1.f;
	SetColorAndOpacity(BorderColor);
	SetVisibility(EVisibility::Visible);
}

void SSequencerTreeFilterStatusBar::HideStatusBar()
{
	SetVisibility(EVisibility::Hidden);
}

void SSequencerTreeFilterStatusBar::FadeOutStatusBar()
{
	// Show and then start fading out the status bar
	OpacityThrobEndTime = FPlatformTime::Seconds() + SequencerOpacityDurationSeconds;
}

#undef LOCTEXT_NAMESPACE
