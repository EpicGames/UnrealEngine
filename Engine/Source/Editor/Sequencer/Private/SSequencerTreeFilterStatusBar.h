// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class STextBlock;
class FSequencer;

/** Widget that shows the metrics for the current tree filter in Sequencer (in the form "Showing {0} of {1} items ({2} selected)") */
class SSequencerTreeFilterStatusBar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSequencerTreeFilterStatusBar){}
	SLATE_END_ARGS()

	/** Construct the status bar */
	void Construct(const FArguments& InArgs, TSharedPtr<FSequencer> InSequencer);

	/** Update the filter text to represent the current filter states in the tree */
	void UpdateText();

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void ShowStatusBar();
	void HideStatusBar();
	void FadeOutStatusBar();

private:

	/** Request the visibility of the clear hyperlink widget based on whether there's a filter active or not */
	EVisibility GetVisibilityFromFilter() const;

	/** Clear all the filters in the tree - called as a result of the user clicking on the 'clear' hyperlink */
	void ClearFilters();

private:

	TWeakPtr<FSequencer> WeakSequencer;
	TSharedPtr<STextBlock> TextBlock;
	double OpacityThrobEndTime = 0;
};