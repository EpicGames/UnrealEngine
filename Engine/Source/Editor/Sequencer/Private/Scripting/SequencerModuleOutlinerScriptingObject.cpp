// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scripting/SequencerModuleOutlinerScriptingObject.h"
#include "Scripting/ViewModelScriptingStruct.h"

#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/SequencerOutlinerViewModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModelPtr.h"

#include "Containers/Array.h"
#include "Misc/FrameNumber.h"
#include "MovieScene.h"
#include "ObjectBindingTagCache.h"

#include "Sequencer.h"
#include "SequencerCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequencerModuleOutlinerScriptingObject)

#define LOCTEXT_NAMESPACE "SequencerModuleOutlinerScriptingObject"

TArray<UMovieSceneSection*> USequencerModuleOutlinerScriptingObject::GetSections(const TArray<FSequencerViewModelScriptingStruct>& InNodes) const
{
	using namespace UE::Sequencer;

	TViewModelPtr<FSequencerOutlinerViewModel> Outliner = CastViewModel<FSequencerOutlinerViewModel>(WeakOutliner.Pin());
	if (!Outliner)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("OutlinerInvalid", "Outliner is no longer valid.").ToString(), ELogVerbosity::Error);
		return TArray<UMovieSceneSection*>();
	}

	TSet<TWeakObjectPtr<UMovieSceneSection> > AllSections;
	TArray<TSharedRef<FViewModel>> Nodes;
	for (const FSequencerViewModelScriptingStruct& InNode : InNodes)
	{
		FViewModelPtr Item = InNode.WeakViewModel.ImplicitPin();
		if (Item)
		{
			SequencerHelpers::GetAllSections(Item, AllSections);
		}
	}

	TArray<UMovieSceneSection*> Sections;
	for (TWeakObjectPtr<UMovieSceneSection> WeakSection : AllSections)
	{
		if (WeakSection.IsValid())
		{
			Sections.Add(WeakSection.Get());
		}
	}
	return Sections;
}

FFrameNumber USequencerModuleOutlinerScriptingObject::GetNextKey(const TArray<FSequencerViewModelScriptingStruct>& InNodes, FFrameNumber FrameNumber, EMovieSceneTimeUnit TimeUnit, const bool bAllowLoop) const
{
	using namespace UE::Sequencer;

	TViewModelPtr<FSequencerOutlinerViewModel> Outliner = CastViewModel<FSequencerOutlinerViewModel>(WeakOutliner.Pin());
	if (!Outliner)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("OutlinerInvalid", "Outliner is no longer valid.").ToString(), ELogVerbosity::Error);
		return FFrameNumber();
	}

	TRange<FFrameNumber> Range = TRange<FFrameNumber>::All();

	TArray<TSharedRef<FViewModel>> Nodes;
	for (const FSequencerViewModelScriptingStruct& InNode : InNodes)
	{
		FViewModelPtr Item = InNode.WeakViewModel.ImplicitPin();
		if (Item)
		{
			Nodes.Add(Item.ToSharedRef());
		}
	}

	return Outliner->GetNextKey(Nodes, FrameNumber, TimeUnit, Range, bAllowLoop);
}

FFrameNumber USequencerModuleOutlinerScriptingObject::GetPreviousKey(const TArray<FSequencerViewModelScriptingStruct>& InNodes, FFrameNumber FrameNumber, EMovieSceneTimeUnit TimeUnit, const bool bAllowLoop) const
{
	using namespace UE::Sequencer;

	TViewModelPtr<FSequencerOutlinerViewModel> Outliner = CastViewModel<FSequencerOutlinerViewModel>(WeakOutliner.Pin());
	if (!Outliner)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("OutlinerInvalid", "Outliner is no longer valid.").ToString(), ELogVerbosity::Error);
		return FFrameNumber();
	}

	TRange<FFrameNumber> Range = TRange<FFrameNumber>::All();

	TArray<TSharedRef<FViewModel>> Nodes;
	for (const FSequencerViewModelScriptingStruct& InNode : InNodes)
	{
		TViewModelPtr<FViewModel> Item = InNode.WeakViewModel.ImplicitPin();
		if (Item)
		{
			Nodes.Add(Item.ToSharedRef());
		}
	}

	return Outliner->GetPreviousKey(Nodes, FrameNumber, TimeUnit, Range, bAllowLoop);
}

void USequencerModuleOutlinerScriptingObject::AddBindingTags(const TArray<FSequencerViewModelScriptingStruct>& InNodes, const TArray<FName>& TagNames)
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("SequencerInvalid", "Sequencer is no longer valid.").ToString(), ELogVerbosity::Error);
		return;
	}

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;

	if (!MovieScene)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("MovieSceneInvalid", "Movie Scene is not an object binding.").ToString(), ELogVerbosity::Error);
		return;
	}

	for (const FSequencerViewModelScriptingStruct& InNode : InNodes)
	{
		TViewModelPtr<FObjectBindingModel> ObjectBindingModel = CastViewModel<FObjectBindingModel>(InNode.WeakViewModel.Pin());
		if (!ObjectBindingModel)
		{
			FFrame::KismetExecutionMessage(*LOCTEXT("NodeInvalid", "Node is not an object binding.").ToString(), ELogVerbosity::Error);
			return;
		}

		const FGuid& ObjectID = ObjectBindingModel->GetObjectGuid();
		FMovieSceneSequenceIDRef SequenceID = Sequencer->GetFocusedTemplateID();

		for (const FName& TagName : TagNames)
		{
			UE::MovieScene::FFixedObjectBindingID BindingID(ObjectID, SequenceID);
			MovieScene->TagBinding(TagName, BindingID);
		}
	}
}

void USequencerModuleOutlinerScriptingObject::RemoveBindingTags(const TArray<FSequencerViewModelScriptingStruct>& InNodes, const TArray<FName>& TagNames)
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("SequencerInvalid", "Sequencer is no longer valid.").ToString(), ELogVerbosity::Error);
		return;
	}

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;

	if (!MovieScene)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("MovieSceneInvalid", "Movie Scene is not an object binding.").ToString(), ELogVerbosity::Error);
		return;
	}	

	for (const FSequencerViewModelScriptingStruct& InNode : InNodes)
	{
		TViewModelPtr<FObjectBindingModel> ObjectBindingModel = CastViewModel<FObjectBindingModel>(InNode.WeakViewModel.Pin());
		if (!ObjectBindingModel)
		{
			FFrame::KismetExecutionMessage(*LOCTEXT("NodeInvalid", "Node is not an object binding.").ToString(), ELogVerbosity::Error);
			return;
		}

		const FGuid& ObjectID = ObjectBindingModel->GetObjectGuid();
		FMovieSceneSequenceIDRef SequenceID = Sequencer->GetFocusedTemplateID();

		for (const FName& TagName : TagNames)
		{
			UE::MovieScene::FFixedObjectBindingID BindingID(ObjectID, SequenceID);
			MovieScene->UntagBinding(TagName, BindingID);
		}
	}
}

TArray<FName> USequencerModuleOutlinerScriptingObject::GetBindingTags(FSequencerViewModelScriptingStruct InNode)
{
	using namespace UE::Sequencer;

	TArray<FName> Tags;

	TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("SequencerInvalid", "Sequencer is no longer valid.").ToString(), ELogVerbosity::Error);
		return Tags;
	}

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;

	if (!MovieScene)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("MovieSceneInvalid", "Movie Scene is not an object binding.").ToString(), ELogVerbosity::Error);
		return Tags;
	}

	TViewModelPtr<FObjectBindingModel> ObjectBindingModel = CastViewModel<FObjectBindingModel>(InNode.WeakViewModel.Pin());
	if (!ObjectBindingModel)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("NodeInvalid", "Node is not an object binding.").ToString(), ELogVerbosity::Error);
		return Tags;
	}

	const FGuid& ObjectID = ObjectBindingModel->GetObjectGuid();
	FMovieSceneSequenceIDRef SequenceID = Sequencer->GetFocusedTemplateID();

	UE::MovieScene::FFixedObjectBindingID BindingID(ObjectID, SequenceID);
	for (auto It = Sequencer->GetObjectBindingTagCache()->IterateTags(BindingID); It; ++It)
	{
		Tags.Add(It.Value());
	}

	return Tags;
}

TSharedPtr<FSequencer> USequencerModuleOutlinerScriptingObject::GetSequencer()
{
	using namespace UE::Sequencer;

	TViewModelPtr<FSequencerOutlinerViewModel> Outliner = CastViewModel<FSequencerOutlinerViewModel>(WeakOutliner.Pin());
	if (!Outliner)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("SequencerOutlinerInvalid", "Sequencer Outliner is no longer valid.").ToString(), ELogVerbosity::Error);
		return nullptr;
	}

	TViewModelPtr<FSequencerEditorViewModel> Editor = CastViewModel<FSequencerEditorViewModel>(Outliner->GetEditor());
	if (!Editor)
	{
		FFrame::KismetExecutionMessage(*LOCTEXT("SequencerEditorInvalid", "Sequencer Editor is no longer valid.").ToString(), ELogVerbosity::Error);
		return nullptr;
	}

	TSharedPtr<FSequencer> Sequencer = Editor->GetSequencerImpl();
	return Sequencer;
}

#undef LOCTEXT_NAMESPACE
