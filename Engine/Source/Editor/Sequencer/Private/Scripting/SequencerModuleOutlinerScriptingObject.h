// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "MovieSceneTimeUnit.h"
#include "Scripting/OutlinerScriptingObject.h"

#include "SequencerModuleOutlinerScriptingObject.generated.h"

struct FFrameNumber;
struct FSequencerViewModelScriptingStruct;

class FSequencer;
class UMovieSceneSection;

UCLASS()
class USequencerModuleOutlinerScriptingObject : public USequencerOutlinerScriptingObject
{
public:

	GENERATED_BODY()

	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	TArray<UMovieSceneSection*> GetSections(const TArray<FSequencerViewModelScriptingStruct>& InNodes) const;

	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	FFrameNumber GetNextKey(const TArray<FSequencerViewModelScriptingStruct>& InNodes, FFrameNumber FrameNumber, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate, bool bAllowLoop = true) const;

	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	FFrameNumber GetPreviousKey(const TArray<FSequencerViewModelScriptingStruct>& InNodes, FFrameNumber FrameNumber, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate, bool bAllowLoop = true) const;

	// Tags
	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	void AddBindingTags(const TArray<FSequencerViewModelScriptingStruct>& InNodes, const TArray<FName>& TagNames);

	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	void RemoveBindingTags(const TArray<FSequencerViewModelScriptingStruct>& InNodes, const TArray<FName>& TagNames);

	UFUNCTION(BlueprintCallable, Category = "Sequencer Editor")
	TArray<FName> GetBindingTags(FSequencerViewModelScriptingStruct InNode);

private:

	TSharedPtr<FSequencer> GetSequencer();
};