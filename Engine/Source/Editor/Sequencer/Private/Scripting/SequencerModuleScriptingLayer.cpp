// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scripting/SequencerModuleScriptingLayer.h"
#include "Scripting/SequencerModuleOutlinerScriptingObject.h"

#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/EditorViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequencerModuleScriptingLayer)

void USequencerModuleScriptingLayer::Initialize(TSharedPtr<UE::Sequencer::FEditorViewModel> InViewModel)
{
	Outliner = NewObject<USequencerModuleOutlinerScriptingObject>(this, "Outliner");
	Outliner->Initialize(InViewModel->GetOutliner());
}

USequencerModuleOutlinerScriptingObject* USequencerModuleScriptingLayer::GetOutliner()
{
	return Cast<USequencerModuleOutlinerScriptingObject>(Outliner);
}
