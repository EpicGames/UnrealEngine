// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "Engine/World.h"
#include "Algo/RemoveIf.h"
#include "Async/ParallelFor.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/Extensions/IClockExtension.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/FolderModel.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/ViewModels/DecorationOutlinerModel.h"
#include "MVVM/ViewModels/SectionOutlinerModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/SequencerOutlinerViewModel.h"
#include "MVVM/ViewModels/SequencerTrackAreaViewModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/PossessableModel.h"
#include "MVVM/Views/SOutlinerView.h"
#include "MVVM/Extensions/IDeletableExtension.h"
#include "MVVM/Extensions/IClockExtension.h"
#include "MVVM/CurveEditorExtension.h"
#include "MVVM/CurveEditorIntegrationExtension.h"
#include "MVVM/ObjectBindingModelStorageExtension.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/TrackModelStorageExtension.h"
#include "MVVM/FolderModelStorageExtension.h"
#include "Camera/PlayerCameraManager.h"
#include "MovieSceneTransformTypes.h"
#include "Misc/MessageDialog.h"
#include "Misc/FeedbackContext.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/TransactionObjectEvent.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"
#include "UObject/MetaData.h"
#include "UObject/PropertyPortFlags.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "GameFramework/PlayerController.h"
#include "Engine/Engine.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Editor.h"
#include "BlueprintActionDatabase.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneTimeWarpChannel.h"
#include "MovieScenePossessable.h"
#include "MovieScene.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Widgets/Layout/SBorder.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Styling/AppStyle.h"
#include "Editor/UnrealEdEngine.h"
#include "Camera/CameraActor.h"
#include "Engine/Selection.h"
#include "EngineUtils.h"
#include "LevelEditorViewport.h"
#include "EditorModeManager.h"
#include "UnrealEdMisc.h"
#include "FileHelpers.h"
#include "UnrealEdGlobals.h"
#include "SequencerCommands.h"
#include "ISequencerSection.h"
#include "MovieSceneClipboard.h"
#include "SequencerCommonHelpers.h"
#include "SequencerMarkedFrameHelper.h"
#include "ISequencerNumericTypeInterface.h"
#include "SSequencer.h"
#include "SSequencerSection.h"
#include "SequencerKeyCollection.h"
#include "SequencerAddKeyOperation.h"
#include "SequencerPropertyKeyedStatus.h"
#include "SequencerSettings.h"
#include "SequencerLog.h"
#include "ISequencerObjectSchema.h"
#include "SequencerEdMode.h"
#include "MovieSceneBindingProxy.h"
#include "MovieSceneSequence.h"
#include "MovieSceneFolder.h"
#include "MovieSceneTracksSettings.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "EditorWidgetsModule.h"
#include "IAssetViewport.h"
#include "EditorSupportDelegates.h"
#include "MVVM/Views/SOutlinerView.h"
#include "ScopedTransaction.h"
#include "ISequencerTrackEditor.h"
#include "MovieSceneToolHelpers.h"
#include "Decorations/IMovieSceneSectionProviderDecoration.h"
#include "Decorations/MovieSceneDecorationContainer.h"
#include "Evaluation/ViewportSettingsPlaybackCapability.h"
#include "Sections/MovieSceneBoolSection.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "TimeSliderArgs.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "MovieSceneObjectBindingIDCustomization.h"
#include "MovieSceneObjectBindingID.h"
#include "ISettingsModule.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "SequencerHotspots.h"
#include "MovieSceneCaptureDialogModule.h"
#include "AutomatedLevelSequenceCapture.h"
#include "MovieSceneCommonHelpers.h"
#include "SceneOutlinerModule.h"
#include "SceneOutlinerPublicTypes.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "PackageTools.h"
#include "SequencerUtilities.h"
#include "CineCameraActor.h"
#include "CameraRig_Rail.h"
#include "CameraRig_Crane.h"
#include "DesktopPlatformModule.h"
#include "Factories.h"
#include "ObjectBindingTagCache.h"
#include "ISequencerEditorObjectBinding.h"
#include "LevelSequence.h"
#include "LevelSequenceActor.h"
#include "IVREditorModule.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ISequencerChannelInterface.h"
#include "IMovieRendererInterface.h"
#include "MVVM/ViewModels/OutlinerColumns/IOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerIndicators/IOutlinerIndicatorBuilder.h"
#include "SequencerKeyCollection.h"
#include "CurveEditor.h"
#include "CurveEditorScreenSpace.h"
#include "CurveDataAbstraction.h"
#include "Fonts/FontMeasure.h"
#include "MovieSceneTimeHelpers.h"
#include "FrameNumberNumericInterface.h"
#include "FrameNumberDetailsCustomization.h"
#include "UObject/StrongObjectPtr.h"
#include "LevelUtils.h"
#include "Engine/Blueprint.h"
#include "MovieSceneSequenceEditor.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "ISerializedRecorder.h"
#include "Features/IModularFeatures.h"
#include "SequencerContextMenus.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "EngineAnalytics.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Components/SkeletalMeshComponent.h"
#include "EntitySystem/MovieSceneInitialValueCache.h"
#include "SSequencerGroupManager.h"
#include "ActorTreeItem.h"
#include "Widgets/Layout/SSpacer.h"
#include "CurveEditorCommands.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Systems/MovieSceneMotionVectorSimulationSystem.h"
#include "IKeyArea.h"
#include "Editor/TransBuffer.h"
#include "Filters/SequencerFilterBar.h"
#include "Sidebar/SidebarDrawerConfig.h"
#include "Widgets/Sidebar/SequencerSelectionDrawer.h"
#include "Misc/Thumbnail/ThumbnailCaptureUtils.h"
#include "ToolMenus.h"
#include "SequencerToolMenuContext.h"
#include "Tools/SequencerSelectionAlignmentUtils.h"

#include "EngineModule.h"
#include "IViewportSelectableObject.h"
#include "Tracks/MovieSceneBindingLifetimeTrack.h"
#include "Sections/MovieSceneBindingLifetimeSection.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "Bindings/MovieSceneSpawnableActorBinding.h"
#include "Bindings/MovieSceneReplaceableActorBinding.h"
#include "CurveEditor/SequencerCurveEditorApp.h"
#include "Decorations/MovieSceneMuteSoloDecoration.h"
#include "Filters/SequencerTrackFilterCommands.h"
#include "Filters/Linking/Mode/LinkedFilterViewModel.h"
#include "MVVM/ViewModels/OutlinerFilterAreaManager.h"
#include "MVVM/ViewModels/SequencerEditorViewModelImpl.h"
#include "ISequencerDecorationEditor.h"
#include "TrackEditors/TimeWarpTrackEditor.h"
#include "Variants/MovieSceneTimeWarpGetter.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Widgets/SSequencerWrapper.h"
#include "Misc/KeyHelperUtils.h"
#include "SimpleView/SimpleViewTimeline.h"

#define LOCTEXT_NAMESPACE "Sequencer"

DEFINE_LOG_CATEGORY(LogSequencer);

static TAutoConsoleVariable<bool> CVarAutoScrub(
	TEXT("Sequencer.AutoScrub"),
	false,
	TEXT("Enable/disable auto-scrubbing"));

static TAutoConsoleVariable<float> CVarAutoScrubSpeed(
	TEXT("Sequencer.AutoScrubSpeed"),
	6.0f,
	TEXT("How fast to scrub forward/backward when auto-scrubbing"));

static TAutoConsoleVariable<float> CVarAutoScrubCurveExponent(
	TEXT("Sequencer.AutoScrubCurveExponent"),
	2.0f,
	TEXT("How much to ramp in and out the scrub speed when auto-scrubbing"));

static TAutoConsoleVariable<bool> CVarTimeUndo(
	TEXT("Sequencer.TimeUndo"),
	false,
	TEXT("Enable/disable ability to undo time when making changes"));

static TAutoConsoleVariable<bool> CVarRestorePreAnimatedStateOnBlueprintPreCompile(
	TEXT("Sequencer.RestorePreAnimatedStateOnBlueprintPreCompile"),
	false,
	TEXT("Enable/disable restoring pre-animated state before Blueprint classes get (re)compiled"));

namespace UE
{
	namespace Sequencer
	{

	static bool bEvaluateOnUndo = true;
	static FAutoConsoleVariableRef CVarEvaluateOnUndo(
		TEXT("Sequencer.EvaluateOnUndo"),
		bEvaluateOnUndo,
		TEXT("Enable/disable Sequencer evaluating on undo/redo"));

		struct FDeferredSignedObjectChangeHandler : UE::MovieScene::IDeferredSignedObjectChangeHandler
		{
			FDeferredSignedObjectChangeHandler()
			{
				Init();
			}

			~FDeferredSignedObjectChangeHandler()
			{
				if (UTransBuffer* TransBuffer = WeakBuffer.Get())
				{
					TransBuffer->OnTransactionStateChanged().RemoveAll(this);
				}
			}

			void Init()
			{
				UTransBuffer* TransBuffer = GUnrealEd ? Cast<UTransBuffer>(GUnrealEd->Trans) : nullptr;
				if (TransBuffer)
				{
					WeakBuffer = TransBuffer;
					TransBuffer->OnTransactionStateChanged().AddRaw(this, &FDeferredSignedObjectChangeHandler::OnTransactionStateChanged);
					if (TransBuffer->IsActive())
					{
						DeferTransactionChanges.Emplace();
					}
				}
				else
				{
					FCoreDelegates::GetOnPostEngineInit().AddLambda([this] { this->Init(); });
				}
			}

			void OnTransactionStateChanged(const FTransactionContext& TransactionContext, ETransactionStateEventType TransactionState)
			{
				/** A transaction has been started. This will be followed by a TransactionCanceled or TransactionFinalized event. */
				switch (TransactionState)
				{
				case ETransactionStateEventType::TransactionStarted:
				case ETransactionStateEventType::UndoRedoStarted:
					DeferTransactionChanges.Emplace();
					break;

				case ETransactionStateEventType::TransactionCanceled:
				case ETransactionStateEventType::PreTransactionFinalized:
				case ETransactionStateEventType::UndoRedoFinalized:
					DeferTransactionChanges.Reset();
					break;
				}
			}

			void Flush() override
			{
				TSet<TWeakObjectPtr<UMovieSceneSignedObject>> SignedObjectsTmp = SignedObjects;
				SignedObjects.Empty();
				// we operate on a copy of the signed objects in case the delegates would modify the array
				for (TWeakObjectPtr<UMovieSceneSignedObject> WeakObject : SignedObjectsTmp)
				{
					if (UMovieSceneSignedObject* Object = WeakObject.Get())
					{
						Object->BroadcastChanged();
					}
				}
			}

			void DeferMarkAsChanged(UMovieSceneSignedObject* SignedObject) override
			{
				SignedObjects.Add(SignedObject);
			}

			bool CreateImplicitScopedModifyDefer() override
			{
				ensure(!DeferImplicitChanges.IsSet());
				DeferImplicitChanges.Emplace();
				return true;
			}

			void ResetImplicitScopedModifyDefer() override
			{
				DeferImplicitChanges.Reset();
			}

			TSet<TWeakObjectPtr<UMovieSceneSignedObject>> SignedObjects;
			TWeakObjectPtr<UTransBuffer>   WeakBuffer;
			TOptional<UE::MovieScene::FScopedSignedObjectModifyDefer> DeferTransactionChanges;
			TOptional<UE::MovieScene::FScopedSignedObjectModifyDefer> DeferImplicitChanges;
		};



		struct FPositionNumericTypeInterface : UE::Sequencer::FSequencerNumericTypeInterface
		{
			FPositionNumericTypeInterface(TSharedRef<FFrameNumberInterface> InInterface)
				: FSequencerNumericTypeInterface(InInterface, ENumericIntent::Position)
			{}
		};

		struct FDurationNumericTypeInterface : UE::Sequencer::FSequencerNumericTypeInterface
		{
			FDurationNumericTypeInterface(TSharedRef<FFrameNumberInterface> InInterface)
				: FSequencerNumericTypeInterface(InInterface, ENumericIntent::Duration)
			{}
		};

	} // namespace Sequencer
} // namespace UE

const FName FSequencer::SelectionDrawerId = TEXT("SelectionDetails");
const FName FSequencer::SelectionDetailsTabId = TEXT("SequencerSelectionDetails");
bool FSequencer::bSelectionLimited = false;

void FSequencer::InitSequencer(const FSequencerInitParams& InitParams
	, const TSharedRef<ISequencerObjectChangeListener>& InObjectChangeListener
	, const TArray<FOnCreateTrackEditor>& TrackEditorDelegates
	, const TArray<FOnCreateDecorationEditor>& DecorationEditorDelegates
	, const TArray<FOnCreateEditorObjectBinding>& EditorObjectBindingDelegates
	, const TArray<FOnCreateOutlinerColumn>& OutlinerColumnDelegates
	, const TArray<FOnCreateOutlinerIndicator>& OutlinerIndicatorDelegates)
{
	using namespace UE::MovieScene;
	using namespace UE::Sequencer;

	const TSharedRef<FSequencer> SharedThisRef = SharedThis(this);

	bIsEditingWithinLevelEditor = InitParams.bEditWithinLevelEditor;
	ScrubStyle = InitParams.ViewParams.ScrubberStyle;
	HostCapabilities = InitParams.HostCapabilities;

	CurrentTimeBreadcrumbs = FMovieSceneTransformBreadcrumbs(EMovieSceneBreadcrumbMode::Dense);

	SilentModeCount = 0;
	bReadOnly = InitParams.ViewParams.bReadOnly;

	GetPlaybackSpeeds = InitParams.ViewParams.OnGetPlaybackSpeeds;
	
	const int32 IndexOfOne = GetPlaybackSpeeds.Execute().Find(1.f);
	check(IndexOfOne != INDEX_NONE);
	CurrentSpeedIndex = IndexOfOne;
	SpeedIndexBeforePlay = CurrentSpeedIndex;

	if (InitParams.SpawnRegister.IsValid())
	{
		SpawnRegister = InitParams.SpawnRegister;
	}
	else
	{
		// Spawnables not supported
		SpawnRegister = MakeShareable(new FNullMovieSceneSpawnRegister);
	}

	EventContextsAttribute = InitParams.EventContexts;
	if (EventContextsAttribute.IsSet())
	{
		CachedEventContexts.Reset();
		for (UObject* Object : EventContextsAttribute.Get())
		{
			CachedEventContexts.Add(Object);
		}
	}

	PlaybackContextAttribute = InitParams.PlaybackContext;
	CachedPlaybackContext = PlaybackContextAttribute.Get(nullptr);

	PlaybackClientAttribute = InitParams.PlaybackClient;
	CachedPlaybackClient = TWeakInterfacePtr<IMovieScenePlaybackClient>(PlaybackClientAttribute.Get(nullptr));

	Settings = USequencerSettingsContainer::GetOrCreate<USequencerSettings>(*InitParams.ViewParams.UniqueName);

	BindToSequencerSettingsDelegates(Settings);

	const FString TracksSettingsUniqueName = FString::Format(TEXT("{0}.TracksSettings"), { InitParams.ViewParams.UniqueName });
	TracksSettings = USequencerSettingsContainer::GetOrCreate<UMovieSceneTracksSettings>(*TracksSettingsUniqueName);

	// Initialize numeric type interfaces
	{

		TAttribute<EFrameNumberDisplayFormats> GetDisplayFormatAttr = MakeAttributeLambda(
			[this]
			{
				return GetSequencerSettings()->GetTimeDisplayFormat();
			}
		);
		TAttribute<uint8> GetZeroPadFramesAttr = MakeAttributeLambda(
			[this]()->uint8
			{
				return GetSequencerSettings()->GetZeroPadFrames();
			}
		);

		TAttribute<FFrameRate> GetTickResolutionAttr = MakeAttributeSP(this, &FSequencer::GetFocusedTickResolution);
		TAttribute<FFrameRate> GetDisplayRateAttr    = MakeAttributeSP(this, &FSequencer::GetFocusedDisplayRate);

		NumericTypeInterfaces.Add(MakeShared<FPositionNumericTypeInterface>(
			MakeShared<FFrameNumberInterface>(GetDisplayFormatAttr, GetZeroPadFramesAttr, GetTickResolutionAttr, GetDisplayRateAttr)
			)
		);

		NumericTypeInterfaces.Add(MakeShared<FDurationNumericTypeInterface>(
			MakeShared<FFrameNumberInterface>(GetDisplayFormatAttr, GetZeroPadFramesAttr, GetTickResolutionAttr, GetDisplayRateAttr)
			)
		);
	}


	ObjectBindingTagCache = MakeUnique<FObjectBindingTagCache>();

	{
		FDelegateHandle OnBlueprintPreCompileHandle = GEditor->OnBlueprintPreCompile().AddLambda([&](UBlueprint* InBlueprint)
		{
			if (CVarRestorePreAnimatedStateOnBlueprintPreCompile->GetBool())
			{
				// Restore pre animate state since objects will be reinstanced and current cached state will no longer be valid.
				if (InBlueprint && InBlueprint->GeneratedClass.Get())
				{
					PreAnimatedState.RestorePreAnimatedState(InBlueprint->GeneratedClass.Get());
				}
			}
		});
		AcquiredResources.Add([=] { GEditor->OnBlueprintPreCompile().Remove(OnBlueprintPreCompileHandle); });

		FDelegateHandle OnBlueprintCompiledHandle = GEditor->OnBlueprintCompiled().AddLambda([&]
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			State.InvalidateExpiredObjects();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			// Force re-evaluation since animated state was restored in PreCompile
			bNeedsEvaluate = true;
		});
		AcquiredResources.Add([=] { GEditor->OnBlueprintCompiled().Remove(OnBlueprintCompiledHandle); });
	}

	{
		FDelegateHandle OnObjectsReplacedHandle = FCoreUObjectDelegates::OnObjectsReplaced.AddLambda([&](const TMap<UObject*, UObject*>& ReplacementMap)
		{
			// Close sequencer if any of the objects being replaced is itself
			TArray<UPackage*> AllSequences;
			if (UMovieSceneSequence* Sequence = RootSequence.Get())
			{
				if (UPackage* Package = Sequence->GetOutermost())
				{
					AllSequences.AddUnique(Package);
				}
			}

			FMovieSceneCompiledDataID DataID = CompiledDataManager->GetDataID(RootSequence.Get());
			if (const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(DataID))
			{
				for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
				{
					if (UMovieSceneSequence* Sequence = Pair.Value.GetSequence())
					{
						if (UPackage* Package = Sequence->GetOutermost())
						{
							AllSequences.AddUnique(Package);
						}
					}
				}
			}

			for (TPair<UObject*, UObject*> ReplacedObject : ReplacementMap)
			{
				if (AllSequences.Contains(ReplacedObject.Value) || AllSequences.Contains(ReplacedObject.Key))
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(GetRootMovieSceneSequence());
					return;
				}
			}

			// Update our playback context and client if they were replaced.
			// This needs to happen before we resting our object bindings (below) because object binding resolution
			// can sometimes try to access the playback context.
			if (UObject* OldPlaybackContext = CachedPlaybackContext.Get())
			{
				if (UObject* const* NewPlaybackContext = ReplacementMap.Find(OldPlaybackContext))
				{
					CachedPlaybackContext = *NewPlaybackContext;
				}
			}
			if (UObject* OldPlaybackClient = CachedPlaybackClient.GetObject())
			{
				if (UObject* const* NewPlaybackClient = ReplacementMap.Find(OldPlaybackClient))
				{
					IMovieScenePlaybackClient* ClientInterface = Cast<IMovieScenePlaybackClient>(*NewPlaybackClient);
					CachedPlaybackClient = TWeakInterfacePtr<IMovieScenePlaybackClient>(ClientInterface);
				}
			}

			if (UObject* PlaybackContext = CachedPlaybackContext.Get())
			{
				if (ReplacementMap.Contains(PlaybackContext->GetClass()))
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(GetRootMovieSceneSequence());
					return;
				}
			}

			// Reset Bindings for replaced objects.
			bool bAnythingReplaced = false;
			for (TPair<UObject*, UObject*> ReplacedObject : ReplacementMap)
			{
				FGuid Guid = GetHandleToObject(ReplacedObject.Key, false);
				if (Guid.IsValid())
				{
					bAnythingReplaced = true;
				}
			}

			if (bAnythingReplaced)
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				State.InvalidateExpiredObjects();
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		});
		AcquiredResources.Add([=] { FCoreUObjectDelegates::OnObjectsReplaced.Remove(OnObjectsReplacedHandle); });
	}

	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
	{
		FDelegateHandle OnAssetRenamedHandle = AssetRegistryModule->Get().OnAssetRenamed().AddLambda([this](const FAssetData& AssetData, const FString& AssetOldName)
		{
			UMovieSceneSequence* MovieScene = GetFocusedMovieSceneSequence();
			if (AssetData.GetAsset() == MovieScene)
			{
				UMovieSceneSequence* RootSequencePtr = RootSequence.Get();
				if (RootSequencePtr)
				{
					ResetToNewRootSequence(*RootSequencePtr);
				}
			}
		});
		AcquiredResources.Add([=] { AssetRegistryModule->Get().OnAssetRenamed().Remove(OnAssetRenamedHandle); });
	}

	ToolkitHost = InitParams.ToolkitHost;

	PlaybackSpeed = 1.f;
	PlaybackSpeedBeforePlay = PlaybackSpeed;
	ShuttleMultiplier = 0;
	ObjectChangeListener = InObjectChangeListener;
	PropertyKeyedStatusHandler = MakeShared<FSequencerPropertyKeyedStatusHandler>(SharedThisRef);

	check( ObjectChangeListener.IsValid() );
	
	RootSequence = InitParams.RootSequence;

	{
		CompiledDataManager = FindObject<UMovieSceneCompiledDataManager>(GetTransientPackage(), TEXT("SequencerCompiledDataManager"));
		if (!CompiledDataManager)
		{
			CompiledDataManager = NewObject<UMovieSceneCompiledDataManager>(GetTransientPackage(), "SequencerCompiledDataManager");
		}
	}

	ActiveTemplateIDs.Add(MovieSceneSequenceID::Root);
	ActiveTemplateStates.Add(true);

	RootTemplateInstance.Initialize(*InitParams.RootSequence, *this, CompiledDataManager);
	Runner = RootTemplateInstance.GetRunner();

	RootTemplateInstance.EnableGlobalPreAnimatedStateCapture();

	InitialValueCache = UE::MovieScene::FInitialValueCache::GetGlobalInitialValues();
	RootTemplateInstance.GetEntitySystemLinker()->AddExtension(InitialValueCache.Get());

	InitRootSequenceInstance();

	// Create tools and bind them to this sequencer
	for( int32 DelegateIndex = 0; DelegateIndex < TrackEditorDelegates.Num(); ++DelegateIndex )
	{
		check( TrackEditorDelegates[DelegateIndex].IsBound() );
		// Tools may exist in other modules, call a delegate that will create one for us
		TSharedRef<ISequencerTrackEditor> TrackEditor = TrackEditorDelegates[DelegateIndex].Execute( SharedThis( this ) );

		if (TrackEditor->SupportsSequence(InitParams.RootSequence))
		{
			TrackEditors.Add( TrackEditor );
		}
	}

	// Create per-sequencer decoration editors via the registered factories.
	for (const FOnCreateDecorationEditor& Factory : DecorationEditorDelegates)
	{
		check(Factory.IsBound());
		DecorationEditors.Add(Factory.Execute(SharedThis(this)));
	}

	// OutlinerColumns are registered to be provided to SOutlinerView by SSequencer
	for (int32 DelegateIndex = 0; DelegateIndex < OutlinerColumnDelegates.Num(); ++DelegateIndex)
	{
		check(OutlinerColumnDelegates[DelegateIndex].IsBound());
		TSharedRef<IOutlinerColumn> OutlinerColumn = OutlinerColumnDelegates[DelegateIndex].Execute();

		if (OutlinerColumn->SupportsSequence(InitParams.RootSequence))
		{
			check(!OutlinerColumns.Contains(OutlinerColumn->GetColumnName()));
			OutlinerColumns.Add(OutlinerColumn->GetColumnName(), OutlinerColumn);
		}
	}

	// OutlinerIndicators are registered to be provided to SOutlinerView by SSequencer
	for (int32 DelegateIndex = 0; DelegateIndex < OutlinerIndicatorDelegates.Num(); ++DelegateIndex)
	{
		check(OutlinerIndicatorDelegates[DelegateIndex].IsBound());
		TSharedRef<IOutlinerIndicatorBuilder> OutlinerIndicator = OutlinerIndicatorDelegates[DelegateIndex].Execute();

		if (OutlinerIndicator->SupportsSequence(InitParams.RootSequence))
		{
			check(!OutlinerIndicators.Contains(OutlinerIndicator->GetIndicatorName()));
			OutlinerIndicators.Add(OutlinerIndicator->GetIndicatorName(), OutlinerIndicator);
		}
	}

	TWeakPtr<UE::MovieScene::IDeferredSignedObjectChangeHandler> WeakDeferredSignedObjectChangeHandler = UMovieSceneSignedObject::GetDeferredHandler();
	if (WeakDeferredSignedObjectChangeHandler.IsValid())
	{
		DeferredSignedObjectChangeHandler = WeakDeferredSignedObjectChangeHandler.Pin();
	}
	else
	{
		DeferredSignedObjectChangeHandler = MakeShared<FDeferredSignedObjectChangeHandler>();
		UMovieSceneSignedObject::SetDeferredHandler(DeferredSignedObjectChangeHandler);
	}
	
	ViewModel = MakeShared<FSequencerEditorViewModelImpl>(SharedThisRef, GetHostCapabilities());
	ViewModel->InitializeEditor();
	ViewModel->SetSequence(InitParams.RootSequence);
	ViewModel->GetRootSequenceModel()->InitializeExtensions();
	

	ViewModel->GetSelection()->OnChanged.AddRaw(this, &FSequencer::OnSelectionChanged);

	FSequencerOutlinerViewModel* OutlinerViewModel = ViewModel->GetOutliner()->CastThisChecked<FSequencerOutlinerViewModel>();
	{
		OutlinerViewModel->OnGetAddMenuContent = InitParams.ViewParams.OnGetAddMenuContent;
		OutlinerViewModel->OnBuildCustomContextMenuForGuid = InitParams.ViewParams.OnBuildCustomContextMenuForGuid;
	}

	NodeTree->SetRootNode(ViewModel->GetRootModel());

	ResetTimeController();

	UpdateTimeBases();
	PlayPosition.Reset(ConvertFrameTime(GetPlaybackRange().GetLowerBoundValue(), GetRootTickResolution(), PlayPosition.GetInputRate()));

	if (!GetSelectionRange().IsEmpty() && GetLoopMode() != SLM_LoopSelectionRange)
	{
		Settings->SetLoopMode(SLM_LoopSelectionRange);
	}

	// Make internal widgets
	SequencerWidget = SNew( SSequencer, SharedThis( this ) )
		.ViewRange( this, &FSequencer::GetViewRange )
		.ClampRange( this, &FSequencer::GetClampRange )
		.PlaybackRange( this, &FSequencer::GetPlaybackRange )
		.TimeBounds( this, &FSequencer::GetTimeBounds )
		.PlaybackStatus( this, &FSequencer::GetPlaybackStatus )
		.SelectionRange( this, &FSequencer::GetSelectionRange )
		.VerticalFrames(this, &FSequencer::GetVerticalFrames)
		.MarkedFrames(this, &FSequencer::GetMarkedFrames)
		.GlobalMarkedFrames(this, &FSequencer::GetGlobalMarkedFrames)
		.OnSetMarkedFrame(this, &FSequencer::SetMarkedFrame)
		.OnAddMarkedFrame(this, &FSequencer::AddMarkedFrame)
		.OnDeleteAllMarkedFrames(this, &FSequencer::DeleteAllMarkedFrames)
		.AreMarkedFramesLocked(this, &FSequencer::AreMarkedFramesLocked)
		.OnToggleMarkedFramesLocked(this, &FSequencer::ToggleMarkedFramesLocked)
		.SubSequenceRange( this, &FSequencer::GetSubSequenceRange )
		.OnPlaybackRangeChanged( this, &FSequencer::SetPlaybackRange )
		.OnPlaybackRangeBeginDrag( this, &FSequencer::OnPlaybackRangeBeginDrag )
		.OnPlaybackRangeEndDrag( this, &FSequencer::OnPlaybackRangeEndDrag )
		.OnSelectionRangeChanged( this, &FSequencer::SetSelectionRange )
		.OnSelectionRangeBeginDrag( this, &FSequencer::OnSelectionRangeBeginDrag )
		.OnSelectionRangeEndDrag( this, &FSequencer::OnSelectionRangeEndDrag )
		.OnMarkBeginDrag(this, &FSequencer::OnMarkBeginDrag)
		.OnMarkEndDrag(this, &FSequencer::OnMarkEndDrag)
		.IsPlaybackRangeLocked( this, &FSequencer::IsPlaybackRangeLocked )
		.OnTogglePlaybackRangeLocked( this, &FSequencer::TogglePlaybackRangeLocked )
		.ScrubPosition( this, &FSequencer::GetScrubPosition )
		.ScrubPositionText( this, &FSequencer::GetFrameTimeText )
		.ScrubPositionParent( this, &FSequencer::GetScrubPositionParent)
		.ScrubPositionParentChain( this, &FSequencer::GetScrubPositionParentChain)
		.OnScrubPositionParentChanged(this, &FSequencer::OnScrubPositionParentChanged)
		.OnBeginScrubbing( this, &FSequencer::OnBeginScrubbing )
		.OnEndScrubbing( this, &FSequencer::OnEndScrubbing )
		.OnScrubPositionChanged( this, &FSequencer::OnScrubPositionChanged )
		.OnViewRangeChanged( this, &FSequencer::SetViewRange )
		.OnClampRangeChanged( this, &FSequencer::OnClampRangeChanged )
		.OnGetNearestKey( this, &FSequencer::OnGetNearestKey )
		.OnGetPlaybackSpeeds(InitParams.ViewParams.OnGetPlaybackSpeeds)
		.OnReceivedFocus(InitParams.ViewParams.OnReceivedFocus)
		.OnInitToolMenuContext(InitParams.ViewParams.OnInitToolMenuContext)
		.AddMenuExtender(InitParams.ViewParams.AddMenuExtender)
		.ToolbarExtender(InitParams.ViewParams.ToolbarExtender)
		.ShowPlaybackRangeInTimeSlider(InitParams.ViewParams.bShowPlaybackRangeInTimeSlider);
	SequencerWrapperWidget = SNew(SSequencerWrapperWidget, SequencerWidget.ToSharedRef());

	if (GetHostCapabilities().bSupportsCurveEditor)
	{
		FCurveEditorExtension* CurveEditorExtension = ViewModel->CastDynamic<FCurveEditorExtension>();
		check(CurveEditorExtension);
		TSharedPtr<FCurveEditor> CurveEditor = CurveEditorExtension->GetCurveEditor();

		CurveEditor->OnCurveArrayChanged.AddRaw(this, &FSequencer::OnCurveModelDisplayChanged);

		SetShowCurveEditor(GetSequencerSettings()->GetCurveEditorVisible());
	}

	// When undo occurs, get a notification so we can make sure our view is up to date
	GEditor->RegisterForUndo(this);

	for (int32 DelegateIndex = 0; DelegateIndex < EditorObjectBindingDelegates.Num(); ++DelegateIndex)
	{
		check(EditorObjectBindingDelegates[DelegateIndex].IsBound());
		// Object bindings may exist in other modules, call a delegate that will create one for us 
		TSharedRef<ISequencerEditorObjectBinding> ObjectBinding = EditorObjectBindingDelegates[DelegateIndex].Execute(SharedThisRef);
		ObjectBindings.Add(ObjectBinding);
	}

	FMovieSceneObjectBindingIDCustomization::BindTo(AsShared());

	ZoomAnimation = FCurveSequence();
	ZoomCurve = ZoomAnimation.AddCurve(0.f, 0.2f, ECurveEaseFunction::QuadIn);
	OverlayAnimation = FCurveSequence();
	OverlayCurve = OverlayAnimation.AddCurve(0.f, 0.2f, ECurveEaseFunction::QuadIn);
	RecordingAnimation = FCurveSequence();
	RecordingAnimation.AddCurve(0.f, 1.5f, ECurveEaseFunction::Linear);
	
	// Update initial movie scene data
	NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::ActiveMovieSceneChanged );

	if (CVarTimeUndo->GetBool())
	{
		TimeUndoRedoHandler.UndoRedoProxy = NewObject<USequencerTimeChangeUndoRedoProxy>(GetTransientPackage(), NAME_None);
		TimeUndoRedoHandler.SetSequencer(SharedThisRef);
		TimeUndoRedoHandler.UndoRedoProxy->SetFlags(RF_Transactional | RF_Transient);
	}

	// Update the view range to the new current time
	UpdateTimeBoundsToFocusedMovieScene();

	// NOTE: Could fill in asset editor commands here!

	BindCommands();

	// Ensure that the director BP is registered with the action database
	if (FMovieSceneSequenceEditor* SequenceEditor = FMovieSceneSequenceEditor::Find(InitParams.RootSequence))
	{
		UBlueprint* Blueprint = SequenceEditor->FindDirectorBlueprint(InitParams.RootSequence);
		if (Blueprint)
		{
			if (FBlueprintActionDatabase* Database = FBlueprintActionDatabase::TryGet())
			{
				Database->RefreshAssetActions(Blueprint);
			}
		}
	}

	for (auto TrackEditor : TrackEditors)
	{
		TrackEditor->OnInitialize();
	}

	UpdateSequencerCustomizations(nullptr);

	AddNodeGroupsCollectionChangedDelegate();

	OnActivateSequenceEvent.Broadcast(ActiveTemplateIDs[0]);

	// Selection Details Drawer
	if (HostCapabilities.bSupportsSidebar)
	{
		FSidebarDrawerConfig DetailsDrawer;
		DetailsDrawer.UniqueId = SelectionDrawerId;
		DetailsDrawer.SortOrder = 0.f;
		DetailsDrawer.ButtonText = LOCTEXT("SelectionDetailsPanelLabel", "Selection");
		DetailsDrawer.ToolTipText = TAttribute<FText>::CreateSP(this, &FSequencer::GetSidebarSelectionDrawerToolTipText);
		DetailsDrawer.Icon = FAppStyle::GetBrush(TEXT("EditorPreferences.TabIcon"));
		DetailsDrawer.InitialState = Settings->GetSidebarState().FindOrAddDrawerState(SelectionDrawerId);
		RegisterDrawer(MoveTemp(DetailsDrawer));

		SequencerWidget->SetSelectionDetailsPoppedOut(Settings->IsSelectionDetailsPoppedOut());
	}

	SynchronizeSequencerSelectionWithExternalSelection();
}

void FSequencer::SetSequencerSettings(USequencerSettings* InSettings)
{
	UnbindFromSequencerSettingsDelegates(Settings);
	Settings = InSettings;
	BindToSequencerSettingsDelegates(Settings);

	OnTimeDisplayFormatChanged();
}

void FSequencer::OnPlaybackContextChanged()
{
	RootTemplateInstance.PlaybackContextChanged(*this);
	// Update runner to match the newly created shared payback state of the root template instance.
	Runner = RootTemplateInstance.GetRunner();
	InitRootSequenceInstance();
}

void FSequencer::OnEvaluateSubSequencesInIsolationChanged()
{
	RestorePreAnimatedState();
	FMovieSceneSequenceID NewOverrideRoot = Settings->ShouldEvaluateSubSequencesInIsolation() ? ActiveTemplateIDs.Top() : MovieSceneSequenceID::Root;
	UMovieSceneEntitySystemLinker* Linker = RootTemplateInstance.GetEntitySystemLinker();
	RootTemplateInstance.FindInstance(MovieSceneSequenceID::Root)->OverrideRootSequence(NewOverrideRoot);
	ForceEvaluate();
}

void FSequencer::InitRootSequenceInstance()
{
	using namespace UE::MovieScene;

	// Add the camera cut playback capability.
	TSharedPtr<FSharedPlaybackState> SharedPlaybackState = RootTemplateInstance.GetSharedPlaybackState();
	if (ensure(SharedPlaybackState.IsValid()))
	{
		SharedPlaybackState->SetOrAddCapabilityRaw<FCameraCutPlaybackCapability>((FCameraCutPlaybackCapability*)this);
		SharedPlaybackState->SetOrAddCapabilityRaw<FCameraCutViewTargetCacheCapability>((FCameraCutViewTargetCacheCapability*)this);
	}
}

FSequencer::FSequencer()
	: SequencerCommandBindings( new FUICommandList )
	, SequencerSharedBindings(new FUICommandList)
	, CurveEditorSharedBindings(new FUICommandList)
	, TargetViewRange(0.f, 5.f)
	, LastViewRange(0.f, 5.f)
	, ViewRangeBeforeZoom(TRange<double>::Empty())
	, PlaybackState( EMovieScenePlayerStatus::Stopped )
	, bPerspectiveViewportPossessionEnabled( true )
	, bPerspectiveViewportCameraCutEnabled( false )
	, bIsEditingWithinLevelEditor( false )
	, bNeedTreeRefresh( false )
	, NodeTree( MakeShareable( new FSequencerNodeTree( *this ) ) )
	, bUpdatingSequencerSelection( false )
	, bUpdatingExternalSelection( false )
	, bNeedsEvaluate(false)
	, bNeedsInvalidateCachedData(false)
	, TimeControllerState(ETimeControllerState::ReadyToPlay)
{
	// Exposes the sequencer and curve editor command lists to subscribers from other systems
	FInputBindingManager::Get().RegisterCommandList(FSequencerCommands::Get().GetContextName(), SequencerCommandBindings);
	FInputBindingManager::Get().RegisterCommandList(FCurveEditorCommands::Get().GetContextName(), CurveEditorSharedBindings);

	TimeOperationDomain = UE::Sequencer::ETimeDomain::Warped;
}


FSequencer::~FSequencer()
{
	// Suppress any deferred modification flushes during destruction.
	// This prevents view model updates from trying to access the dying Sequencer
	// when the deferred flush is triggered during teardown.

	UE::MovieScene::FScopedSignedObjectModifySuppress SuppressFlush;
	if (Runner)
	{
		Runner->QueueFinalUpdate(RootTemplateInstance.GetRootInstanceHandle());
		Runner->Flush();
	}

	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}

	if (!GExitPurge)
	{
		if (OldMaxTickRate.IsSet())
		{
			GEngine->SetMaxFPS(OldMaxTickRate.GetValue());
			OldMaxTickRate.Reset();
		}
	}

	for (auto TrackEditor : TrackEditors)
	{
		TrackEditor->OnRelease();
	}

	AcquiredResources.Release();

	if (ViewModel && ViewModel->GetSelection())
	{
		ViewModel->GetSelection()->OnChanged.RemoveAll(this);
	}

	ViewModel.Reset();
	DestroySequencerWidget();

	TrackEditors.Empty();

	RootTemplateInstance.TearDown();
}

void FSequencer::Close()
{
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC != nullptr)
		{
			LevelVC->ViewModifiers.RemoveAll(this);
		}
	}

	if (OldMaxTickRate.IsSet())
	{
		GEngine->SetMaxFPS(OldMaxTickRate.GetValue());
		OldMaxTickRate.Reset();
	}

	if (Runner)
	{
		Runner->QueueFinalUpdate(RootTemplateInstance.GetRootInstanceHandle());
		Runner->Flush();
	}

	// Save the user's preference for curve editor visibility since it can be closed outside of Sequencer's control (ie. close tab)
	if (GetHostCapabilities().bSupportsCurveEditor)
	{
		GetSequencerSettings()->SetCurveEditorVisible(GetCurveEditorIsVisible());
	}

	RestorePreAnimatedState();

	for (auto TrackEditor : TrackEditors)
	{
		TrackEditor->OnRelease();
	}

	DestroySequencerWidget();
	TrackEditors.Empty();

	GUnrealEd->UpdatePivotLocationForSelection();

	// Redraw viewports after restoring pre animated state in case viewports are not set to realtime
	GEditor->RedrawLevelEditingViewports();

	CachedViewState.RestoreViewState();

	OnCloseEventDelegate.Broadcast(AsShared());
}

TSharedPtr<ISequencerTrackEditor> FSequencer::GetTrackEditor(UMovieSceneTrack* InTrack)
{
	UClass* TrackClass = InTrack->GetClass();
	FObjectKey TrackClassKey(TrackClass);

	TSharedPtr<ISequencerTrackEditor> TrackEditor = TrackEditorsByType.FindRef(TrackClassKey);
	if (!TrackEditor)
	{
		for (TSharedPtr<ISequencerTrackEditor> OtherTrackEditor : TrackEditors)
		{
			if (OtherTrackEditor->SupportsType(TrackClass))
			{
				TrackEditor = OtherTrackEditor;
				TrackEditorsByType.Add(TrackClassKey, TrackEditor);
				break;
			}
		}
	}

	ensureAlwaysMsgf(TrackEditor, TEXT("Unable to find a track editor for track type %s"), *TrackClass->GetName());
	return TrackEditor;
}

namespace
{
	// Used by FindDecorationOwner when no class-specific editor has been registered.
	// Inherits the base ISequencerDecorationEditor::AddOrUpdateKey, which keys the
	// channel via the standard channel editor interface.
	class FDefaultDecorationEditor : public ISequencerDecorationEditor
	{
	public:
		virtual UClass* GetDecorationClass() const override { return nullptr; }
	};
}

ISequencerDecorationEditor* FSequencer::FindDecorationEditor(UClass* DecorationClass) const
{
	if (!DecorationClass)
	{
		return nullptr;
	}
	for (const TSharedPtr<ISequencerDecorationEditor>& Editor : DecorationEditors)
	{
		if (Editor && Editor->GetDecorationClass() == DecorationClass)
		{
			return Editor.Get();
		}
	}
	return nullptr;
}

ISequencerDecorationEditor* FSequencer::GetOrCreateDefaultDecorationEditor() const
{
	if (!DefaultDecorationEditor.IsValid())
	{
		DefaultDecorationEditor = MakeShared<FDefaultDecorationEditor>();
	}
	return DefaultDecorationEditor.Get();
}

TTuple<ISequencerDecorationEditor*, UObject*> FSequencer::FindDecorationOwner(
	const FMovieSceneChannelHandle& ChannelHandle, UMovieSceneSection* Section) const
{
	if (const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData())
	{
		UObject* OwningObject = MetaData->WeakOwningObject.Get();
		if (OwningObject && OwningObject != static_cast<UObject*>(Section))
		{
			ISequencerDecorationEditor* Editor = FindDecorationEditor(OwningObject->GetClass());
			if (!Editor)
			{
				Editor = GetOrCreateDefaultDecorationEditor();
			}
			return MakeTuple(Editor, OwningObject);
		}
	}
	return MakeTuple(nullptr, static_cast<UObject*>(nullptr));
}

void FSequencer::Tick(float InDeltaTime)
{
	using namespace UE::MovieScene;
	using namespace UE::Sequencer;

	static bool bEnableRefCountCheck = true;

	if (!FSlateApplication::Get().AnyMenusVisible())
	{
		if (bEnableRefCountCheck)
		{
			const int32 SequencerRefCount = AsShared().GetSharedReferenceCount() - 1;
			ensureAlwaysMsgf(SequencerRefCount == 1, TEXT("Multiple persistent shared references detected for Sequencer. There should only be one persistent authoritative reference. Found %d additional references which will result in FSequencer not being released correctly."), SequencerRefCount - 1);
		}
	}

	TSharedPtr<FSharedViewModelData> SharedModelData = ViewModel->GetRootModel()->GetSharedData();
	if (SharedModelData)
	{
		SharedModelData->ReportLatentHierarchicalOperations();
	}

	{
		FViewModelHierarchyOperation Operation(GetViewModel()->GetRootSequenceModel()->GetSharedData());
		UMovieSceneSignedObject::ResetImplicitScopedModifyDefer();
	}
	
	if (bNeedsInvalidateCachedData)
	{
		InvalidateCachedData();
		bNeedsInvalidateCachedData = false;
	}

	// Ensure the time bases for our playback position are kept up to date with the root sequence
	UpdateTimeBases();

	UMovieSceneSequence* RootSequencePtr = RootSequence.Get();
	ObjectBindingTagCache->ConditionalUpdate(RootSequencePtr);

	UpdateCachedPlaybackContextAndClient();

	{
		if (CompiledDataManager->IsDirty(RootSequencePtr))
		{
			// Try and preserve the current local time across compilations.
			//     When a modification results in a change of transform or time-warp,
			//     this helps to stop the play head from jumping around
			FMovieSceneSequenceTransform CachedRootToUnwarpedLocalTransform = RootToUnwarpedLocalTransform;
			FFrameTime OldLocalTime = GetUnwarpedLocalTime().Time;

			CompiledDataManager->Compile(RootSequencePtr);

			// Reset to the root sequence if the focused sequence no longer exists. This can happen if either the subsequence has been deleted or the hierarchy has changed.
			if (!GetFocusedMovieSceneSequence() || !GetFocusedMovieSceneSequence()->GetMovieScene())
			{
				PopToSequenceInstance(MovieSceneSequenceID::Root);
			}

			// Suppress auto evaluation if the sequence signature matches the one to be suppressed
			if (!SuppressAutoEvalSignature.IsSet())
			{
				bNeedsEvaluate = true;
			}
			else
			{
				UMovieSceneSequence* SuppressSequence = SuppressAutoEvalSignature->Get<0>().Get();
				const FGuid& SuppressSignature = SuppressAutoEvalSignature->Get<1>();

				if (!SuppressSequence || SuppressSequence->GetSignature() != SuppressSignature)
				{
					bNeedsEvaluate = true;
				}
			}

			if (CachedRootToUnwarpedLocalTransform != RootToUnwarpedLocalTransform)
			{
				FTimeDomainOverride TimeDomain = OverrideTimeDomain(ETimeDomain::Unwarped);
				SetLocalTime(OldLocalTime, ESnapTimeMode::STM_None, false /* bEvaluate */);
			}

			SuppressAutoEvalSignature.Reset();
		}
	}

	if (bNeedTreeRefresh || NodeTree->NeedsFilterUpdate())
	{
		EMovieScenePlayerStatus::Type StoredPlaybackState = GetPlaybackStatus();
		SetPlaybackStatus(EMovieScenePlayerStatus::Stopped);

		SelectionPreview.Empty();
		RefreshTree();

		SetPlaybackStatus(StoredPlaybackState);
	}


	UObject* PlaybackContext = GetPlaybackContext();
	UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
	float Dilation = World ? World->GetWorldSettings()->CinematicTimeDilation : 1.f;

	if (TimeControllerState == ETimeControllerState::PreparingToPlay && TimeController->IsReadyToPlay())
	{
		SetPlaybackStatus(EMovieScenePlayerStatus::Playing);
		TimeControllerState = ETimeControllerState::ReadyToPlay;
	}

	TimeController->Tick(InDeltaTime, PlaybackSpeed * Dilation);

	FQualifiedFrameTime GlobalTime = GetGlobalTime();

	static const float AutoScrollFactor = 0.1f;

	UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;

	// Animate the autoscroll offset if it's set
	if (AutoscrollOffset.IsSet())
	{
		float Offset = AutoscrollOffset.GetValue() * AutoScrollFactor;
		SetViewRange(TRange<double>(TargetViewRange.GetLowerBoundValue() + Offset, TargetViewRange.GetUpperBoundValue() + Offset), EViewRangeInterpolation::Immediate);
	}
	else if (MovieScene)
	{
		FMovieSceneEditorData& EditorData = MovieScene->GetEditorData();
		if (EditorData.GetViewRange() != TargetViewRange)
		{
			SetViewRange(EditorData.GetViewRange(), EViewRangeInterpolation::Immediate);
		}
	}

	// Animate the autoscrub offset if it's set
	if (AutoscrubOffset.IsSet() && PlaybackState == EMovieScenePlayerStatus::Scrubbing )
	{
		FQualifiedFrameTime CurrentTime = GetLocalTime();
		FFrameTime Offset = (AutoscrubOffset.GetValue() * AutoScrollFactor) * CurrentTime.Rate;
		const bool bForwards = PlaybackSpeed > 0;
		SetLocalTimeLooped(CurrentTime.Time + Offset, CurrentTimeBreadcrumbs, bForwards);
	}

	if (GetSelectionRange().IsEmpty() && GetLoopMode() == SLM_LoopSelectionRange)
	{
		Settings->SetLoopMode(SLM_Loop);
	}

	if (PlaybackState == EMovieScenePlayerStatus::Playing)
	{
		FFrameTime NewGlobalTime = TimeController->RequestCurrentTime(GlobalTime, PlaybackSpeed * Dilation, GetFocusedDisplayRate());

		// Put the time into clamped local space
		FMovieSceneTransformBreadcrumbs Breadcrumbs;
		FFrameTime LocalTime = RootToUnwarpedLocalTransform.TransformTime(NewGlobalTime, FTransformTimeParams().HarvestBreadcrumbs(Breadcrumbs).IgnoreClamps());

		FTimeDomainOverride DomainOverride = OverrideTimeDomain(ETimeDomain::Unwarped);
		const bool bForwards = PlaybackSpeed > 0;
		SetLocalTimeLooped(LocalTime, Breadcrumbs, bForwards);

		if (GetPlaybackStatus() == EMovieScenePlayerStatus::Playing)
		{
			const float ThresholdPercentage = 0.15f;
			UpdateAutoScroll(GetLocalTime().Time / GetFocusedTickResolution(), ThresholdPercentage);
		}
	}
	else
	{
		PlayPosition.Reset(GetGlobalTime().ConvertTo(PlayPosition.GetInputRate()));
	}

	if (AutoScrubTarget.IsSet())
	{
		const double ScrubSpeed = CVarAutoScrubSpeed->GetFloat();		// How fast to scrub at peak curve speed
		const double AutoScrubExp = CVarAutoScrubCurveExponent->GetFloat();	// How long to ease in and out.  Bigger numbers allow for longer easing.

		const double SecondsPerFrame = GetFocusedTickResolution().AsInterval() / ScrubSpeed;
		const int32 TotalFrames = FMath::Abs(AutoScrubTarget.GetValue().DestinationTime.GetFrame().Value - AutoScrubTarget.GetValue().SourceTime.GetFrame().Value);
		const double TargetSeconds = (double)TotalFrames * SecondsPerFrame;

		double ElapsedSeconds = FPlatformTime::Seconds() - AutoScrubTarget.GetValue().StartTime;
		float Alpha = ElapsedSeconds / TargetSeconds;
		Alpha = FMath::Clamp(Alpha, 0.f, 1.f);
		int32 NewFrameNumber = FMath::InterpEaseInOut(AutoScrubTarget.GetValue().SourceTime.GetFrame().Value, AutoScrubTarget.GetValue().DestinationTime.GetFrame().Value, Alpha, AutoScrubExp);

		FAutoScrubTarget CachedTarget = AutoScrubTarget.GetValue();

		SetPlaybackStatus(EMovieScenePlayerStatus::Scrubbing);
		PlayPosition.SetTimeBase(GetRootTickResolution(), GetRootTickResolution(), EMovieSceneEvaluationType::WithSubFrames);
		SetLocalTimeDirectly(FFrameNumber(NewFrameNumber));

		AutoScrubTarget = CachedTarget;

		if (FMath::IsNearlyEqual(Alpha, 1.f, KINDA_SMALL_NUMBER))
		{
			SetPlaybackStatus(EMovieScenePlayerStatus::Stopped);
			AutoScrubTarget.Reset();
		}
	}

	if (PendingScrubPosition.IsSet())
	{
		const bool bEvaluate = false;
		FTimeDomainOverride DomainOverride = OverrideTimeDomain(PendingScrubPosition->Get<1>());
		SetLocalTimeDirectly(PendingScrubPosition->Get<0>(), bEvaluate);
		PendingScrubPosition.Reset();

		bNeedsEvaluate = true;
	}

	UpdateSubSequenceData();

	// Tick all the tools we own as well
	for (int32 EditorIndex = 0; EditorIndex < TrackEditors.Num(); ++EditorIndex)
	{
		TrackEditors[EditorIndex]->Tick(InDeltaTime);
	}

	// Tick the per-sequencer decoration editors.
	for (const TSharedPtr<ISequencerDecorationEditor>& DecorationEditor : DecorationEditors)
	{
		if (DecorationEditor)
		{
			DecorationEditor->Tick(InDeltaTime);
		}
	}

	if (!IsInSilentMode())
	{
		if (bNeedsEvaluate)
		{
			EvaluateInternal(PlayPosition.GetCurrentPositionAsRange());
		}
	}

	// Reset any player controllers that we were possessing, if we're not possessing them any more
	if (!IsPerspectiveViewportCameraCutEnabled() && PrePossessionViewTargets.Num())
	{
		for (const FCachedViewTarget& CachedView : PrePossessionViewTargets)
		{
			APlayerController* PlayerController = CachedView.PlayerController.Get();
			AActor* ViewTarget = CachedView.ViewTarget.Get();

			if (PlayerController && ViewTarget)
			{
				PlayerController->SetViewTarget(ViewTarget);
			}
		}
		PrePossessionViewTargets.Reset();
	}

	UpdateCachedCameraActors();

	UpdateLevelViewportClientsActorLocks();

	if (!bGlobalMarkedFramesCached)
	{
		UpdateGlobalMarkedFramesCache();
	}

	for (TSharedRef<UE::Sequencer::FSequencerNumericTypeInterface> Interface : GetNumericTypeInterfaces())
	{
		if (Interface->Interface->GetOnSettingChanged() != nullptr)
		{
			Interface->Interface->GetOnSettingChanged()->Broadcast();
		}
	}
}


TSharedRef<SWidget> FSequencer::GetSequencerWidget() const
{
	return SequencerWrapperWidget.ToSharedRef();
}

UE::Sequencer::ISequencerWidget& FSequencer::GetSequencerWidgetInterface() const
{
	return *SequencerWrapperWidget;
}

UMovieSceneSequence* FSequencer::GetRootMovieSceneSequence() const
{
	return RootSequence.Get();
}

FMovieSceneSequenceTransform FSequencer::GetLocalTimeWarpTransform() const
{
	return LocalToWarpedLocalTransform;
}

FMovieSceneSequenceTransform FSequencer::GetGlobalPlaybackWarpTransform() const
{
	return GlobalPlaybackWarpTransform;
}

FMovieSceneSequenceTransform FSequencer::GetFocusedMovieSceneSequenceTransform() const
{
	return RootToWarpedLocalTransform;
}

FMovieSceneSequenceTransform FSequencer::GetFocusedMovieSceneSequenceTransformUnwarped() const
{
	return RootToUnwarpedLocalTransform;
}

UMovieSceneSequence* FSequencer::GetFocusedMovieSceneSequence() const
{
	// the last item is the focused movie scene
	if (ActiveTemplateIDs.Num())
	{
		return RootTemplateInstance.GetSequence(ActiveTemplateIDs.Last());
	}

	return nullptr;
}

UMovieSceneSubSection* FSequencer::FindSubSection(FMovieSceneSequenceID SequenceID) const
{
	if (SequenceID == MovieSceneSequenceID::Root)
	{
		return nullptr;
	}

	FMovieSceneCompiledDataID DataID = CompiledDataManager->Compile(RootSequence.Get());
	const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(DataID);
	if (!Hierarchy)
	{
		return nullptr;
	}
	
	const FMovieSceneSequenceHierarchyNode* SequenceNode = Hierarchy->FindNode(SequenceID);
	const FMovieSceneSubSequenceData*       SubData      = Hierarchy->FindSubData(SequenceID);

	if (SubData && SequenceNode)
	{
		UMovieSceneSequence* ParentSequence   = RootTemplateInstance.GetSequence(SequenceNode->ParentID);
		UMovieScene*         ParentMovieScene = ParentSequence ? ParentSequence->GetMovieScene() : nullptr;

		if (ParentMovieScene)
		{
			return FindObject<UMovieSceneSubSection>(ParentMovieScene, *SubData->SectionPath.ToString());
		}
	}

	return nullptr;
}


void FSequencer::ResetToNewRootSequence(UMovieSceneSequence& NewSequence)
{
	using namespace UE::MovieScene;

	OnPreNavigateToSequenceEvent.Broadcast();

	RemoveNodeGroupsCollectionChangedDelegate();

	const UMovieSceneSequence* PreviousRootSequence = RootSequence.Get();
	RootSequence = &NewSequence;
	RestorePreAnimatedState();

	// Ensure that the director BP is registered with the action database
	if (FMovieSceneSequenceEditor* SequenceEditor = FMovieSceneSequenceEditor::Find(&NewSequence))
	{
		UBlueprint* Blueprint = SequenceEditor->FindDirectorBlueprint(&NewSequence);
		if (Blueprint)
		{
			if (FBlueprintActionDatabase* Database = FBlueprintActionDatabase::TryGet())
			{
				Database->RefreshAssetActions(Blueprint);
			}
		}
	}

	if (Runner)
	{
		Runner->QueueFinalUpdate(RootTemplateInstance.GetRootInstanceHandle());
		Runner->Flush();
	}

	ActiveTemplateIDs.Reset();
	ActiveTemplateIDs.Add(MovieSceneSequenceID::Root);
	ActiveTemplateStates.Reset();
	ActiveTemplateStates.Add(true);

	RootTemplateInstance.Initialize(NewSequence, *this, CompiledDataManager);
	RootTemplateInstance.EnableGlobalPreAnimatedStateCapture();

	UpdateSubSequenceData();

	CurrentTimeBreadcrumbs.Reset();

	ResetPerMovieSceneData();
	SequencerWidget->ResetBreadcrumbs();

	PlayPosition.Reset(ConvertFrameTime(GetPlaybackRange().GetLowerBoundValue(), GetRootTickResolution(), PlayPosition.GetInputRate()));
	TimeController->Reset(GetGlobalTime());

	UpdateSequencerCustomizations(PreviousRootSequence);

	AddNodeGroupsCollectionChangedDelegate();

	InitRootSequenceInstance();

	OnActivateSequenceEvent.Broadcast(ActiveTemplateIDs.Top());
}


void FSequencer::FocusSequenceInstance(UMovieSceneSubSection& InSubSection)
{
	OnPreNavigateToSequenceEvent.Broadcast();

	RemoveNodeGroupsCollectionChangedDelegate();

	const UMovieSceneSequence* PreviousFocusedSequence = GetFocusedMovieSceneSequence();

	TemplateIDBackwardStack.Push(ActiveTemplateIDs.Top());
	TemplateIDForwardStack.Reset();

	UE::MovieScene::FSubSequencePath Path;

	// Ensure the hierarchy is up to date
	FMovieSceneCompiledDataID DataID = CompiledDataManager->Compile(RootSequence.Get());
	const FMovieSceneSequenceHierarchy& Hierarchy = CompiledDataManager->GetHierarchyChecked(DataID);

	Path.Reset(ActiveTemplateIDs.Last(), &Hierarchy);

	// Root out the SequenceID for the sub section
	FMovieSceneSequenceID SequenceID = Path.ResolveChildSequenceID(InSubSection.GetSequenceID());

	// If the sequence isn't found, reset to the root and dive in from there
	if (!Hierarchy.FindSubData(SequenceID))
	{
		// Pop until the root and reset breadcrumbs
		while (MovieSceneSequenceID::Root != ActiveTemplateIDs.Last())
		{
			ActiveTemplateIDs.Pop();
			ActiveTemplateStates.Pop();
		}
		SequencerWidget->ResetBreadcrumbs();

		// Find the requested subsequence's sequence ID
		SequenceID = MovieSceneSequenceID::Invalid;
		for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy.AllSubSequenceData())
		{
			if (Pair.Value.DeterministicSequenceID == InSubSection.GetSequenceID())
			{
				SequenceID = Pair.Key;
				break;
			}
		}

		// Gather the parent chain's sequence IDs
		TArray<FMovieSceneSequenceID> ParentChain;
		const FMovieSceneSequenceHierarchyNode* SequenceNode = Hierarchy.FindNode(SequenceID);
		FMovieSceneSequenceID ParentID = SequenceNode ? SequenceNode->ParentID : MovieSceneSequenceID::Invalid;
		while (ParentID.IsValid() && ParentID != MovieSceneSequenceID::Root)
		{
			ParentChain.Add(ParentID);

			const FMovieSceneSequenceHierarchyNode* ParentNode = Hierarchy.FindNode(ParentID);
			ParentID = ParentNode ? ParentNode->ParentID : MovieSceneSequenceID::Invalid;
		}

		// Push each sequence ID in the parent chain, updating the breadcrumb as we go
		for (int32 ParentIDIndex = ParentChain.Num() - 1; ParentIDIndex >= 0; --ParentIDIndex)
		{
			UMovieSceneSubSection* ParentSubSection = FindSubSection(ParentChain[ParentIDIndex]);
			if (ParentSubSection)
			{
				ActiveTemplateIDs.Push(ParentChain[ParentIDIndex]);
				ActiveTemplateStates.Push(ParentSubSection->IsActive());

				SequencerWidget->UpdateBreadcrumbs();
			}
		}

		Path.Reset(ActiveTemplateIDs.Last(), &Hierarchy);

		// Root out the SequenceID for the sub section
		SequenceID = Path.ResolveChildSequenceID(InSubSection.GetSequenceID());
	}

	if (!ensure(Hierarchy.FindSubData(SequenceID)))
	{
		return;
	}

	ActiveTemplateIDs.Push(SequenceID);
	ActiveTemplateStates.Push(InSubSection.IsActive());

	if (Settings->ShouldEvaluateSubSequencesInIsolation())
	{
		RestorePreAnimatedState();

		UMovieSceneEntitySystemLinker* Linker = RootTemplateInstance.GetEntitySystemLinker();
		RootTemplateInstance.FindInstance(MovieSceneSequenceID::Root)->OverrideRootSequence(ActiveTemplateIDs.Top());
	}

	UpdateSubSequenceData();

	UpdateSequencerCustomizations(PreviousFocusedSequence);

	ScrubPositionParent.Reset();

	// Always reset the hotspot so that it can be regenerated when rebuilding
	ViewModel->GetTrackArea()->SetHotspot(nullptr);

	// Reset data that is only used for the previous movie scene
	ResetPerMovieSceneData();
	SequencerWidget->UpdateBreadcrumbs();

	UMovieSceneSequence* FocusedSequence = GetFocusedMovieSceneSequence();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!State.FindSequence(SequenceID))
	{
		State.AssignSequence(SequenceID, *FocusedSequence, *this);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Ensure that the director BP is registered with the action database
	if (FMovieSceneSequenceEditor* SequenceEditor = FMovieSceneSequenceEditor::Find(FocusedSequence))
	{
		UBlueprint* Blueprint = SequenceEditor->FindDirectorBlueprint(FocusedSequence);
		if (Blueprint)
		{
			if (FBlueprintActionDatabase* Database = FBlueprintActionDatabase::TryGet())
			{
				Database->RefreshAssetActions(Blueprint);
			}
		}
	}

	OnActivateSequenceEvent.Broadcast(ActiveTemplateIDs.Top());

	AddNodeGroupsCollectionChangedDelegate();

	if (Settings->ShouldResetPlayheadWhenNavigating())
	{
		FQualifiedFrameTime CurrentTime = GetLocalTime();
		if (!SubSequenceRange.Contains(CurrentTime.Time.FloorToFrame()))
		{
			SetLocalTime(SubSequenceRange.GetLowerBoundValue(), ESnapTimeMode::STM_None);
		}
	}

	bNeedsEvaluate = true;
	bGlobalMarkedFramesCached = false;
}

TSharedPtr<UE::Sequencer::FSequencerEditorViewModel> FSequencer::GetViewModel() const
{
	return ViewModel;
}

void FSequencer::SuppressAutoEvaluation(UMovieSceneSequence* Sequence, const FGuid& InSequenceSignature)
{
	SuppressAutoEvalSignature = MakeTuple(MakeWeakObjectPtr(Sequence), InSequenceSignature);
}

FGuid FSequencer::CreateBinding(UObject& InObject, const UE::Sequencer::FCreateBindingParams& InParams)
{
    return FSequencerUtilities::CreateBinding(AsShared(), InObject, InParams);
}

UObject* FSequencer::GetPlaybackContext() const
{
	return CachedPlaybackContext.Get();
}

IMovieScenePlaybackClient* FSequencer::GetPlaybackClient()
{
	if (UObject* Obj = CachedPlaybackClient.GetObject())
	{
		return Cast<IMovieScenePlaybackClient>(Obj);
	}
	return nullptr;
}

TArray<UObject*> FSequencer::GetEventContexts() const
{
	TArray<UObject*> Temp;
	CopyFromWeakArray(Temp, CachedEventContexts);
	return Temp;
}

void FSequencer::GetKeysFromSelection(TUniquePtr<FSequencerKeyCollection>& KeyCollection, float DuplicateThresholdSeconds)
{
	using namespace UE::Sequencer;

	if (!KeyCollection.IsValid())
	{
		KeyCollection.Reset(new FSequencerKeyCollection);
	}

	TArray<TSharedRef<FViewModel>> SelectedItems;
	SelectedItems.Reserve(ViewModel->GetSelection()->Outliner.Num());
	for (FViewModelPtr SelectedItem : ViewModel->GetSelection()->Outliner)
	{
		SelectedItems.Add(SelectedItem.AsModel().ToSharedRef());
	}

	int64 TotalMaxSeconds = static_cast<int64>(TNumericLimits<int32>::Max() / GetFocusedTickResolution().AsDecimal());

	FFrameNumber ThresholdFrames = (DuplicateThresholdSeconds * GetFocusedTickResolution()).FloorToFrame();
	if (ThresholdFrames.Value < -TotalMaxSeconds)
	{
		ThresholdFrames.Value = TotalMaxSeconds;
	}
	else if (ThresholdFrames.Value > TotalMaxSeconds)
	{
		ThresholdFrames.Value = TotalMaxSeconds;
	}
	
	KeyCollection->Update(FSequencerKeyCollectionSignature::FromNodesRecursive(SelectedItems, ThresholdFrames));
}

FSequencerKeyCollection* FSequencer::GetKeyCollection()
{
	if (!SelectedKeyCollection.IsValid())
	{
		SelectedKeyCollection.Reset(new FSequencerKeyCollection);
	}

	return SelectedKeyCollection.Get();
}

void FSequencer::GetAllKeys(TUniquePtr<FSequencerKeyCollection>& KeyCollection, float DuplicateThresholdSeconds) const
{
	using namespace UE::Sequencer;

	if (!KeyCollection.IsValid())
	{
		KeyCollection.Reset(new FSequencerKeyCollection);
	}

	const FFrameNumber ThresholdFrames = (DuplicateThresholdSeconds * GetFocusedTickResolution()).FloorToFrame();
	KeyCollection->Update(FSequencerKeyCollectionSignature::FromNodesRecursive({NodeTree->GetRootNode()->AsShared()}, ThresholdFrames));
}


void FSequencer::PopToSequenceInstance(FMovieSceneSequenceIDRef SequenceID)
{
	if( ActiveTemplateIDs.Num() > 1 )
	{
		OnPreNavigateToSequenceEvent.Broadcast();

		TemplateIDBackwardStack.Push(ActiveTemplateIDs.Top());
		TemplateIDForwardStack.Reset();

		RemoveNodeGroupsCollectionChangedDelegate();

		const UMovieSceneSequence* PreviousFocusedSequence = GetFocusedMovieSceneSequence();

		// Pop until we find the movie scene to focus
		while( SequenceID != ActiveTemplateIDs.Last() )
		{
			ActiveTemplateIDs.Pop();
			ActiveTemplateStates.Pop();
		}

		check( ActiveTemplateIDs.Num() > 0 );
		UpdateSubSequenceData();

		ResetPerMovieSceneData();

		if (SequenceID == MovieSceneSequenceID::Root)
		{
			SequencerWidget->ResetBreadcrumbs();
		}
		else
		{
			SequencerWidget->UpdateBreadcrumbs();
		}

		if (Settings->ShouldEvaluateSubSequencesInIsolation())
		{
			UMovieSceneEntitySystemLinker* Linker = RootTemplateInstance.GetEntitySystemLinker();
			RootTemplateInstance.FindInstance(MovieSceneSequenceID::Root)->OverrideRootSequence(ActiveTemplateIDs.Top());
		}

		UpdateSequencerCustomizations(PreviousFocusedSequence);

		AddNodeGroupsCollectionChangedDelegate();

		ScrubPositionParent.Reset();

		OnActivateSequenceEvent.Broadcast(ActiveTemplateIDs.Top());

		bNeedsEvaluate = true;
		bGlobalMarkedFramesCached = false;
	}
}

void FSequencer::UpdateSubSequenceData()
{
	using namespace UE::MovieScene;

	SubSequenceRange = TRange<FFrameNumber>::Empty();

	const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(RootTemplateInstance.GetCompiledDataID());

	// If the root transform for the whole sequence is in the Play Rate domain, we assign a GlobalPlaybackWarpTransform
	//     that mutates the evaluation ranges as we evaluate. This is not necessary for Time domain transforms since that is handled
	//     by the sequence updaters themselves
	FMovieSceneSequenceTransform RootTransform = Hierarchy ? Hierarchy->GetRootTransform() : FMovieSceneSequenceTransform();;
	if (Hierarchy && Hierarchy->GetRootTransform().FindFirstWarpDomain() == ETimeWarpChannelDomain::PlayRate)
	{
		GlobalPlaybackWarpTransform = Hierarchy->GetRootTransform();
	}
	else
	{
		GlobalPlaybackWarpTransform = FMovieSceneSequenceTransform();
	}

	// Find the parent sub section and set up the sub sequence range, if necessary
	if (ActiveTemplateIDs.Num() <= 1)
	{
		// Ensure in this case we also reset the CurrentTimeBreadcrumbs.
		CurrentTimeBreadcrumbs.Reset();

		// Reset everything first
		RootToUnwarpedLocalTransform = RootToWarpedLocalTransform = LocalToWarpedLocalTransform = FMovieSceneSequenceTransform();

		if (Hierarchy)
		{
			LocalToWarpedLocalTransform = Hierarchy->GetRootTransform();
		}
		RootToWarpedLocalTransform = LocalToWarpedLocalTransform;
		return;
	}

	check(Hierarchy);

	const FMovieSceneSubSequenceData* SubSequenceData = Hierarchy->FindSubData(ActiveTemplateIDs.Top());

	if (SubSequenceData)
	{
		SubSequenceRange = SubSequenceData->PlayRange.Value;

		RootToUnwarpedLocalTransform = SubSequenceData->RootToUnwarpedLocalTransform * RootTransform;
		RootToWarpedLocalTransform   = SubSequenceData->RootToSequenceTransform * RootTransform;
		LocalToWarpedLocalTransform  = SubSequenceData->LocalToWarpedLocalTransform;

		const bool bIsScrubbing = GetPlaybackStatus() == EMovieScenePlayerStatus::Scrubbing;

		FFrameTime CurrentTime = GetGlobalTime().Time;

		CurrentTimeBreadcrumbs.Reset();
		RootToWarpedLocalTransform.TransformTime(CurrentTime, FTransformTimeParams().HarvestBreadcrumbs(CurrentTimeBreadcrumbs).IgnoreClamps());

		// Inner play range in unwarped space
		UMovieSceneSequence* SubSequence   = SubSequenceData->GetSequence();
		TRange<FFrameNumber> PlaybackRange = SubSequence->GetMovieScene()->GetPlaybackRange();

		FMovieSceneInverseSequenceTransform LocalToRootTransform = RootToUnwarpedLocalTransform.Inverse();
		const int32 PlaybackSize = UE::MovieScene::DiscreteSize(PlaybackRange);

		TOptional<FFrameTime> PlayStart = LocalToRootTransform.TryTransformTime(PlaybackRange.GetLowerBoundValue(), CurrentTimeBreadcrumbs, EInverseEvaluateFlags::Backwards | EInverseEvaluateFlags::Cycle);
		TOptional<FFrameTime> PlayEnd   = LocalToRootTransform.TryTransformTime(PlaybackRange.GetUpperBoundValue(), CurrentTimeBreadcrumbs, EInverseEvaluateFlags::Forwards  | EInverseEvaluateFlags::Cycle);

		TRange<FFrameTime> ValidRange = TRange<FFrameTime>::All();
		if (PlayStart)
		{
			ValidRange.SetLowerBound(TRangeBound<FFrameTime>::Inclusive(PlayStart.GetValue()));
		}
		if (PlayEnd)
		{
			ValidRange.SetUpperBound(TRangeBound<FFrameTime>::Inclusive(PlayEnd.GetValue()));
		}

		for (int32 Index = 0; Index < ActiveTemplateIDs.Num(); ++Index)
		{
			FMovieSceneSequenceID SequenceID = ActiveTemplateIDs[Index];
			const FMovieSceneSubSequenceData* SubData = Hierarchy->FindSubData(SequenceID);
			if (SubData)
			{
				// Clamp to the sub section range recursively
				if (!SubData->ParentPlayRange.Value.Contains(CurrentTime.FrameNumber))
				{
					FFrameNumber Lower = DiscreteInclusiveLower(SubData->ParentPlayRange.Value);
					FFrameNumber Upper = DiscreteExclusiveUpper(SubData->ParentPlayRange.Value);

					if (CurrentTime.FrameNumber < Lower)
					{
						CurrentTime = FFrameTime(Lower);
					}
					else if (CurrentTime.FrameNumber >= Upper)
					{
						CurrentTime = FFrameTime(Upper - 1, FFrameTime::MaxSubframe);
					}
				}

				CurrentTime = SubData->OuterToInnerTransform.TransformTime(CurrentTime);

				// Clamp the range and transform it into inner space
				ValidRange = TRange<FFrameTime>::Intersection(ValidRange, ConvertToFrameTimeRange(SubData->ParentPlayRange.Value));
				if (ValidRange.IsEmpty())
				{
					break;
				}

				ValidRange = SubData->OuterToInnerTransform.ComputeTraversedHull(ValidRange);
			}
		}

		SubSequenceRange = ConvertToDiscreteRange(ValidRange);
	}
	else
	{
		// Ensure in this case we also reset the CurrentTimeBreadcrumbs.
		CurrentTimeBreadcrumbs.Reset();
	}
}

void FSequencer::UpdateSequencerCustomizations(const UMovieSceneSequence* PreviousFocusedSequence)
{
	UMovieSceneSequence* FocusedSequence = GetFocusedMovieSceneSequence();
	check(FocusedSequence != nullptr);
	ViewModel->UpdateSequencerCustomizations(PreviousFocusedSequence);
	TArrayView<const FSequencerCustomizationInfo> CustomizationInfos = ViewModel->GetActiveCustomizationInfos();

	// Apply customizations to our editor.
	SequencerWidget->ApplySequencerCustomizations(CustomizationInfos);

	// Apply customizations to ourselves.
	OnPaste.Reset();

	for (const FSequencerCustomizationInfo& CustomizationInfo : CustomizationInfos)
	{
		if (CustomizationInfo.OnPaste.IsBound())
		{
			OnPaste.Add(CustomizationInfo.OnPaste);
		}
	}

	// TODO: Allow customization of which custom bindings are allowed? For now just iterate over all subclasses and sort by priority

	// Cache custom spawnable types
	RefreshSupportedCustomBindingTypes();
}

void FSequencer::RerunConstructionScripts()
{
	TSet<TWeakObjectPtr<AActor> > BoundActors;

	FMovieSceneRootEvaluationTemplateInstance& RootTemplate = GetEvaluationTemplate();
		
	UMovieSceneSequence* Sequence = RootTemplate.GetSequence(MovieSceneSequenceID::Root);
	if (!Sequence)
	{
		return;
	}

	TArray < TPair<FMovieSceneSequenceID, FGuid> > BoundGuids;

	GetConstructionScriptActors(Sequence->GetMovieScene(), MovieSceneSequenceID::Root, BoundActors, BoundGuids);

	const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(RootTemplateInstance.GetCompiledDataID());
	if (Hierarchy)
	{
		FMovieSceneEvaluationTreeRangeIterator Iter = Hierarchy->GetTree().IterateFromTime(PlayPosition.GetCurrentPosition().FrameNumber);

		for (const FMovieSceneSubSequenceTreeEntry& Entry : Hierarchy->GetTree().GetAllData(Iter.Node()))
		{
			UMovieSceneSequence* SubSequence = Hierarchy->FindSubSequence(Entry.SequenceID);
			if (SubSequence)
			{
				GetConstructionScriptActors(SubSequence->GetMovieScene(), Entry.SequenceID, BoundActors, BoundGuids);
			}
		}
	}

	for (TWeakObjectPtr<AActor> BoundActor : BoundActors)
	{
		if (BoundActor.IsValid())
		{
			BoundActor.Get()->RerunConstructionScripts();
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (TPair<FMovieSceneSequenceID, FGuid> BoundGuid : BoundGuids)
	{
		State.Invalidate(BoundGuid.Value, BoundGuid.Key);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FSequencer::GetConstructionScriptActors(UMovieScene* MovieScene, FMovieSceneSequenceIDRef SequenceID, TSet<TWeakObjectPtr<AActor> >& BoundActors, TArray < TPair<FMovieSceneSequenceID, FGuid> >& BoundGuids)
{
	auto ShouldRerunConstructionScriptsForClass = [](UClass* Class)
	{
		// If any blueprints in the class hierarchy are marked as bRunConstructionScriptInSequencer, we need to rerun them
		while (Class)
		{
			UBlueprint* Blueprint = Cast<UBlueprint>(Class->ClassGeneratedBy);
			if (!Blueprint)
			{
				// As soon as we find a non BP class, we can't re-run construction scripts any more
				return false;
			}
			else if (Blueprint->bRunConstructionScriptInSequencer)
			{
				return true;
			}

			Class = Class->GetSuperClass();
		}
		return false;
	};

	for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
	{
		FGuid ThisGuid = MovieScene->GetPossessable(Index).GetGuid();

		FMovieSceneBinding* MovieSceneBinding = MovieScene->FindBinding(ThisGuid);
		if (MovieSceneBinding)
		{
			if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = MovieSceneBinding->FindDecoration<UMovieSceneMuteSoloDecoration>())
			{
				if (MuteSoloDecoration->IsMuted())
				{
					continue;
				}
			}
		}
		
		for (TWeakObjectPtr<> WeakObject : FindBoundObjects(ThisGuid, SequenceID))
		{
			AActor* Actor = Cast<AActor>(WeakObject.Get());
			if (Actor && ShouldRerunConstructionScriptsForClass(Actor->GetClass()))
			{
				BoundActors.Add(Actor);
				BoundGuids.Add(TPair<FMovieSceneSequenceID, FGuid>(SequenceID, ThisGuid));
			}
		}
	}

	for (int32 Index = 0; Index < MovieScene->GetSpawnableCount(); ++Index)
	{
		FGuid ThisGuid = MovieScene->GetSpawnable(Index).GetGuid();

		FMovieSceneBinding* MovieSceneBinding = MovieScene->FindBinding(ThisGuid);
		if (MovieSceneBinding)
		{
			if (UMovieSceneMuteSoloDecoration* MuteSoloDecoration = MovieSceneBinding->FindDecoration<UMovieSceneMuteSoloDecoration>())
			{
				if (MuteSoloDecoration->IsMuted())
				{
					continue;
				}
			}
		}

		for (TWeakObjectPtr<> WeakObject : FindBoundObjects(ThisGuid, SequenceID))
		{
			AActor* Actor = Cast<AActor>(WeakObject.Get());
			if (Actor && ShouldRerunConstructionScriptsForClass(Actor->GetClass()))
			{
				BoundActors.Add(Actor);
				BoundGuids.Add(TPair<FMovieSceneSequenceID, FGuid>(SequenceID, ThisGuid));
			}
		}
	}
}

void FSequencer::DeleteSections(const TSet<UMovieSceneSection*>& Sections)
{
	if (IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	UMovieScene* MovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();
	bool bAnythingRemoved = false;

	FScopedTransaction DeleteSectionTransaction( NSLOCTEXT("Sequencer", "DeleteSection_Transaction", "Delete Section") );

	for (UMovieSceneSection* Section : Sections)
	{
		if (!Section || Section->IsLocked())
		{
			continue;
		}

		// Check if section interface allows deletion
		TSharedPtr<UE::Sequencer::FSectionModel> SectionModel = GetNodeTree()->GetSectionModel(Section);
		if (SectionModel && SectionModel->GetSectionInterface() && !SectionModel->GetSectionInterface()->IsDeletable())
		{
			continue;
		}

		UObject* SectionOuter = Section->GetOuter();

		// Check if section comes from a decoration that provides sections
		if (IMovieSceneSectionProviderDecoration* SectionProvider = Cast<IMovieSceneSectionProviderDecoration>(SectionOuter))
		{
			// Section is provided by a decoration - remove section if there are more than one, or remove the decoration from its container if it's the last section
			UMovieSceneDecorationContainerObject* Container = Cast<UMovieSceneDecorationContainerObject>(SectionOuter->GetOuter());
			if (Container)
			{
				Container->SetFlags(RF_Transactional);
				Container->Modify();
				if (SectionProvider->SupportsMultipleSections() && SectionProvider->GetSections().Num() > 1)
				{
					SectionOuter->SetFlags(RF_Transactional);
					SectionOuter->Modify();
					SectionProvider->RemoveSection(*Section);
					if (UMovieSceneTrack* OuterTrack = Section->GetTypedOuter<UMovieSceneTrack>())
					{
						OuterTrack->Modify();
						OuterTrack->UpdateEasing();
					}
				}
				else
				{
					Container->RemoveDecoration(SectionOuter->GetClass());
				}
				bAnythingRemoved = true;
			}
		}
		else if (UMovieSceneTrack* Track = Cast<UMovieSceneTrack>(SectionOuter))
		{
			// Standard case: section is owned by a track
			Track->SetFlags(RF_Transactional);
			Track->Modify();
			Track->RemoveSection(*Section);
			Track->UpdateEasing();
			bAnythingRemoved = true;
		}
	}

	if (bAnythingRemoved)
	{
		// Full refresh required just in case the last section was removed from any track.
		NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemRemoved );
	}

	ViewModel->GetSelection()->TrackArea.Empty();
}


void FSequencer::DeleteSelectedKeys()
{
	using namespace UE::Sequencer;
	const bool bUseParallel = true;

	FScopedTransaction DeleteKeysTransaction( NSLOCTEXT("Sequencer", "DeleteSelectedKeys_Transaction", "Delete Selected Keys") );
	bool bAnythingRemoved = false;

	FSelectedKeysByChannel KeysByChannel(ViewModel->GetSelection()->KeySelection);
	TSet<UObject*> ModifiedObjects;
	for (const FSelectedChannelInfo& ChannelInfo : KeysByChannel.SelectedChannels)
	{
		if (ChannelInfo.OwningSection->IsReadOnly() || ChannelInfo.KeyHandles.Num() == 0)
		{
			continue;
		}

		UObject* Owner = nullptr;
		const FMovieSceneChannelMetaData* ChannelMetaData = ChannelInfo.Channel.GetMetaData();
		if (ChannelMetaData)
		{
			Owner = ChannelMetaData->WeakOwningObject.Get();
		}
		if (!Owner)
		{
			Owner = ChannelInfo.OwningSection;
		}

		if (!ModifiedObjects.Contains(Owner))
		{
			Owner->Modify();
			ModifiedObjects.Add(Owner);
			bAnythingRemoved = true;
		}
	}

	ParallelFor(KeysByChannel.SelectedChannels.Num(), [this, &KeysByChannel](int32 Index)
	{
		const FSelectedChannelInfo& ChannelInfo = KeysByChannel.SelectedChannels[Index];

		if (ChannelInfo.OwningSection->IsReadOnly())
		{
			return;
		}

		if (ChannelInfo.KeyHandles.Num() == 0)
		{
			return;
		}

		TSharedPtr<FChannelModel> Channel = ViewModel->GetSelection()->KeySelection.GetModelForKey(ChannelInfo.KeyHandles[0]);
		if (!Channel)
		{
			return;
		}

		TSharedPtr<IKeyArea> KeyArea = Channel->GetKeyArea();
		if (!KeyArea)
		{
			return;
		}

		KeyArea->DeleteKeys(ChannelInfo.KeyHandles, GetLocalTime().Time.FloorToFrame());

	},bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

	if (bAnythingRemoved)
	{
		NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::TrackValueChanged );

		ViewModel->GetSelection()->KeySelection.Empty();
	}
}


void FSequencer::SetInterpTangentMode(ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode)
{
	if (ViewModel->GetSelection()->KeySelection.Num() == 0)
	{
		return;
	}

	FScopedTransaction SetInterpTangentModeTransaction(NSLOCTEXT("Sequencer", "SetInterpTangentMode_Transaction", "Set Interpolation and Tangent Mode"));
	bool bAnythingChanged = false;

	FSelectedKeysByChannel KeysByChannel(ViewModel->GetSelection()->KeySelection);
	TSet<UMovieSceneSection*> ModifiedSections;

	const FName FloatChannelTypeName = FMovieSceneFloatChannel::StaticStruct()->GetFName();
	const FName DoubleChannelTypeName = FMovieSceneDoubleChannel::StaticStruct()->GetFName();

	// @todo: sequencer-timecode: move this float-specific logic elsewhere to make it extensible for any channel type
	for (const FSelectedChannelInfo& ChannelInfo : KeysByChannel.SelectedChannels)
	{
		FMovieSceneChannel* ChannelPtr = ChannelInfo.Channel.Get();
		if (!ChannelPtr)
		{
			continue;
		}

		const FName ChannelTypeName = ChannelInfo.Channel.GetChannelTypeName();
		if (ChannelTypeName == FloatChannelTypeName || ChannelTypeName == DoubleChannelTypeName)
		{
			if (!ModifiedSections.Contains(ChannelInfo.OwningSection))
			{
				ChannelInfo.OwningSection->Modify();
				ModifiedSections.Add(ChannelInfo.OwningSection);
			}

			if (ChannelTypeName == FloatChannelTypeName)
			{
				FMovieSceneFloatChannel* Channel = static_cast<FMovieSceneFloatChannel*>(ChannelPtr);
				TMovieSceneChannelData<FMovieSceneFloatValue> ChannelData = Channel->GetData();

				TArrayView<FMovieSceneFloatValue> Values = ChannelData.GetValues();

				for (FKeyHandle Handle : ChannelInfo.KeyHandles)
				{
					const int32 KeyIndex = ChannelData.GetIndex(Handle);
					if (KeyIndex != INDEX_NONE)
					{
						Values[KeyIndex].InterpMode = InterpMode;
						Values[KeyIndex].TangentMode = TangentMode;
						bAnythingChanged = true;
					}
				}

				Channel->AutoSetTangents();
			}
			else if (ChannelTypeName == DoubleChannelTypeName)
			{
				FMovieSceneDoubleChannel* Channel = static_cast<FMovieSceneDoubleChannel*>(ChannelPtr);
				TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = Channel->GetData();

				TArrayView<FMovieSceneDoubleValue> Values = ChannelData.GetValues();

				for (FKeyHandle Handle : ChannelInfo.KeyHandles)
				{
					const int32 KeyIndex = ChannelData.GetIndex(Handle);
					if (KeyIndex != INDEX_NONE)
					{
						Values[KeyIndex].InterpMode = InterpMode;
						Values[KeyIndex].TangentMode = TangentMode;
						bAnythingChanged = true;
					}
				}

				Channel->AutoSetTangents();
			}
		}
	}

	if (bAnythingChanged)
	{
		NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::TrackValueChanged );
	}
}

void FSequencer::ToggleInterpTangentWeightMode()
{
	// @todo: sequencer-timecode: move this float-specific logic elsewhere to make it extensible for any channel type
	if (ViewModel->GetSelection()->KeySelection.Num() == 0)
	{
		return;
	}

	FScopedTransaction SetInterpTangentWeightModeTransaction(NSLOCTEXT("Sequencer", "ToggleInterpTangentWeightMode_Transaction", "Toggle Tangent Weight Mode"));
	bool bAnythingChanged = false;

	FSelectedKeysByChannel KeysByChannel(ViewModel->GetSelection()->KeySelection);
	TSet<UMovieSceneSection*> ModifiedSections;

	const FName FloatChannelTypeName = FMovieSceneFloatChannel::StaticStruct()->GetFName();
	const FName DoubleChannelTypeName = FMovieSceneDoubleChannel::StaticStruct()->GetFName();

	// Remove all tangent weights unless we find a compatible key that does not have weights yet
	ERichCurveTangentWeightMode WeightModeToApply = RCTWM_WeightedNone;

	// First off iterate all the current keys and find any that don't have weights
	for (const FSelectedChannelInfo& ChannelInfo : KeysByChannel.SelectedChannels)
	{
		FMovieSceneChannel* ChannelPtr = ChannelInfo.Channel.Get();
		if (!ChannelPtr)
		{
			continue;
		}

		if (ChannelInfo.Channel.GetChannelTypeName() == FloatChannelTypeName)
		{
			FMovieSceneFloatChannel* Channel = static_cast<FMovieSceneFloatChannel*>(ChannelPtr);
			TMovieSceneChannelData<FMovieSceneFloatValue> ChannelData = Channel->GetData();

			TArrayView<FMovieSceneFloatValue> Values = ChannelData.GetValues();

			for (FKeyHandle Handle : ChannelInfo.KeyHandles)
			{
				const int32 KeyIndex = ChannelData.GetIndex(Handle);
				if (KeyIndex != INDEX_NONE && Values[KeyIndex].InterpMode == RCIM_Cubic && Values[KeyIndex].Tangent.TangentWeightMode == RCTWM_WeightedNone)
				{
					WeightModeToApply = RCTWM_WeightedBoth;
					goto assign_weights;
				}
			}
		}
		else if (ChannelInfo.Channel.GetChannelTypeName() == DoubleChannelTypeName)
		{
			FMovieSceneDoubleChannel* Channel = static_cast<FMovieSceneDoubleChannel*>(ChannelPtr);
			TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = Channel->GetData();

			TArrayView<FMovieSceneDoubleValue> Values = ChannelData.GetValues();

			for (FKeyHandle Handle : ChannelInfo.KeyHandles)
			{
				const int32 KeyIndex = ChannelData.GetIndex(Handle);
				if (KeyIndex != INDEX_NONE && Values[KeyIndex].InterpMode == RCIM_Cubic && Values[KeyIndex].Tangent.TangentWeightMode == RCTWM_WeightedNone)
				{
					WeightModeToApply = RCTWM_WeightedBoth;
					goto assign_weights;
				}
			}
		}
	}

assign_weights:

	// Assign the new weight mode for all cubic keys
	for (const FSelectedChannelInfo& ChannelInfo : KeysByChannel.SelectedChannels)
	{
		FMovieSceneChannel* ChannelPtr = ChannelInfo.Channel.Get();
		if (!ChannelPtr)
		{
			continue;
		}

		const FName ChannelTypeName = ChannelInfo.Channel.GetChannelTypeName();
		if (ChannelTypeName == FloatChannelTypeName || ChannelTypeName == DoubleChannelTypeName)
		{
			if (!ModifiedSections.Contains(ChannelInfo.OwningSection))
			{
				ChannelInfo.OwningSection->Modify();
				ModifiedSections.Add(ChannelInfo.OwningSection);
			}

			if (ChannelTypeName == FloatChannelTypeName)
			{
				FMovieSceneFloatChannel* Channel = static_cast<FMovieSceneFloatChannel*>(ChannelPtr);
				TMovieSceneChannelData<FMovieSceneFloatValue> ChannelData = Channel->GetData();

				TArrayView<FMovieSceneFloatValue> Values = ChannelData.GetValues();

				for (FKeyHandle Handle : ChannelInfo.KeyHandles)
				{
					const int32 KeyIndex = ChannelData.GetIndex(Handle);
					if (KeyIndex != INDEX_NONE && Values[KeyIndex].InterpMode == RCIM_Cubic)
					{
						Values[KeyIndex].Tangent.TangentWeightMode = WeightModeToApply;
						bAnythingChanged = true;
					}
				}

				Channel->AutoSetTangents();
			}
			else if (ChannelTypeName == DoubleChannelTypeName)
			{
				FMovieSceneDoubleChannel* Channel = static_cast<FMovieSceneDoubleChannel*>(ChannelPtr);
				TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = Channel->GetData();

				TArrayView<FMovieSceneDoubleValue> Values = ChannelData.GetValues();

				for (FKeyHandle Handle : ChannelInfo.KeyHandles)
				{
					const int32 KeyIndex = ChannelData.GetIndex(Handle);
					if (KeyIndex != INDEX_NONE && Values[KeyIndex].InterpMode == RCIM_Cubic)
					{
						Values[KeyIndex].Tangent.TangentWeightMode = WeightModeToApply;
						bAnythingChanged = true;
					}
				}

				Channel->AutoSetTangents();
			}
		}
	}

	if (bAnythingChanged)
	{
		NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}
}

void FSequencer::SnapToFrame()
{
	FScopedTransaction SnapToFrameTransaction(NSLOCTEXT("Sequencer", "SnapToFrame_Transaction", "Snap Selected Keys to Frame"));
	bool bAnythingChanged = false;

	FSelectedKeysByChannel KeysByChannel(ViewModel->GetSelection()->KeySelection);
	TSet<UMovieSceneSection*> ModifiedSections;

	TArray<FFrameNumber> KeyTimesScratch;
	for (const FSelectedChannelInfo& ChannelInfo : KeysByChannel.SelectedChannels)
	{
		FMovieSceneChannel* Channel = ChannelInfo.Channel.Get();
		if (Channel)
		{
			if (!ModifiedSections.Contains(ChannelInfo.OwningSection))
			{
				ChannelInfo.OwningSection->Modify();
				ModifiedSections.Add(ChannelInfo.OwningSection);
			}

			const int32 NumKeys = ChannelInfo.KeyHandles.Num();
			KeyTimesScratch.Reset(NumKeys);
			KeyTimesScratch.SetNum(NumKeys);

			Channel->GetKeyTimes(ChannelInfo.KeyHandles, KeyTimesScratch);

			FFrameRate TickResolution  = GetFocusedTickResolution();
			FFrameRate DisplayRate     = GetFocusedDisplayRate();

			for (FFrameNumber& Time : KeyTimesScratch)
			{
				// Convert to frame
				FFrameNumber PlayFrame    = FFrameRate::TransformTime(Time,      TickResolution, DisplayRate).RoundToFrame();
				FFrameNumber SnappedFrame = FFrameRate::TransformTime(PlayFrame, DisplayRate, TickResolution).RoundToFrame();

				Time = SnappedFrame;
			}

			Channel->SetKeyTimes(ChannelInfo.KeyHandles, KeyTimesScratch);
			bAnythingChanged = true;
		}
	}

	if (bAnythingChanged)
	{
		NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::TrackValueChanged );
	}
}


bool FSequencer::CanSnapToFrame() const
{
	const bool bKeysSelected = ViewModel->GetSelection()->KeySelection.Num() > 0;

	return bKeysSelected;
}

void FSequencer::TransformSelection(FFrameTime InDeltaTime, float InScale)
{
	using namespace UE::Sequencer;

	if (!ViewModel.IsValid())
	{
		return;
	}

	const TViewModelPtr<FSequenceModel> RootSequenceModel = ViewModel->GetRootSequenceModel();
	if (!RootSequenceModel.IsValid())
	{
		return;
	}

	TSet<FSequencerSelectedKey> SelectedKeys;
	FSequencerSelectedKey::AppendKeySelection(SelectedKeys, GetSelection().KeySelection);

	TSet<TSharedPtr<FChannelModel>> AllChannels;
	SequencerHelpers::GetAllChannels(RootSequenceModel, AllChannels);

	TSet<TWeakViewModelPtr<FChannelModel>> WeakChannels;
	for (const TSharedPtr<FChannelModel>& Channel : AllChannels)
	{
		if (Channel.IsValid())
		{
			WeakChannels.Add(TWeakViewModelPtr<FChannelModel>(Channel));
		}
	}

	const TSet<int32> SelectedMarkedFrames = GetSelection().MarkedFrames.GetSelected();
	const TSet<UMovieSceneSection*> SelectedSections = GetSelection().GetSelectedSections();
	if (SelectedMarkedFrames.IsEmpty() && SelectedSections.IsEmpty())
	{
		KeyHelperUtils::TransformSelectedOrRelativeKeys(*this, WeakChannels, SelectedKeys, InDeltaTime, InScale);
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("TransformSelection_Transaction", "Transform Selection"));

	for (TSet<FSequencerSelectedKey>::TIterator It = SelectedKeys.CreateIterator(); It; ++It)
	{
		if (SelectedSections.Contains(It->Section))
		{
			It.RemoveCurrent();
		}
	}

	const bool bSectionsChanged = KeyHelperUtils::TransformSectionSelection(*this, SelectedSections, InDeltaTime, InScale, false);
	const bool bKeysChanged = KeyHelperUtils::TransformKeySelection(*this, SelectedKeys, InDeltaTime, InScale, false);
	const bool bMarksChanged = KeyHelperUtils::TransformMarkedFrameSelection(*this, SelectedMarkedFrames, InDeltaTime, InScale);

	if (bMarksChanged && !bKeysChanged && !bSectionsChanged)
	{
		NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}
}

void FSequencer::TranslateSelection(bool bTranslateLeft)
{
	if (IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	int32 Shift = bTranslateLeft ? -1 : 1;
	FFrameTime Delta = FQualifiedFrameTime(Shift, GetFocusedDisplayRate()).ConvertTo(GetFocusedTickResolution());
	TransformSelection(Delta, 1.f);
}

void FSequencer::StretchTime(FFrameTime InDeltaTime)
{
	// From the current time, find all the keys and sections to the right and move them by InDeltaTime
	UMovieScene* FocusedMovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	FScopedTransaction StretchTimeTransaction(NSLOCTEXT("Sequencer", "StretchTime", "Stretch Time"));

	TRange<FFrameNumber> CachedSelectionRange = GetSelectionRange();

	TRange<FFrameNumber> SelectionRange;

	if (InDeltaTime > 0)
	{
		SelectionRange.SetLowerBound(GetLocalTime().Time.FrameNumber+1);
		SelectionRange.SetUpperBound(TRangeBound<FFrameNumber>::Open());
	}
	else
	{
		SelectionRange.SetUpperBound(GetLocalTime().Time.FrameNumber-1);
		SelectionRange.SetLowerBound(TRangeBound<FFrameNumber>::Open());
	}

	FocusedMovieScene->SetSelectionRange(SelectionRange);
	SelectInSelectionRange(true, true, true);
	TransformSelection(InDeltaTime, 1.f);

	// Return state
	FocusedMovieScene->SetSelectionRange(CachedSelectionRange);
	ViewModel->GetSelection()->Empty(); //todo restore key and section selection
}

void FSequencer::ShrinkTime(FFrameTime InDeltaTime)
{
	using namespace UE::Sequencer;

	// From the current time, find all the keys and sections to the right and move them by -InDeltaTime
	UMovieScene* FocusedMovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	FScopedTransaction StretchTimeTransaction(NSLOCTEXT("Sequencer", "ShrinkTime", "Shrink Time"));

	TRange<FFrameNumber> CachedSelectionRange = GetSelectionRange();

	// First, check if there's any keys/sections within InDeltaTime

	TRange<FFrameNumber> CheckRange;

	if (InDeltaTime > 0)
	{
		CheckRange.SetLowerBound(GetLocalTime().Time.FrameNumber + 1);
		CheckRange.SetUpperBound(GetLocalTime().Time.FrameNumber + InDeltaTime.FrameNumber);
	}
	else
	{
		CheckRange.SetUpperBound(GetLocalTime().Time.FrameNumber - InDeltaTime.FrameNumber);
		CheckRange.SetLowerBound(GetLocalTime().Time.FrameNumber - 1);
	}

	FocusedMovieScene->SetSelectionRange(CheckRange);
	SelectInSelectionRange(true, true, true);

	TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();
	if (Selection->KeySelection.Num() > 0)
	{
		FNotificationInfo Info(FText::Format(NSLOCTEXT("Sequencer", "ShrinkTimeFailedKeys", "Shrink failed. There are {0} keys in between"), Selection->KeySelection.Num()));
		Info.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);

		// Return state
		FocusedMovieScene->SetSelectionRange(CachedSelectionRange);
		Selection->Empty(); //todo restore key and section selection
		return;
	}

	if (Selection->GetSelectedSections().Num() > 0)
	{
		FNotificationInfo Info(FText::Format(NSLOCTEXT("Sequencer", "ShrinkTimeFailedSections", "Shrink failed. There are {0} sections in between"), Selection->GetSelectedSections().Num()));
		Info.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);

		// Return state
		FocusedMovieScene->SetSelectionRange(CachedSelectionRange);
		Selection->Empty(); //todo restore key and section selection
		return;
	}

	TRange<FFrameNumber> SelectionRange;

	if (InDeltaTime > 0)
	{
		SelectionRange.SetLowerBound(GetLocalTime().Time.FrameNumber + 1);
		SelectionRange.SetUpperBound(TRangeBound<FFrameNumber>::Open());
	}
	else
	{
		SelectionRange.SetUpperBound(GetLocalTime().Time.FrameNumber - 1);
		SelectionRange.SetLowerBound(TRangeBound<FFrameNumber>::Open());
	}

	FocusedMovieScene->SetSelectionRange(SelectionRange);
	SelectInSelectionRange(true, true, true);
	TransformSelection(-InDeltaTime, 1.f);

	// Return state
	FocusedMovieScene->SetSelectionRange(CachedSelectionRange);
	Selection->Empty(); //todo restore key and section selection
}

TSharedPtr<UE::Sequencer::SimpleView::FSimpleViewTimeline> FSequencer::GetSimpleViewTimeline() const
{
	return SequencerWidget.IsValid() ? SequencerWidget->GetSimpleViewTimeline() : nullptr;
}

bool FSequencer::CanAddTransformKeysForSelectedObjects(const EMovieSceneTransformChannel InChannel) const
{
	const TSharedPtr<UE::Sequencer::SimpleView::FSimpleViewTimeline> Timeline = GetSimpleViewTimeline();
	if (Timeline.IsValid() && Timeline->IsInSimpleView())
	{
		return Timeline->CanExecuteTransformKeyCommand(InChannel);
	}

	return CanAddTransformKeysForSelectedObjectsInternal(InChannel);
}

bool FSequencer::CanAddTransformKeysForSelectedObjectsInternal(const EMovieSceneTransformChannel InChannel) const
{
	for (int32 i = 0; i < TrackEditors.Num(); ++i)
	{
		if (TrackEditors[i]->HasTransformKeyBindings() && TrackEditors[i]->CanAddTransformKeysForSelectedObjects())
		{
			return true;
		}
	}

	return false;
}

bool FSequencer::CanDeleteTransformKeysForSelectedObjects(const EMovieSceneTransformChannel InChannel) const
{
	const TSharedPtr<UE::Sequencer::SimpleView::FSimpleViewTimeline> Timeline = GetSimpleViewTimeline();
	return (Timeline.IsValid() && Timeline->IsInSimpleView())
		? Timeline->CanExecuteDeleteTransformKeyCommand(InChannel)
		: CanAddTransformKeysForSelectedObjects(InChannel);
}

void FSequencer::OnAddTransformKeysForSelectedObjects(const EMovieSceneTransformChannel InChannel)
{
	const TSharedPtr<UE::Sequencer::SimpleView::FSimpleViewTimeline> Timeline = GetSimpleViewTimeline();
	if (Timeline.IsValid() && Timeline->IsInSimpleView())
	{
		Timeline->ExecuteTransformKeyCommand(InChannel);
		return;
	}

	AddTransformKeysForSelectedObjectsInternal(InChannel);
}

void FSequencer::AddTransformKeysForSelectedObjectsInternal(const EMovieSceneTransformChannel InChannel)
{
	FScopedTransaction SetKeyTransaction(NSLOCTEXT("Sequencer", "SetTransformKey_Transaction", "Set Transform Key"));

	TArray<TSharedPtr<ISequencerTrackEditor>> PossibleTrackEditors;
	bool AtLeastOneHasPriority = false;
	for (int32 i = 0; i < TrackEditors.Num(); ++i)
	{
		if (TrackEditors[i]->HasTransformKeyBindings()  && TrackEditors[i]->CanAddTransformKeysForSelectedObjects())
		{
			PossibleTrackEditors.Add(TrackEditors[i]);
			if (TrackEditors[i]->HasTransformKeyOverridePriority())
			{
				AtLeastOneHasPriority = true;
			}
		}
	}
	for (int32 i = 0; i < PossibleTrackEditors.Num(); ++i)
	{
		if (AtLeastOneHasPriority)
		{
			if (PossibleTrackEditors[i]->HasTransformKeyOverridePriority())
			{
				PossibleTrackEditors[i]->OnAddTransformKeysForSelectedObjects(InChannel);
			}
		}
		else
		{
			PossibleTrackEditors[i]->OnAddTransformKeysForSelectedObjects(InChannel);
		}
	}
}

void FSequencer::OnDeleteTransformKeysForSelectedObjects(const EMovieSceneTransformChannel InChannel)
{
	const TSharedPtr<UE::Sequencer::SimpleView::FSimpleViewTimeline> Timeline = GetSimpleViewTimeline();
	if (Timeline.IsValid() && Timeline->IsInSimpleView())
	{
		Timeline->ExecuteDeleteTransformKeyCommand(InChannel);
		return;
	}

	FScopedTransaction DeleteKeyTransaction(NSLOCTEXT("Sequencer", "DeleteTransformKey_Transaction", "Delete Transform Key"));

	TArray<TSharedPtr<ISequencerTrackEditor>> PossibleTrackEditors;
	bool AtLeastOneHasPriority = false;
	for (int32 i = 0; i < TrackEditors.Num(); ++i)
	{
		if (TrackEditors[i]->HasTransformKeyBindings() && TrackEditors[i]->CanAddTransformKeysForSelectedObjects())
		{
			PossibleTrackEditors.Add(TrackEditors[i]);
			if (TrackEditors[i]->HasTransformKeyOverridePriority())
			{
				AtLeastOneHasPriority = true;
			}
		}
	}
	for (int32 i = 0; i < PossibleTrackEditors.Num(); ++i)
	{
		if (AtLeastOneHasPriority)
		{
			if (PossibleTrackEditors[i]->HasTransformKeyOverridePriority())
			{
				PossibleTrackEditors[i]->OnDeleteTransformKeysForSelectedObjects(InChannel);
			}
		}
		else
		{
			PossibleTrackEditors[i]->OnDeleteTransformKeysForSelectedObjects(InChannel);
		}
	}
}

void FSequencer::OnTogglePilotCamera()
{
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC != nullptr && LevelVC->AllowsCinematicControl() && LevelVC->GetViewMode() != VMI_Unknown)
		{
			bool bLockedAny = false;

			// If locked to the camera cut track, pilot the camera that the camera cut track is locked to
			if (IsPerspectiveViewportCameraCutEnabled())
			{
				SetPerspectiveViewportCameraCutEnabled(false);

				if (LevelVC->GetCinematicActorLock().HasValidLockedActor())
				{
					LevelVC->SetActorLock(LevelVC->GetCinematicActorLock().GetLockedActor());
					LevelVC->SetCinematicActorLock(nullptr);
					LevelVC->bLockedCameraView = true;
					LevelVC->UpdateViewForLockedActor();
					LevelVC->Invalidate();
					bLockedAny = true;
				}
			}
			else if (!LevelVC->GetActorLock().HasValidLockedActor())
			{
				// If NOT piloting, and was previously piloting a camera, start piloting that previous camera
				if (LevelVC->GetPreviousActorLock().HasValidLockedActor())
				{
					LevelVC->SetCinematicActorLock(nullptr);
					LevelVC->SetActorLock(LevelVC->GetPreviousActorLock().GetLockedActor());
					LevelVC->bLockedCameraView = true;
					LevelVC->UpdateViewForLockedActor();
					LevelVC->Invalidate();
					bLockedAny = true;
				}
				// If NOT piloting, and was previously locked to the camera cut track, start piloting the camera that the camera cut track was previously locked to
				else if (LevelVC->GetPreviousCinematicActorLock().HasValidLockedActor())
				{
					LevelVC->SetCinematicActorLock(nullptr);
					LevelVC->SetActorLock(LevelVC->GetPreviousCinematicActorLock().GetLockedActor());
					LevelVC->bLockedCameraView = true;
					LevelVC->UpdateViewForLockedActor();
					LevelVC->Invalidate();
					bLockedAny = true;
				}
			}
			
			if (!bLockedAny)
			{
				LevelVC->SetCinematicActorLock(nullptr);
				LevelVC->SetActorLock(nullptr);
				LevelVC->bLockedCameraView = false;
				LevelVC->UpdateViewForLockedActor();
				LevelVC->Invalidate();
			}
		}
	}
}

bool FSequencer::IsPilotCamera() const
{
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC != nullptr && LevelVC->AllowsCinematicControl() && LevelVC->GetViewMode() != VMI_Unknown)
		{
			if (LevelVC->GetActorLock().HasValidLockedActor())
			{
				return true;
			}
		}
	}

	return false;
}

void FSequencer::OnActorsDropped( const TArray<TWeakObjectPtr<AActor> >& Actors )
{
	AddActors(Actors);
}


void FSequencer::NotifyMovieSceneDataChangedInternal()
{
	NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::Unknown );
}


void FSequencer::NotifyMovieSceneDataChanged( EMovieSceneDataChangeType DataChangeType )
{
	if (!GetFocusedMovieSceneSequence()->GetMovieScene())
	{
		if (RootSequence.IsValid())
		{
			ResetToNewRootSequence(*RootSequence.Get());
		}
		else
		{
			UE_LOGF(LogSequencer, Error, "Fatal error, focused movie scene no longer valid and there is no root sequence to default to.");
		}
	}
	bool bResetCache = true;
	if (DataChangeType == EMovieSceneDataChangeType::RefreshTree)
	{
		bNeedTreeRefresh = true;
		OnMovieSceneDataChangedDelegate.Broadcast(DataChangeType);
		return;
	}
	else if ( DataChangeType == EMovieSceneDataChangeType::MovieSceneStructureItemRemoved ||
		DataChangeType == EMovieSceneDataChangeType::MovieSceneStructureItemsChanged ||
		DataChangeType == EMovieSceneDataChangeType::Unknown )
	{
		// When structure items are removed, or we don't know what may have changed, refresh the tree and instances immediately so that the data
		// is in a consistent state when the UI is updated during the next tick.
		EMovieScenePlayerStatus::Type StoredPlaybackState = GetPlaybackStatus();
		SetPlaybackStatus( EMovieScenePlayerStatus::Stopped );
		SelectionPreview.Empty();
		RefreshTree();
		SetPlaybackStatus( StoredPlaybackState );
	}
	else if (DataChangeType == EMovieSceneDataChangeType::TrackValueChangedRefreshImmediately)
	{
		// Evaluate now
		EvaluateInternal(PlayPosition.GetCurrentPositionAsRange());
		bResetCache = false;
	}
	else if (DataChangeType == EMovieSceneDataChangeType::RefreshAllImmediately)
	{
		RefreshTree();

		// Evaluate now
		EvaluateInternal(PlayPosition.GetCurrentPositionAsRange());
		//not sure ? bResetCache = false;
	}
	else
	{
		if (DataChangeType != EMovieSceneDataChangeType::TrackValueChanged)
		{
			bNeedTreeRefresh = true;
		}
		else if (NodeTree->UpdateFiltersOnTrackValueChanged())
		{
			bNeedTreeRefresh = true;
		}
		if (DataChangeType == EMovieSceneDataChangeType::TrackValueChanged)
		{
			bResetCache = false;
		}
	}

	bGlobalMarkedFramesCached = false;
	bNeedsEvaluate = true;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if(bResetCache == true)
	{ 
		State.ClearObjectCaches(*this);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UpdatePlaybackRange();
	OnMovieSceneDataChangedDelegate.Broadcast(DataChangeType);
}

static bool bRefreshTreeGuard = false;
void FSequencer::RefreshTree()
{
	if (bRefreshTreeGuard == false)
	{
		TGuardValue<bool> Guard(bRefreshTreeGuard, true);

		SequencerWidget->UpdateLayoutTree();
		bNeedTreeRefresh = false;
		OnTreeViewChangedDelegate.Broadcast();
	}
}

void FSequencer::RecreateCurveEditor()
{
	using namespace UE::Sequencer;

	FSequencerCurveEditorApp* CurveEditorIntegration = FSequencerCurveEditorApp::Get(*this);
	if (ensure(CurveEditorIntegration))
	{
		CurveEditorIntegration->ResetCurveEditor();
		CurveEditorIntegration->RequestSyncSelection();
	}
}

FAnimatedRange FSequencer::GetViewRange() const
{
	FAnimatedRange AnimatedRange(FMath::Lerp(LastViewRange.GetLowerBoundValue(), TargetViewRange.GetLowerBoundValue(), ZoomCurve.GetLerp()),
		FMath::Lerp(LastViewRange.GetUpperBoundValue(), TargetViewRange.GetUpperBoundValue(), ZoomCurve.GetLerp()));

	if (ZoomAnimation.IsPlaying())
	{
		AnimatedRange.AnimationTarget = TargetViewRange;
	}

	return AnimatedRange;
}


FAnimatedRange FSequencer::GetClampRange() const
{
	return GetFocusedMovieSceneSequence()->GetMovieScene()->GetEditorData().GetWorkingRange();
}


void FSequencer::SetClampRange(TRange<double> InNewClampRange)
{
	FMovieSceneEditorData& EditorData = GetFocusedMovieSceneSequence()->GetMovieScene()->GetEditorData();
	EditorData.WorkStart = InNewClampRange.GetLowerBoundValue();
	EditorData.WorkEnd   = InNewClampRange.GetUpperBoundValue();
}


TOptional<TRange<FFrameNumber>> FSequencer::GetSubSequenceRange() const
{
	if (Settings->ShouldEvaluateSubSequencesInIsolation() || ActiveTemplateIDs.Num() == 1)
	{
		return TOptional<TRange<FFrameNumber>>();
	}
	return SubSequenceRange;
}


TRange<FFrameNumber> FSequencer::GetSelectionRange() const
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return TRange<FFrameNumber>();
	}

	return FocusedMovieScene->GetSelectionRange();
}


void FSequencer::SetSelectionRange(TRange<FFrameNumber> Range)
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	const bool bInitiallyEmpty = GetSelectionRange().IsEmpty();

	const FScopedTransaction Transaction(LOCTEXT("SetSelectionRange_Transaction", "Set Selection Range"));
	FocusedMovieScene->Modify();
	FocusedMovieScene->SetSelectionRange(Range);

	if (bInitiallyEmpty && GetLoopMode() != ESequencerLoopMode::SLM_LoopSelectionRange)
	{
		Settings->SetLoopMode(ESequencerLoopMode::SLM_LoopSelectionRange);
	}
}


void FSequencer::SetSelectionRangeEnd(FFrameTime EndFrame)
{
	using namespace UE::MovieScene;

	const bool bInitiallyEmpty = GetSelectionRange().IsEmpty();
	SnapSequencerTime(EndFrame);
	const FFrameNumber LocalTime = EndFrame.FrameNumber;

	const FFrameNumber StartFrame = (bInitiallyEmpty || GetSelectionRange().GetLowerBoundValue() >= LocalTime)
		? DiscreteInclusiveLower(GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange())
		: GetSelectionRange().GetLowerBoundValue();

	if (StartFrame >= LocalTime)
	{
		SetSelectionRange(TRange<FFrameNumber>(LocalTime - 1, LocalTime));
	}
	else
	{
		SetSelectionRange(TRange<FFrameNumber>(StartFrame, LocalTime));
	}

	if (bInitiallyEmpty && GetLoopMode() != ESequencerLoopMode::SLM_LoopSelectionRange)
	{
		Settings->SetLoopMode(ESequencerLoopMode::SLM_LoopSelectionRange);
	}
}


void FSequencer::SetSelectionRangeStart(FFrameTime StartFrame)
{
	using namespace UE::MovieScene;

	const bool bInitiallyEmpty = GetSelectionRange().IsEmpty();
	SnapSequencerTime(StartFrame);
	const FFrameNumber LocalTime = StartFrame.FrameNumber;

	const FFrameNumber EndFrame = (bInitiallyEmpty || GetSelectionRange().GetUpperBoundValue() <= LocalTime)
		? DiscreteExclusiveUpper(GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange())
		: GetSelectionRange().GetUpperBoundValue();

	if (EndFrame <= LocalTime)
	{
		SetSelectionRange(TRange<FFrameNumber>(LocalTime, LocalTime + 1));
	}
	else
	{
		SetSelectionRange(TRange<FFrameNumber>(LocalTime, EndFrame));
	}

	if (bInitiallyEmpty && GetLoopMode() != ESequencerLoopMode::SLM_LoopSelectionRange)
	{
		Settings->SetLoopMode(ESequencerLoopMode::SLM_LoopSelectionRange);
	}
}


void FSequencer::SelectInSelectionRange(const TSharedPtr<UE::Sequencer::FViewModel>& Item, const TRange<FFrameNumber>& SelectionRange, bool bSelectKeys, bool bSelectSections)
{
	using namespace UE::Sequencer;

	IOutlinerExtension* Outliner = Item->CastThis<IOutlinerExtension>();
	if (Outliner && Outliner->IsFilteredOut())
	{
		return;
	}

	TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();

	TSet<UMovieSceneSection*> SectionsWithKeys;

	if (bSelectKeys)
	{
		TArray<FKeyHandle> HandlesScratch;

		TSet<TSharedPtr<FChannelModel>> Channels;
		SequencerHelpers::GetAllChannels(Item, Channels);

		for (TSharedPtr<FChannelModel> Channel : Channels)
		{
			TSharedPtr<IKeyArea> KeyArea = Channel->GetKeyArea();
			UMovieSceneSection*  Section = KeyArea->GetOwningSection();

			if (Section)
			{
				HandlesScratch.Reset();
				KeyArea->GetKeyHandles(HandlesScratch, SelectionRange);

				if (HandlesScratch.Num() > 0)
				{
					SectionsWithKeys.Add(Section);
				}

				for (int32 Index = 0; Index < HandlesScratch.Num(); ++Index)
				{
					Selection->KeySelection.Select(Channel, HandlesScratch[Index]);
				}
			}
		}
	}

	if (bSelectSections)
	{
		for (TSharedPtr<FSectionModel> SectionModel : Item->GetDescendantsOfType<FSectionModel>())
		{
			TRange<FFrameNumber> SectionRange = SectionModel->GetRange();
			if (SectionRange.Overlaps(SelectionRange) && SectionRange.GetLowerBound().IsClosed() && SectionRange.GetUpperBound().IsClosed())
			{
				UMovieSceneSection* Section = SectionModel->GetSection();
				if (!SectionsWithKeys.Contains(Section))
				{
					Selection->TrackArea.Select(SectionModel);
				}
			}
		}
	}

	for (TSharedPtr<FViewModel> Child : Item->GetChildren())
	{
		SelectInSelectionRange(Child, SelectionRange, bSelectKeys, bSelectSections);
	}
}

void FSequencer::ClearSelectionRange()
{
	SetSelectionRange(TRange<FFrameNumber>::Empty());
}

void FSequencer::SelectInSelectionRange(bool bSelectKeys, bool bSelectSections, bool bSelectMarkers)
{
	using namespace UE::Sequencer;

	UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	TRange<FFrameNumber> SelectionRange = MovieScene->GetSelectionRange();

	TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();

	FSelectionEventSuppressor EventSuppressor = Selection->SuppressEvents();

	// Don't empty all selection, just keys and sections
	Selection->KeySelection.Empty();
	Selection->TrackArea.Empty();
	Selection->MarkedFrames.Empty();

	for (const TViewModelPtr<IOutlinerExtension>& DisplayNode : NodeTree->GetRootNodes())
	{
		SelectInSelectionRange(DisplayNode, SelectionRange, bSelectKeys, bSelectSections);
	}

	if (bSelectMarkers)
	{
		SelectMarkersInRange(SelectionRange);
	}
}

void FSequencer::SelectMarkersInRange(const TRange<FFrameNumber>& Range)
{
	using namespace UE::Sequencer;

	UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = Sequence->GetMovieScene();

	TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();

	for (int32 Index = 0; Index < MovieScene->GetMarkedFrames().Num(); ++Index)
	{
		if (Range.Contains(MovieScene->GetMarkedFrames()[Index].FrameNumber))
		{
			Selection->MarkedFrames.Select(Index);
		}
	}
}

void FSequencer::SelectForward()
{
	using namespace UE::Sequencer;

	FFrameRate TickResolution = GetFocusedTickResolution();
	FFrameNumber CurrentFrame = GetLocalTime().ConvertTo(TickResolution).CeilToFrame();
	TRange<FFrameNumber> SelectionRange(CurrentFrame, TNumericLimits<FFrameNumber>::Max());

	TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();

	FSelectionEventSuppressor EventSuppressor = Selection->SuppressEvents();

	Selection->KeySelection.Empty();
	Selection->TrackArea.Empty();
	Selection->MarkedFrames.Empty();

	if (Selection->GetNodesWithSelectedKeysOrSections().Num() != 0)
	{
		for (TViewModelPtr<IOutlinerExtension> Item : Selection->IterateIndirectOutlinerSelection())
		{
			SelectInSelectionRange(Item, SelectionRange, true, true);
		}
	}
	else if (Selection->Outliner.Num() != 0)
	{
		for (TViewModelPtr<IOutlinerExtension> Item : Selection->Outliner)
		{
			SelectInSelectionRange(Item, SelectionRange, true, true);
		}
	}
	else
	{
		SelectInSelectionRange(ViewModel->GetRootModel(), SelectionRange, true, true);
	}

	SelectMarkersInRange(SelectionRange);
}


void FSequencer::SelectBackward()
{
	using namespace UE::Sequencer;

	FFrameRate TickResolution = GetFocusedTickResolution();
	FFrameNumber CurrentFrame = GetLocalTime().ConvertTo(TickResolution).CeilToFrame();
	TRange<FFrameNumber> SelectionRange(TNumericLimits<FFrameNumber>::Min(), CurrentFrame);

	TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();

	FSelectionEventSuppressor EventSuppressor = Selection->SuppressEvents();

	Selection->KeySelection.Empty();
	Selection->TrackArea.Empty();
	Selection->MarkedFrames.Empty();

	if (Selection->GetNodesWithSelectedKeysOrSections().Num() != 0)
	{
		for (TViewModelPtr<IOutlinerExtension> Item : Selection->IterateIndirectOutlinerSelection())
		{
			SelectInSelectionRange(Item, SelectionRange, true, true);
		}
	}
	else if (Selection->Outliner.Num() != 0)
	{
		for (TViewModelPtr<IOutlinerExtension> Item : Selection->Outliner)
		{
			SelectInSelectionRange(Item, SelectionRange, true, true);
		}
	}
	else
	{
		SelectInSelectionRange(ViewModel->GetRootModel(), SelectionRange, true, true);
	}

	SelectMarkersInRange(SelectionRange);
}


TRange<FFrameNumber> FSequencer::GetPlaybackRange() const
{
	return GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();
}


void FSequencer::SetPlaybackRange(TRange<FFrameNumber> Range)
{
	if (ensure(Range.HasLowerBound() && Range.HasUpperBound()))
	{
		if (!IsPlaybackRangeLocked())
		{
			UMovieScene* FocusedMovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();
			if (FocusedMovieScene)
			{
				TRange<FFrameNumber> CurrentRange = FocusedMovieScene->GetPlaybackRange();
				if (CurrentRange == Range)
				{
					return;
				}

				TOptional<FScopedTransaction> Transaction;

				if (!bIsDraggingPlaybackRange)
				{
					Transaction.Emplace(LOCTEXT("SetPlaybackRange_Transaction", "Set Playback Range"));
				}
				else if (!bPlaybackRangeChangedDuringDrag)
				{
					FocusedMovieScene->Modify();
				}

				FocusedMovieScene->SetPlaybackRange(Range, !bIsDraggingPlaybackRange);

				// If we're in a subsequence, compensate the start offset, so that it appears decoupled from the 
				// playback range (ie. the cut in frame remains the same)
				if (ActiveTemplateIDs.Num() > 1)
				{
					if (UMovieSceneSubSection* SubSection = FindSubSection(ActiveTemplateIDs.Last()))
					{
						FFrameNumber LowerBoundDiff = Range.GetLowerBoundValue() - CurrentRange.GetLowerBoundValue();
						FFrameNumber StartFrameOffset = SubSection->Parameters.StartFrameOffset - LowerBoundDiff;

						if (!bIsDraggingPlaybackRange || !bPlaybackRangeChangedDuringDrag)
						{
							SubSection->Modify();
						}
						SubSection->Parameters.StartFrameOffset = StartFrameOffset;
					}
				}

				bNeedsEvaluate = true;
				if (bIsDraggingPlaybackRange)
				{
					bPlaybackRangeChangedDuringDrag = true;
				}
				else
				{
					NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
				}
			}
		}
	}
}

void FSequencer::SetSelectionRangeToShot(const bool bNextShot)
{
	UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = Sequence->GetMovieScene();

	UMovieSceneTrack* CinematicShotTrack = MovieScene->FindTrack(UMovieSceneCinematicShotTrack::StaticClass());
	if (!CinematicShotTrack)
	{
		return;
	}

	UMovieSceneSection* TargetShotSection = MovieSceneHelpers::FindNextSection(CinematicShotTrack->GetAllSections(), GetLocalTime().Time.FloorToFrame());

	TRange<FFrameNumber> NewSelectionRange = TargetShotSection ? TargetShotSection->GetRange() : TRange<FFrameNumber>::All();
	if (NewSelectionRange.GetLowerBound().IsClosed() && NewSelectionRange.GetUpperBound().IsClosed())
	{
		SetSelectionRange(NewSelectionRange);
	}
}

void FSequencer::SetPlaybackRangeToAllShots()
{
	UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
	UMovieScene* OwnerMovieScene = Sequence->GetMovieScene();

	UMovieSceneTrack* CinematicShotTrack = OwnerMovieScene->FindTrack(UMovieSceneCinematicShotTrack::StaticClass());
	if (!CinematicShotTrack || CinematicShotTrack->GetAllSections().Num() == 0)
	{
		return;
	}

	TRange<FFrameNumber> NewRange = CinematicShotTrack->GetAllSections()[0]->GetRange();

	for (UMovieSceneSection* ShotSection : CinematicShotTrack->GetAllSections())
	{
		if (ShotSection && ShotSection->HasStartFrame() && ShotSection->HasEndFrame())
		{
			NewRange = TRange<FFrameNumber>::Hull(ShotSection->GetRange(), NewRange);
		}
	}

	SetPlaybackRange(NewRange);
}

bool FSequencer::IsPlaybackRangeLocked() const
{
	if (IsReadOnly())
	{
		return true;
	}
	
	UMovieSceneSequence* FocusedMovieSceneSequence = GetFocusedMovieSceneSequence();
	if (FocusedMovieSceneSequence != nullptr)
	{
		UMovieScene* MovieScene = FocusedMovieSceneSequence->GetMovieScene();

		if (MovieScene->IsReadOnly())
		{
			return true;
		}
	
		return MovieScene->IsPlaybackRangeLocked();
	}

	return false;
}

void FSequencer::TogglePlaybackRangeLocked()
{
	UMovieSceneSequence* FocusedMovieSceneSequence = GetFocusedMovieSceneSequence();
	if ( FocusedMovieSceneSequence != nullptr )
	{
		UMovieScene* MovieScene = FocusedMovieSceneSequence->GetMovieScene();

		if (MovieScene->IsReadOnly())
		{
			FSequencerUtilities::ShowReadOnlyError();
			return;
		}

		FScopedTransaction TogglePlaybackRangeLockTransaction( NSLOCTEXT( "Sequencer", "TogglePlaybackRangeLocked", "Toggle playback range lock" ) );
		MovieScene->Modify();
		MovieScene->SetPlaybackRangeLocked( !MovieScene->IsPlaybackRangeLocked() );
	}
}

void FSequencer::FocusPlaybackTime()
{
	const double CurrentTime = GetLocalTime().AsSeconds();

	TRange<double> NewViewRange = GetViewRange();

	double MidRange = (NewViewRange.GetUpperBoundValue() - NewViewRange.GetLowerBoundValue()) / 2.0 + NewViewRange.GetLowerBoundValue();

	NewViewRange.SetLowerBoundValue(NewViewRange.GetLowerBoundValue() - (MidRange - CurrentTime));
	NewViewRange.SetUpperBoundValue(NewViewRange.GetUpperBoundValue() - (MidRange - CurrentTime));

	SetViewRange(NewViewRange, EViewRangeInterpolation::Animated);
}

void FSequencer::ResetViewRange()
{
	TRange<double> PlayRangeSeconds = GetPlaybackRange() / GetFocusedTickResolution();
	const double OutputViewSize = PlayRangeSeconds.Size<double>();
	const double OutputChange = OutputViewSize * 0.1f;

	if (OutputChange > 0)
	{
		PlayRangeSeconds = UE::MovieScene::ExpandRange(PlayRangeSeconds, OutputChange);

		SetClampRange(PlayRangeSeconds);
		SetViewRange(PlayRangeSeconds, EViewRangeInterpolation::Animated);
	}
}


void FSequencer::ZoomViewRange(float InZoomDelta)
{
	float LocalViewRangeMax = TargetViewRange.GetUpperBoundValue();
	float LocalViewRangeMin = TargetViewRange.GetLowerBoundValue();

	const double CurrentTime = GetLocalTime().AsSeconds();
	const double OutputViewSize = LocalViewRangeMax - LocalViewRangeMin;
	const double OutputChange = OutputViewSize * InZoomDelta;

	float CurrentPositionFraction = (CurrentTime - LocalViewRangeMin) / OutputViewSize;

	double NewViewOutputMin = LocalViewRangeMin - (OutputChange * CurrentPositionFraction);
	double NewViewOutputMax = LocalViewRangeMax + (OutputChange * (1.f - CurrentPositionFraction));

	if (NewViewOutputMin < NewViewOutputMax)
	{
		SetViewRange(TRange<double>(NewViewOutputMin, NewViewOutputMax), EViewRangeInterpolation::Animated);
	}
}


void FSequencer::ZoomInViewRange()
{
	ZoomViewRange(-0.1f);
}


void FSequencer::ZoomOutViewRange()
{
	ZoomViewRange(0.1f);
}

void FSequencer::UpdatePlaybackRange()
{
	if (!Settings->ShouldKeepPlayRangeInSectionBounds())
	{
		return;
	}

	UMovieScene* FocusedMovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	TArray<UMovieSceneSection*> AllSections = FocusedMovieScene->GetAllSections();

	if (AllSections.Num() > 0 && !IsPlaybackRangeLocked())
	{
		TRange<FFrameNumber> NewBounds = TRange<FFrameNumber>::Empty();
		for (UMovieSceneSection* Section : AllSections)
		{
			NewBounds = TRange<FFrameNumber>::Hull(Section->ComputeEffectiveRange(), NewBounds);
		}

		// When the playback range is determined by the section bounds, don't mark the change in the playback range otherwise the scene will be marked dirty
		if (!NewBounds.IsDegenerate() && !NewBounds.IsEmpty())
		{
			// Playback ranges should always have exclusive upper bounds
			if (NewBounds.GetUpperBound().IsInclusive())
			{
				NewBounds.SetUpperBound(TRangeBound<FFrameNumber>::Exclusive(NewBounds.GetUpperBound().GetValue() + 1));
			}
			const bool bAlwaysMarkDirty = false;
			FocusedMovieScene->SetPlaybackRange(NewBounds, bAlwaysMarkDirty);
		}
	}
}


EAutoChangeMode FSequencer::GetAutoChangeMode() const 
{
	return Settings->GetAutoChangeMode();
}


void FSequencer::SetAutoChangeMode(EAutoChangeMode AutoChangeMode)
{
	Settings->SetAutoChangeMode(AutoChangeMode);
}


EAllowEditsMode FSequencer::GetAllowEditsMode() const 
{
	return Settings->GetAllowEditsMode();
}


void FSequencer::SetAllowEditsMode(EAllowEditsMode AllowEditsMode)
{
	Settings->SetAllowEditsMode(AllowEditsMode);
}


EKeyGroupMode FSequencer::GetKeyGroupMode() const
{
	return Settings->GetKeyGroupMode();
}


void FSequencer::SetKeyGroupMode(EKeyGroupMode Mode)
{
	Settings->SetKeyGroupMode(Mode);
}


EMovieSceneKeyInterpolation FSequencer::GetKeyInterpolation() const
{
	return Settings->GetKeyInterpolation();
}


void FSequencer::SetKeyInterpolation(EMovieSceneKeyInterpolation InKeyInterpolation)
{
	Settings->SetKeyInterpolation(InKeyInterpolation);
}


bool FSequencer::GetInfiniteKeyAreas() const
{
	return Settings->GetInfiniteKeyAreas();
}


void FSequencer::SetInfiniteKeyAreas(bool bInfiniteKeyAreas)
{
	Settings->SetInfiniteKeyAreas(bInfiniteKeyAreas);
}


bool FSequencer::GetAutoSetTrackDefaults() const
{
	return Settings->GetAutoSetTrackDefaults();
}


FQualifiedFrameTime FSequencer::GetLocalTime() const
{
	using namespace UE::MovieScene;

	const FFrameRate    FocusedResolution = GetFocusedTickResolution();
	FQualifiedFrameTime CurrentPosition   = GetGlobalTime();

	CurrentPosition.Time = RootToWarpedLocalTransform.TransformTime(CurrentPosition.Time, FTransformTimeParams().IgnoreClamps());

	return CurrentPosition;
}


TOptional<int32> FSequencer::GetLocalLoopIndex() const
{
	using namespace UE::MovieScene;

	TOptional<int32> LoopIndex;
	RootToUnwarpedLocalTransform.TransformTime(GetGlobalTime().Time, FTransformTimeParams().TrackCycleCounts(&LoopIndex));
	return LoopIndex;
}


FQualifiedFrameTime FSequencer::GetGlobalTime() const
{
	FFrameTime RootTime = ConvertFrameTime(PlayPosition.GetCurrentPosition(), PlayPosition.GetInputRate(), PlayPosition.GetOutputRate());
	return FQualifiedFrameTime(RootTime, PlayPosition.GetOutputRate());
}

FQualifiedFrameTime FSequencer::GetUnwarpedLocalTime() const
{
	using namespace UE::MovieScene;

	const FFrameRate    FocusedResolution = GetFocusedTickResolution();
	FQualifiedFrameTime CurrentPosition   = GetGlobalTime();

	CurrentPosition.Time = RootToUnwarpedLocalTransform.TransformTime(CurrentPosition.Time, FTransformTimeParams().IgnoreClamps());
	return CurrentPosition;
}

FFrameTime FSequencer::GetScrubPosition() const
{
	if (Settings->GetTimeWarpDisplayMode() == ESequencerTimeWarpDisplay::WarpedTime)
	{
		return GetLocalTime().Time;
	}
	return GetUnwarpedLocalTime().Time;
}

FFrameTime FSequencer::GetLastEvaluatedLocalTime() const 
{
	return LastEvaluatedLocalTime;
}

UE::Sequencer::FTimeDomainOverride FSequencer::OverrideTimeDomain(UE::Sequencer::ETimeDomain NewDomain)
{
	return UE::Sequencer::FTimeDomainOverride(&TimeOperationDomain, NewDomain);
}

void FSequencer::SetLocalTime( FFrameTime NewTime, ESnapTimeMode SnapTimeMode, bool bEvaluate)
{
	FFrameRate LocalResolution = GetFocusedTickResolution();

	// Ensure the time is in the current view
	if (IsAutoScrollEnabled())
	{
		ScrollIntoView(NewTime / LocalResolution);
	}

	// Perform snapping
	if ((SnapTimeMode & ESnapTimeMode::STM_Interval) && Settings->GetForceWholeFrames())
	{
		FFrameRate LocalDisplayRate = GetFocusedDisplayRate();

		NewTime = FFrameRate::TransformTime(FFrameRate::TransformTime(NewTime, LocalResolution, LocalDisplayRate).RoundToFrame(), LocalDisplayRate, LocalResolution);
	}

	if (SnapTimeMode & ESnapTimeMode::STM_Keys)
	{
		ENearestKeyOption NearestKeyOption = ENearestKeyOption::NKO_None;

		if (Settings->GetSnapPlayTimeToKeys() || FSlateApplication::Get().GetModifierKeys().IsShiftDown())
		{
			EnumAddFlags(NearestKeyOption, ENearestKeyOption::NKO_SearchKeys);
		}

		if (Settings->GetSnapPlayTimeToSections() || FSlateApplication::Get().GetModifierKeys().IsShiftDown())
		{
			EnumAddFlags(NearestKeyOption, ENearestKeyOption::NKO_SearchSections);
		}

		if (Settings->GetSnapPlayTimeToMarkers() || FSlateApplication::Get().GetModifierKeys().IsShiftDown())
		{
			EnumAddFlags(NearestKeyOption, ENearestKeyOption::NKO_SearchMarkers);
		}

		NewTime = OnGetNearestKey(NewTime, NearestKeyOption);
	}

	SetLocalTimeDirectly(NewTime, bEvaluate);
}

void FSequencer::SetLocalTimeDirectly(FFrameTime NewTime, bool bEvaluate)
{
	using namespace UE::MovieScene;
	using namespace UE::Sequencer;

	const FMovieSceneTransformBreadcrumbs& Breadcrumbs = GetPlaybackStatus() == EMovieScenePlayerStatus::Scrubbing
		? ScrubStartBreadcrumbs
		: CurrentTimeBreadcrumbs;

	FMovieSceneInverseSequenceTransform LocalToRootTransform = TimeOperationDomain == ETimeDomain::Warped
		? RootToWarpedLocalTransform.Inverse()
		: RootToUnwarpedLocalTransform.Inverse();

	// Transform the time to the root time-space
	TOptional<FFrameTime> NewGlobalTime = LocalToRootTransform.TryTransformTime(NewTime + ScrubLinearOffset, Breadcrumbs,
		EInverseEvaluateFlags::AnyDirection | EInverseEvaluateFlags::Cycle | EInverseEvaluateFlags::IgnoreClamps);

	// If we still didn't find a time there's nothing we can do
	if (NewGlobalTime.IsSet())
	{
		FTimeDomainOverride DomainOverride = OverrideTimeDomain(ETimeDomain::Unwarped);
		SetGlobalTime(NewGlobalTime.GetValue(), bEvaluate);
	}
}


void FSequencer::SetGlobalTime(FFrameTime NewTime, bool bEvaluate)
{
	using namespace UE::MovieScene;
	using namespace UE::Sequencer;

	NewTime = ConvertFrameTime(NewTime, GetRootTickResolution(), PlayPosition.GetInputRate());
	if (PlayPosition.GetEvaluationType() == EMovieSceneEvaluationType::FrameLocked)
	{
		NewTime = NewTime.FloorToFrame();
	}

	// Don't update the sequence if the time hasn't changed as this will cause duplicate events and the like to fire.
	// If we need to reevaluate the sequence at the same time for whetever reason, we should call ForceEvaluate()
	if (PlayPosition.GetCurrentPosition() != NewTime)
	{
		// Make sure breadcrumbs are up to date
		RootToWarpedLocalTransform.TransformTime(PlayPosition.GetCurrentPosition(), FTransformTimeParams().HarvestBreadcrumbs(CurrentTimeBreadcrumbs).IgnoreClamps());

		FMovieSceneEvaluationRange EvalRange = PlayPosition.JumpTo(NewTime);
		if (bEvaluate)
		{
			EvaluateInternal(EvalRange);
		}
	}

	if (AutoScrubTarget.IsSet())
	{
		SetPlaybackStatus(EMovieScenePlayerStatus::Stopped);
		AutoScrubTarget.Reset();
	}
}

void FSequencer::PlayTo(FMovieSceneSequencePlaybackParams PlaybackParams)
{
	FFrameTime PlayToTime = GetLocalTime().Time;

	if (PlaybackParams.PositionType == EMovieScenePositionType::Frame)
	{
		PlayToTime = PlaybackParams.Frame;
	}
	else if (PlaybackParams.PositionType == EMovieScenePositionType::Time)
	{
		PlayToTime = PlaybackParams.Time * GetFocusedTickResolution();
	}
	else if (PlaybackParams.PositionType == EMovieScenePositionType::MarkedFrame)
	{
		UMovieSceneSequence* FocusedMovieSequence = GetFocusedMovieSceneSequence();
		if (FocusedMovieSequence != nullptr)
		{
			UMovieScene* FocusedMovieScene = FocusedMovieSequence->GetMovieScene();
			if (FocusedMovieScene != nullptr)
			{
				int32 MarkedIndex = FocusedMovieScene->FindMarkedFrameByLabel(PlaybackParams.MarkedFrame);

				if (MarkedIndex != INDEX_NONE)
				{
					PlayToTime = FocusedMovieScene->GetMarkedFrames()[MarkedIndex].FrameNumber;
				}
			}
		}
	}

	if (GetLocalTime().Time < PlayToTime)
	{
		PlaybackSpeed = FMath::Abs(PlaybackSpeed);
	}
	else
	{
		PlaybackSpeed = -FMath::Abs(PlaybackSpeed);
	}
		
	OnPlay(false);
	PauseOnFrame = PlayToTime;
}

void FSequencer::SnapSequencerTime(FFrameTime& ScrubTime)
{
	// Clamp first, snap to frame last
	TRange<FFrameNumber> PlayheadScrubbingRange = GetSequencerSettings()->GetPlayheadScrubbingRange(TWeakPtr<ISequencer>(AsShared()));
	if (!PlayheadScrubbingRange.IsEmpty())
	{
		ScrubTime = UE::MovieScene::ClampToDiscreteRange(ScrubTime, PlayheadScrubbingRange);
	}

	if (GetSequencerSettings()->GetForceWholeFrames())
	{
		FFrameRate TickResolution = GetFocusedTickResolution();
		FFrameRate DisplayRate = GetFocusedDisplayRate();

		// Set the style of the scrub handle
		if (GetScrubStyle() == ESequencerScrubberStyle::FrameBlock)
		{
			// Floor to the display frame
			ScrubTime = ConvertFrameTime(ConvertFrameTime(ScrubTime, TickResolution, DisplayRate).FloorToFrame(), DisplayRate, TickResolution);
		}
		else
		{
			// Snap (round) to display rate
			ScrubTime = FFrameRate::Snap(ScrubTime, TickResolution, DisplayRate);
		}
	}
	
	if (GetSequencerSettings()->GetIsSnapEnabled())
	{
		using namespace UE::Sequencer;
		TViewModelPtr<IClockExtension> Clock = GetViewModel()->GetRootSequenceModel().ImplicitCast();
		if (Clock.IsValid() && Clock->SupportsSnapping() && Clock->ShouldSnapFrameTime())
		{
			ScrubTime = Clock->SnapFrameTime(ScrubTime);
		}
	}
}

void FSequencer::ForceEvaluate()
{
	EvaluateInternal(PlayPosition.GetCurrentPositionAsRange());
}

void FSequencer::EvaluateInternal(FMovieSceneEvaluationRange InRange, bool bHasJumped)
{
	using namespace UE::MovieScene;

	// Ensure breadcrumbs are up to date
	RootToUnwarpedLocalTransform.TransformTime(InRange.GetTime(), FTransformTimeParams().HarvestBreadcrumbs(CurrentTimeBreadcrumbs).IgnoreClamps());

	if (!GlobalPlaybackWarpTransform.IsIdentity())
	{
		TRange<FFrameTime> NewRange = GlobalPlaybackWarpTransform.ComputeTraversedHull(InRange.GetRange());
		InRange = FMovieSceneEvaluationRange(NewRange, InRange.GetFrameRate(), InRange.GetDirection());
	}

	LastEvaluatedLocalTime = GetLocalTime().Time;

	if (Settings->ShouldCompileDirectorOnEvaluate())
	{
		RecompileDirtyDirectors();
	}

	bNeedsEvaluate = false;

	UpdateCachedPlaybackContextAndClient();
	if (EventContextsAttribute.IsBound())
	{
		CachedEventContexts.Reset();
		for (UObject* Object : EventContextsAttribute.Get())
		{
			CachedEventContexts.Add(Object);
		}
	}

	if (IMovieScenePlaybackClient* PlaybackClient = GetPlaybackClient())
	{
		PlaybackClient->WarpEvaluationRange(InRange);
	}

	FMovieSceneContext Context = FMovieSceneContext(InRange, PlaybackState).SetIsSilent(SilentModeCount != 0);
	Context.SetHasJumped(bHasJumped);

	
	RootTemplateInstance.EvaluateSynchronousBlocking(Context);
	SuppressAutoEvalSignature.Reset();

	if (Settings->ShouldRerunConstructionScripts())
	{
		RerunConstructionScripts();
	}

	if (!IsInSilentMode())
	{
		OnGlobalTimeChangedDelegate.Broadcast();
		GetRendererModule().InvalidatePathTracedOutput();
	}

}

void FSequencer::UpdateCachedPlaybackContextAndClient()
{
	TWeakObjectPtr<UObject> NewPlaybackContext;
	TWeakInterfacePtr<IMovieScenePlaybackClient> NewPlaybackClient;

	if (PlaybackContextAttribute.IsBound())
	{
		NewPlaybackContext = PlaybackContextAttribute.Get();
	}
	if (PlaybackClientAttribute.IsBound())
	{
		NewPlaybackClient = TWeakInterfacePtr<IMovieScenePlaybackClient>(PlaybackClientAttribute.Get());
	}

	if (CachedPlaybackContext != NewPlaybackContext || CachedPlaybackClient != NewPlaybackClient)
	{
		PrePossessionViewTargets.Reset();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		State.ClearObjectCaches(*this);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		RestorePreAnimatedState();

		CachedPlaybackContext = NewPlaybackContext;
		CachedPlaybackClient = NewPlaybackClient;

		OnPlaybackContextChanged();
	}
}

void FSequencer::UpdateCachedCameraActors()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const uint32 CurrentStateSerial = State.GetSerialNumber();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (CurrentStateSerial == LastKnownStateSerial)
	{
		return;
	}
	
	LastKnownStateSerial = CurrentStateSerial;
	CachedCameraActors.Reset();

	TArray<FMovieSceneSequenceID> SequenceIDs;
	SequenceIDs.Add(MovieSceneSequenceID::Root);
	if (const FMovieSceneSequenceHierarchy* Hierarchy = RootTemplateInstance.GetHierarchy())
	{
		Hierarchy->AllSubSequenceIDs(SequenceIDs);
	}

	for (FMovieSceneSequenceID SequenceID : SequenceIDs)
	{
		if (UMovieSceneSequence* Sequence = RootTemplateInstance.GetSequence(SequenceID))
		{
			if (UMovieScene* MovieScene = Sequence->GetMovieScene())
			{
				TArray<FGuid> BindingGuids;

				for (uint32 SpawnableIndex = 0, SpawnableCount = MovieScene->GetSpawnableCount(); 
						SpawnableIndex < SpawnableCount; 
						++SpawnableIndex)
				{
					const FMovieSceneSpawnable& Spawnable = MovieScene->GetSpawnable(SpawnableIndex);
					BindingGuids.Add(Spawnable.GetGuid());
				}

				for (uint32 PossessableIndex = 0, PossessableCount = MovieScene->GetPossessableCount(); 
						PossessableIndex < PossessableCount; 
						++PossessableIndex)
				{
					const FMovieScenePossessable& Possessable = MovieScene->GetPossessable(PossessableIndex);
					BindingGuids.Add(Possessable.GetGuid());
				}

				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				const FMovieSceneObjectCache& ObjectCache = State.GetObjectCache(SequenceID);
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				for (const FGuid& BindingGuid : BindingGuids)
				{
					for (TWeakObjectPtr<> BoundObject : ObjectCache.IterateBoundObjects(BindingGuid))
					{
						if (AActor* BoundActor = Cast<AActor>(BoundObject.Get()))
						{
							UCameraComponent* CameraComponent = MovieSceneHelpers::CameraComponentFromActor(BoundActor);
							if (CameraComponent)
							{
								CachedCameraActors.Add(BoundActor, BindingGuid);
							}
						}
					}
				}
			}
		}
	}
}

void FSequencer::ScrollIntoView(float InLocalTime)
{
	float RangeOffset = CalculateAutoscrollEncroachment(InLocalTime).Get(0.f);
		
	// When not scrubbing, we auto scroll the view range immediately
	if (RangeOffset != 0.f)
	{
		TRange<double> WorkingRange = GetClampRange();

		// Adjust the offset so that the target range will be within the working range.
		if (TargetViewRange.GetLowerBoundValue() + RangeOffset < WorkingRange.GetLowerBoundValue())
		{
			RangeOffset = WorkingRange.GetLowerBoundValue() - TargetViewRange.GetLowerBoundValue();
		}
		else if (TargetViewRange.GetUpperBoundValue() + RangeOffset > WorkingRange.GetUpperBoundValue())
		{
			RangeOffset = WorkingRange.GetUpperBoundValue() - TargetViewRange.GetUpperBoundValue();
		}

		SetViewRange(TRange<double>(TargetViewRange.GetLowerBoundValue() + RangeOffset, TargetViewRange.GetUpperBoundValue() + RangeOffset), EViewRangeInterpolation::Immediate);
	}
}

void FSequencer::UpdateAutoScroll(double NewTime, float ThresholdPercentage)
{
	if (!IsAutoScrollEnabled())
	{
		return;
	}

	AutoscrollOffset = CalculateAutoscrollEncroachment(NewTime, ThresholdPercentage);

	if (!AutoscrollOffset.IsSet())
	{
		AutoscrubOffset.Reset();
		return;
	}

	TRange<double> ViewRange = GetViewRange();
	const double Threshold = (ViewRange.GetUpperBoundValue() - ViewRange.GetLowerBoundValue()) * ThresholdPercentage;
	const bool bForwards = PlaybackSpeed > 0;

	const FQualifiedFrameTime LocalTime = GetLocalTime();

	// If we have no autoscrub offset yet, we move the scrub position to the boundary of the autoscroll threasdhold, then autoscrub from there
	if (!AutoscrubOffset.IsSet())
	{
		if (AutoscrollOffset.GetValue() < 0 && LocalTime.AsSeconds() > ViewRange.GetLowerBoundValue() + Threshold)
		{
			SetLocalTimeLooped( (ViewRange.GetLowerBoundValue() + Threshold) * LocalTime.Rate, CurrentTimeBreadcrumbs, bForwards);
		}
		else if (AutoscrollOffset.GetValue() > 0 && LocalTime.AsSeconds() < ViewRange.GetUpperBoundValue() - Threshold)
		{
			SetLocalTimeLooped( (ViewRange.GetUpperBoundValue() - Threshold) * LocalTime.Rate, CurrentTimeBreadcrumbs, bForwards);
		}
	}

	// Don't autoscrub if we're at the extremes of the movie scene range
	const FMovieSceneEditorData& EditorData = GetFocusedMovieSceneSequence()->GetMovieScene()->GetEditorData();
	if (NewTime < EditorData.WorkStart + Threshold ||
		NewTime > EditorData.WorkEnd - Threshold
		)
	{
		AutoscrubOffset.Reset();
		return;
	}

	// Scrub at the same rate we scroll
	AutoscrubOffset = AutoscrollOffset;
}


TOptional<float> FSequencer::CalculateAutoscrollEncroachment(double NewTime, float ThresholdPercentage) const
{
	enum class EDirection { Positive, Negative };
	const EDirection Movement = NewTime - GetLocalTime().AsSeconds() >= 0 ? EDirection::Positive : EDirection::Negative;

	const TRange<double> CurrentRange = GetViewRange();
	const double RangeMin = CurrentRange.GetLowerBoundValue(), RangeMax = CurrentRange.GetUpperBoundValue();
	const double AutoScrollThreshold = (RangeMax - RangeMin) * ThresholdPercentage;

	if (Movement == EDirection::Negative && NewTime < RangeMin + AutoScrollThreshold)
	{
		// Scrolling backwards in time, and have hit the threshold
		return NewTime - (RangeMin + AutoScrollThreshold);
	}
	
	if (Movement == EDirection::Positive && NewTime > RangeMax - AutoScrollThreshold)
	{
		// Scrolling forwards in time, and have hit the threshold
		return NewTime - (RangeMax - AutoScrollThreshold);
	}

	return TOptional<float>();
}


void FSequencer::AutoScrubToTime(FFrameTime DestinationTime)
{
	AutoScrubTarget = FAutoScrubTarget(DestinationTime, GetLocalTime().Time, FPlatformTime::Seconds());
}

void FSequencer::SetPerspectiveViewportPossessionEnabled(bool bEnabled)
{
	bPerspectiveViewportPossessionEnabled = bEnabled;
}


void FSequencer::SetPerspectiveViewportCameraCutEnabled(bool bEnabled)
{
	if (bPerspectiveViewportCameraCutEnabled == bEnabled)
	{
		return;
	}

	bPerspectiveViewportCameraCutEnabled = bEnabled;
}

FString FSequencer::GetMovieRendererName() const
{
	// If blank, default to the first available since we don't want the be using the Legacy one anyway, unless the user explicitly chooses it.
	FString MovieRendererName = Settings->GetMovieRendererName();
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	if (MovieRendererName.IsEmpty() && SequencerModule.GetMovieRendererNames().Num() > 0)
	{
		MovieRendererName = SequencerModule.GetMovieRendererNames()[0];

		Settings->SetMovieRendererName(MovieRendererName);
	}

	return MovieRendererName;
}

void FSequencer::RenderMovie(const TArray<UMovieSceneCinematicShotSection*>& InSections) const
{
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	if (IMovieRendererInterface* MovieRenderer = SequencerModule.GetMovieRenderer(GetMovieRendererName()))
	{
		MovieRenderer->RenderMovie(GetRootMovieSceneSequence(), InSections);
		return;
	}

	if (InSections.Num() != 0)
	{
		RenderMovieInternal(InSections[0]->GetRange(), true);
	}
}

void FSequencer::RenderMovieInternal(TRange<FFrameNumber> Range, bool bSetFrameOverrides) const
{
	using namespace UE::MovieScene;

	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	if (IMovieRendererInterface* MovieRenderer = SequencerModule.GetMovieRenderer(GetMovieRendererName()))
	{
		TArray<UMovieSceneCinematicShotSection*> ShotSections;
		if (UMovieSceneCinematicShotSection* ShotSection = Cast<UMovieSceneCinematicShotSection>(FindSubSection(GetFocusedTemplateID())))
		{
			ShotSections.Add(ShotSection);
		}

		MovieRenderer->RenderMovie(GetRootMovieSceneSequence(), ShotSections);
		return;
	}

	if (Range.GetLowerBound().IsOpen() || Range.GetUpperBound().IsOpen())
	{
		Range = TRange<FFrameNumber>::Hull(Range, GetPlaybackRange());
	}

	// If focused on a subsequence, transform the playback range to the root in order to always render from the root
	if (GetRootMovieSceneSequence() != GetFocusedMovieSceneSequence())
	{
		bSetFrameOverrides = true;

		if (const FMovieSceneSubSequenceData* SubSequenceData = RootTemplateInstance.FindSubData(GetFocusedTemplateID()))
		{
			auto Visit = [&Range](TRange<FFrameTime> RootRange)
			{
				Range = ConvertToDiscreteRange(RootRange);
				return false;
			};
			SubSequenceData->RootToSequenceTransform.Inverse().TransformFiniteRangeWithinRange(ConvertToFrameTimeRange(Range), Visit, CurrentTimeBreadcrumbs, CurrentTimeBreadcrumbs);
		}
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));

	// Create a new movie scene capture object for an automated level sequence, and open the tab
	UAutomatedLevelSequenceCapture* MovieSceneCapture = NewObject<UAutomatedLevelSequenceCapture>(GetTransientPackage(), UAutomatedLevelSequenceCapture::StaticClass(), UMovieSceneCapture::MovieSceneCaptureUIName, RF_Transient);
	MovieSceneCapture->LoadFromConfig();

	// Always render from the root
	MovieSceneCapture->LevelSequenceAsset = GetRootMovieSceneSequence()->GetMovieScene()->GetOuter()->GetPathName();

	FFrameRate DisplayRate = GetFocusedDisplayRate();
	FFrameRate TickResolution = GetFocusedTickResolution();

	MovieSceneCapture->Settings.FrameRate = DisplayRate;
	MovieSceneCapture->Settings.ZeroPadFrameNumbers = Settings->GetZeroPadFrames();
	MovieSceneCapture->Settings.bUseRelativeFrameNumbers = false;

	FFrameNumber StartFrame = UE::MovieScene::DiscreteInclusiveLower(Range);
	FFrameNumber EndFrame = UE::MovieScene::DiscreteExclusiveUpper(Range);

	FFrameNumber RoundedStartFrame = FFrameRate::TransformTime(StartFrame, TickResolution, DisplayRate).CeilToFrame();
	FFrameNumber RoundedEndFrame = FFrameRate::TransformTime(EndFrame, TickResolution, DisplayRate).CeilToFrame();

	if (bSetFrameOverrides)
	{
		MovieSceneCapture->SetFrameOverrides(RoundedStartFrame, RoundedEndFrame);
	}
	else
	{
		if (!MovieSceneCapture->bUseCustomStartFrame)
		{
			MovieSceneCapture->CustomStartFrame = RoundedStartFrame;
		}

		if (!MovieSceneCapture->bUseCustomEndFrame)
		{
			MovieSceneCapture->CustomEndFrame = RoundedEndFrame;
		}
	}

	// We create a new Numeric Type Interface that ties it's Capture/Resolution rates to the Capture Object so that it converts UI entries
	// to the correct resolution for the capture, and not for the original sequence.
	USequencerSettings* LocalSettings = Settings;

	TAttribute<EFrameNumberDisplayFormats> GetDisplayFormatAttr = MakeAttributeLambda(
		[LocalSettings]
		{
			if (LocalSettings)
			{
				return LocalSettings->GetTimeDisplayFormat();
			}
			return EFrameNumberDisplayFormats::Frames;
		}
	);

	TAttribute<uint8> GetZeroPadFramesAttr = MakeAttributeLambda(
		[LocalSettings]()->uint8
		{
			if (LocalSettings)
			{
				return LocalSettings->GetZeroPadFrames();
			}
			return 0;
		}
	);

	// By using a TickResolution/DisplayRate that match the numbers entered via the numeric interface don't change frames of reference.
	// This is used here because the movie scene capture works entirely on play rate resolution and has no knowledge of the internal resolution
	// so we don't need to convert the user's input into internal resolution.
	TAttribute<FFrameRate> GetFrameRateAttr = MakeAttributeLambda(
		[MovieSceneCapture]
		{
			if (MovieSceneCapture)
			{
				return MovieSceneCapture->GetSettings().FrameRate;
			}
			return FFrameRate(30, 1);
		}
	);

	// Create our numeric type interface so we can pass it to the time slider below.
	TSharedPtr<INumericTypeInterface<double>> MovieSceneCaptureNumericInterface = MakeShareable(new FFrameNumberInterface(GetDisplayFormatAttr, GetZeroPadFramesAttr, GetFrameRateAttr, GetFrameRateAttr));

	IMovieSceneCaptureDialogModule::Get().OpenDialog(LevelEditorModule.GetLevelEditorTabManager().ToSharedRef(), MovieSceneCapture, MovieSceneCaptureNumericInterface);
}

void FSequencer::EnterSilentMode()
{
	++SilentModeCount;
}

void FSequencer::ExitSilentMode()
{ 
	--SilentModeCount;
	ensure(SilentModeCount >= 0);
}

ISequencer::FOnActorAddedToSequencer& FSequencer::OnActorAddedToSequencer()
{
	return OnActorAddedToSequencerEvent;
}

ISequencer::FOnPreSave& FSequencer::OnPreSave()
{
	return OnPreSaveEvent;
}

ISequencer::FOnPostSave& FSequencer::OnPostSave()
{
	return OnPostSaveEvent;
}

ISequencer::FOnPreNavigateToSequence& FSequencer::OnPreNavigateToSequence()
{
	return OnPreNavigateToSequenceEvent;
}

ISequencer::FOnActivateSequence& FSequencer::OnActivateSequence()
{
	return OnActivateSequenceEvent;
}

ISequencer::FOnCameraCut& FSequencer::OnCameraCut()
{
	return OnCameraCutEvent;
}

void FSequencer::AddNumericTypeInterface(TSharedRef<UE::Sequencer::FSequencerNumericTypeInterface> InNumericTypeInterface)
{
	NumericTypeInterfaces.AddUnique(InNumericTypeInterface);
}

void FSequencer::RemoveNumericTypeInterface(TSharedRef<UE::Sequencer::FSequencerNumericTypeInterface> InNumericTypeInterface)
{
	NumericTypeInterfaces.Remove(InNumericTypeInterface);
}

TArrayView<const TSharedRef<UE::Sequencer::FSequencerNumericTypeInterface>> FSequencer::GetNumericTypeInterfaces() const
{
	return NumericTypeInterfaces;
}

TSharedPtr<INumericTypeInterface<double>> FSequencer::GetNumericTypeInterface(UE::Sequencer::ENumericIntent Intent) const
{
	TSharedPtr<INumericTypeInterface<double>> Result;

	int32 Score = MIN_int32;
	for (TSharedRef<UE::Sequencer::FSequencerNumericTypeInterface> Interface : NumericTypeInterfaces)
	{
		if (Interface->Intent != Intent)
		{
			continue;
		}
		const int32 ThisScore = Interface->GetRelevancyScore(*this, nullptr);
		if (ThisScore > Score)
		{
			Result = Interface->Interface;
			Score = ThisScore;
		}
	}

	check(Result);
	return Result.ToSharedRef();
}

TSharedRef<IPropertyTypeCustomization> FSequencer::MakeFrameNumberDetailsCustomization()
{
	return MakeShared<FFrameNumberDetailsCustomization>(TWeakPtr<ISequencer>(AsShared()));
}

TSharedRef<SWidget> FSequencer::MakeTimeRange(const TSharedRef<SWidget>& InnerContent, bool bShowWorkingRange, bool bShowViewRange, bool bShowPlaybackRange)
{
	return SequencerWidget->MakeTimeRange(InnerContent, bShowWorkingRange, bShowViewRange, bShowPlaybackRange);
}

/** Attempt to find an object binding ID that relates to an unspawned spawnable object */
FGuid FSequencer::FindUnspawnedObjectGuid(UObject& InObject)
{
	if (UMovieSceneSequence* FocusedMovieSceneSequence = GetFocusedMovieSceneSequence())
	{
		UMovieScene* MovieScene = FocusedMovieSceneSequence->GetMovieScene();

		// If the object is an archetype, the it relates to an unspawned spawnable.
		UObject* ParentObject = FocusedMovieSceneSequence->GetParentObject(&InObject);
		if (ParentObject && FMovieSceneSpawnable::IsSpawnableTemplate(*ParentObject))
		{
			FMovieSceneSpawnable* ParentSpawnable = MovieScene->FindSpawnable([&](FMovieSceneSpawnable& InSpawnable){
				return InSpawnable.GetObjectTemplate() == ParentObject;
			});

			if (ParentSpawnable)
			{
				// TODO: Won't this always fail to find anything if the spawnable actor is unspawned?

				// The only way to find the object now is to resolve all the child bindings, and see if they are the same
				for (const FGuid& ChildGuid : ParentSpawnable->GetChildPossessables())
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					TArrayView<TWeakObjectPtr<>> BoundObjects = State.FindBoundObjects(ChildGuid, GetFocusedTemplateID(), GetSharedPlaybackState());
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
					const bool bHasObject = BoundObjects.Contains(&InObject);
					if (bHasObject)
					{
						return ChildGuid;
					}
				}
			}
		}
		else if (FMovieSceneSpawnable::IsSpawnableTemplate(InObject))
		{
			FMovieSceneSpawnable* SpawnableByArchetype = MovieScene->FindSpawnable([&](FMovieSceneSpawnable& InSpawnable){
				return InSpawnable.GetObjectTemplate() == &InObject;
			});

			if (SpawnableByArchetype)
			{
				return SpawnableByArchetype->GetGuid();
			}

			// Also check for custom spawnables
			if (const FMovieSceneBindingReferences* BindingReferences = FocusedMovieSceneSequence->GetBindingReferences())
			{
				for (const FMovieSceneBindingReference& BindingReference : BindingReferences->GetAllReferences())
				{
					if (BindingReference.CustomBinding)
					{
						if (UMovieSceneSpawnableBindingBase* SpawnableBinding = BindingReference.CustomBinding->AsSpawnable(GetSharedPlaybackState()))
						{
							if (SpawnableBinding->SupportsObjectTemplates() && SpawnableBinding->GetObjectTemplate() == &InObject)
							{
								return BindingReference.ID;
							}
						}
					}
				}
			}
		}
	}

	return FGuid();
}

void FSequencer::DestroySequencerWidget()
{
	const TWeakPtr<SSequencer> WeakSequencerWidget = SequencerWidget.ToWeakPtr();
	
	if (SequencerWidget)
	{
		SequencerWidget.Reset();
	}
	
	if (SequencerWrapperWidget)
	{
		SequencerWrapperWidget->DestroySequencerWidget();
		SequencerWrapperWidget.Reset();
	}
	
	// Last reference to SSequencer should have been cleared now.
	ensure(!WeakSequencerWidget.IsValid());
}

UMovieSceneFolder* FSequencer::CreateFoldersRecursively(const TArray<FName>& FolderPath, int32 FolderPathIndex, UMovieScene* OwningMovieScene, UMovieSceneFolder* ParentFolder, TArrayView<UMovieSceneFolder* const> FoldersToSearch)
{
	// An empty folder path won't create a folder
	if (FolderPath.Num() == 0)
	{
		return ParentFolder;
	}

	check(FolderPathIndex < FolderPath.Num());

	// Look to see if there's already a folder with the right name
	UMovieSceneFolder* FolderToUse = nullptr;
	FName DesiredFolderName = FolderPath[FolderPathIndex];

	for (UMovieSceneFolder* Folder : FoldersToSearch)
	{
		if (Folder->GetFolderName() == DesiredFolderName)
		{
			FolderToUse = Folder;
			break;
		}
	}

	// If we didn't find a folder with the desired name then we create a new folder as a sibling of the existing folders.
	if (FolderToUse == nullptr)
	{
		FolderToUse = NewObject<UMovieSceneFolder>(OwningMovieScene, NAME_None, RF_Transactional);
		FolderToUse->SetFolderName(DesiredFolderName);
		if (ParentFolder)
		{
			// Add the new folder as a sibling of the folders we were searching in.
			ParentFolder->AddChildFolder(FolderToUse);
		}
		else
		{
			// If we have no parent folder then we must be at the root so we add it to the root of the movie scene
			OwningMovieScene->Modify();
			OwningMovieScene->AddRootFolder(FolderToUse);
		}
	}

	// Increment which part of the path we're searching in and then recurse inside of the folder we found (or created).
	FolderPathIndex++;
	if (FolderPathIndex < FolderPath.Num())
	{
		return CreateFoldersRecursively(FolderPath, FolderPathIndex, OwningMovieScene, FolderToUse, FolderToUse->GetChildFolders());
	}

	// We return the tail folder created so that the user can add things to it.
	return FolderToUse;
}

FGuid FSequencer::GetHandleToObject( UObject* Object, bool bCreateHandleIfMissing, const FName& CreatedFolderName )
{
	using namespace UE::Sequencer;

	if (Object == nullptr)
	{
		return FGuid();
	}

	UMovieSceneSequence* FocusedMovieSceneSequence = GetFocusedMovieSceneSequence();
	UMovieScene* FocusedMovieScene = FocusedMovieSceneSequence ? FocusedMovieSceneSequence->GetMovieScene() : nullptr;
	
	if (!FocusedMovieScene)
	{
		return FGuid();
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		return FGuid();
	}

	ISequencerModule& Module = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	if (TSharedPtr<IObjectSchema> Schema = Module.FindObjectSchema(Object))
	{
		Object = Schema->GetRedirectedBoundObject(Object);
		if (!Object)
		{
			return FGuid();
		}
	}

	// Attempt to resolve the object through the movie scene instance first, 
	FGuid ObjectGuid = bCreateHandleIfMissing ?
		FindObjectId(*Object, ActiveTemplateIDs.Top()) :
		FindCachedObjectId(*Object, ActiveTemplateIDs.Top());

	if (ObjectGuid.IsValid())
	{
		// Check here for spawnable and custom spawnables otherwise spawnables get recreated as possessables, which doesn't make sense
		if (MovieSceneHelpers::IsBoundToAnySpawnable(FocusedMovieSceneSequence, ObjectGuid, GetSharedPlaybackState()))
		{
			return ObjectGuid;
		}

		// Make sure that the possessable is still valid, if it's not remove the binding so new one 
		// can be created.  This can happen due to undo.
		FMovieScenePossessable* Possessable = FocusedMovieScene->FindPossessable(ObjectGuid);
		if(Possessable == nullptr)
		{
			FocusedMovieSceneSequence->UnbindPossessableObjects(ObjectGuid);
			ObjectGuid.Invalidate();
		}
	}
	else
	{
		ObjectGuid = FindUnspawnedObjectGuid(*Object);
	}

	if (ObjectGuid.IsValid() || IsReadOnly())
	{
		return ObjectGuid;
	}

	if (bCreateHandleIfMissing)
	{
		UObject* PlaybackContext = PlaybackContextAttribute.Get(nullptr);
		if (FocusedMovieSceneSequence->CanPossessObject(*Object, PlaybackContext))
		{
			ObjectGuid = FSequencerUtilities::CreateBinding(AsShared(), *Object);

			if (CreatedFolderName != NAME_None)
			{
				// Find the outermost object and put it in a 
				FMovieScenePossessable* Possessable = FocusedMovieScene->FindPossessable(ObjectGuid);
				if (!Possessable || !Possessable->GetParent().IsValid())
				{
					UMovieSceneFolder* Folder = FSequencer::CreateFoldersRecursively(TArray<FName>{ CreatedFolderName }, 0, FocusedMovieScene, nullptr, FocusedMovieScene->GetRootFolders());
					if (Folder)
					{
						Folder->AddChildObjectBinding(ObjectGuid);
					}
				}
			}

			NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemAdded );
		}
	}
	return ObjectGuid;
}


ISequencerObjectChangeListener& FSequencer::GetObjectChangeListener()
{ 
	return *ObjectChangeListener;
}

ISequencerPropertyKeyedStatusHandler& FSequencer::GetPropertyKeyedStatusHandler()
{
	return *PropertyKeyedStatusHandler;
}

TSharedPtr<class ITimeSlider> FSequencer::GetTopTimeSliderWidget() const
{
	return SequencerWidget->GetTopTimeSliderWidget();
}

void FSequencer::UpdateLevelViewportClientsActorLocks()
{
	// Nothing to do if we are not editing level sequence, as these are the only kinds of sequences right now
	// that have some aspect ratio constraints settings.
	const ALevelSequenceActor* LevelSequenceActor = Cast<ALevelSequenceActor>(GetPlaybackClient());
	if (LevelSequenceActor == nullptr)
	{
		return;
	}

	TOptional<EAspectRatioAxisConstraint> AspectRatioAxisConstraint;
	if (LevelSequenceActor->CameraSettings.bOverrideAspectRatioAxisConstraint)
	{
		AspectRatioAxisConstraint = LevelSequenceActor->CameraSettings.AspectRatioAxisConstraint;
	}

	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC != nullptr)
		{
			// If there is an actor lock on an actor that turns out to be one of our cameras, set the
			// aspect ratio axis constraint on it.
			FLevelViewportActorLock& ActorLock = LevelVC->GetActorLock();
			if (AActor* LockedActor = ActorLock.GetLockedActor())
			{
				if (CachedCameraActors.Find(LockedActor))
				{
					ActorLock.AspectRatioAxisConstraint = AspectRatioAxisConstraint;
				}
			}
			// If we are in control of the entire viewport, also set the aspect ratio axis constraint.
			if (IsPerspectiveViewportCameraCutEnabled())
			{
				FLevelViewportActorLock& CinematicLock = LevelVC->GetCinematicActorLock();
				if (AActor* LockedActor = CinematicLock.GetLockedActor())
				{
					CinematicLock.AspectRatioAxisConstraint = AspectRatioAxisConstraint;
				}
			}
		}
	}
}

void FSequencer::GetCameraObjectBindings(TArray<FGuid>& OutBindingIDs)
{
	CachedCameraActors.GenerateValueArray(OutBindingIDs);
}

void FSequencer::NotifyBindingsChanged()
{
	ISequencer::NotifyBindingsChanged();

	OnMovieSceneBindingsChangedDelegate.Broadcast();
}


void FSequencer::SetViewportSettings(const TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap)
{
	if (!IsPerspectiveViewportPossessionEnabled())
	{
		return;
	}

	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC)
		{
			if (LevelVC->AllowsCinematicControl())
			{
				if (ViewportParamsMap.Contains(LevelVC))
				{
					const EMovieSceneViewportParams* ViewportParams = ViewportParamsMap.Find(LevelVC);
					if (ViewportParams->SetWhichViewportParam & EMovieSceneViewportParams::SVP_FadeAmount)
					{
						LevelVC->FadeAmount = ViewportParams->FadeAmount;
						LevelVC->bEnableFading = true;
					}
					if (ViewportParams->SetWhichViewportParam & EMovieSceneViewportParams::SVP_FadeColor)
					{
						LevelVC->FadeColor = ViewportParams->FadeColor;
						LevelVC->bEnableFading = true;
					}
					if (ViewportParams->SetWhichViewportParam & EMovieSceneViewportParams::SVP_ColorScaling)
					{
						LevelVC->bEnableColorScaling = ViewportParams->bEnableColorScaling;
						LevelVC->ColorScale = ViewportParams->ColorScale;
					}
				}
			}
			else
			{
				LevelVC->bEnableFading = false;
				LevelVC->bEnableColorScaling = false;
			}
		}
	}
}


void FSequencer::GetViewportSettings(TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) const
{
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && LevelVC->AllowsCinematicControl())
		{
			EMovieSceneViewportParams ViewportParams;
			ViewportParams.FadeAmount = LevelVC->FadeAmount;
			ViewportParams.FadeColor = FLinearColor(LevelVC->FadeColor);
			ViewportParams.ColorScale = LevelVC->ColorScale;

			ViewportParamsMap.Add(LevelVC, ViewportParams);
		}
	}
}

UMovieSceneEntitySystemLinker* FSequencer::ConstructEntitySystemLinker()
{
	UObject* PlaybackContext = GetPlaybackContext();
	UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

	UObject* Outer = GetTransientPackage();
	FName EntitySystemLinkerName = MakeUniqueObjectName(Outer, UMovieSceneEntitySystemLinker::StaticClass(), TEXT("SequencerEntitySystemLinker"));
	UMovieSceneEntitySystemLinker* EntitySystemLinker = NewObject<UMovieSceneEntitySystemLinker>(Outer, EntitySystemLinkerName);

	EntitySystemLinker->SetWorld(World);
	EntitySystemLinker->SetLinkerRole(UE::MovieScene::EEntitySystemLinkerRole::Standalone);
	return EntitySystemLinker;
}

EMovieScenePlayerStatus::Type FSequencer::GetPlaybackStatus() const
{
	return PlaybackState;
}


void FSequencer::SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus)
{
	PlaybackState = InPlaybackStatus;
	PauseOnFrame.Reset();

	// Inform the renderer when Sequencer is in a 'paused' state for the sake of inter-frame effects
	ESequencerState SequencerState = ESS_None;
	if (InPlaybackStatus == EMovieScenePlayerStatus::Playing)
	{
		SequencerState = ESS_Playing;
	}
	else if (InPlaybackStatus == EMovieScenePlayerStatus::Stopped || InPlaybackStatus == EMovieScenePlayerStatus::Scrubbing || InPlaybackStatus == EMovieScenePlayerStatus::Stepping)
	{
		// We may have been waiting for our time controller to tell us it was ready to play,
		// so TimeControllerState may be ETimeControllerState::PreparingToPlay. Here we reset that back to its
		// initialized state of true.
		TimeControllerState = ETimeControllerState::ReadyToPlay;
		SequencerState = ESS_Paused;
	}
	
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && LevelVC->AllowsCinematicControl())
		{
			LevelVC->ViewState.GetReference()->SetSequencerState(SequencerState);
		}
	}

	if (InPlaybackStatus == EMovieScenePlayerStatus::Playing)
	{
		if (Settings->GetCleanPlaybackMode())
		{
			CachedViewState.StoreViewState();
		}

		// override max frame rate
		if (PlayPosition.GetEvaluationType() == EMovieSceneEvaluationType::FrameLocked)
		{
			if (!OldMaxTickRate.IsSet())
			{
				OldMaxTickRate = GEngine->GetMaxFPS();
			}

			GEngine->SetMaxFPS(1.f / PlayPosition.GetInputRate().AsInterval());
		}
	}
	else
	{
		CachedViewState.RestoreViewState();

		StopAutoscroll();

		if (OldMaxTickRate.IsSet())
		{
			GEngine->SetMaxFPS(OldMaxTickRate.GetValue());
			OldMaxTickRate.Reset();
		}

		ShuttleMultiplier = 0;
	}

	TimeController->PlayerStatusChanged(PlaybackState, GetGlobalTime());
}

void FSequencer::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject( CompiledDataManager );
	Collector.AddReferencedObject( Settings );
	if (TimeUndoRedoHandler.UndoRedoProxy)
	{
		Collector.AddReferencedObject(TimeUndoRedoHandler.UndoRedoProxy);
	}
	if (RootSequence.Get())
	{
		Collector.AddReferencedObject( RootSequence );
	}

	Collector.AddPropertyReferences(FMovieSceneRootEvaluationTemplateInstance::StaticStruct(), &RootTemplateInstance, nullptr);
}

FString FSequencer::GetReferencerName() const
{
	return TEXT("FSequencer");
}

void FSequencer::ResetPerMovieSceneData()
{
	using namespace UE::Sequencer;

	//@todo Sequencer - We may want to preserve selections when moving between movie scenes
	ViewModel->GetSelection()->Empty();

	// This will reinitialize all extensions to rebuild the view-model hierarchy
	UMovieSceneSequence* CurrentSequence = GetFocusedMovieSceneSequence();
	TSharedPtr<FSequenceModel> RootSequenceModel = ViewModel->GetRootModel().ImplicitCast();
	RootSequenceModel->SetSequence(CurrentSequence, ActiveTemplateIDs.Top());

	RefreshTree();

	UpdateTimeBoundsToFocusedMovieScene();

	SuppressAutoEvalSignature.Reset();

	for (TSharedRef<UE::Sequencer::FSequencerNumericTypeInterface> Interface : GetNumericTypeInterfaces())
	{
		if (Interface->Interface->GetOnSettingChanged() != nullptr)
		{
			Interface->Interface->GetOnSettingChanged()->Broadcast();
		}
	}

	// @todo run through all tracks for new movie scene changes
	//  needed for audio track decompression
}

void FSequencer::RefreshUI()
{
	using namespace UE::Sequencer;

	ViewModel->GetSelection()->Empty();

	UMovieSceneSequence* FocusedSequence = GetFocusedMovieSceneSequence();
	FMovieSceneSequenceID SequenceID = GetFocusedTemplateID();

	TSharedPtr<FSequenceModel> RootSequenceModel = ViewModel->GetRootModel().ImplicitCast();
	RootSequenceModel->SetSequence(nullptr, MovieSceneSequenceID::Root);
	RootSequenceModel->SetSequence(FocusedSequence, SequenceID);

	RefreshTree();
}

TSharedRef<SWidget> FSequencer::MakeTransportControls(bool bExtended)
{
	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::Get().LoadModuleChecked<FEditorWidgetsModule>( "EditorWidgets" );

	FTransportControlArgs TransportControlArgs;
	{
		TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(FOnMakeTransportWidget::CreateSP(this, &FSequencer::OnCreateTransportRecord)));

		TransportControlArgs.OnBackwardEnd.BindSP( this, &FSequencer::OnJumpToStart );
		TransportControlArgs.OnBackwardStep.BindSP( this, &FSequencer::OnStepBackward, FFrameNumber(1) );
		TransportControlArgs.OnForwardPlay.BindSP( this, &FSequencer::OnPlayForward, true );
		TransportControlArgs.OnBackwardPlay.BindSP( this, &FSequencer::OnPlayBackward, true );
		TransportControlArgs.OnForwardStep.BindSP( this, &FSequencer::OnStepForward, FFrameNumber(1) );
		TransportControlArgs.OnForwardEnd.BindSP( this, &FSequencer::OnJumpToEnd );
		TransportControlArgs.OnGetPlaybackMode.BindSP( this, &FSequencer::GetPlaybackMode );

		if(bExtended)
		{
			TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(FOnMakeTransportWidget::CreateSP(this, &FSequencer::OnCreateTransportSetPlaybackStart)));
		}
		TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::BackwardEnd));
		if(bExtended)
		{
			TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(FOnMakeTransportWidget::CreateSP(this, &FSequencer::OnCreateTransportJumpToPreviousKey)));
		}
		TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::BackwardStep));
		TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::BackwardPlay));
		TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::ForwardPlay));
		TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::ForwardStep));
		if(bExtended)
		{
			TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(FOnMakeTransportWidget::CreateSP(this, &FSequencer::OnCreateTransportJumpToNextKey)));
		}
		TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::ForwardEnd));
		if(bExtended)
		{
			TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(FOnMakeTransportWidget::CreateSP(this, &FSequencer::OnCreateTransportSetPlaybackEnd)));
		}
		TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(FOnMakeTransportWidget::CreateSP(this, &FSequencer::OnCreateTransportLoopMode)));
		TransportControlArgs.bAreButtonsFocusable = false;
	}

	return EditorWidgetsModule.CreateTransportControl( TransportControlArgs );
}

TSharedRef<STemporarilyFocusedSpinBox<double>> FSequencer::MakePlayTimeDisplay()
{
	return SNew(STemporarilyFocusedSpinBox<double>)
		.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>("Sequencer.PlayTimeSpinBox"))
		.PreventThrottling(true)
		.Value_Lambda([this]() -> double {
			return GetLocalTime().Time.GetFrame().Value;
		})
		.OnValueChanged_Lambda([this](double InFrame) {
			double CurrentTime = GetLocalTime().Time.AsDecimal();
			if (CurrentTime != InFrame)
			{
				SequencerWidget->SetPlayTimeClampedByWorkingRange(InFrame);
			}
		})
		.OnValueCommitted_Lambda([this](double InFrame, ETextCommit::Type) {
			double CurrentTime = GetLocalTime().Time.AsDecimal();
			if (CurrentTime != InFrame)
			{
				SequencerWidget->SetPlayTime(InFrame);
			}
		})
		.MinValue(TOptional<double>())
		.MaxValue(TOptional<double>())
		.ToolTipText_Lambda([this] {
			FFrameRate TickResolution = GetFocusedTickResolution();
			FFrameRate DisplayRate = GetFocusedDisplayRate();

			FFrameNumber CurrentFrame = GetLocalTime().Time.GetFrame();
			TRange<FFrameNumber> CurrentRange = GetSubSequenceRange().IsSet() ? GetSubSequenceRange().GetValue() : GetPlaybackRange();

			FFrameNumber FrameCount = FFrameRate::TransformTime((CurrentFrame - CurrentRange.GetLowerBoundValue() + 1).Value, TickResolution, DisplayRate).CeilToFrame();
			FFrameNumber FrameDuration = FFrameRate::TransformTime(CurrentRange.Size<FFrameNumber>().Value, TickResolution, DisplayRate).CeilToFrame();

			return FText::Format(LOCTEXT("FrameCountTooltip", "{0} of {1}"), FrameCount.Value, FrameDuration.Value);
		})
		.TypeInterface(this, &FSequencer::GetNumericTypeInterface, UE::Sequencer::ENumericIntent::Position)
		.Delta(this, &FSequencer::GetDisplayRateDeltaFrameCount)
		.LinearDeltaSensitivity(25)
		.MinDesiredWidth_Lambda([this] {
			TRange<double> ViewRange = GetViewRange();

			FString LowerBoundStr = GetNumericTypeInterface()->ToString(ViewRange.GetLowerBoundValue());
			FString UpperBoundStr = GetNumericTypeInterface()->ToString(ViewRange.GetUpperBoundValue());

			// Always measure with the negative and subframe indicator so that the size doesn't change when there is and isn't a subframe
			if (!LowerBoundStr.Contains(TEXT("*")))
			{
				LowerBoundStr += TEXT("*");
			}
			if (!LowerBoundStr.Contains(TEXT("-")))
			{
				LowerBoundStr += TEXT("-");
			}
			if (!UpperBoundStr.Contains(TEXT("*")))
			{
				UpperBoundStr += TEXT("*");
			}
			if (!UpperBoundStr.Contains(TEXT("-")))
			{
				UpperBoundStr += TEXT("-");
			}

			const FSlateFontInfo NormalFont = FCoreStyle::Get().GetFontStyle(TEXT("NormalFont"));

			const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

			FVector2D LowerTextSize = FontMeasureService->Measure(LowerBoundStr, NormalFont);
			FVector2D UpperTextSize = FontMeasureService->Measure(UpperBoundStr, NormalFont);

			return FMath::Max(LowerTextSize.X, UpperTextSize.X);
		});
}

TSharedRef<SWidget> FSequencer::OnCreateTransportSetPlaybackStart()
{
	FText SetPlaybackStartToolTip = FText::Format(LOCTEXT("SetPlayStart_Tooltip", "Set playback start to the current position ({0})"), FSequencerCommands::Get().SetStartPlaybackRange->GetInputText());

	return SNew(SButton)
		.OnClicked(this, &FSequencer::SetPlaybackStart)
		.ToolTipText(SetPlaybackStartToolTip)
		.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
		.ContentPadding(0.0f)
		.IsFocusable(false)
		[
			SNew(SImage)
				.ColorAndOpacity_Lambda([this]() { return Settings->GetPlaybackRangeStartColor(); })
				.Image(FAppStyle::Get().GetBrush("Sequencer.Transport.SetPlayStart"))
		];
}

TSharedRef<SWidget> FSequencer::OnCreateTransportJumpToPreviousKey()
{
	FText JumpToPreviousKeyToolTip = FText::Format(LOCTEXT("JumpToPreviousKey_Tooltip", "Jump to the previous key in the selected track(s) ({0})"), FSequencerCommands::Get().StepToPreviousKey->GetInputText());

	return SNew(SButton)
		.OnClicked(this, &FSequencer::JumpToPreviousKey)
		.ToolTipText(JumpToPreviousKeyToolTip)
		.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
		.ContentPadding(0.0f)
		.IsFocusable(false)
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.Image(FAppStyle::Get().GetBrush("Sequencer.Transport.JumpToPreviousKey"))
		];
}

TSharedRef<SWidget> FSequencer::OnCreateTransportJumpToNextKey()
{
	FText JumpToNextKeyToolTip = FText::Format(LOCTEXT("JumpToNextKey_Tooltip", "Jump to the next key in the selected track(s) ({0})"), FSequencerCommands::Get().StepToNextKey->GetInputText());

	return SNew(SButton)
		.OnClicked(this, &FSequencer::JumpToNextKey)
		.ToolTipText(JumpToNextKeyToolTip)
		.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
		.ContentPadding(0.0f)
		.IsFocusable(false)
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.Image(FAppStyle::Get().GetBrush("Sequencer.Transport.JumpToNextKey"))
		];
}

TSharedRef<SWidget> FSequencer::OnCreateTransportSetPlaybackEnd()
{
	FText SetPlaybackEndToolTip = FText::Format(LOCTEXT("SetPlayEnd_Tooltip", "Set playback end to the current position ({0})"), FSequencerCommands::Get().SetEndPlaybackRange->GetInputText());

	return SNew(SButton)
		.OnClicked(this, &FSequencer::SetPlaybackEnd)
		.ToolTipText(SetPlaybackEndToolTip)
		.ButtonStyle(FAppStyle::Get(), "Animation.PlayControlsButton")
		.ContentPadding(0.0f)
		.IsFocusable(false)
		[
			SNew(SImage)
			.ColorAndOpacity_Lambda([this]() { return Settings->GetPlaybackRangeEndColor(); })
			.Image(FAppStyle::Get().GetBrush("Sequencer.Transport.SetPlayEnd"))
		];
}

TSharedRef<SWidget> FSequencer::OnCreateTransportLoopMode()
{
	TSharedRef<SButton> LoopButton = SNew(SButton)
		.OnClicked(this, &FSequencer::OnCycleLoopMode)
		.ButtonStyle( FAppStyle::Get(), "Animation.PlayControlsButton" )
		.IsFocusable(false)
		.ToolTipText_Lambda([&]()
		{ 
			if (GetLoopMode() == ESequencerLoopMode::SLM_NoLoop)
			{
				return LOCTEXT("LoopModeNoLoop_Tooltip", "No looping");
			}
			else if (GetLoopMode() == ESequencerLoopMode::SLM_Loop)
			{
				return LOCTEXT("LoopModeLoop_Tooltip", "Loop playback range");
			}
			else
			{
				return LOCTEXT("LoopModeLoopSelectionRange_Tooltip", "Loop selection range");
			}
		})
		.ContentPadding(0.0f);

	TWeakPtr<SButton> WeakButton = LoopButton;

	LoopButton->SetContent(SNew(SImage)
		.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		.Image_Lambda([&, WeakButton]()
		{
			if (GetLoopMode() == ESequencerLoopMode::SLM_NoLoop)
			{
				return FAppStyle::Get().GetBrush("Animation.Loop.Disabled");
			}
			else if (GetLoopMode() == ESequencerLoopMode::SLM_Loop)
			{
				return FAppStyle::Get().GetBrush("Animation.Loop.Enabled");
			}
			else
			{
				return FAppStyle::Get().GetBrush("Animation.Loop.SelectionRange");
			}
		})
	);

	return LoopButton;
}

TSharedRef<SWidget> FSequencer::OnCreateTransportRecord()
{
	TSharedRef<SButton> RecordButton = SNew(SButton)
		.OnClicked(this, &FSequencer::OnRecord)
		.ButtonStyle( FAppStyle::Get(), "Animation.PlayControlsButton" )
		.IsFocusable(false)
		.ToolTipText_Lambda([&]()
		{
			FText OutTooltipText;
			if (OnGetCanRecord().IsBound())
			{
				OnGetCanRecord().Execute(OutTooltipText);
			}

			if (!OutTooltipText.IsEmpty())
			{
				return OutTooltipText;
			}
			else
			{
				return OnGetIsRecording().IsBound() && OnGetIsRecording().Execute() ? LOCTEXT("StopRecord_Tooltip", "Stop recording") : LOCTEXT("Record_Tooltip", "Start recording"); 
			}
		})
		.Visibility_Lambda([&] { return HostCapabilities.bSupportsRecording && OnGetCanRecord().IsBound() ? EVisibility::Visible : EVisibility::Collapsed; })
		.IsEnabled_Lambda([&] { FText OutErrorText; return OnGetCanRecord().IsBound() && OnGetCanRecord().Execute(OutErrorText); })
		.ContentPadding(0.0f);

	TWeakPtr<SButton> WeakButton = RecordButton;

	RecordButton->SetContent(SNew(SImage)
		.Image(FAppStyle::Get().GetBrush("Animation.Record"))
		.ColorAndOpacity_Lambda([this]()
		{
			if (OnGetIsRecording().IsBound() && OnGetIsRecording().Execute())
			{
				if (!RecordingAnimation.IsPlaying())
				{
					RecordingAnimation.Play(SequencerWidget.ToSharedRef(), true);
				}
				
				FLinearColor Color = FLinearColor::White;
				return FSlateColor(Color.CopyWithNewOpacity(0.2f + 0.8f * RecordingAnimation.GetLerp()));
			}

			RecordingAnimation.Pause();
			return FSlateColor::UseSubduedForeground();
		})
	);


	TSharedRef<SHorizontalBox> RecordBox = SNew(SHorizontalBox);
	RecordBox->AddSlot()
	.AutoWidth()
	[
		RecordButton
	];
	RecordBox->AddSlot()
	[
		// Intentionally add some space to separate the record button so it's not easily pressed
		SNew(SSpacer)
		.Size(FVector2D(10.0f, 0.0f))
	];

	return RecordBox;
}

UObject* FSequencer::FindSpawnedObjectOrTemplate(const FGuid& BindingId)
{
	TArrayView<TWeakObjectPtr<>> Objects = FindObjectsInCurrentSequence(BindingId);
	if (Objects.Num())
	{
		return Objects[0].Get();
	}

	UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return nullptr;
	}

	UMovieScene* FocusedMovieScene = Sequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return nullptr;
	}

	FMovieScenePossessable* Possessable = FocusedMovieScene->FindPossessable(BindingId);
	// If we're a possessable with a parent spawnable and we don't have the object, we look the object up within the default object of the spawnable
	if (Possessable && Possessable->GetParent().IsValid())
	{
		// If we're a spawnable and we don't have the object, use the default object to build up the track menu
		UObject* ParentObject = MovieSceneHelpers::GetObjectTemplate(Sequence, Possessable->GetParent(), GetSharedPlaybackState());

		if (ParentObject)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			TArrayView<TWeakObjectPtr<>> BoundObjects = State.FindBoundObjects(BindingId, GetFocusedTemplateID(), GetSharedPlaybackState());
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			for (TWeakObjectPtr<> WeakObj : BoundObjects)
			{
				if (UObject* Obj = WeakObj.Get())
				{
					return Obj;
				}
			}
		}
	}
	// If we're a possessable that references a spawnable in another sequence via SpawnableObjectBindingID,
	// resolve the spawnable's template from that sequence (needed for operations like Bake Animation Sequence
	// when evaluating in isolation mode where the referenced spawnable isn't actually spawned)
	else if (Possessable && Possessable->GetSpawnableObjectBindingID().IsValid())
	{
		const FMovieSceneObjectBindingID& SpawnableBindingID = Possessable->GetSpawnableObjectBindingID();
		FMovieSceneSequenceID ResolvedSequenceID = SpawnableBindingID.ResolveSequenceID(GetFocusedTemplateID(), GetSharedPlaybackState());
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UMovieSceneSequence* ResolvedSequence = State.FindSequence(ResolvedSequenceID);			
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (ResolvedSequence)
		{
			if (UObject* Template = MovieSceneHelpers::GetObjectTemplate(ResolvedSequence, SpawnableBindingID.GetGuid(), GetSharedPlaybackState()))
			{
				return Template;
			}
		}
	}
	// If we're a spawnable and we don't have the object, use the default object to build up the track menu
	else if (UObject* Template = MovieSceneHelpers::GetObjectTemplate(Sequence, BindingId, GetSharedPlaybackState()))
	{
		return Template;
	}

	return nullptr;
}


void
FSequencer::FCachedViewState::StoreViewState()
{
	if (bValid)
	{
		return;
	}

	bValid = true;

	GameViewStates.Empty();
	for (int32 ViewIndex = 0; ViewIndex < GEditor->GetLevelViewportClients().Num(); ++ViewIndex)
	{
		FLevelEditorViewportClient* LevelVC = GEditor->GetLevelViewportClients()[ViewIndex];
		if (LevelVC && LevelVC->AllowsCinematicControl())
		{
			GameViewStates.Add(TPair<int32, bool>(ViewIndex, LevelVC->IsInGameView()));
			LevelVC->SetGameView(true);
		}
	}
}

void
FSequencer::FCachedViewState::RestoreViewState()
{
	if (!bValid)
	{
		return;
	}

	bValid = false;

	for (int32 Index = 0; Index < GameViewStates.Num(); ++Index)
	{
		int32 ViewIndex = GameViewStates[Index].Key;
		if (GEditor->GetLevelViewportClients().IsValidIndex(ViewIndex))
		{
			FLevelEditorViewportClient* LevelVC = GEditor->GetLevelViewportClients()[ViewIndex];
			if (LevelVC && LevelVC->AllowsCinematicControl())
			{
				LevelVC->SetGameView(GameViewStates[Index].Value);
				//if turn off game view now need to make sure widget/gizmo is on
				if (GameViewStates[Index].Value == false)
				{
					LevelVC->ShowWidget(true);
				}
			}
		}
	}
	GameViewStates.Empty();
}

FReply FSequencer::OnPlay(bool bTogglePlay)
{
	if( PlaybackState == EMovieScenePlayerStatus::Playing && bTogglePlay )
	{
		Pause();
	}
	else
	{
		TRange<FFrameNumber> LocalTimeBounds = GetTimeBounds();

		FMovieSceneInverseSequenceTransform LocalToRootTransform = RootToUnwarpedLocalTransform.Inverse();

		TOptional<FFrameTime> MinInclusiveTime = LocalToRootTransform.TryTransformTime(UE::MovieScene::DiscreteInclusiveLower(LocalTimeBounds));
		TOptional<FFrameTime> MaxInclusiveTime = LocalToRootTransform.TryTransformTime(UE::MovieScene::DiscreteExclusiveUpper(LocalTimeBounds) - 1);

		FFrameTime GlobalTime = GlobalPlaybackWarpTransform.TransformTime(GetGlobalTime().Time);

		if (PlaybackSpeed > 0)
		{
			if (MinInclusiveTime.IsSet())
			{
				if (GlobalTime <= MinInclusiveTime.GetValue())
				{
					SetGlobalTime(MinInclusiveTime.GetValue());
				}
				else if (MaxInclusiveTime.IsSet() && GlobalTime >= MaxInclusiveTime.GetValue())
				{
					SetGlobalTime(MinInclusiveTime.GetValue());
				}
			}
		}
		else
		{
			if (MaxInclusiveTime.IsSet())
			{
				if (GlobalTime >= MaxInclusiveTime.GetValue())
				{
					SetGlobalTime(MaxInclusiveTime.GetValue());
				}
				else if (MinInclusiveTime.IsSet() && GlobalTime <= MinInclusiveTime.GetValue())
				{
					SetGlobalTime(MaxInclusiveTime.GetValue());
				}
			}
		}

		SpeedIndexBeforePlay = CurrentSpeedIndex;
		PlaybackSpeedBeforePlay = PlaybackSpeed;

		TimeController->PrepareToPlay(GetGlobalTime());
		if (TimeController->IsReadyToPlay())
		{
			SetPlaybackStatus(EMovieScenePlayerStatus::Playing);
			TimeControllerState = ETimeControllerState::ReadyToPlay;
		}
		else
		{
			TimeControllerState = ETimeControllerState::PreparingToPlay;
		}

		// Make sure Slate ticks during playback
		SequencerWidget->RegisterActiveTimerForPlayback();

		OnPlayDelegate.Broadcast();
	}

	return FReply::Handled();
}

FReply FSequencer::OnRecord()
{
	OnRecordDelegate.Broadcast();
	return FReply::Handled();
}

FReply FSequencer::OnPlayForward(bool bTogglePlay)
{
	if (PlaybackSpeed < 0)
	{
		PlaybackSpeed = -PlaybackSpeed;
		if (PlaybackState != EMovieScenePlayerStatus::Playing)
		{
			OnPlay(false);
		}
	}
	else
	{
		OnPlay(bTogglePlay);
	}
	return FReply::Handled();
}

FReply FSequencer::OnPlayBackward(bool bTogglePlay)
{
	if (PlaybackSpeed > 0)
	{
		PlaybackSpeed = -PlaybackSpeed;
		if (PlaybackState != EMovieScenePlayerStatus::Playing)
		{
			OnPlay(false);
		}
	}
	else
	{
		OnPlay(bTogglePlay);
	}
	return FReply::Handled();
}

FReply FSequencer::OnStepForward(FFrameNumber Increment)
{
	using namespace UE::Sequencer;

	SetPlaybackStatus(EMovieScenePlayerStatus::Stepping);

	FFrameRate          DisplayRate = GetFocusedDisplayRate();
	FQualifiedFrameTime CurrentTime = GetUnwarpedLocalTime();

	FFrameTime NewPosition = FFrameRate::TransformTime((CurrentTime.ConvertTo(DisplayRate) + Increment).FloorToFrame(), DisplayRate, CurrentTime.Rate);

	FTimeDomainOverride TimeDomain = OverrideTimeDomain(ETimeDomain::Unwarped);
	if (GetLoopMode() == ESequencerLoopMode::SLM_Loop || GetLoopMode() == ESequencerLoopMode::SLM_LoopSelectionRange)
	{
		const bool bForwards = true;
		SetLocalTimeLooped(NewPosition, CurrentTimeBreadcrumbs, bForwards);
	}
	else
	{
		SetLocalTime(NewPosition, ESnapTimeMode::STM_Interval);
	}
	return FReply::Handled();
}


FReply FSequencer::OnStepBackward(FFrameNumber Increment)
{
	using namespace UE::Sequencer;

	SetPlaybackStatus(EMovieScenePlayerStatus::Stepping);

	FFrameRate          DisplayRate = GetFocusedDisplayRate();
	FQualifiedFrameTime CurrentTime = GetUnwarpedLocalTime();

	FFrameTime NewPosition = FFrameRate::TransformTime((CurrentTime.ConvertTo(DisplayRate) - Increment).FloorToFrame(), DisplayRate, CurrentTime.Rate);

	FTimeDomainOverride TimeDomain = OverrideTimeDomain(ETimeDomain::Unwarped);
	if (GetLoopMode() == ESequencerLoopMode::SLM_Loop || GetLoopMode() == ESequencerLoopMode::SLM_LoopSelectionRange)
	{
		const bool bForwards = false;
		SetLocalTimeLooped(NewPosition, CurrentTimeBreadcrumbs, bForwards);
	}
	else
	{
		SetLocalTime(NewPosition, ESnapTimeMode::STM_Interval);
	}
	return FReply::Handled();
}


FReply FSequencer::OnJumpToStart()
{
	
	const bool bWasPlaying = Settings->GetStopPlayingWhenJumpingToStartOrEnd() == false && (GetPlaybackStatus() == EMovieScenePlayerStatus::Playing);

	SetPlaybackStatus(EMovieScenePlayerStatus::Stepping);
	SetLocalTime(UE::MovieScene::DiscreteInclusiveLower(GetTimeBounds()), ESnapTimeMode::STM_None);
	
	if (bWasPlaying)
	{
		SetPlaybackStatus(EMovieScenePlayerStatus::Playing);
	}
	return FReply::Handled();
}


FReply FSequencer::OnJumpToEnd()
{
	const bool bWasPlaying = Settings->GetStopPlayingWhenJumpingToStartOrEnd() == false && (GetPlaybackStatus() == EMovieScenePlayerStatus::Playing);

	SetPlaybackStatus(EMovieScenePlayerStatus::Stepping);
	const bool bInsetDisplayFrame = ScrubStyle == ESequencerScrubberStyle::FrameBlock && Settings->GetForceWholeFrames();

	FFrameRate LocalResolution = GetFocusedTickResolution();
	FFrameRate DisplayRate = GetFocusedDisplayRate();

	// Calculate an offset from the end to go to. If they have snapping on (and the scrub style is a block) the last valid frame is represented as one
	// whole display rate frame before the end, otherwise we just subtract a single frame which matches the behavior of hitting play and letting it run to the end.
	FFrameTime OneFrame = bInsetDisplayFrame ? FFrameRate::TransformTime(FFrameTime(1), DisplayRate, LocalResolution) : FFrameTime(1);
	FFrameTime NewTime = UE::MovieScene::DiscreteExclusiveUpper(GetTimeBounds()) - OneFrame;

	SetLocalTime(NewTime, ESnapTimeMode::STM_None);

	if (bWasPlaying)
	{
		SetPlaybackStatus(EMovieScenePlayerStatus::Playing);
	}
	return FReply::Handled();
}


FReply FSequencer::OnCycleLoopMode()
{
	ESequencerLoopMode LoopMode = Settings->GetLoopMode();
	if (LoopMode == ESequencerLoopMode::SLM_NoLoop)
	{
		Settings->SetLoopMode(ESequencerLoopMode::SLM_Loop);
	}
	else if (LoopMode == ESequencerLoopMode::SLM_Loop && !GetSelectionRange().IsEmpty())
	{
		Settings->SetLoopMode(ESequencerLoopMode::SLM_LoopSelectionRange);
	}
	else if (LoopMode == ESequencerLoopMode::SLM_LoopSelectionRange || GetSelectionRange().IsEmpty())
	{
		Settings->SetLoopMode(ESequencerLoopMode::SLM_NoLoop);
	}
	return FReply::Handled();
}


FReply FSequencer::SetPlaybackEnd()
{
	const UMovieSceneSequence* FocusedSequence = GetFocusedMovieSceneSequence();
	if (FocusedSequence)
	{
		FFrameNumber         CurrentFrame = GetLocalTime().Time.FloorToFrame();
		TRange<FFrameNumber> CurrentRange = FocusedSequence->GetMovieScene()->GetPlaybackRange();
		if (CurrentFrame >= UE::MovieScene::DiscreteInclusiveLower(CurrentRange))
		{
			CurrentRange.SetUpperBoundValue(CurrentFrame);
			SetPlaybackRange(CurrentRange);
		}
	}
	return FReply::Handled();
}

FReply FSequencer::SetPlaybackStart()
{
	const UMovieSceneSequence* FocusedSequence = GetFocusedMovieSceneSequence();
	if (FocusedSequence)
	{
		FFrameNumber         CurrentFrame = GetLocalTime().Time.FloorToFrame();
		TRange<FFrameNumber> CurrentRange = FocusedSequence->GetMovieScene()->GetPlaybackRange();
		if (CurrentFrame < UE::MovieScene::DiscreteExclusiveUpper(CurrentRange))
		{
			CurrentRange.SetLowerBound(CurrentFrame);
			SetPlaybackRange(CurrentRange);
		}
	}
	return FReply::Handled();
}

FReply FSequencer::JumpToPreviousKey()
{
	if (ViewModel->GetSelection()->Outliner.Num())
	{
		GetKeysFromSelection(SelectedKeyCollection, SMALL_NUMBER);
	}
	else
	{
		GetAllKeys(SelectedKeyCollection, SMALL_NUMBER);
	}

	if (SelectedKeyCollection.IsValid())
	{
		TRange<FFrameNumber> Range = GetTimeBounds();
		FFrameNumber FrameNumber = GetLocalTime().Time.FloorToFrame();
		const bool bAllowLoop = GetLoopMode() != SLM_NoLoop;
		TOptional<FFrameNumber> NewTime = SelectedKeyCollection->GetNextKey(FrameNumber, EFindKeyDirection::Backwards, Range, EFindKeyType::FKT_All, bAllowLoop);
		if (NewTime.IsSet())
		{
			SetPlaybackStatus(EMovieScenePlayerStatus::Stepping);

			// Ensure the time is in the current view
			FFrameRate LocalResolution = GetFocusedTickResolution();
			ScrollIntoView(NewTime.GetValue() / LocalResolution);

			SetLocalTimeDirectly(NewTime.GetValue());
		}
	}
	return FReply::Handled();
}

FReply FSequencer::JumpToNextKey()
{
	if (ViewModel->GetSelection()->Outliner.Num())
	{
		GetKeysFromSelection(SelectedKeyCollection, SMALL_NUMBER);
	}
	else
	{
		GetAllKeys(SelectedKeyCollection, SMALL_NUMBER);
	}

	if (SelectedKeyCollection.IsValid())
	{
		TRange<FFrameNumber> Range = GetTimeBounds();
		FFrameNumber FrameNumber = GetLocalTime().Time.FloorToFrame();
		const bool bAllowLoop = GetLoopMode() != SLM_NoLoop;
		TOptional<FFrameNumber> NewTime = SelectedKeyCollection->GetNextKey(FrameNumber, EFindKeyDirection::Forwards, Range, EFindKeyType::FKT_All, bAllowLoop);
		if (NewTime.IsSet())
		{
			SetPlaybackStatus(EMovieScenePlayerStatus::Stepping);

			// Ensure the time is in the current view
			FFrameRate LocalResolution = GetFocusedTickResolution();
			ScrollIntoView(NewTime.GetValue() / LocalResolution);

			SetLocalTimeDirectly(NewTime.GetValue());
		}
	}

	return FReply::Handled();
}

ESequencerLoopMode FSequencer::GetLoopMode() const
{
	return Settings->GetLoopMode();
}

void FSequencer::SetLocalTimeLooped(FFrameTime NewLocalTime, const FMovieSceneTransformBreadcrumbs& Breadcrumbs, bool bForwards)
{
	using namespace UE::MovieScene;
	using namespace UE::Sequencer;

	TOptional<EMovieScenePlayerStatus::Type> NewPlaybackStatus;

	FMovieSceneInverseSequenceTransform LocalToRootTransform = TimeOperationDomain == ETimeDomain::Unwarped
		? RootToUnwarpedLocalTransform.Inverse()
		: RootToWarpedLocalTransform.Inverse();

	// Default to the CurrentTimeBreadcrumbs
	const FMovieSceneTransformBreadcrumbs& BreadcrumbsToUse = Breadcrumbs.Num() == 0 ? CurrentTimeBreadcrumbs : Breadcrumbs;

	TOptional<FFrameTime> NewGlobalTime;

	bool         bResetPosition       = false;

	bool bHasJumped = false;
	bool bRestarted = false;

	if (PauseOnFrame.IsSet() && ((bForwards && NewLocalTime > PauseOnFrame.GetValue()) || (!bForwards && NewLocalTime < PauseOnFrame.GetValue())))
	{
		NewGlobalTime = LocalToRootTransform.TryTransformTime(PauseOnFrame.GetValue(), BreadcrumbsToUse);

		if (NewGlobalTime)
		{
			PauseOnFrame.Reset();
			bResetPosition = true;
			NewPlaybackStatus = EMovieScenePlayerStatus::Stopped;
		}
	}
	else if (GetLoopMode() == ESequencerLoopMode::SLM_Loop || GetLoopMode() == ESequencerLoopMode::SLM_LoopSelectionRange)
	{
		TRange<FFrameNumber> TimeBounds = GetTimeBounds();
		FFrameNumber MinInclusiveTime = UE::MovieScene::DiscreteInclusiveLower(TimeBounds);
		FFrameNumber MaxInclusiveTime = UE::MovieScene::DiscreteExclusiveUpper(TimeBounds) - 1;

		FFrameTime LoopTime = NewLocalTime;
		if (TimeOperationDomain == ETimeDomain::Unwarped && LocalToWarpedLocalTransform.FindFirstWarpDomain() == ETimeWarpChannelDomain::PlayRate)
		{
			LoopTime = LocalToWarpedLocalTransform.TransformTime(LoopTime);
		}

		if (LoopTime < MinInclusiveTime || LoopTime > MaxInclusiveTime)
		{
			NewGlobalTime = LocalToRootTransform.TryTransformTime((bForwards ? MinInclusiveTime : MaxInclusiveTime), BreadcrumbsToUse);
			if (NewGlobalTime)
			{
				bResetPosition = true;
				bHasJumped = true;
			}
		}
		else
		{
			NewGlobalTime = LocalToRootTransform.TryTransformTime(NewLocalTime, BreadcrumbsToUse);
		}
	}
	else
	{
		NewGlobalTime = LocalToRootTransform.TryTransformTime(NewLocalTime, BreadcrumbsToUse);

		if (NewGlobalTime)
		{
			FFrameTime NewWarpedGlobalTime = GlobalPlaybackWarpTransform.TransformTime(NewGlobalTime.GetValue());
			FFrameTime OldWarpedGlobalTime = GlobalPlaybackWarpTransform.TransformTime(GetGlobalTime().Time);

			TRange<FFrameNumber> RootTimeBounds = GetRootTimeBounds();
			FFrameNumber MinInclusiveTime = UE::MovieScene::DiscreteInclusiveLower(RootTimeBounds);
			FFrameNumber MaxInclusiveTime = UE::MovieScene::DiscreteExclusiveUpper(RootTimeBounds) - 1;
			bool bReachedEnd = false;
			if (bForwards)
			{
				bReachedEnd = OldWarpedGlobalTime <= MaxInclusiveTime && NewWarpedGlobalTime >= MaxInclusiveTime;
			}
			else
			{
				bReachedEnd = OldWarpedGlobalTime >= MinInclusiveTime && NewWarpedGlobalTime <= MinInclusiveTime;
			}

			// Stop if we hit the playback range end
			if (bReachedEnd)
			{
				NewGlobalTime = GlobalPlaybackWarpTransform.Inverse().TryTransformTime(bForwards ? MaxInclusiveTime : MinInclusiveTime);
				NewPlaybackStatus = EMovieScenePlayerStatus::Stopped;
			}
		}
	}

	if (!NewGlobalTime)
	{
		NewGlobalTime = LocalToRootTransform.TryTransformTime(NewLocalTime, BreadcrumbsToUse);
	}

	if (!NewGlobalTime && RootToUnwarpedLocalTransform.NestedTransforms.Num() > 0)
	{
		// If we couldn't transform the time, we try one last-ditch attempt to guess a cycling or looping
		//     transformation based on the linear transformation between each of the breadcrumbs.
		FMovieSceneTransformBreadcrumbs DenseBreadcrumbs;
		FFrameTime CurrentLocalTime = RootToUnwarpedLocalTransform.TransformTime(GetGlobalTime().Time, FTransformTimeParams().HarvestBreadcrumbs(DenseBreadcrumbs));

		FFrameTime LocalDeltaGuess   = NewLocalTime - CurrentLocalTime;
		FFrameTime NewLocalTimeGuess = RootToUnwarpedLocalTransform.NestedTransforms.Last().TransformTime(LocalDeltaGuess);

		NewGlobalTime = LocalToRootTransform.TryTransformTime(NewLocalTimeGuess, DenseBreadcrumbs);
	}

	if (!NewGlobalTime)
	{
		return;
	}

	FFrameRate   RootTickResolution = GetRootTickResolution();
	// Ensure the time is in the current view - must occur before the time cursor changes
	UMovieScene* MovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();
	if (IsAutoScrollEnabled())
	{
		ScrollIntoView((NewGlobalTime.GetValue() * RootToUnwarpedLocalTransform) / RootTickResolution);
	}

	FFrameTime NewPlayPosition = ConvertFrameTime(NewGlobalTime.GetValue(), RootTickResolution, PlayPosition.GetInputRate());

	// Reset the play cursor if we're looping or have otherwise jumpted to a new position in the sequence
	if (bResetPosition)
	{
		PlayPosition.Reset(NewPlayPosition);
		TimeController->Reset(FQualifiedFrameTime(NewGlobalTime.GetValue(), RootTickResolution));
	}

	// Ensure breadcrumbs are up to date
	RootToUnwarpedLocalTransform.TransformTime(NewGlobalTime.GetValue(), FTransformTimeParams().HarvestBreadcrumbs(CurrentTimeBreadcrumbs).IgnoreClamps());

	// Evaluate the sequence
	FMovieSceneEvaluationRange EvalRange = PlayPosition.PlayTo(NewPlayPosition);
	EvaluateInternal(EvalRange, bHasJumped);

	// Set the playback status if we need to
	if (NewPlaybackStatus.IsSet())
	{
		SetPlaybackStatus(NewPlaybackStatus.GetValue());
		// Evaluate the sequence with the new status
		EvaluateInternal(PlayPosition.GetCurrentPositionAsRange());

		if (NewPlaybackStatus.GetValue() == EMovieScenePlayerStatus::Stopped)
		{
			RestorePlaybackSpeedAfterPlay();

			OnStopDelegate.Broadcast();
		}
	}
}

void FSequencer::SetPlaybackSpeed(float InPlaybackSpeed)
{
	PlaybackSpeed = InPlaybackSpeed;

	const bool bExactOnly = true;
	CurrentSpeedIndex = FindClosestPlaybackSpeed(InPlaybackSpeed, bExactOnly);
}

void FSequencer::RestorePlaybackSpeedAfterPlay()
{
	// Reset the speed to what it was before we started playing, in case the user increased/decreased the speed while it
	// was playing.
	CurrentSpeedIndex = SpeedIndexBeforePlay;
	PlaybackSpeed = PlaybackSpeedBeforePlay;
}

EPlaybackMode::Type FSequencer::GetPlaybackMode() const
{
	if (PlaybackState == EMovieScenePlayerStatus::Playing)
	{
		if (PlaybackSpeed > 0)
		{
			return EPlaybackMode::PlayingForward;
		}
		else
		{
			return EPlaybackMode::PlayingReverse;
		}
	}
		
	return EPlaybackMode::Stopped;
}

void FSequencer::UpdateTimeBoundsToFocusedMovieScene()
{
	UMovieScene* FocusedMovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	// We have to update subsequence data before getting the local time on a new subsequence
	UpdateSubSequenceData();

	FQualifiedFrameTime CurrentTime = GetLocalTime();

	// Set the view range to:
	// 1. The moviescene view range
	// 2. The moviescene playback range
	// 3. Some sensible default
	TRange<double> NewRange = FocusedMovieScene->GetEditorData().GetViewRange();

	if (NewRange.IsEmpty() || NewRange.IsDegenerate())
	{
		NewRange = FocusedMovieScene->GetPlaybackRange() / CurrentTime.Rate;
	}
	if (NewRange.IsEmpty() || NewRange.IsDegenerate())
	{
		NewRange = TRange<double>(0.0, 5.0);
	}

	// Set the view range to the new range
	SetViewRange(NewRange, EViewRangeInterpolation::Immediate);
}


TRange<FFrameNumber> FSequencer::GetTimeBounds() const
{
	const UMovieSceneSequence* FocusedSequence = GetFocusedMovieSceneSequence();

	if(!FocusedSequence)
	{
		return TRange<FFrameNumber>( -100000, 100000 );
	}
	
	if (GetLoopMode() == ESequencerLoopMode::SLM_LoopSelectionRange)
	{
		if (!GetSelectionRange().IsEmpty())
		{
			return GetSelectionRange();
		}
	}

	if (ActiveTemplateIDs.Num() == 1)
	{
		return GetRootTimeBounds();
	}

	if (Settings->ShouldEvaluateSubSequencesInIsolation())
	{
		return FocusedSequence->GetMovieScene()->GetPlaybackRange();
	}

	return SubSequenceRange;
}

TRange<FFrameNumber> FSequencer::GetRootTimeBounds() const
{
	const UMovieSceneSequence* RootMovieSceneSequence = GetRootMovieSceneSequence();
	if (!RootMovieSceneSequence)
	{
		return TRange<FFrameNumber>(-100000, 100000);
	}

	return RootMovieSceneSequence->GetMovieScene()->GetPlaybackRange();
}

void FSequencer::RefreshSupportedCustomBindingTypes()
{
	MovieSceneHelpers::GetPrioritySortedCustomBindingTypes(SupportedCustomBindingTypes);
}

void FSequencer::SetViewRange(TRange<double> NewViewRange, EViewRangeInterpolation Interpolation)
{
	if (!ensure(NewViewRange.HasUpperBound() && NewViewRange.HasLowerBound() && !NewViewRange.IsDegenerate()))
	{
		return;
	}

	const float AnimationLengthSeconds = Interpolation == EViewRangeInterpolation::Immediate ? 0.f : 0.1f;
	if (AnimationLengthSeconds != 0.f)
	{
		if (ZoomAnimation.GetCurve(0).DurationSeconds != AnimationLengthSeconds)
		{
			ZoomAnimation = FCurveSequence();
			ZoomCurve = ZoomAnimation.AddCurve(0.f, AnimationLengthSeconds, ECurveEaseFunction::QuadIn);
		}

		if (!ZoomAnimation.IsPlaying())
		{
			LastViewRange = TargetViewRange;
			ZoomAnimation.Play( SequencerWidget.ToSharedRef() );
		}
		TargetViewRange = NewViewRange;
	}
	else
	{
		TargetViewRange = LastViewRange = NewViewRange;
		ZoomAnimation.JumpToEnd();
	}


	UMovieSceneSequence* FocusedMovieSequence = GetFocusedMovieSceneSequence();
	if (FocusedMovieSequence != nullptr)
	{
		UMovieScene* FocusedMovieScene = FocusedMovieSequence->GetMovieScene();
		if (FocusedMovieScene != nullptr)
		{
			FMovieSceneEditorData& EditorData = FocusedMovieScene->GetEditorData();
			EditorData.ViewStart = TargetViewRange.GetLowerBoundValue();
			EditorData.ViewEnd   = TargetViewRange.GetUpperBoundValue();

			// Always ensure the working range is big enough to fit the view range
			EditorData.WorkStart = FMath::Min(TargetViewRange.GetLowerBoundValue(), EditorData.WorkStart);
			EditorData.WorkEnd   = FMath::Max(TargetViewRange.GetUpperBoundValue(), EditorData.WorkEnd);
		}
	}
}


void FSequencer::OnClampRangeChanged( TRange<double> NewClampRange )
{
	if (!NewClampRange.IsEmpty())
	{
		FMovieSceneEditorData& EditorData =  GetFocusedMovieSceneSequence()->GetMovieScene()->GetEditorData();

		EditorData.WorkStart = NewClampRange.GetLowerBoundValue();
		EditorData.WorkEnd   = NewClampRange.GetUpperBoundValue();
	}
}

FFrameNumber FSequencer::OnGetNearestKey(FFrameTime InTime, ENearestKeyOption NearestKeyOption)
{
	const FFrameNumber CurrentTime = InTime.FloorToFrame();

	const bool bComputeNearest = EnumHasAnyFlags(NearestKeyOption, ENearestKeyOption::NKO_SearchKeys | ENearestKeyOption::NKO_SearchSections | ENearestKeyOption::NKO_SearchMarkers);
	if (!bComputeNearest)
	{
		return CurrentTime;
	}

	if (ViewModel->GetSelection()->Outliner.Num() == 0)
	{
		GetAllKeys(SelectedKeyCollection, SMALL_NUMBER);
	}
	else
	{
		GetKeysFromSelection(SelectedKeyCollection, SMALL_NUMBER);
	}

	TOptional<FFrameNumber> NearestTime;

	if (EnumHasAnyFlags(NearestKeyOption, ENearestKeyOption::NKO_SearchKeys) && SelectedKeyCollection.IsValid())
	{
		TOptional<FFrameNumber> NearestKeyTime;

		TRange<FFrameNumber> FindRangeBackwards(TRangeBound<FFrameNumber>::Open(), CurrentTime);
		TOptional<FFrameNumber> NewTimeBackwards = SelectedKeyCollection->FindFirstKeyInRange(FindRangeBackwards, EFindKeyDirection::Backwards, EFindKeyType::FKT_Keys);

		TRange<FFrameNumber> FindRangeForwards(CurrentTime, TRangeBound<FFrameNumber>::Open());
		TOptional<FFrameNumber> NewTimeForwards = SelectedKeyCollection->FindFirstKeyInRange(FindRangeForwards, EFindKeyDirection::Forwards, EFindKeyType::FKT_Keys);
		if (NewTimeForwards.IsSet())
		{
			if (NewTimeBackwards.IsSet())
			{
				if (FMath::Abs(NewTimeForwards.GetValue() - CurrentTime) < FMath::Abs(NewTimeBackwards.GetValue() - CurrentTime))
				{
					NearestKeyTime = NewTimeForwards.GetValue();
				}
				else
				{
					NearestKeyTime = NewTimeBackwards.GetValue();
				}
			}
			else
			{
				NearestKeyTime = NewTimeForwards.GetValue();
			}
		}
		else if (NewTimeBackwards.IsSet())
		{
			NearestKeyTime = NewTimeBackwards.GetValue();
		}

		if (NearestKeyTime.IsSet())
		{
			NearestTime = NearestKeyTime.GetValue();
		}
	}

	if (EnumHasAnyFlags(NearestKeyOption, ENearestKeyOption::NKO_SearchSections) && SelectedKeyCollection.IsValid())
	{
		TOptional<FFrameNumber> NearestSectionTime;

		TRange<FFrameNumber> FindRangeBackwards(TRangeBound<FFrameNumber>::Open(), CurrentTime);
		TOptional<FFrameNumber> NewTimeBackwards = SelectedKeyCollection->FindFirstKeyInRange(FindRangeBackwards, EFindKeyDirection::Backwards, EFindKeyType::FKT_Sections);

		TRange<FFrameNumber> FindRangeForwards(CurrentTime, TRangeBound<FFrameNumber>::Open());
		TOptional<FFrameNumber> NewTimeForwards = SelectedKeyCollection->FindFirstKeyInRange(FindRangeForwards, EFindKeyDirection::Forwards, EFindKeyType::FKT_Sections);
		if (NewTimeForwards.IsSet())
		{
			if (NewTimeBackwards.IsSet())
			{
				if (FMath::Abs(NewTimeForwards.GetValue() - CurrentTime) < FMath::Abs(NewTimeBackwards.GetValue() - CurrentTime))
				{
					NearestSectionTime = NewTimeForwards.GetValue();
				}
				else
				{
					NearestSectionTime = NewTimeBackwards.GetValue();
				}
			}
			else
			{
				NearestSectionTime = NewTimeForwards.GetValue();
			}
		}
		else if (NewTimeBackwards.IsSet())
		{
			NearestSectionTime = NewTimeBackwards.GetValue();
		}

		if (NearestSectionTime.IsSet())
		{
			if (!NearestTime.IsSet())
			{
				NearestTime = NearestSectionTime.GetValue();
			}
			else if (FMath::Abs(NearestSectionTime.GetValue() - CurrentTime) < FMath::Abs(NearestTime.GetValue() - CurrentTime))
			{
				NearestTime = NearestSectionTime.GetValue();
			}
		}
	}
	
	if (EnumHasAnyFlags(NearestKeyOption, ENearestKeyOption::NKO_SearchMarkers))
	{
		TArray<FFrameNumber> Markers;

		TArray<FMovieSceneMarkedFrame> MarkedFrames = GetMarkedFrames();
		for (const FMovieSceneMarkedFrame& MarkedFrame : MarkedFrames)
		{
			Markers.Add(MarkedFrame.FrameNumber);
		}

		TArray<FMovieSceneMarkedFrame> GlobalMarkedFrames = GetGlobalMarkedFrames();
		for (const FMovieSceneMarkedFrame& GlobalMarkedFrame : GlobalMarkedFrames)
		{
			Markers.Add(GlobalMarkedFrame.FrameNumber);
		}

		if (UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence())
		{
			const bool bInsetDisplayFrame = ScrubStyle == ESequencerScrubberStyle::FrameBlock && Settings->GetForceWholeFrames();
			FFrameRate LocalResolution = GetFocusedTickResolution();
			FFrameRate DisplayRate = GetFocusedDisplayRate();
			FFrameTime OneFrame = bInsetDisplayFrame ? FFrameRate::TransformTime(FFrameTime(1), DisplayRate, LocalResolution) : FFrameTime(1);

			TRange<FFrameNumber> PlaybackRange = Sequence->GetMovieScene()->GetPlaybackRange();
			Markers.Add(UE::MovieScene::DiscreteInclusiveLower(PlaybackRange));
			Markers.Add(UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange) - OneFrame.FrameNumber);

			TRange<FFrameNumber> SelectionRange = Sequence->GetMovieScene()->GetSelectionRange();
			Markers.Add(UE::MovieScene::DiscreteInclusiveLower(SelectionRange));
			Markers.Add(UE::MovieScene::DiscreteExclusiveUpper(SelectionRange) - OneFrame.FrameNumber);
		}

		for (const FFrameNumber& FrameNumber : Markers)
		{
			if (!NearestTime.IsSet())
			{
				NearestTime = FrameNumber;
			}
			else if (FMath::Abs(FrameNumber - CurrentTime) < FMath::Abs(NearestTime.GetValue() - CurrentTime))
			{
				NearestTime = FrameNumber;
			}
		}
	}

	return NearestTime.IsSet() ? NearestTime.GetValue() : CurrentTime;
}

void FSequencer::OnScrubPositionChanged( FFrameTime NewScrubPosition, bool bScrubbing , bool bEvaluate)
{
	if (PlaybackState == EMovieScenePlayerStatus::Scrubbing)
	{
		if (!bScrubbing)
		{
			NewScrubPosition += ScrubLinearOffset;
			OnEndScrubbing();
		}
		else
		{
			UpdateAutoScroll(NewScrubPosition / GetFocusedTickResolution());
			
			// When scrubbing, we animate auto-scrolled scrub position in Tick()
			if (AutoscrubOffset.IsSet())
			{
				return;
			}
		}
	}

	if (!bScrubbing && CVarAutoScrub->GetBool() && FSlateApplication::Get().GetModifierKeys().IsShiftDown())
	{
		AutoScrubToTime(NewScrubPosition);
	}
	else if (bEvaluate)
	{
		// Evaluation can be expensive and we may receive multiple OnScrubPositionChanged events in
		// a frame, so defer the evaluation until the next tick
		PendingScrubPosition = MakeTuple(NewScrubPosition, TimeOperationDomain);
	}
	else
	{
		SetLocalTimeDirectly(NewScrubPosition, bEvaluate);
	}
}


void FSequencer::OnBeginScrubbing()
{
	SetPlaybackStatus(EMovieScenePlayerStatus::Scrubbing);
	SequencerWidget->RegisterActiveTimerForPlayback();

	ScrubLinearOffset = FFrameTime(0);
	ScrubStartBreadcrumbs = CurrentTimeBreadcrumbs;

	OnBeginScrubbingDelegate.Broadcast();
}


void FSequencer::OnEndScrubbing()
{
	SetPlaybackStatus(EMovieScenePlayerStatus::Stopped);
	AutoscrubOffset.Reset();
	StopAutoscroll();

	ScrubStartBreadcrumbs = FMovieSceneTransformBreadcrumbs();
	ScrubLinearOffset = FFrameTime(0);

	OnEndScrubbingDelegate.Broadcast();
}


void FSequencer::OnPlaybackRangeBeginDrag()
{
	bIsDraggingPlaybackRange = true;
	bPlaybackRangeChangedDuringDrag = false;

	GEditor->BeginTransaction(LOCTEXT("SetPlaybackRange_Transaction", "Set Playback Range"));
}


void FSequencer::OnPlaybackRangeEndDrag()
{
	GEditor->EndTransaction();

	bIsDraggingPlaybackRange = false;

	if (bPlaybackRangeChangedDuringDrag)
	{
		bPlaybackRangeChangedDuringDrag = false;

		NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}
}


void FSequencer::OnSelectionRangeBeginDrag()
{
	GEditor->BeginTransaction(LOCTEXT("SetSelectionRange_Transaction", "Set Selection Range"));
}


void FSequencer::OnSelectionRangeEndDrag()
{
	GEditor->EndTransaction();
}


void FSequencer::OnMarkBeginDrag()
{	
	bDraggingMarks = true;
	GEditor->BeginTransaction(LOCTEXT("SetMark_Transaction", "Set Mark"));
}


void FSequencer::OnMarkEndDrag()
{
	bDraggingMarks = false;
	UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
	UMovieScene* OwnerMovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (OwnerMovieScene)
	{
		OwnerMovieScene->SortMarkedFrames();
	}
	GEditor->EndTransaction();
}



FString FSequencer::GetFrameTimeText() const
{
	using namespace UE::MovieScene;

	const bool bWarpedTime = Settings->GetTimeWarpDisplayMode() == ESequencerTimeWarpDisplay::WarpedTime;

	const FMovieSceneSequenceTransform* RootToParentChainTransform = bWarpedTime
		? &RootToWarpedLocalTransform
		: &RootToUnwarpedLocalTransform;

	FFrameTime LocalTime = bWarpedTime ? GlobalPlaybackWarpTransform.TransformTime(GetGlobalTime().Time) : GetGlobalTime().Time;

	if (ScrubPositionParent.IsSet() && ScrubPositionParent.GetValue() != MovieSceneSequenceID::Root)
	{
		if (const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(RootTemplateInstance.GetCompiledDataID()))
		{
			for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
			{
				if (Pair.Key == ScrubPositionParent.GetValue())
				{
					RootToParentChainTransform = bWarpedTime 
						? &Pair.Value.RootToSequenceTransform
						: &Pair.Value.RootToUnwarpedLocalTransform;
					break;
				}
			}
		}
		LocalTime = RootToParentChainTransform->TransformTime(GetGlobalTime().Time, FTransformTimeParams().IgnoreClamps());
	}

	return GetNumericTypeInterface()->ToString(LocalTime.GetFrame().Value);
}

FMovieSceneSequenceID FSequencer::GetScrubPositionParent() const
{
	if (ScrubPositionParent.IsSet())
	{
		return ScrubPositionParent.GetValue();
	}
	return MovieSceneSequenceID::Invalid;
}
	
	
TArray<FMovieSceneSequenceID> FSequencer::GetScrubPositionParentChain() const
{
	TArray<FMovieSceneSequenceID> ParentChain;
	for (FMovieSceneSequenceID SequenceID : ActiveTemplateIDs)
	{
		ParentChain.Add(SequenceID);
	}
	return ParentChain;
}

void FSequencer::OnScrubPositionParentChanged(FMovieSceneSequenceID InScrubPositionParent)
{
	if (InScrubPositionParent == GetFocusedTemplateID())
	{
		ScrubPositionParent.Reset();
	}
	else
	{
		ScrubPositionParent = InScrubPositionParent;
	}
}

void FSequencer::StartAutoscroll(float UnitsPerS)
{
	AutoscrollOffset = UnitsPerS;
}


void FSequencer::StopAutoscroll()
{
	AutoscrollOffset.Reset();
	AutoscrubOffset.Reset();
}


void FSequencer::OnToggleAutoScroll()
{
	Settings->SetAutoScrollEnabled(!Settings->GetAutoScrollEnabled());
}


bool FSequencer::IsAutoScrollEnabled() const
{
	return Settings->GetAutoScrollEnabled();
}


void FSequencer::FindInContentBrowser()
{
	if (GetFocusedMovieSceneSequence())
	{
		TArray<UObject*> ObjectsToFocus;
		ObjectsToFocus.Add(GetCurrentAsset());

		GEditor->SyncBrowserToObjects(ObjectsToFocus);
	}
}

void FSequencer::BrowseToObject()
{
	using namespace UE::Sequencer;

	TArray<UObject*> ObjectsToFocus;

	for (TViewModelPtr<FSectionModel> SectionModel : ViewModel->GetSelection()->TrackArea.Filter<FSectionModel>())
	{
		if (UMovieSceneSection* Section = SectionModel->GetSection())
		{	
			if (UObject* ObjectToFocus = Section->GetSourceObject())
			{
				ObjectsToFocus.Add(ObjectToFocus);
			}
		}
	}

	if (ObjectsToFocus.Num() > 0)
	{
		GEditor->SyncBrowserToObjects(ObjectsToFocus);
	}
}

UObject* FSequencer::GetCurrentAsset() const
{
	// For now we find the asset by looking at the root movie scene's outer.
	// @todo: this may need refining if/when we support editing movie scene instances
	return GetFocusedMovieSceneSequence()->GetMovieScene()->GetOuter();
}

bool FSequencer::IsReadOnly() const
{
	return bReadOnly || (GetFocusedMovieSceneSequence() && GetFocusedMovieSceneSequence()->GetMovieScene() && GetFocusedMovieSceneSequence()->GetMovieScene()->IsReadOnly());
}

void FSequencer::ExternalSelectionHasChanged()
{
	SynchronizeSequencerSelectionWithExternalSelection();
}

FGuid FSequencer::MakeNewSpawnable(UObject& Object, UActorFactory* ActorFactory, bool bSetupDefaults)
{
	const FScopedTransaction Transaction(LOCTEXT("UndoAddingObject", "Add Object to MovieScene"));

	TArray<UMovieSceneFolder*> SelectedParentFolders;
	FString NewNodePath;
	CalculateSelectedFolderAndPath(SelectedParentFolders, NewNodePath);

	FGuid NewGuid = FSequencerUtilities::MakeNewSpawnable(AsShared(), Object, ActorFactory, bSetupDefaults);
	
	if (SelectedParentFolders.Num() > 0)
	{
		SelectedParentFolders[0]->AddChildObjectBinding(NewGuid);
	}

	return NewGuid;
}

void FSequencer::AddSubSequence(UMovieSceneSequence* Sequence)
{
	// @todo Sequencer - sub-moviescenes This should be moved to the sub-moviescene editor

	// Grab the MovieScene that is currently focused.  This is the movie scene that will contain the sub-moviescene
	UMovieScene* OwnerMovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();

	if (OwnerMovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	// @todo sequencer: Undo doesn't seem to be working at all
	const FScopedTransaction Transaction( LOCTEXT("UndoAddingObject", "Add Object to MovieScene") );
	OwnerMovieScene->Modify();

	UMovieSceneSubTrack* SubTrack = OwnerMovieScene->AddTrack<UMovieSceneSubTrack>();

	FFrameNumber Duration = ConvertFrameTime(
		Sequence->GetMovieScene()->GetPlaybackRange().Size<FFrameNumber>(),
		Sequence->GetMovieScene()->GetTickResolution(),
		OwnerMovieScene->GetTickResolution()).FloorToFrame();

	SubTrack->AddSequence(Sequence, GetLocalTime().Time.FloorToFrame(), Duration.Value);
}


bool FSequencer::OnHandleAssetDropped(UObject* DroppedAsset, const FGuid& TargetObjectGuid)
{
	bool bWasConsumed = false;
	for (int32 i = 0; i < TrackEditors.Num(); ++i)
	{
		bool bWasHandled = TrackEditors[i]->HandleAssetAdded(DroppedAsset, TargetObjectGuid);
		if (bWasHandled)
		{
			// @todo Sequencer - This will crash if multiple editors try to handle a single asset
			// Should we allow this? How should it consume then?
			// gmp 10/7/2015: the user should be presented with a dialog asking what kind of track they want to create
			check(!bWasConsumed);
			bWasConsumed = true;
		}
	}
	return bWasConsumed;
}

bool FSequencer::OnRequestNodeDeleted( TSharedRef<FViewModel> NodeToBeDeleted, const bool bKeepState )
{
	using namespace UE::Sequencer;
	using namespace UE::MovieScene;

	// Find out which moviescene this is in
	TViewModelPtr<FSequenceModel> OwnerModel = NodeToBeDeleted->FindAncestorOfType<FSequenceModel>();
	if (!OwnerModel)
	{
		return false;
	}

	UMovieScene* OwnerMovieScene = OwnerModel->GetMovieScene();
	if (OwnerMovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return false;
	}

	if (bKeepState)
	{
		FMovieSceneSequenceID SequenceID = OwnerModel->GetSequenceID();

		for (const TViewModelPtr<IObjectBindingExtension>& ObjectBinding :
			NodeToBeDeleted->GetDescendantsOfType<IObjectBindingExtension>(true, EViewModelListType::Outliner))
		{
			for (TWeakObjectPtr<> WeakObject : FindBoundObjects(ObjectBinding->GetObjectGuid(), SequenceID))
			{
				if (WeakObject.IsValid())
				{
					TArray<UObject*> SubObjects;
					GetObjectsWithOuter(WeakObject.Get(), SubObjects);

					PreAnimatedState.DiscardAndRemoveEntityTokensForObject(*WeakObject.Get());

					for (UObject* SubObject : SubObjects)
					{
						if (SubObject)
						{
							PreAnimatedState.DiscardAndRemoveEntityTokensForObject(*SubObject);
						}
					}
				}
			}
		}
	}

	if (IDeletableExtension* Deletable = NodeToBeDeleted->CastThis<IDeletableExtension>())
	{
		if (!Deletable->CanDelete(nullptr))
		{
			return false;
		}

		Deletable->Delete();

		const TViewModelPtr<IOutlinerExtension> OutlinerItem = NodeToBeDeleted->CastThisShared<IOutlinerExtension>();

		const TSharedPtr<FSequencerFilterBar> FilterBar = GetFilterBar();
		if (FilterBar->HasIsolatedTracks())
		{
			FilterBar->UnisolateTracks({ OutlinerItem });
		}

		if (FilterBar->HasHiddenTracks())
		{
			FilterBar->UnhideTracks({ OutlinerItem });
		}

		return true;
	}

	return false;
}

bool FSequencer::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjects) const
{
	// Check if we care about the undo/redo
	for (const TPair<UObject*, FTransactionObjectEvent>& TransactionObjectPair : TransactionObjects)
	{
		if (TransactionObjectPair.Value.HasPendingKillChange())
		{
			return true;
		}

		UObject* Object = TransactionObjectPair.Key;
		while (Object != nullptr)
		{
			if (Object->GetClass()->IsChildOf(UMovieSceneSignedObject::StaticClass()))
			{
				return true;
			}
			Object = Object->GetOuter();
		}
	}
	return false;
}

void FSequencer::PostUndo(bool bSuccess)
{
	if (UE::Sequencer::bEvaluateOnUndo)
	{
		NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::Unknown);
	}
	SynchronizeSequencerSelectionWithExternalSelection();
	OnNodeGroupsCollectionChanged();

	UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
	UMovieScene* OwnerMovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (OwnerMovieScene)
	{
		OwnerMovieScene->SortMarkedFrames();
	}

	NodeTree->SortAllNodesAndDescendants();
	NodeTree->RequestFilterUpdate();

	OnActivateSequenceEvent.Broadcast(ActiveTemplateIDs.Top());
}

void FSequencer::OnNewActorsDropped(const TArray<UObject*>& DroppedObjects, const TArray<AActor*>& DroppedActors)
{
	UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
	UMovieScene* OwnerMovieScene = Sequence->GetMovieScene();

	bool bAddSpawnable = FSlateApplication::Get().GetModifierKeys().IsShiftDown() && Sequence->AllowsSpawnableObjects();
	bool bAddPossessable = FSlateApplication::Get().GetModifierKeys().IsControlDown();
	bool bAddReplaceable = FSlateApplication::Get().GetModifierKeys().IsAltDown();

	if (bAddSpawnable || bAddPossessable || bAddReplaceable)
	{
		TArray<AActor*> SpawnedActors;

		const FScopedTransaction Transaction(LOCTEXT("UndoAddActors", "Add Actors to Sequencer"));

		if (OwnerMovieScene->IsReadOnly())
		{
			FSequencerUtilities::ShowReadOnlyError();
			return;
		}

		Sequence->Modify();

		TArray<UMovieSceneFolder*> SelectedParentFolders;
		FString NewNodePath;
		CalculateSelectedFolderAndPath(SelectedParentFolders, NewNodePath);
		UMovieSceneFolder* ParentFolder = SelectedParentFolders.Num() > 0 ? SelectedParentFolders[0] : nullptr;

		for ( AActor* Actor : DroppedActors )
		{
			if (AActor* NewActor = Actor)
			{
				FGuid PossessableGuid = FSequencerUtilities::CreateBinding(AsShared(), *NewActor);
				FGuid NewGuid = PossessableGuid;

				if (bAddSpawnable || bAddReplaceable)
				{
					TSubclassOf<UMovieSceneCustomBinding> CustomBindingClass = bAddSpawnable ? UMovieSceneSpawnableActorBinding::StaticClass() : UMovieSceneReplaceableActorBinding::StaticClass();

					const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences();

					if (BindingReferences)
					{
						for (const FMovieSceneBindingReference& Reference : BindingReferences->GetReferences(PossessableGuid))
						{
							for (const TSubclassOf<UMovieSceneCustomBinding>& SupportedCustomBindingType : SupportedCustomBindingTypes)
							{
								if (SupportedCustomBindingType && SupportedCustomBindingType->IsChildOf(CustomBindingClass) && 
									SupportedCustomBindingType->GetDefaultObject<UMovieSceneCustomBinding>()->SupportsConversionFromBinding(Reference, Actor))
								{
									FMovieScenePossessable* NewPossessable = FSequencerUtilities::ConvertToCustomBinding(AsShared(), NewGuid, CustomBindingClass);

									if (NewPossessable)
									{
										for (TWeakObjectPtr<> WeakObject : FindBoundObjects(NewPossessable->GetGuid(), ActiveTemplateIDs.Top()))
										{
											AActor* SpawnedActor = Cast<AActor>(WeakObject.Get());
											if (SpawnedActor)
											{
												SpawnedActors.Add(SpawnedActor);
												NewActor = SpawnedActor;
											}
										}

										NewGuid = NewPossessable->GetGuid();
									}
									break;
								}
							}
						}
					}
				}

				OnActorAddedToSequencerEvent.Broadcast(NewActor, NewGuid);

				if (NewActor && (NewActor->GetClass() == ACameraRig_Rail::StaticClass() || NewActor->GetClass() == ACameraRig_Crane::StaticClass()))
				{
					ACineCameraActor* OutActor;
					FSequencerUtilities::CreateCameraWithRig(AsShared(), NewActor, bAddSpawnable, OutActor);
				}

				if (ParentFolder)
				{
					ParentFolder->AddChildObjectBinding(NewGuid);
				}
			}
		}

		if (SpawnedActors.Num())
		{
			const bool bNotifySelectionChanged = true;
			const bool bDeselectBSP = true;
			const bool bWarnAboutTooManyActors = false;
			const bool bSelectEvenIfHidden = false;
	
			GEditor->GetSelectedActors()->Modify();
			GEditor->GetSelectedActors()->BeginBatchSelectOperation();
			GEditor->SelectNone( bNotifySelectionChanged, bDeselectBSP, bWarnAboutTooManyActors );
			for (auto SpawnedActor : SpawnedActors)
			{
				GEditor->SelectActor( SpawnedActor, true, bNotifySelectionChanged, bSelectEvenIfHidden );
			}
			GEditor->GetSelectedActors()->EndBatchSelectOperation();
			GEditor->NoteSelectionChange();
		}

		NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemsChanged );

		SynchronizeSequencerSelectionWithExternalSelection();
	}
}

void FSequencer::SetShowCurveEditor(bool bInShowCurveEditor)
{
	SequencerWidget->OnCurveEditorVisibilityChanged(bInShowCurveEditor);

	GetSequencerSettings()->SetCurveEditorVisible(bInShowCurveEditor);
}

bool FSequencer::GetCurveEditorIsVisible() const
{
	using namespace UE::Sequencer;

	// Some Sequencer usages don't support the Curve Editor
	if (!GetHostCapabilities().bSupportsCurveEditor)
	{
		return false;
	}

	// We always want to retrieve this directly from the UI instead of mirroring it to a local bool as there are
	// a lot of ways the UI could get out of sync with a local bool (such as previously restored tab layouts)
	FCurveEditorExtension* CurveEditorExtension = ViewModel->CastDynamic<FCurveEditorExtension>();
	if (ensure(CurveEditorExtension))
	{
		return CurveEditorExtension->IsCurveEditorOpen();
	}

	return false;
}

void FSequencer::SaveCurrentMovieScene()
{
	UE::Sequencer::CaptureThumbnailForAssetBlocking(
		*GetCurrentAsset(),
		*this,
		Settings->GetThumbnailCaptureSettings()
		);

	OnPreSaveEvent.Broadcast(*this);

	TArray<UPackage*> PackagesToSave;
	TArray<UMovieScene*> MovieScenesToSave;
	MovieSceneHelpers::GetDescendantMovieScenes(GetRootMovieSceneSequence(), MovieScenesToSave);
	for (UMovieScene* MovieSceneToSave : MovieScenesToSave)
	{
		UPackage* MovieScenePackageToSave = MovieSceneToSave->GetOuter()->GetOutermost();
		if (MovieScenePackageToSave->IsDirty())
		{
			PackagesToSave.AddUnique(MovieScenePackageToSave);
		}
	}

	// If there's more than 1 movie scene to save, prompt the user whether to save all dirty movie scenes.
	const bool bCheckDirty = PackagesToSave.Num() > 1;
	const bool bPromptToSave = PackagesToSave.Num() > 1;

	FEditorFileUtils::PromptForCheckoutAndSave( PackagesToSave, bCheckDirty, bPromptToSave );

	ForceEvaluate();

	OnPostSaveEvent.Broadcast(*this);
}

void FSequencer::SaveCurrentMovieSceneAs()
{
	if (!GetHostCapabilities().bSupportsSaveMovieSceneAsset)
	{
		return;
	}

	TSharedPtr<IToolkitHost> MyToolkitHost = GetToolkitHost();
	check(MyToolkitHost);

	TArray<UObject*> AssetsToSave;
	AssetsToSave.Add(GetCurrentAsset());

	TArray<UObject*> SavedAssets;
	FEditorFileUtils::SaveAssetsAs(AssetsToSave, SavedAssets);

	if (SavedAssets.Num() == 0)
	{
		return;
	}

	if ((SavedAssets[0] != AssetsToSave[0]) && (SavedAssets[0] != nullptr))
	{
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		AssetEditorSubsystem->CloseAllEditorsForAsset(AssetsToSave[0]);
		AssetEditorSubsystem->OpenEditorForAssets_Advanced(SavedAssets, EToolkitMode::Standalone, MyToolkitHost.ToSharedRef());
	}
}

TArray<FGuid> FSequencer::AddActors(const TArray<TWeakObjectPtr<AActor> >& InActors, bool bSelectActors)
{
	using namespace UE::Sequencer;

	const FScopedTransaction Transaction(LOCTEXT("UndoPossessingObject", "Possess Object in Sequencer"));

	TArray<FGuid> PossessableGuids = FSequencerUtilities::AddActors(AsShared(), InActors);

	if (PossessableGuids.Num())
	{
		// Check if a folder is selected so we can add the actors to the selected folder.
		TArray<UMovieSceneFolder*> SelectedParentFolders;
		FString NewNodePath;
		if (ViewModel->GetSelection()->Outliner.Num() > 0)
		{
			for (FViewModelPtr CurrentItem : ViewModel->GetSelection()->Outliner)
			{
				if (TSharedPtr<FFolderModel> Folder = CurrentItem->FindAncestorOfType<FFolderModel>(true))
				{
					SelectedParentFolders.Add(Folder->GetFolder());

					// The first valid folder we find will be used to put the new actors into, so it's the node that we
					// want to know the path from.
					if (NewNodePath.Len() == 0)
					{
						// Add an extra delimiter (".") as we know that the new objects will be appended onto the end of this.
						NewNodePath = FString::Printf(TEXT("%s."), *IOutlinerExtension::GetPathName(*Folder));

						// Make sure the folder is expanded too so that adding objects to hidden folders become visible.
						Folder->SetExpansion(true);
					}
				}
			}
		}

		if (bSelectActors)
		{
			// Clear our editor selection so we can make the selection our added actors.
			// This has to be done after we know if the actor is going to be added to a
			// folder, otherwise it causes the folder we wanted to pick to be deselected.
			USelection* SelectedActors = GEditor->GetSelectedActors();
			SelectedActors->BeginBatchSelectOperation();
			SelectedActors->Modify();
			GEditor->SelectNone(false, true);
			for (TWeakObjectPtr<AActor> WeakActor : InActors)
			{
				if (AActor* Actor = WeakActor.Get())
				{
					GEditor->SelectActor(Actor, true, false);
				}
			}
			SelectedActors->EndBatchSelectOperation();
			GEditor->NoteSelectionChange();
		}

		// Add the possessables as children of the first selected folder
		if (SelectedParentFolders.Num() > 0)
		{
			for (const FGuid& Possessable : PossessableGuids)
			{
				SelectedParentFolders[0]->Modify();
				SelectedParentFolders[0]->AddChildObjectBinding(Possessable);
			}
		}

		// Now add them all to the selection set to be selected after a tree rebuild.
		if (bSelectActors)
		{
			for (const FGuid& Possessable : PossessableGuids)
			{
				FString PossessablePath = NewNodePath + Possessable.ToString();

				// Object Bindings use their FGuid as their unique key.
				SequencerWidget->AddAdditionalPathToSelectionSet(PossessablePath);
			}
		}
	}

	RefreshTree();

	SynchronizeSequencerSelectionWithExternalSelection();

	return PossessableGuids;
}

FGuid FSequencer::AddEmptyBinding()
{
	using namespace UE::Sequencer;

	const FScopedTransaction Transaction(LOCTEXT("UndoAddEmptyBinding", "Add Empty Binding to Sequencer"));

	FGuid PossessableGuid;

	UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return PossessableGuid;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return PossessableGuid;
	}

	if (MovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return PossessableGuid;
	}

	Sequence->Modify();

	UObject* Context = GetPlaybackContext();

	// Create a new binding for this object
	TArray<FName> PossessableNames;
	for (int32 i = 0; i < MovieScene->GetPossessableCount(); ++i)
	{
		PossessableNames.Add(*MovieScene->GetPossessable(i).GetName());
	}
	FName PossessableName = FSequencerUtilities::GetUniqueName(TEXT("EmptyBinding"), PossessableNames);

	UE::Sequencer::FCreateBindingParams CreateBindingParams;
	CreateBindingParams.bAllowCustomBinding = false;
	CreateBindingParams.BindingNameOverride = PossessableName.ToString();
	CreateBindingParams.bAllowEmptyBinding = true;
	PossessableGuid = FSequencerUtilities::CreateOrReplaceBinding(SharedThis(this), nullptr, CreateBindingParams);
	RefreshTree();

	return PossessableGuid;
}

void FSequencer::OnSelectionChanged()
{
	HandleSelectedOutlinerNodesChanged();
}

void FSequencer::HandleSelectedOutlinerNodesChanged()
{
	using namespace UE::Sequencer;

	SynchronizeExternalSelectionWithSequencerSelection();

	TSet<UMovieSceneTrack*> SelectedTracks;
	for (TViewModelPtr<ITrackExtension> TrackModel : ViewModel->GetSelection()->Outliner.Filter<ITrackExtension>())
	{
		if (UMovieSceneTrack* Track = TrackModel->GetTrack())
		{
			SelectedTracks.Add(Track);
		}
	}	
	
	for (TViewModelPtr<ITrackRowExtension> TrackRowModel : ViewModel->GetSelection()->Outliner.Filter<ITrackRowExtension>())
	{
		if (UMovieSceneTrack* Track = TrackRowModel->GetParentTrack())
		{
			SelectedTracks.Add(Track);
		}
	}

	TArray<UMovieSceneSection*> SelectedSections;
	for (TViewModelPtr<FSectionModel> SectionModel : ViewModel->GetSelection()->TrackArea.Filter<FSectionModel>())
	{
		if (UMovieSceneSection* Section = SectionModel->GetSection())
		{
			SelectedSections.Add(Section);
		}
	}

	OnSelectionChangedObjectGuidsDelegate.Broadcast(ViewModel->GetSelection()->GetBoundObjectsGuids().Array());
	OnSelectionChangedTracksDelegate.Broadcast(SelectedTracks.Array());
	OnSelectionChangedSectionsDelegate.Broadcast(SelectedSections);
}

void FSequencer::AddNodeGroupsCollectionChangedDelegate()
{
	UMovieSceneSequence* MovieSceneSequence = GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr;
	if (ensure(MovieScene))
	{
		if (!MovieScene->GetNodeGroups().OnNodeGroupCollectionChanged().IsBoundToObject(this))
		{
			MovieScene->GetNodeGroups().OnNodeGroupCollectionChanged().AddSP(this, &FSequencer::OnNodeGroupsCollectionChanged);
		}
	}
}

void FSequencer::RemoveNodeGroupsCollectionChangedDelegate()
{
	UMovieSceneSequence* MovieSceneSequence = GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr;
	if (MovieScene)
	{
		MovieScene->GetNodeGroups().OnNodeGroupCollectionChanged().RemoveAll(this);
	}
}

void FSequencer::OnNodeGroupsCollectionChanged()
{
	TSharedPtr<SSequencerGroupManager> NodeGroupManager = SequencerWidget->GetNodeGroupsManager();
	if (NodeGroupManager)
	{
		NodeGroupManager->RefreshNodeGroups();
	}

	NodeTree->NodeGroupsCollectionChanged();
}

void FSequencer::AddSelectedNodesToNewNodeGroup()
{
	using namespace UE::Sequencer;

	UMovieScene* MovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();

	if (MovieScene->IsReadOnly())
	{
		return;
	}

	if (ViewModel->GetSelection()->Outliner.Num() == 0)
	{
		return;
	}

	TSet<FString> NodesToAdd;
	for (FViewModelPtr Item : ViewModel->GetSelection()->Outliner)
	{
		TSharedPtr<IGroupableExtension> Groupable = Item->FindAncestorOfType<IGroupableExtension>(true);
		if (Groupable)
		{
			NodesToAdd.Add(IOutlinerExtension::GetPathName(Item));
		}
	}

	if (NodesToAdd.Num() == 0)
	{
		return;
	}

	TArray<FName> ExistingGroupNames;
	for (const UMovieSceneNodeGroup* NodeGroup : MovieScene->GetNodeGroups())
	{
		ExistingGroupNames.Add(NodeGroup->GetName());
	}

	const FScopedTransaction Transaction(LOCTEXT("CreateNewGroupTransaction", "Create New Group"));

	UMovieSceneNodeGroup* NewNodeGroup = NewObject<UMovieSceneNodeGroup>(&MovieScene->GetNodeGroups(), NAME_None, RF_Transactional);
	NewNodeGroup->SetName(FSequencerUtilities::GetUniqueName(FName("Group"), ExistingGroupNames));

	for (const FString& NodeToAdd : NodesToAdd)
	{
		NewNodeGroup->AddNode(NodeToAdd);
	}

	MovieScene->GetNodeGroups().AddNodeGroup(NewNodeGroup);

	SequencerWidget->OpenNodeGroupsManager();
	SequencerWidget->GetNodeGroupsManager()->RequestRenameNodeGroup(NewNodeGroup);
}

void FSequencer::AddSelectedNodesToExistingNodeGroup(UMovieSceneNodeGroup* NodeGroup)
{
	AddNodesToExistingNodeGroup(ViewModel->GetSelection()->Outliner.GetSelected().Array(), NodeGroup);
}

void FSequencer::AddNodesToExistingNodeGroup(TArrayView<const UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>> InItems, UMovieSceneNodeGroup* InNodeGroup)
{
	using namespace UE::Sequencer;

	UMovieScene* MovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();

	if (MovieScene->IsReadOnly())
	{
		return;
	}

	if (!MovieScene->GetNodeGroups().Contains(InNodeGroup))
	{
		return;
	}

	TSet<FString> NodesToAdd;
	for (const TWeakViewModelPtr<IOutlinerExtension>& WeakItem : InItems)
	{
		FViewModelPtr                   Item      = WeakItem.Pin();
		TSharedPtr<IGroupableExtension> Groupable = Item ? Item->FindAncestorOfType<IGroupableExtension>(true) : nullptr;

		if (Groupable)
		{
			NodesToAdd.Add(IOutlinerExtension::GetPathName(Item));
		}
	}

	if (NodesToAdd.Num() == 0)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddNodesToGroupTransaction", "Add Nodes to Group"));

	for (const FString& NodeToAdd : NodesToAdd)
	{
		if (!InNodeGroup->ContainsNode(NodeToAdd))
		{
			InNodeGroup->AddNode(NodeToAdd);
		}
	}

	TSharedPtr<SSequencerGroupManager> NodeGroupManager = SequencerWidget->GetNodeGroupsManager();
	if (NodeGroupManager)
	{
		NodeGroupManager->RefreshNodeGroups();
	}
}

void FSequencer::ClearFilters()
{
	GetFilterBar()->SetTextFilterString(FString());
	GetFilterBar()->EnableFilters(false);
	GetSequencerSettings()->SetShowSelectedNodesOnly(false);

	UMovieSceneSequence* FocusedMovieSequence = GetFocusedMovieSceneSequence();
	UMovieScene* FocusedMovieScene = FocusedMovieSequence ? FocusedMovieSequence->GetMovieScene() : nullptr;
	if (FocusedMovieScene)
	{
		for (UMovieSceneNodeGroup* NodeGroup : FocusedMovieScene->GetNodeGroups())
		{
			NodeGroup->SetEnableFilter(false);
		}
	}
	
	NodeTree->RequestFilterUpdate();
}

void FSequencer::SynchronizeExternalSelectionWithSequencerSelection()
{
	using namespace UE::Sequencer;
	
	if (!ViewModel)
	{
		return;
	}

	if ( bUpdatingSequencerSelection || !IsLevelEditorSequencer() )
	{
		return;
	}

	TGuardValue<bool> Guard(bUpdatingExternalSelection, true);

	TSet<AActor*> SelectedSequencerActors;
	TSet<UActorComponent*> SelectedSequencerComponents;

	const FName ControlRigEditModeModeName("EditMode.ControlRig");
	const bool bHACK_ControlRigEditMode = GLevelEditorModeTools().GetActiveMode(ControlRigEditModeModeName) != nullptr;

	for (TWeakViewModelPtr<IOutlinerExtension> WeakOutlinerItem : ViewModel->GetSelection()->Outliner.GetSelected())
	{
		FViewModelPtr OutlinerItem = WeakOutlinerItem.Pin();
		if (!OutlinerItem)
		{
			continue;
		}

		//HACK for DHI, if we have an active control rig then one is selected so don't find a parent actor or compomonent to select	
		//but if we do select the actor/compoent directly we still select it.
		TViewModelPtr<IObjectBindingExtension> ObjectBinding = bHACK_ControlRigEditMode
			? CastViewModel<IObjectBindingExtension>(OutlinerItem)
			: OutlinerItem->FindAncestorOfType<IObjectBindingExtension>(true);

		// If the closest node is an object node, try to get the actor/component nodes from it.
		if (ObjectBinding)
		{
			for (auto RuntimeObject : FindBoundObjects(ObjectBinding->GetObjectGuid(), ActiveTemplateIDs.Top()) )
			{
				AActor* Actor = Cast<AActor>(RuntimeObject.Get());
				if ( Actor != nullptr )
				{
					ULevel* ActorLevel = Actor->GetLevel();
					if (!FLevelUtils::IsLevelLocked(ActorLevel))
					{
						SelectedSequencerActors.Add(Actor);
					}
				}

				UActorComponent* ActorComponent = Cast<UActorComponent>(RuntimeObject.Get());
				if ( ActorComponent != nullptr )
				{
					if (!FLevelUtils::IsLevelLocked(ActorComponent->GetOwner()->GetLevel()))
					{	
						SelectedSequencerComponents.Add(ActorComponent);
						Actor = ActorComponent->GetOwner();
						if (Actor != nullptr)
						{
							SelectedSequencerActors.Add(Actor);
						}	
					}
				}
			}
		}
	}

	const bool bNotifySelectionChanged = false;
	const bool bDeselectBSP = true;
	const bool bWarnAboutTooManyActors = false;
	const bool bSelectEvenIfHidden = true;

	if (SelectedSequencerComponents.Num() + SelectedSequencerActors.Num() == 0)
	{
		if (GEditor->GetSelectedActorCount())
		{
			const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "UpdatingActorComponentSelectionNone", "Select None"), !GIsTransacting);
			GEditor->SelectNone( bNotifySelectionChanged, bDeselectBSP, bWarnAboutTooManyActors );
			GEditor->NoteSelectionChange();
		}
		return;
	}

	// We need to check if the selection has changed. Rebuilding the selection set if it hasn't changed can cause unwanted side effects.
	bool bIsSelectionChanged = false;

	// Check if any actors have been added to the selection
	for (AActor* SelectedSequencerActor : SelectedSequencerActors)
	{
		if (!GEditor->GetSelectedActors()->IsSelected(SelectedSequencerActor))
		{
			bIsSelectionChanged = true;
			break;
		}
	}

	// Check if any actors have been removed from the selection
	if (!bIsSelectionChanged)
	{
		for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
		{
			if (AActor* CurrentlySelectedActor = Cast<AActor>(*It))
			{
				if (!SelectedSequencerActors.Contains(CurrentlySelectedActor))
				{
					bIsSelectionChanged = true;
					break;
				}
			}
		}
	}

	// Check if any components have been added to the selection
	if (!bIsSelectionChanged)
	{
		for (UActorComponent* SelectedSequencerComponent : SelectedSequencerComponents)
		{
			if (!GEditor->GetSelectedComponents()->IsSelected(SelectedSequencerComponent))
			{
				bIsSelectionChanged = true;
				break;
			}
		}
	}

	// Check if any components have been removed from the selection
	if (!bIsSelectionChanged)
	{
		for (FSelectionIterator It(GEditor->GetSelectedComponentIterator()); It; ++It)
		{
			if (UActorComponent* CurrentlySelectedComponent = Cast<UActorComponent>(*It))
			{
				if (!SelectedSequencerComponents.Contains(CurrentlySelectedComponent))
				{
					bIsSelectionChanged = true;
					break;
				}
			}
		}
	}

	if (!bIsSelectionChanged)
	{
		return;
	}

	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "UpdatingActorComponentSelection", "Select Actors/Components"), !GIsTransacting);


	GEditor->GetSelectedActors()->Modify();
	GEditor->GetSelectedActors()->BeginBatchSelectOperation();

	GEditor->SelectNone( bNotifySelectionChanged, bDeselectBSP, bWarnAboutTooManyActors );

	for (AActor* SelectedSequencerActor : SelectedSequencerActors)
	{
		GEditor->SelectActor(SelectedSequencerActor, true, bNotifySelectionChanged, bSelectEvenIfHidden);
	}

	GEditor->GetSelectedActors()->EndBatchSelectOperation();

	GEditor->NoteSelectionChange();

	// Ensure that the (newer) typed element selection broadcasts its changes immediately so we don't get an
	//   end of frame update which might overwrite a newly created track area selection
	if (UTypedElementSelectionSet* TypedElements = GEditor->GetSelectedActors()->GetElementSelectionSet())
	{
		TypedElements->NotifyPendingChanges();
	}

	if (SelectedSequencerComponents.Num())
	{
		GEditor->GetSelectedComponents()->Modify();
		GEditor->GetSelectedComponents()->BeginBatchSelectOperation();

		for (UActorComponent* SelectedSequencerComponent : SelectedSequencerComponents)
		{
			GEditor->SelectComponent(SelectedSequencerComponent, true, bNotifySelectionChanged, bSelectEvenIfHidden);
		}

		GEditor->GetSelectedComponents()->EndBatchSelectOperation();

		GEditor->NoteSelectionChange();

		// Ensure that the (newer) typed element selection broadcasts its changes immediately so we don't get an
		//   end of frame update which might overwrite a newly created track area selection
		if (UTypedElementSelectionSet* TypedElements = GEditor->GetSelectedComponents()->GetElementSelectionSet())
		{
			TypedElements->NotifyPendingChanges();
		}
	}
}


void FSequencer::SynchronizeSequencerSelectionWithExternalSelection()
{
	using namespace UE::Sequencer;

	if ( bUpdatingExternalSelection )
	{
		return;
	}

	UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
	if( !IsLevelEditorSequencer() )
	{
		// Only level sequences have a full update here, but we still want filters to update for UMG animations
		NodeTree->RequestFilterUpdate();
		return;
	}

	const UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return;
	}

	TGuardValue<bool> Guard(bUpdatingSequencerSelection, true);

	// If all nodes are already selected, do nothing. This ensures that when an undo event happens, 
	// nodes are not cleared and reselected, which can cause issues with the curve editor auto-fitting 
	// based on selection.
	bool bAllAlreadySelected = true;

	USelection* ActorSelection = GEditor->GetSelectedActors();
	
	// Get the selected sequencer keys for viewport interaction
	TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();

	FObjectBindingModelStorageExtension* ObjectModelStorage = ViewModel->GetRootModel()->CastDynamic<FObjectBindingModelStorageExtension>();
	check(ObjectModelStorage);

	TSet<TSharedPtr<FObjectBindingModel>> NodesToSelect;
	for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
	{
		TSharedPtr<FObjectBindingModel> ObjectModel = ObjectModelStorage->FindModelForObjectBinding(Binding.GetObjectGuid());
		if (!ObjectModel)
		{
			continue;
		}

		for ( TWeakObjectPtr<> WeakObject : FindBoundObjects(Binding.GetObjectGuid(), ActiveTemplateIDs.Top()) )
		{
			UObject* RuntimeObject = WeakObject.Get();
			if (RuntimeObject == nullptr)
			{
				continue;
			}

			AActor* Actor = Cast<AActor>(RuntimeObject);
			const bool bActorSelected = Actor ? ActorSelection->IsSelected(Actor) : false;
						
			UActorComponent* ActorComponent = Cast<UActorComponent>(RuntimeObject);
			const bool bComponentSelected = ActorComponent ? GEditor->GetSelectedComponents()->IsSelected(ActorComponent) : false;

			if (bActorSelected || bComponentSelected)
			{
				NodesToSelect.Add( ObjectModel );

				if (bAllAlreadySelected && !Selection->Outliner.IsSelected(ObjectModel))
				{
					bool bAnyChildrenSelected = false;
					for (TViewModelPtr<IOutlinerExtension> Child : ObjectModel->GetDescendantsOfType<IOutlinerExtension>())
					{
						if (Selection->Outliner.IsSelected(Child) || Selection->NodeHasSelectedKeysOrSections(Child))
						{
							bAnyChildrenSelected = true;
							break;
						}
					}

					if (!bAnyChildrenSelected)
					{
						bAllAlreadySelected = false;
					}
				}
			}
			else if (Selection->Outliner.IsSelected(ObjectModel))
			{
				bAllAlreadySelected = false;
			}
		}
	}
	//Only test if none are selected if we are not transacting, otherwise it will clear out control rig's incorrectly.

	if (!bAllAlreadySelected || (!GIsTransacting && (NodesToSelect.Num() == 0 && Selection->Outliner.Num())))
	{
		FSelectionEventSuppressor EventSuppressor = Selection->SuppressEvents();

		Selection->Outliner.Empty();
		for (TSharedPtr<FObjectBindingModel> NodeToSelect : NodesToSelect)
		{
			Selection->Outliner.Select(NodeToSelect);
		}
		
		TSharedPtr<SOutlinerView> TreeView = SequencerWidget->GetTreeView();
		bool bScrolledIntoView = false;
		for (TViewModelPtr<IOutlinerExtension> Node : Selection->Outliner)
		{
			for (TViewModelPtr<IOutlinerExtension> Parent : Node.AsModel()->GetAncestorsOfType<IOutlinerExtension>())
			{
				TreeView->SetItemExpansion(Parent, true);
				Parent = Parent.AsModel()->FindAncestorOfType<IOutlinerExtension>();
			}

			if (!bScrolledIntoView)
			{
				bScrolledIntoView = true;
				TreeView->RequestScrollIntoView(Node);
			}
		}
	}
}

void FSequencer::SelectNodesByPath(const TSet<FString>& NodePaths)
{
	using namespace UE::Sequencer;

	if (bUpdatingExternalSelection)
	{
		return;
	}

	UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
	if (!Sequence->GetMovieScene())
	{
		return;
	}

	// If all nodes are already selected, do nothing. This ensures that when an undo event happens, 
	// nodes are not cleared and reselected, which can cause issues with the curve editor auto-fitting 
	// based on selection.
	bool bAllAlreadySelected = true;
	const FOutlinerSelection& CurrentSelection = ViewModel->GetSelection()->Outliner;

	TSet<TViewModelPtr<IOutlinerExtension>> NodesToSelect;
	for (TViewModelPtr<IOutlinerExtension> DisplayNode : ViewModel->GetRootModel()->GetDescendantsOfType<IOutlinerExtension>())
	{
		if (NodePaths.Contains(IOutlinerExtension::GetPathName(DisplayNode)))
		{
			NodesToSelect.Add(DisplayNode);
			if (bAllAlreadySelected && !CurrentSelection.IsSelected(DisplayNode))
			{
				bAllAlreadySelected = false;
			}
		}
	}

	if (!bAllAlreadySelected || (NodesToSelect.Num() != CurrentSelection.Num()))
	{
		TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();

		FSelectionEventSuppressor EventSuppressor = Selection->SuppressEvents();

		Selection->Outliner.Empty();
		for (TViewModelPtr<IOutlinerExtension> NodeToSelect : NodesToSelect)
		{
			Selection->Outliner.Select( NodeToSelect );
		}

		TSharedPtr<SOutlinerView> TreeView = SequencerWidget->GetTreeView();
		if (const TWeakViewModelPtr<IOutlinerExtension>* Node = Selection->Outliner.GetSelected().FindArbitraryElement())
		{
			for (TViewModelPtr<IOutlinerExtension> Parent : Node->Pin().AsModel()->GetAncestorsOfType<IOutlinerExtension>())
			{
				TreeView->SetItemExpansion(Parent, true);
					TreeView->SetItemExpansion(Parent, true);
					Parent = Parent.AsModel()->FindAncestorOfType<IOutlinerExtension>();
			}

			TreeView->RequestScrollIntoView(*Node);
		}
	}
}

bool FSequencer::IsBindingVisible(const FMovieSceneBinding& InBinding)
{
	if (Settings->GetShowSelectedNodesOnly() && OnGetIsBindingVisible().IsBound())
	{
		return OnGetIsBindingVisible().Execute(InBinding);
	}

	return true;
}

bool FSequencer::IsTrackVisible(const UMovieSceneTrack* InTrack)
{
	if (Settings->GetShowSelectedNodesOnly() && OnGetIsTrackVisible().IsBound())
	{
		return OnGetIsTrackVisible().Execute(InTrack);
	}

	return true;
}

void FSequencer::OnSelectedNodesOnlyChanged()
{
	RefreshTree();
	
	SynchronizeSequencerSelectionWithExternalSelection();
}

void FSequencer::OnTimeDisplayFormatChanged()
{
	using namespace UE::Sequencer;

	for (TSharedRef<FSequencerNumericTypeInterface> Interface : GetNumericTypeInterfaces())
	{
		if (Interface->Interface->GetOnSettingChanged() != nullptr)
		{
			Interface->Interface->GetOnSettingChanged()->Broadcast();
		}
	}
}

void FSequencer::ZoomToFit()
{
	using namespace UE::Sequencer;

	FFrameRate TickResolution = GetFocusedTickResolution();

	TRange<FFrameNumber> BoundsHull = TRange<FFrameNumber>::All();
	
	TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();

	for (FKeyHandle Key : Selection->KeySelection)
	{
		TSharedPtr<FChannelModel> Channel = Selection->KeySelection.GetModelForKey(Key);
		if (Channel)
		{
			FFrameNumber KeyTime = Channel->GetKeyArea()->GetKeyTime(Key);
			if (!BoundsHull.HasLowerBound() || BoundsHull.GetLowerBoundValue() > KeyTime)
			{
				BoundsHull.SetLowerBound(TRange<FFrameNumber>::BoundsType::Inclusive(KeyTime));
			}
			if (!BoundsHull.HasUpperBound() || BoundsHull.GetUpperBoundValue() < KeyTime)
			{
				BoundsHull.SetUpperBound(TRange<FFrameNumber>::BoundsType::Inclusive(KeyTime));
			}
		}
	}

	for (TWeakObjectPtr<UMovieSceneSection> SelectedSection : Selection->GetSelectedSections())
	{
		if (SelectedSection->GetRange().HasUpperBound() && SelectedSection->GetRange().HasLowerBound())
		{
			if (BoundsHull == TRange<FFrameNumber>::All())
			{
				BoundsHull = SelectedSection->GetRange();
			}
			else
			{
				BoundsHull = TRange<FFrameNumber>::Hull(SelectedSection->GetRange(), BoundsHull);
			}
		}
	}
	
	if (BoundsHull.IsEmpty() || BoundsHull == TRange<FFrameNumber>::All())
	{
		BoundsHull = GetTimeBounds();
	}

	if (!BoundsHull.IsEmpty() && !BoundsHull.IsDegenerate())
	{
		const double Tolerance = KINDA_SMALL_NUMBER;

		// Zoom back to last view range if already expanded
		if (!ViewRangeBeforeZoom.IsEmpty() &&
			FMath::IsNearlyEqual(BoundsHull.GetLowerBoundValue() / TickResolution, GetViewRange().GetLowerBoundValue(), Tolerance) &&
			FMath::IsNearlyEqual(BoundsHull.GetUpperBoundValue() / TickResolution, GetViewRange().GetUpperBoundValue(), Tolerance))
		{
			SetViewRange(ViewRangeBeforeZoom, EViewRangeInterpolation::Animated);
		}
		else
		{
			ViewRangeBeforeZoom = GetViewRange();

			TRange<double> BoundsHullSeconds = BoundsHull / TickResolution;
			const double OutputViewSize = BoundsHullSeconds.Size<double>();
			const double OutputChange = OutputViewSize * 0.1f;

			if (OutputChange > 0)
			{
				BoundsHullSeconds = UE::MovieScene::ExpandRange(BoundsHullSeconds, OutputChange);
	
				SetViewRange(BoundsHullSeconds, EViewRangeInterpolation::Animated);
			}
		}
	}
}

void FSequencer::ZoomToFitSelectionRange()
{
	FFrameRate TickResolution = GetFocusedTickResolution();

	// Use the movie scene selection range if set, otherwise fall back to time bounds
	TRange<FFrameNumber> BoundsHull = TRange<FFrameNumber>::Empty();
	if (const UMovieSceneSequence* FocusedSequence = GetFocusedMovieSceneSequence())
	{
		if (const UMovieScene* MovieScene = FocusedSequence->GetMovieScene())
		{
			BoundsHull = MovieScene->GetSelectionRange();
		}
	}
	if (BoundsHull.IsEmpty())
	{
		BoundsHull = GetTimeBounds();
	}

	if (!BoundsHull.IsEmpty() && !BoundsHull.IsDegenerate())
	{
		const double Tolerance = KINDA_SMALL_NUMBER;

		if (!ViewRangeBeforeZoom.IsEmpty() &&
			FMath::IsNearlyEqual(BoundsHull.GetLowerBoundValue() / TickResolution,
			                      GetViewRange().GetLowerBoundValue(), Tolerance) &&
			FMath::IsNearlyEqual(BoundsHull.GetUpperBoundValue() / TickResolution,
			                      GetViewRange().GetUpperBoundValue(), Tolerance))
		{
			SetViewRange(ViewRangeBeforeZoom, EViewRangeInterpolation::Animated);
		}
		else
		{
			ViewRangeBeforeZoom = GetViewRange();

			TRange<double> BoundsHullSeconds = BoundsHull / TickResolution;
			const double OutputViewSize = BoundsHullSeconds.Size<double>();
			const double OutputChange   = OutputViewSize * 0.1f;

			if (OutputChange > 0)
			{
				BoundsHullSeconds = UE::MovieScene::ExpandRange(BoundsHullSeconds, OutputChange);
				SetViewRange(BoundsHullSeconds, EViewRangeInterpolation::Animated);
			}
		}
	}
}

bool FSequencer::CanKeyProperty(FCanKeyPropertyParams CanKeyPropertyParams) const
{
	FPropertyPath PropertyPath;
	return ObjectChangeListener->CanKeyProperty(CanKeyPropertyParams, PropertyPath);
}


void FSequencer::KeyProperty(FKeyPropertyParams KeyPropertyParams) 
{
	ObjectChangeListener->KeyProperty(KeyPropertyParams);
}

EPropertyKeyedStatus FSequencer::GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const
{
	return PropertyKeyedStatusHandler->GetPropertyKeyedStatus(PropertyHandle);
}

FSequencerSelectionPreview& FSequencer::GetSelectionPreview()
{
	return SelectionPreview;
}

void FSequencer::GetSelectedTracks(TArray<UMovieSceneTrack*>& OutSelectedTracks)
{
	OutSelectedTracks.Append(ViewModel->GetSelection()->GetSelectedTracks().Array());
}

void FSequencer::GetSelectedTrackRows(TArray<TPair<UMovieSceneTrack*, int32>>& OutSelectedTrackRows)
{
	OutSelectedTrackRows.Append(ViewModel->GetSelection()->GetSelectedTrackRows().Array());
}

void FSequencer::GetSelectedSections(TArray<UMovieSceneSection*>& OutSelectedSections)
{
	for (TWeakObjectPtr<UMovieSceneSection> WeakSection : ViewModel->GetSelection()->GetSelectedSections())
	{
		if (UMovieSceneSection* Section = WeakSection.Get())
		{
			OutSelectedSections.Add(Section);
		}
	}
}

void FSequencer::GetSelectedFolders(TArray<UMovieSceneFolder*>& OutSelectedFolders)
{
	FString OutNewNodePath;
	CalculateSelectedFolderAndPath(OutSelectedFolders, OutNewNodePath);
}

void FSequencer::GetSelectedObjects(TArray<FGuid>& Objects)
{
	Objects = GetSelection().GetBoundObjectsGuids().Array();
}

void FSequencer::GetSelectedKeyAreas(TArray<const IKeyArea*>& OutSelectedKeyAreas, bool bIncludeSelectedKeys)
{
	using namespace UE::Sequencer;

	TArray<TSharedRef<FViewModel>> NodesToKey;
	{
		TSet<TSharedRef<FViewModel>> ChildNodes;
		for (FViewModelPtr Node : ViewModel->GetSelection()->Outliner)
		{
			NodesToKey.Add(Node.AsModel().ToSharedRef());

			// No need to gather key areas from binding/tracks because they have no key areas
			if (Node->IsA<IObjectBindingExtension>() || Node->IsA<ITrackExtension>() || Node->IsA<ITrackRowExtension>() || Node->IsA<FFolderModel>())
			{
				continue;
			}

			ChildNodes.Reset();
			SequencerHelpers::GetDescendantNodes(Node.AsModel().ToSharedRef(), ChildNodes);

			for (TSharedRef<FViewModel> ChildNode : ChildNodes)
			{
				NodesToKey.Remove(ChildNode);
			}
		}
	}

	TSet<TSharedPtr<IKeyArea>> KeyAreas;
	TSet<UMovieSceneSection*>  ModifiedSections;

	for (TSharedRef<FViewModel> Node : NodesToKey)
	{
		//if object or track selected we don't want all of the children only if spefically selected.
		if (!Node->IsA<ITrackExtension>() && !Node->IsA<ITrackRowExtension>() && !Node->IsA<IObjectBindingExtension>() && !Node->IsA<FFolderModel>())
		{
			SequencerHelpers::GetAllKeyAreas(Node, KeyAreas);
		}
	}

	if (bIncludeSelectedKeys)
	{
		for (FKeyHandle Key : ViewModel->GetSelection()->KeySelection)
		{
			TSharedPtr<FChannelModel> Channel = ViewModel->GetSelection()->KeySelection.GetModelForKey(Key);
			if (Channel)
			{
				KeyAreas.Add(Channel->GetKeyArea()); 
			}
		}
	}
	for (TSharedPtr<IKeyArea> KeyArea : KeyAreas)
	{
		const IKeyArea* KeyAreaPtr = KeyArea.Get();
		OutSelectedKeyAreas.Add(KeyAreaPtr);
	}
}

void FSequencer::SelectByNthCategoryNode(UMovieSceneSection* Section, int Index, bool bSelect)
{
	using namespace UE::Sequencer;

	TSet<TSharedRef<FViewModel>> Nodes;
	TArray<TViewModelPtr<IOutlinerExtension>> NodesToSelect;

	TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();
	FSelectionEventSuppressor EventSuppressor = Selection->SuppressEvents();

	TSharedPtr<FSectionModel> SectionHandle = NodeTree->GetSectionModel(Section);
	int32 Count = 0;
	if (SectionHandle)
	{
		TSharedPtr<FViewModel> TrackNode = SectionHandle->GetParentSectionOwnerModel();
		for (TViewModelPtr<FCategoryGroupModel> Node : TrackNode->GetChildrenOfType<FCategoryGroupModel>())
		{
			if (Count++ == Index)
			{
				bool bAlreadySelected = false;
				if (bSelect == true)
				{
					bAlreadySelected = Selection->Outliner.IsSelected(Node);
				}
				if (bAlreadySelected == false)
				{
					NodesToSelect.Add(Node);
					if (bSelect == false) //make sure all children not selected
					{
						for (TViewModelPtr<IOutlinerExtension> ChildNode : Node->GetChildrenOfType<IOutlinerExtension>())
						{
							NodesToSelect.Add(ChildNode);
						}
					}
				}
			}
		}
	}
	if (bSelect)
	{
		if (Settings->GetAutoExpandNodesOnSelection())
		{
			for (FViewModelPtr DisplayNode : NodesToSelect)
			{
				FViewModelPtr Parent = DisplayNode->GetParent();
				TViewModelPtr<ITrackExtension> ParentTrack = Parent.ImplicitCast();
				TViewModelPtr<ITrackRowExtension> ParentTrackRow = Parent.ImplicitCast();
				if (ParentTrack || ParentTrackRow)
				{
					if (TViewModelPtr<IOutlinerExtension> ParentOutliner = Parent.ImplicitCast())
					{
						if (!ParentOutliner->IsExpanded())
						{
							ParentOutliner->SetExpansion(true);
						}
					}
					break;
				}
			}
		}

		if (NodesToSelect.Num() > 0)
		{
			SequencerWidget->GetTreeView()->RequestScrollIntoView(CastViewModel<IOutlinerExtension>(NodesToSelect[0]));

			Selection->Outliner.SelectRange(NodesToSelect);
		}
	}
	else if (NodesToSelect.Num() > 0)
	{
		for (const TViewModelPtr<IOutlinerExtension>& DisplayNode : NodesToSelect)
		{
			Selection->Outliner.Deselect(DisplayNode);
		}
	}
}

void FSequencer::SelectByChannels(UMovieSceneSection* Section, TArrayView<const FMovieSceneChannelHandle> InChannels, bool bSelectParentInstead, bool bSelect)
{
	using namespace UE::Sequencer;

	TSharedPtr<FSectionModel> SectionHandle = NodeTree->GetSectionModel(Section);
	if (!SectionHandle)
	{
		return;
	}
	
	const TSharedPtr<FViewModel>& TrackNode = SectionHandle->GetParentSectionOwnerModel();
	auto FindChannelGroupModel = [TrackNode](const FMovieSceneChannelHandle& Handle) -> TSharedPtr<FChannelGroupModel>
	{
		for (TSharedPtr<FChannelGroupModel> KeyAreaNode : TrackNode->GetDescendantsOfType<FChannelGroupModel>())
		{
			const TArrayView<const TWeakViewModelPtr<FChannelModel>>& Channels = KeyAreaNode->GetChannels();
			const bool bContainsHandle = Channels.ContainsByPredicate([&Handle](const TWeakViewModelPtr<FChannelModel>& WeakChannel)
			{
				if (const TSharedPtr<FChannelModel>& Channel = WeakChannel.Pin())
				{
					const TSharedPtr<IKeyArea>& KeyArea = Channel->GetKeyArea();
					return KeyArea && KeyArea->GetChannel() == Handle;
				}
				return false;
			});
			
			if (bContainsHandle)
			{
				return KeyAreaNode;
			}
		}
		return nullptr;
	};
	
	TSet<TViewModelPtr<IOutlinerExtension>> NodesToUnselect, NodesToSelect;
	TViewModelPtr<IOutlinerExtension> FirstSelectedNode = nullptr;
		
	for (const FMovieSceneChannelHandle& Channel: InChannels)
	{
		if (const TSharedPtr<FChannelGroupModel>& ChannelGroupModel = FindChannelGroupModel(Channel))
		{
			if (bSelectParentInstead || !bSelect)
			{
				if (TViewModelPtr<IOutlinerExtension> OutlinerNode = ChannelGroupModel->FindAncestorOfType<IOutlinerExtension>())
				{
					if (bSelect)
					{
						NodesToSelect.Add(OutlinerNode);
						if (!FirstSelectedNode)
						{
							FirstSelectedNode = OutlinerNode;
						}
					}
					else
					{
						NodesToUnselect.Add(OutlinerNode);
					}
				}
			}
		
			if (!bSelectParentInstead || !bSelect)
			{
				// Handle top-level channels which might need to select their parent model
				TViewModelPtr<IOutlinerExtension> OutlinerNode = CastViewModel<IOutlinerExtension>(ChannelGroupModel);
				if (!OutlinerNode)
				{
					OutlinerNode = ChannelGroupModel->FindAncestorOfType<IOutlinerExtension>();
				}
			
				if (OutlinerNode)
				{
					if (bSelect)
					{
						NodesToSelect.Add(OutlinerNode);
						if (!FirstSelectedNode)
						{
							FirstSelectedNode = OutlinerNode;
						}
					}
					else
					{
						NodesToUnselect.Add(OutlinerNode);
					}
				}
			}
		}
	}
	
	TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();
	FSelectionEventSuppressor EventSuppressor = Selection->SuppressEvents();

	if (bSelect)
	{
		if (NodesToSelect.Num() > 0)
		{
			//todo hide behind preference 
			SequencerWidget->GetTreeView()->RequestScrollIntoView(CastViewModel<IOutlinerExtension>(FirstSelectedNode));
			Selection->Outliner.SelectRange(NodesToSelect);
		}
	}
	else if (NodesToUnselect.Num() > 0)
	{
		for (const TViewModelPtr<IOutlinerExtension>& DisplayNode : NodesToUnselect)
		{
			Selection->Outliner.Deselect(DisplayNode);
		}
	}
}

void FSequencer::SelectByChannels(UMovieSceneSection* Section, const TArray<FName>& InChannelNames, bool bSelectParentInstead, bool bSelect)
{
	using namespace UE::Sequencer;

	TSet<TViewModelPtr<IOutlinerExtension>> Nodes;
	TArray<TViewModelPtr<IOutlinerExtension>> NodesToSelect;

	TSharedPtr<FSectionModel> SectionHandle = NodeTree->GetSectionModel(Section);
	if (SectionHandle)
	{
		TSharedPtr<FViewModel> TrackNode = SectionHandle->GetParentSectionOwnerModel();
		for (TSharedPtr<FChannelGroupOutlinerModel> KeyAreaNode : TrackNode->GetDescendantsOfType<FChannelGroupOutlinerModel>())
		{
			TViewModelPtr<IOutlinerExtension> Parent = KeyAreaNode->FindAncestorOfType<IOutlinerExtension>();
			if (Parent && InChannelNames.Contains(*Parent->GetLabel().ToString()))
			{
				Nodes.Add(Parent);
			}

			for (TSharedPtr<IKeyArea> KeyArea : KeyAreaNode->GetAllKeyAreas())
			{
				FMovieSceneChannelHandle ThisChannel = KeyArea->GetChannel();

				const FMovieSceneChannelMetaData* MetaData = ThisChannel.GetMetaData();

				if (MetaData && InChannelNames.Contains(MetaData->Name))
				{
					if (bSelectParentInstead || bSelect == false)
					{
						Nodes.Add(Parent);
					}
					if (!bSelectParentInstead || bSelect == false)
					{
						Nodes.Add(KeyAreaNode);
					}
				}
			}
		}
	}

	TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();
	FSelectionEventSuppressor EventSuppressor = Selection->SuppressEvents();

	if (bSelect)
	{
		for (const TViewModelPtr<IOutlinerExtension>& DisplayNode : Nodes)
		{
			if (Settings->GetAutoExpandNodesOnSelection())
			{
				TViewModelPtr<IOutlinerExtension> Parent = DisplayNode.AsModel()->FindAncestorOfType<IOutlinerExtension>();
				if (Parent && (Parent.AsModel()->IsA<ITrackExtension>() || Parent.AsModel()->IsA<ITrackRowExtension>()) && !Parent->IsExpanded())
				{
					Parent->SetExpansion(true);
				}
			}
			NodesToSelect.Add(DisplayNode);
		}

		if (NodesToSelect.Num() > 0)
		{
			SequencerWidget->GetTreeView()->RequestScrollIntoView(CastViewModel<IOutlinerExtension>(NodesToSelect[0]));

			Selection->Outliner.SelectRange(NodesToSelect);
		}
	}
	else if (Nodes.Num() > 0)
	{
		for (const TViewModelPtr<IOutlinerExtension>& DisplayNode : Nodes)
		{
			Selection->Outliner.Deselect(DisplayNode);
		}
	}
}

void FSequencer::SelectObject(FGuid ObjectBinding)
{
	using namespace UE::Sequencer;

	FObjectBindingModelStorageExtension* ObjectStorage = ViewModel->GetRootModel()->CastDynamic<FObjectBindingModelStorageExtension>();
	check(ObjectStorage);

	if (TSharedPtr<FObjectBindingModel> Model = ObjectStorage->FindModelForObjectBinding(ObjectBinding))
	{
		ViewModel->GetSelection()->Outliner.Select(Model);
	}
}

void FSequencer::DeselectObject(FGuid ObjectBinding)
{
	using namespace UE::Sequencer;

	FObjectBindingModelStorageExtension* ObjectStorage = ViewModel->GetRootModel()->CastDynamic<FObjectBindingModelStorageExtension>();
	check(ObjectStorage);

	if (TSharedPtr<FObjectBindingModel> Model = ObjectStorage->FindModelForObjectBinding(ObjectBinding))
	{
		ViewModel->GetSelection()->Outliner.Deselect(Model);
	}
}

void FSequencer::SelectTrack(UMovieSceneTrack* Track)
{
	using namespace UE::Sequencer;

	FTrackModelStorageExtension* TrackStorage = ViewModel->GetRootModel()->CastDynamic<FTrackModelStorageExtension>();
	check(TrackStorage);

	TViewModelPtr<ITrackExtension> TrackExtension = TrackStorage->FindTrackExtensionForTrack(Track);
	if (TrackExtension)
	{
		ViewModel->GetSelection()->Outliner.Select(TrackExtension.ImplicitCast());
	}
}

void FSequencer::SelectSection(UMovieSceneSection* Section)
{
	using namespace UE::Sequencer;

	FSectionModelStorageExtension* SectionModelStorage = ViewModel->GetRootModel()->CastDynamic<FSectionModelStorageExtension>();
	check(SectionModelStorage);

	TSharedPtr<FSectionModel> SectionModel = SectionModelStorage->FindModelForSection(Section);
	if (SectionModel)
	{
		ViewModel->GetSelection()->TrackArea.Select(SectionModel);
	}
}

void FSequencer::SelectKey(UMovieSceneSection* InSection, TSharedPtr<UE::Sequencer::FChannelModel> InChannel, FKeyHandle KeyHandle, bool bToggle)
{
	if (bToggle && ViewModel->GetSelection()->KeySelection.IsSelected(KeyHandle))
	{
		ViewModel->GetSelection()->KeySelection.Deselect(KeyHandle);
	}
	else
	{
		ViewModel->GetSelection()->KeySelection.Select(InChannel, KeyHandle);
	}
}

void FSequencer::SelectByPropertyPaths(const TArray<FString>& InPropertyPaths)
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();
	FSelectionEventSuppressor EventSuppressor = Selection->SuppressEvents();

	Selection->Empty();

	for (TViewModelPtr<ITrackExtension> Track : ViewModel->GetRootModel()->GetDescendantsOfType<ITrackExtension>())
	{
		if (UMovieScenePropertyTrack* PropertyTrack = Cast<UMovieScenePropertyTrack>(Track->GetTrack()))
		{
			FString Path = PropertyTrack->GetPropertyPath().ToString();
			for (const FString& PropertyPath : InPropertyPaths)
			{
				if (Path == PropertyPath)
				{
					Selection->Outliner.Select(CastViewModelChecked<IOutlinerExtension>(Track));
					break;
				}
			}
		}
	}
}


void FSequencer::SelectFolder(UMovieSceneFolder* Folder)
{
	using namespace UE::Sequencer;

	FFolderModelStorageExtension* FolderModelStorage = ViewModel->GetRootModel()->CastDynamic<FFolderModelStorageExtension>();
	check(FolderModelStorage);

	TSharedPtr<FFolderModel> FolderModel = FolderModelStorage->FindModelForFolder(Folder);
	if (FolderModel)
	{
		ViewModel->GetSelection()->TrackArea.Select(FolderModel);
	}
}


void FSequencer::EmptySelection()
{
	ViewModel->GetSelection()->Empty();
}

void FSequencer::ThrobKeySelection()
{
	using namespace UE::Sequencer;

	SSequencerSection::ThrobKeySelection();
}

void FSequencer::ThrobSectionSelection()
{
	using namespace UE::Sequencer;

	// Scrub to the beginning of newly created sections if they're out of view
	TOptional<FFrameNumber> ScrubFrame;
	for (UMovieSceneSection* SelectedSectionPtr : ViewModel->GetSelection()->GetSelectedSections())
	{
		if (SelectedSectionPtr && SelectedSectionPtr->HasStartFrame())
		{
			if (!ScrubFrame.IsSet() || (ScrubFrame.GetValue() > SelectedSectionPtr->GetInclusiveStartFrame()))
			{
				ScrubFrame = SelectedSectionPtr->GetInclusiveStartFrame();
			}
		}
	}

	if (ScrubFrame.IsSet())
	{
		float ScrubTime = GetFocusedDisplayRate().AsSeconds(FFrameRate::TransformTime(ScrubFrame.GetValue(), GetFocusedTickResolution(), GetFocusedDisplayRate()));

		TRange<double> NewViewRange = GetViewRange();

		if (!NewViewRange.Contains(ScrubTime))
		{
			double MidRange = (NewViewRange.GetUpperBoundValue() - NewViewRange.GetLowerBoundValue()) / 2.0 + NewViewRange.GetLowerBoundValue();

			NewViewRange.SetLowerBoundValue(NewViewRange.GetLowerBoundValue() - (MidRange - ScrubTime));
			NewViewRange.SetUpperBoundValue(NewViewRange.GetUpperBoundValue() - (MidRange - ScrubTime));

			SetViewRange(NewViewRange, EViewRangeInterpolation::Animated);
		}
	}

	SSequencerSection::ThrobSectionSelection();
}

float FSequencer::GetOverlayFadeCurve() const
{
	return OverlayCurve.GetLerp();
}


void FSequencer::DeleteSelectedItems()
{
	if (ViewModel->GetSelection()->KeySelection.Num())
	{
		FScopedTransaction DeleteKeysTransaction( NSLOCTEXT("Sequencer", "DeleteKeys_Transaction", "Delete Keys") );
		
		DeleteSelectedKeys();
	}
	else if (ViewModel->GetSelection()->GetSelectedSections().Num())
	{
		FScopedTransaction DeleteSectionsTransaction( NSLOCTEXT("Sequencer", "DeleteSections_Transaction", "Delete Sections") );
	
		DeleteSections(ViewModel->GetSelection()->GetSelectedSections());
	}
	else if (ViewModel->GetSelection()->Outliner.Num())
	{
		DeleteSelectedNodes(false);
	}
}


void FSequencer::DeleteNode(TSharedRef<FViewModel> NodeToBeDeleted, const bool bKeepState)
{
	using namespace UE::Sequencer;

	// If this node is selected, delete all selected nodes
	if (GetSelection().Outliner.IsSelected(CastViewModelChecked<IOutlinerExtension>(NodeToBeDeleted)))
	{
		DeleteSelectedNodes(bKeepState);
	}
	else
	{
		const FScopedTransaction Transaction( NSLOCTEXT("Sequencer", "UndoDeletingObject", "Delete Node") );
		bool bAnythingDeleted = OnRequestNodeDeleted(NodeToBeDeleted, bKeepState);
		if ( bAnythingDeleted )
		{
			NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemRemoved );
		}
	}
}


void FSequencer::DeleteSelectedNodes(const bool bKeepState)
{
	using namespace UE::Sequencer;

	if (ViewModel->GetSelection()->Outliner.Num() == 0)
	{
		return;
	}

	const FScopedTransaction Transaction( NSLOCTEXT("Sequencer", "UndoDeletingObject", "Delete Node") );

	TArray<TSharedPtr<FViewModel>> SelectedNodesCopy;
	SelectedNodesCopy.Reserve(ViewModel->GetSelection()->Outliner.Num());
	for (FViewModelPtr OutlinerNode : ViewModel->GetSelection()->Outliner)
	{
		SelectedNodesCopy.Emplace(MoveTemp(OutlinerNode));
	}
	ViewModel->GetSelection()->Outliner.Empty();

	bool bAnythingDeleted = false;

	for (TSharedPtr<FViewModel>& SelectedNode : SelectedNodesCopy)
	{
		if (!SelectedNode->CastThisChecked<IOutlinerExtension>()->IsFilteredOut())
		{
			// Delete everything in the entire node
			bAnythingDeleted |= OnRequestNodeDeleted( SelectedNode.ToSharedRef(), bKeepState );
		}

		// Null out the node, which should have the effect of completely destroying it
		SelectedNode = nullptr;
	}

	if ( bAnythingDeleted )
	{
		NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemRemoved );
	}
}

void FSequencer::MoveNodeToFolder(TSharedRef<FViewModel> NodeToMove, UMovieSceneFolder* DestinationFolder)
{
	using namespace UE::Sequencer;

	TSharedPtr<FViewModel> ParentNode = NodeToMove->GetParent();
	
	if (DestinationFolder == nullptr)
	{
		return;
	}

	DestinationFolder->Modify();

	if (FFolderModel* FolderNode = NodeToMove->CastThis<FFolderModel>())
	{
		if (ParentNode.IsValid() && ParentNode->IsA<FFolderModel>())
		{
			FFolderModel* NodeParentFolder = ParentNode->CastThisChecked<FFolderModel>();
			NodeParentFolder->GetFolder()->Modify();
			NodeParentFolder->GetFolder()->RemoveChildFolder(FolderNode->GetFolder());
		}
		else
		{
			UMovieScene* FocusedMovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();
			if (FocusedMovieScene)
			{
				FocusedMovieScene->Modify();
				FocusedMovieScene->RemoveRootFolder(FolderNode->GetFolder());
			}
		}

		DestinationFolder->AddChildFolder(FolderNode->GetFolder());
	}
	else if (ITrackExtension* TrackNode = NodeToMove->CastThis<ITrackExtension>())
	{
		if (ParentNode.IsValid() && ParentNode->IsA<FFolderModel>())
		{
			FFolderModel* NodeParentFolder = ParentNode->CastThisChecked<FFolderModel>();
			NodeParentFolder->GetFolder()->Modify();
			NodeParentFolder->GetFolder()->RemoveChildTrack(TrackNode->GetTrack());
		}

		DestinationFolder->AddChildTrack(TrackNode->GetTrack());
	}
	else if (IObjectBindingExtension* ObjectBindingNode = NodeToMove->CastThis<IObjectBindingExtension>())
	{
		if (ParentNode.IsValid() && ParentNode->IsA<FFolderModel>())
		{
			FFolderModel* NodeParentFolder = ParentNode->CastThisChecked<FFolderModel>();
			NodeParentFolder->GetFolder()->Modify();
			NodeParentFolder->GetFolder()->RemoveChildObjectBinding(ObjectBindingNode->GetObjectGuid());
		}

		DestinationFolder->AddChildObjectBinding(ObjectBindingNode->GetObjectGuid());
	}
}

TArray<TSharedRef<UE::Sequencer::FViewModel>> FSequencer::GetSelectedNodesToMove()
{
	using namespace UE::Sequencer;

	TArray<TSharedRef<FViewModel>> NodesToMove;

	// Build a list of the nodes we want to move.
	for (TViewModelPtr<IDraggableOutlinerExtension> DraggableModel : GetSelection().Outliner.Filter<IDraggableOutlinerExtension>())
	{
		// Only nodes that can be dragged can be moved in to a folder.
		if (DraggableModel->CanDrag())
		{
			NodesToMove.Add(DraggableModel.AsModel().ToSharedRef());
		}
	}

	if (!NodesToMove.Num())
	{
		return NodesToMove;
	}

	TArray<int32> NodesToRemove;

	// Find nodes that are children of other nodes in the list
	for (int32 NodeIndex = 0; NodeIndex < NodesToMove.Num(); ++NodeIndex)
	{
		TSharedPtr<FViewModel> Node = NodesToMove[NodeIndex];

		for (TSharedRef<FViewModel> ParentNode : NodesToMove)
		{
			if (ParentNode == Node)
			{
				continue;
			}

			for (TSharedPtr<FViewModel> Child : ParentNode->GetDescendants(true))
			{
				if (Child == Node)
				{
					NodesToRemove.AddUnique(NodeIndex);
					break;
				}
			}
		}
	}

	// Remove the nodes that are children of other nodes in the list, as moving the parent will already be relocating them
	while (NodesToRemove.Num() > 0)
	{
		int32 NodeIndex = NodesToRemove.Pop();
		NodesToMove.RemoveAt(NodeIndex);
	}

	return NodesToMove;
}

TArray<TSharedRef<UE::Sequencer::FViewModel> > FSequencer::GetSelectedNodesInFolders()
{
	using namespace UE::Sequencer;

	TArray<TSharedRef<FViewModel> > NodesToFolders;
	for (FViewModelPtr SelectedNode : GetSelection().Outliner)
	{
		TSharedPtr<FFolderModel> Folder = SelectedNode->FindAncestorOfType<FFolderModel>();
		if (Folder.IsValid())
		{
			if (IObjectBindingExtension* ObjectBindingNode = SelectedNode->CastThis<IObjectBindingExtension>())
			{
				if (Folder->GetFolder()->GetChildObjectBindings().Contains(ObjectBindingNode->GetObjectGuid()))
				{
					NodesToFolders.Add(SelectedNode.AsModel().ToSharedRef());
				}
			}
			else if (ITrackExtension* TrackNode = SelectedNode->CastThis<ITrackExtension>())
			{
				if (TrackNode->GetTrack())
				{
					if (Folder->GetFolder()->GetChildTracks().Contains(TrackNode->GetTrack()))
					{
						NodesToFolders.Add(SelectedNode.AsModel().ToSharedRef());
					}
				}
			}
			else if (ITrackRowExtension* TrackRowNode = SelectedNode->CastThis<ITrackRowExtension>())
			{
				if (TrackRowNode->GetParentTrack())
				{
					if (Folder->GetFolder()->GetChildTracks().Contains(TrackRowNode->GetParentTrack()))
					{
						NodesToFolders.Add(SelectedNode.AsModel().ToSharedRef());
					}
				}
			}
		}
	}

	return NodesToFolders;
}

void FSequencer::MoveSelectedNodesToFolder(UMovieSceneFolder* DestinationFolder)
{
	using namespace UE::Sequencer;

	if (!DestinationFolder)
	{
		return;
	}

	UMovieScene* FocusedMovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();

	if (!FocusedMovieScene)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	TArray<TSharedRef<FViewModel> > NodesToMove = GetSelectedNodesToMove();

	for (TSharedRef<FViewModel> Node : NodesToMove)
	{
		// If this node is the destination folder, don't try to move it
		if (FFolderModel* FolderNode = Node->CastThis<FFolderModel>())
		{
			if (FolderNode->GetFolder() == DestinationFolder)
			{
				NodesToMove.Remove(Node);
				break;
			}
		}
	}

	if (!NodesToMove.Num())
	{
		return;
	}

	TArray<TArray<FString> > NodePathSplits;
	int32 SharedPathLength = TNumericLimits<int32>::Max();

	// Build a list of the paths for each node, split in to folder names
	for (TSharedRef<FViewModel> Node : NodesToMove)
	{
		// Split the node's path in to segments
		TArray<FString>& NodePath = NodePathSplits.AddDefaulted_GetRef();
		IOutlinerExtension::GetPathName(*Node).ParseIntoArray(NodePath, TEXT("."));

		// Shared path obviously won't be larger than the shortest path
		SharedPathLength = FMath::Min(SharedPathLength, NodePath.Num() - 1);
	}

	// If we have more than one, find the deepest folder shared by all paths
	if (NodePathSplits.Num() > 1)
	{
		// Since we are looking for the shared path, we can arbitrarily choose the first path to compare against
		TArray<FString>& ShareNodePathSplit = NodePathSplits[0];
		for (int NodeIndex = 1; NodeIndex < NodePathSplits.Num(); ++NodeIndex)
		{
			if (SharedPathLength == 0)
			{
				break;
			}

			// Since all paths are at least as long as the shortest, we don't need to bounds check the path splits
			for (int PathSplitIndex = 0; PathSplitIndex < SharedPathLength; ++PathSplitIndex)
			{
				if (NodePathSplits[NodeIndex][PathSplitIndex].Compare(ShareNodePathSplit[PathSplitIndex]))
				{
					SharedPathLength = PathSplitIndex;
					break;
				}
			}
		}
	}

	UMovieSceneFolder* ParentFolder = nullptr;

	TArray<FName> FolderPath;

	// Walk up the shared path to find the deepest shared folder
	for (int32 FolderPathIndex = 0; FolderPathIndex < SharedPathLength; ++FolderPathIndex)
	{
		FolderPath.Add(FName(*NodePathSplits[0][FolderPathIndex]));
		FName DesiredFolderName = FolderPath[FolderPathIndex];

		TArray<UMovieSceneFolder*> FoldersToSearch;
		if (!ParentFolder)
		{
			FoldersToSearch = FocusedMovieScene->GetRootFolders();
		}
		else
		{
			FoldersToSearch = ParentFolder->GetChildFolders();
		}

		for (UMovieSceneFolder* Folder : FoldersToSearch)
		{
			if (Folder->GetFolderName() == DesiredFolderName)
			{
				ParentFolder = Folder;
				break;
			}
		}
	}

	FScopedTransaction Transaction(LOCTEXT("MoveTracksToFolder", "Move to Folder"));

	ViewModel->GetSelection()->Empty();

	// Find the path to the displaynode of our destination folder
	FString DestinationFolderPath;
	if (DestinationFolder)
	{
		TArray<TSharedRef<FViewModel>> AllNodes;
		NodeTree->GetAllNodes(AllNodes);
		for (TSharedRef<FViewModel> Node : AllNodes)
		{
			// If this node is the destination folder, don't try to move it
			if (FFolderModel* FolderNode = Node->CastThis<FFolderModel>())
			{
				if (FolderNode->GetFolder() == DestinationFolder)
				{
					DestinationFolderPath = IOutlinerExtension::GetPathName(*Node);

					// Expand the folders to our destination
					TSharedPtr<FViewModel> ParentNode = Node;
					while (ParentNode)
					{
						if (ParentNode->IsA<IOutlinerExtension>())
						{
							ParentNode->CastThisChecked<IOutlinerExtension>()->SetExpansion(true);
						}
						
						ParentNode = ParentNode->GetParent();
					}
					break;
				}
			}
		}
	}

	for (int32 NodeIndex = 0; NodeIndex < NodesToMove.Num(); ++NodeIndex)
	{
		TSharedRef<FViewModel> Node = NodesToMove[NodeIndex];
		TArray<FString>& NodePathSplit = NodePathSplits[NodeIndex];

		// Reset the relative path
		FolderPath.Reset(NodePathSplit.Num());

		FString NewPath = DestinationFolderPath;

		if (!NewPath.IsEmpty())
		{
			NewPath += TEXT(".");
		}

		// Append any relative path for the node
		for (int32 FolderPathIndex = SharedPathLength; FolderPathIndex < NodePathSplit.Num() - 1; ++FolderPathIndex)
		{
			FolderPath.Add(FName(*NodePathSplit[FolderPathIndex]));
			NewPath += NodePathSplit[FolderPathIndex] + TEXT(".");
		}

		NewPath += Node->CastThis<IOutlinerExtension>()->GetIdentifier().ToString();

		UMovieSceneFolder* NodeDestinationFolder = CreateFoldersRecursively(FolderPath, 0, FocusedMovieScene, DestinationFolder, DestinationFolder->GetChildFolders());
		MoveNodeToFolder(Node, NodeDestinationFolder);

		SequencerWidget->AddAdditionalPathToSelectionSet(NewPath);
	}

	NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);

}

void FSequencer::MoveSelectedNodesToNewFolder()
{
	using namespace UE::Sequencer;

	UMovieScene* FocusedMovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();

	if (!FocusedMovieScene)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	TArray<TSharedRef<FViewModel> > NodesToMove = GetSelectedNodesToMove();

	if (!NodesToMove.Num())
	{
		return;
	}

	TArray<TArray<FString> > NodePathSplits;
	int32 SharedPathLength = TNumericLimits<int32>::Max();

	// Build a list of the paths for each node, split in to folder names
	for (TSharedRef<FViewModel> Node : NodesToMove)
	{
		// Split the node's path in to segments
		TArray<FString>& NodePath = NodePathSplits.AddDefaulted_GetRef();
		IOutlinerExtension::GetPathName(*Node).ParseIntoArray(NodePath, TEXT("."));

		// Shared path obviously won't be larger than the shortest path
		SharedPathLength = FMath::Min(SharedPathLength, NodePath.Num() - 1);
	}

	// If we have more than one, find the deepest folder shared by all paths
	if (NodePathSplits.Num() > 1)
	{
		// Since we are looking for the shared path, we can arbitrarily choose the first path to compare against
		TArray<FString>& ShareNodePathSplit = NodePathSplits[0];
		for (int NodeIndex = 1; NodeIndex < NodePathSplits.Num(); ++NodeIndex)
		{
			if (SharedPathLength == 0)
			{
				break;
			}

			// Since all paths are at least as long as the shortest, we don't need to bounds check the path splits
			for (int PathSplitIndex = 0; PathSplitIndex < SharedPathLength; ++PathSplitIndex)
			{
				if (NodePathSplits[NodeIndex][PathSplitIndex].Compare(ShareNodePathSplit[PathSplitIndex]))
				{
					SharedPathLength = PathSplitIndex;
					break;
				}
			}
		}
	}

	UMovieSceneFolder* ParentFolder = nullptr;

	TArray<FName> FolderPath;
	
	// Walk up the shared path to find the deepest shared folder
	for (int32 FolderPathIndex = 0; FolderPathIndex < SharedPathLength; ++FolderPathIndex)
	{
		FolderPath.Add(FName(*NodePathSplits[0][FolderPathIndex]));
		FName DesiredFolderName = FolderPath[FolderPathIndex];

		TArray<UMovieSceneFolder*> FoldersToSearch;
		if (!ParentFolder)
		{
			FoldersToSearch = FocusedMovieScene->GetRootFolders();
		}
		else
		{
			FoldersToSearch = ParentFolder->GetChildFolders();
		}

		for (UMovieSceneFolder* Folder : FoldersToSearch)
		{
			if (Folder->GetFolderName() == DesiredFolderName)
			{
				ParentFolder = Folder;
				break;
			}
		}
	}

	TArray<FName> ExistingFolderNames;
	if (!ParentFolder)
	{
		for (UMovieSceneFolder* SiblingFolder : FocusedMovieScene->GetRootFolders())
		{
			ExistingFolderNames.Add(SiblingFolder->GetFolderName());
		}
	}
	else
	{
		for (UMovieSceneFolder* SiblingFolder : ParentFolder->GetChildFolders())
		{
			ExistingFolderNames.Add(SiblingFolder->GetFolderName());
		}
	}

	FString NewFolderPath;
	for (FName PathSection : FolderPath)
	{
		NewFolderPath.Append(PathSection.ToString());
		NewFolderPath.AppendChar('.');
	}

	FScopedTransaction Transaction(LOCTEXT("MoveTracksToNewFolder", "Move to New Folder"));

	// Create SharedFolder
	FName UniqueName = FSequencerUtilities::GetUniqueName(FName("New Folder"), ExistingFolderNames);
	UMovieSceneFolder* SharedFolder = NewObject<UMovieSceneFolder>( FocusedMovieScene, NAME_None, RF_Transactional );
	SharedFolder->SetFolderName(UniqueName);
	NewFolderPath.Append(UniqueName.ToString());

	FolderPath.Add(UniqueName);
	int SharedFolderPathLen = FolderPath.Num();

	if (!ParentFolder)
	{
		FocusedMovieScene->Modify();
		FocusedMovieScene->AddRootFolder(SharedFolder);
	}
	else
	{
		ParentFolder->Modify();
		ParentFolder->AddChildFolder(SharedFolder);
	}

	for (int32 NodeIndex = 0; NodeIndex < NodesToMove.Num() ; ++NodeIndex)
	{
		TSharedRef<FViewModel> Node = NodesToMove[NodeIndex];
		TArray<FString>& NodePathSplit = NodePathSplits[NodeIndex];

		// Reset to just the path to the shared folder
		FolderPath.SetNum(SharedFolderPathLen);

		// Append any relative path for the node
		for (int32 FolderPathIndex = SharedPathLength; FolderPathIndex < NodePathSplit.Num() - 1; ++FolderPathIndex)
		{
			FolderPath.Add(FName(*NodePathSplit[FolderPathIndex]));
		}

		UMovieSceneFolder* DestinationFolder = CreateFoldersRecursively(FolderPath, 0, FocusedMovieScene, nullptr, FocusedMovieScene->GetRootFolders());
		
		MoveNodeToFolder(Node, DestinationFolder);
	}

	// Set the newly created folder as our selection
	ViewModel->GetSelection()->Empty();
	SequencerWidget->AddAdditionalPathToSelectionSet(NewFolderPath);


	NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}


void FSequencer::RemoveSelectedNodesFromFolders()
{
	using namespace UE::Sequencer;

	UMovieScene* FocusedMovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();

	if (!FocusedMovieScene)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	TArray<TSharedRef<FViewModel> > NodesToFolders = GetSelectedNodesInFolders();
	if (!NodesToFolders.Num())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveNodeFromFolder", "Remove from Folder"));

	FocusedMovieScene->Modify();

	for (TSharedRef<FViewModel> NodeInFolder : NodesToFolders)
	{
		TSharedPtr<FFolderModel> Folder = NodeInFolder->FindAncestorOfType<FFolderModel>();
		if (Folder.IsValid())
		{
			if (IObjectBindingExtension* ObjectBindingNode = NodeInFolder->CastThis<IObjectBindingExtension>())
			{
				Folder->GetFolder()->RemoveChildObjectBinding(ObjectBindingNode->GetObjectGuid());
			}
			else if (ITrackExtension* TrackNode = NodeInFolder->CastThis<ITrackExtension>())
			{
				if (TrackNode->GetTrack())
				{
					Folder->GetFolder()->RemoveChildTrack(TrackNode->GetTrack());
				}
			}
			else if (ITrackRowExtension* TrackRowNode = NodeInFolder->CastThis<ITrackRowExtension>())
			{
				if (TrackRowNode->GetParentTrack())
				{
					Folder->GetFolder()->RemoveChildTrack(TrackRowNode->GetParentTrack());
				}
			}
		}
	}

	NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

TArray<TSharedPtr<FMovieSceneClipboard>> GClipboardStack;

void FSequencer::CopySelectedObjects(TArray<TSharedPtr<UE::Sequencer::FObjectBindingModel>>& ObjectNodes, const TArray<UMovieSceneFolder*>& Folders, FString& ExportedText)
{
	using namespace UE::Sequencer;

	// Gather guids for the object nodes and any child object nodes
	TSet<FMovieSceneBindingProxy> Bindings;
	for (TSharedPtr<FObjectBindingModel> ObjectNode : ObjectNodes)
	{
		Bindings.Add(FMovieSceneBindingProxy(ObjectNode->GetObjectGuid(), GetFocusedMovieSceneSequence()));

		TSet<TSharedRef<FViewModel> > DescendantNodes;

		SequencerHelpers::GetDescendantNodes(ObjectNode.ToSharedRef(), DescendantNodes);

		for (auto DescendantNode : DescendantNodes)
		{
			if (FObjectBindingModel* DescendantObjectNode = DescendantNode->CastThis<FObjectBindingModel>())
			{
				Bindings.Add(FMovieSceneBindingProxy(DescendantObjectNode->GetObjectGuid(), GetFocusedMovieSceneSequence()));
			}
		}
	}

	TArray<FMovieSceneBindingProxy> BindingsArray = Bindings.Array();
	FSequencerUtilities::CopyBindings(AsShared(), BindingsArray, Folders, ExportedText);
		
	// Make sure to clear the clipboard for the keys
	GClipboardStack.Empty();
}

void FSequencer::CopySelectedTracks(TArray<TSharedPtr<FViewModel>>& TrackNodes, const TArray<UMovieSceneFolder*>& Folders, FString& ExportedText)
{
	using namespace UE::Sequencer;

	TArray<UMovieSceneTrack*> CopyableTracks;
	TArray<UMovieSceneSection*> CopyableSections;
	for (TSharedPtr<FViewModel> TrackNode : TrackNodes)
	{
		bool bIsParentSelected = false;
		TSharedPtr<FViewModel> ParentNode = TrackNode->GetParent();
		while (ParentNode.IsValid())
		{
			if (ViewModel->GetSelection()->Outliner.IsSelected(CastViewModel<IOutlinerExtension>(ParentNode)))
			{
				bIsParentSelected = true;
				break;
			}
			ParentNode = ParentNode->GetParent();
		}

		if (!bIsParentSelected)
		{
			// If this is a track row, only copy the sections that belong to this row. otherwise copying the entire track will copy all the sections across all the rows
			if (ITrackRowExtension* TrackRowExtension = TrackNode->CastThis<ITrackRowExtension>())
			{
				if (ISectionOwnerExtension* SectionOwner = TrackNode->CastThis<ISectionOwnerExtension>())
				{
					TArray<UMovieSceneSection*> Sections = SectionOwner->GetSections();
					CopyableSections.Append(Sections);
				}
			}
			else if (UMovieSceneTrack* Track = TrackNode->CastThisChecked<ITrackExtension>()->GetTrack())
			{
				CopyableTracks.Add(Track);
			}
		}
	}

	if (CopyableSections.Num() + CopyableTracks.Num() > 0)
	{
		FString SectionsExportedText;
		FSequencerUtilities::CopySections(CopyableSections, SectionsExportedText);

		FString TracksExportedText;
		FSequencerUtilities::CopyTracks(CopyableTracks, Folders, TracksExportedText);

		ExportedText.Empty();
		ExportedText += SectionsExportedText;
		ExportedText += TracksExportedText;

		// Make sure to clear the clipboard for the keys
		GClipboardStack.Empty();
	}
}

void FSequencer::CopySelectedFolders(const TArray<UMovieSceneFolder*>& Folders, FString& FoldersExportedText, FString& TracksExportedText, FString& ObjectExportedText)
{
	FSequencerUtilities::CopyFolders(AsShared(), Folders, FoldersExportedText, TracksExportedText, ObjectExportedText);

	// Make sure to clear the clipboard for the keys
	GClipboardStack.Empty();
}

bool FSequencer::DoPaste(bool bClearSelection)
{
	if (IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		// If we cancel the paste due to being read-only, count that as having handled the paste operation
		return true;
	}
	
	// Call into the active customizations.
	ESequencerPasteSupport PasteSupport = ESequencerPasteSupport::All;
	for (const FOnSequencerPaste& Callback : OnPaste)
	{
		const ESequencerPasteSupport CurSupport = Callback.Execute();
		PasteSupport = (PasteSupport & CurSupport);
	}
	if (PasteSupport == ESequencerPasteSupport::None)
	{
		// We handled the paste operation but we didn't support doing anything.
		return true;
	}

	// Grab the text to paste from the clipboard
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	FScopedTransaction Transaction(FGenericCommands::Get().Paste->GetDescription());

	TArray<UMovieSceneFolder*> SelectedParentFolders;
	FString NewNodePath;
	CalculateSelectedFolderAndPath(SelectedParentFolders, NewNodePath);
	UMovieSceneFolder* ParentFolder = SelectedParentFolders.Num() > 0 ? SelectedParentFolders[0] : nullptr;

	TArray<FNotificationInfo> PasteErrors;
	bool bAnythingPasted = false;
	TArray<UMovieSceneFolder*> PastedFolders;
	if (EnumHasAnyFlags(PasteSupport, ESequencerPasteSupport::Folders)) 
	{
		bAnythingPasted |= FSequencerUtilities::PasteFolders(TextToImport, FMovieScenePasteFoldersParams(GetFocusedMovieSceneSequence(), ParentFolder), PastedFolders, PasteErrors);
	}
	if (EnumHasAnyFlags(PasteSupport, ESequencerPasteSupport::ObjectBindings)) 
	{
		EPasteResult PasteObjectBindingsResult = PasteObjectBindings(TextToImport, ParentFolder, PastedFolders, PasteErrors, bClearSelection);
		if (PasteObjectBindingsResult == EPasteResult::Cancelled)
		{
			return false;
		}

		bAnythingPasted |= PasteObjectBindingsResult == EPasteResult::Success;
	}
	if (EnumHasAnyFlags(PasteSupport, ESequencerPasteSupport::Tracks)) 
	{
		EPasteResult PasteTracksResult = PasteTracks(TextToImport, ParentFolder, PastedFolders, PasteErrors, bClearSelection);
		if (PasteTracksResult == EPasteResult::Cancelled)
		{
			return false;
		}

		bAnythingPasted |= PasteTracksResult == EPasteResult::Success;
	}

	// Decorations and tracks coexist in mixed clipboards; run unconditionally so
	// neither path eats the other.
	if (EnumHasAnyFlags(PasteSupport, ESequencerPasteSupport::Decorations))
	{
		EPasteResult PasteDecorationsResult = PasteDecorations(TextToImport, PasteErrors);
		if (PasteDecorationsResult == EPasteResult::Cancelled)
		{
			return false;
		}

		bAnythingPasted |= PasteDecorationsResult == EPasteResult::Success;
	}

	// Sections still gate on no-other-paste because a decoration's text contains
	// nested section blocks that FSectionObjectTextFactory would otherwise lift
	// to top-level and double-paste.
	if (!bAnythingPasted)
	{
		if (EnumHasAnyFlags(PasteSupport, ESequencerPasteSupport::Sections))
		{
			EPasteResult PasteSectionsResult = PasteSections(TextToImport, PasteErrors);
			if (PasteSectionsResult == EPasteResult::Cancelled)
			{
				return false;
			}

			bAnythingPasted |= PasteSectionsResult == EPasteResult::Success;
		}
	}

	if (!bAnythingPasted)
	{
		for (auto NotificationInfo : PasteErrors)
		{
			NotificationInfo.bUseLargeFont = false;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}

	return bAnythingPasted;
}

FSequencer::EPasteResult FSequencer::PasteObjectBindings(const FString& TextToImport, UMovieSceneFolder* InParentFolder, const TArray<UMovieSceneFolder*>& InFolders, TArray<FNotificationInfo>& PasteErrors, bool bClearSelection)
{
	using namespace UE::Sequencer;

	TArray<FMovieSceneBindingProxy> TargetBindings;
	if (!bClearSelection)
	{
		for (TViewModelPtr<FObjectBindingModel> ObjectNode : ViewModel->GetSelection()->Outliner.Filter<FObjectBindingModel>())
		{
			TargetBindings.Add(FMovieSceneBindingProxy(ObjectNode->GetObjectGuid(), GetFocusedMovieSceneSequence()));
		}
	}

	UObject* PlaybackContext = GetPlaybackContext();
	UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

	bool bDuplicateExistingActors = false;
	TArray<FString> ObjectNames = FSequencerUtilities::GetPasteBindingsObjectNames(AsShared(), TextToImport);

	// If the current movie scene has bindings that are bound to actors with the same name of the paste buffer, prompt the user what to do
	UMovieScene* MovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();
	bool bWasPrompted = false;

	for (int32 Index = 0; Index < MovieScene->GetPossessableCount() && !bWasPrompted; ++Index)
	{
		FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);

		FGuid ThisGuid = Possessable.GetGuid();

		for (TWeakObjectPtr<> WeakObject : FindBoundObjects(ThisGuid, GetFocusedTemplateID()))
		{
			if (WeakObject.IsValid())
			{
				AActor* Actor = Cast<AActor>(WeakObject.Get());

				if (Actor && ObjectNames.Contains(Actor->GetPathName(World)))
				{
					FText DuplicateActorsMsg = FText::Format(LOCTEXT("DuplicateActorsForPastedBinding", "Attempting to paste a binding that is already bound to {0}.\nShould the existing actor be duplicated for the pasted binding?"), FText::FromString(Actor->GetActorLabel()));

					EAppReturnType::Type YesNoCancelReply = FMessageDialog::Open(EAppMsgType::YesNoCancel, DuplicateActorsMsg);

					switch (YesNoCancelReply)
					{
					case EAppReturnType::Yes:
						bDuplicateExistingActors = true;
						break;

					case EAppReturnType::No:
						bDuplicateExistingActors = false;
						break;

					case EAppReturnType::Cancel:
						return EPasteResult::Cancelled;
					}
					bWasPrompted = true;				
					break;
				}
			}
		}
	}

	TArray<FMovieSceneBindingProxy> OutBindings;
	if (FSequencerUtilities::PasteBindings(TextToImport, AsShared(), FMovieScenePasteBindingsParams(TargetBindings, InParentFolder, InFolders, bDuplicateExistingActors), OutBindings, PasteErrors))
	{
		return EPasteResult::Success;
	}
	return EPasteResult::Failure;
}

FSequencer::EPasteResult FSequencer::PasteTracks(const FString& TextToImport, UMovieSceneFolder* InParentFolder, const TArray<UMovieSceneFolder*>& InFolders, TArray<FNotificationInfo>& PasteErrors, bool bClearSelection)
{
	using namespace UE::Sequencer;

	TArray<FMovieSceneBindingProxy> TargetBindings;
	if (!bClearSelection)
	{
		for (TViewModelPtr<FObjectBindingModel> ObjectNode : ViewModel->GetSelection()->Outliner.Filter<FObjectBindingModel>())
		{
			TargetBindings.Add(FMovieSceneBindingProxy(ObjectNode->GetObjectGuid(), GetFocusedMovieSceneSequence()));
		}
	}

	TArray<UMovieSceneTrack*> OutTracks;
	bool bSuccess = FSequencerUtilities::PasteTracks(TextToImport, FMovieScenePasteTracksParams(GetFocusedMovieSceneSequence(), TargetBindings, InParentFolder, InFolders), OutTracks, PasteErrors);

	if (bSuccess)
	{
		NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
		return EPasteResult::Success;
	}
	
	return EPasteResult::Failure;
}

FSequencer::EPasteResult FSequencer::PasteSections(const FString& TextToImport, TArray<FNotificationInfo>& PasteErrors)
{
	using namespace UE::Sequencer;

	FScopedTransaction Transaction(FGenericCommands::Get().Paste->GetDescription());

	TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();
	FSelectionEventSuppressor EventSuppressor = Selection->SuppressEvents();

	TArray<TSharedRef<FViewModel>> SelectedNodes;
	SelectedNodes.Reserve(Selection->Outliner.Num());
	for (FViewModelPtr SelectedNode : Selection->Outliner)
	{
		SelectedNodes.Add(SelectedNode.AsModel().ToSharedRef());
	}

	TSet<UMovieSceneSection*> SelectedSections = Selection->GetSelectedSections();
	if (SelectedNodes.Num() == 0)
	{
		if (Selection->GetNodesWithSelectedKeysOrSections().Num() != 0)
		{
			SelectedNodes.Reserve(Selection->GetNodesWithSelectedKeysOrSections().Num());
			for (TViewModelPtr<IOutlinerExtension> Item : Selection->IterateIndirectOutlinerSelection())
			{
				SelectedNodes.Add(Item.AsModel().ToSharedRef());
			}
		}
	}

	if (SelectedNodes.Num() == 0)
	{
		for (const TViewModelPtr<IOutlinerExtension>& RootNode : NodeTree->GetRootNodes())
		{
			TSet<TWeakObjectPtr<UMovieSceneSection>> Sections;
			SequencerHelpers::GetAllSections(RootNode.AsModel(), Sections);
			for (TWeakObjectPtr<UMovieSceneSection> Section : Sections)
			{
				if (SelectedSections.Contains(Section.Get()))
				{
					SelectedNodes.Add(RootNode.AsModel().ToSharedRef());
					break;
				}
			}
		}
	}

	if (SelectedNodes.Num() == 0)
	{
		FNotificationInfo Info(LOCTEXT("PasteSections_NoSelectedTracks", "Can't paste section. No selected tracks to paste sections onto"));
		PasteErrors.Add(Info);
		return EPasteResult::Failure;
	}

	FMovieScenePasteSectionsParams PasteParams;
	PasteParams.Time = GetLocalTime().Time;
	for (TSharedRef<FViewModel> Node : SelectedNodes)
	{
		if (FDecorationOutlinerModel* DecoNode = Node->CastThis<FDecorationOutlinerModel>())
		{
			UMovieSceneDecorationContainerObject* Container = DecoNode->GetDecorationContainer().Get();
			UObject* Decoration = DecoNode->GetDecoration();
			if (Container || Decoration)
			{
				PasteParams.AddTarget(/*Track*/ nullptr, INDEX_NONE, Container, Decoration);
			}
			continue;
		}

		// Prefer the row interface so we get a row index for per-row container
		// lookup (a node may implement both, e.g. FAnimationMixerLayerModel).
		if (ITrackRowExtension* TrackRowNode = Node->CastThis<ITrackRowExtension>())
		{
			UMovieSceneTrack* TrackForNode = TrackRowNode->GetParentTrack();
			if (TrackForNode)
			{
				const int32 RowIndex = TrackRowNode->GetRowIndex();
				UMovieSceneDecorationContainerObject* Container = TrackForNode->GetDecorationContainerForRow(RowIndex);
				PasteParams.AddTarget(TrackForNode, RowIndex, Container);
			}
		}
		else if (ITrackExtension* TrackNode = Node->CastThis<ITrackExtension>())
		{
			UMovieSceneTrack* TrackForNode = TrackNode->GetTrack();
			if (TrackForNode)
			{
				UMovieSceneDecorationContainerObject* Container = TrackForNode->GetDecorationContainerForRow(INDEX_NONE);
				PasteParams.AddTarget(TrackForNode, INDEX_NONE, Container);
			}
		}
	}

	// Otherwise, look at all child nodes
	if (PasteParams.Tracks.Num() == 0 && PasteParams.DecorationTargets.Num() == 0)
	{
		for (TSharedRef<FViewModel> Node : SelectedNodes)
		{
			TSet<TSharedRef<FViewModel> > DescendantNodes;
			SequencerHelpers::GetDescendantNodes(Node, DescendantNodes);

			for (TSharedRef<FViewModel> DescendantNode : DescendantNodes)
			{
				// Don't automatically paste onto row-level models because that would lead to multiple paste destinations
				if (ITrackRowExtension* DescendantTrackRowNode = DescendantNode->CastThis<ITrackRowExtension>())
				{
					continue;
				}

				if (ITrackExtension* TrackNode = DescendantNode->CastThis<ITrackExtension>())
				{
					UMovieSceneTrack* TrackForNode = TrackNode->GetTrack();
					if (TrackForNode)
					{
						// DescendantNode is guaranteed not to be an ITrackRowExtension
						// (filtered by the continue above), so no per-row context here.
						UMovieSceneDecorationContainerObject* Container = TrackForNode->GetDecorationContainerForRow(INDEX_NONE);
						PasteParams.AddTarget(TrackForNode, INDEX_NONE, Container);
					}
				}
			}
		}
	}

	TArray<UMovieSceneSection*> NewSections;
	if (!FSequencerUtilities::PasteSections(TextToImport, PasteParams, NewSections, PasteErrors))
	{
		return EPasteResult::Failure;
	}

	FSectionModelStorageExtension* SectionModelStorage = ViewModel->GetRootModel()->CastDynamic<FSectionModelStorageExtension>();
	{
		UE::MovieScene::FScopedSignedObjectModifyDefer ForceFlush(true);
		check(SectionModelStorage);
	}

	EmptySelection();

	for (UMovieSceneSection* NewSection : NewSections)
	{
		TSharedPtr<FSectionModel> SectionModel = SectionModelStorage->FindModelForSection(NewSection);
		if (ensure(SectionModel))
		{
			Selection->TrackArea.Select(SectionModel);
		}
	}

	ThrobSectionSelection();

	NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);

	return EPasteResult::Success;
}

FSequencer::EPasteResult FSequencer::PasteDecorations(const FString& TextToImport, TArray<FNotificationInfo>& PasteErrors)
{
	using namespace UE::Sequencer;

	FScopedTransaction Transaction(FGenericCommands::Get().Paste->GetDescription());

	TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();
	FSelectionEventSuppressor EventSuppressor = Selection->SuppressEvents();

	TArray<UMovieSceneDecorationContainerObject*> Containers;

	auto AddContainerUnique = [&Containers](UMovieSceneDecorationContainerObject* Container)
	{
		if (Container)
		{
			Containers.AddUnique(Container);
		}
	};

	for (FViewModelPtr SelectedNode : Selection->Outliner)
	{
		TSharedPtr<FViewModel> Node = SelectedNode.AsModel();
		if (!Node)
		{
			continue;
		}

		if (FDecorationOutlinerModel* DecoNode = Node->CastThis<FDecorationOutlinerModel>())
		{
			AddContainerUnique(DecoNode->GetDecorationContainer().Get());
			continue;
		}

		if (FSectionOutlinerModel* SectionOutliner = Node->CastThis<FSectionOutlinerModel>())
		{
			AddContainerUnique(SectionOutliner->GetSection());
			continue;
		}

		// Prefer per-row container, fall back to the track itself.
		if (ITrackRowExtension* TrackRowNode = Node->CastThis<ITrackRowExtension>())
		{
			if (UMovieSceneTrack* TrackForNode = TrackRowNode->GetParentTrack())
			{
				if (UMovieSceneDecorationContainerObject* RowContainer = TrackForNode->GetDecorationContainerForRow(TrackRowNode->GetRowIndex()))
				{
					AddContainerUnique(RowContainer);
				}
				else
				{
					AddContainerUnique(TrackForNode);
				}
			}
			continue;
		}

		if (ITrackExtension* TrackNode = Node->CastThis<ITrackExtension>())
		{
			AddContainerUnique(TrackNode->GetTrack());
		}
	}

	if (Containers.Num() == 0)
	{
		FNotificationInfo Info(LOCTEXT("PasteDecorations_NoTargets", "Can't paste decoration. No selected target to paste onto."));
		PasteErrors.Add(Info);
		return EPasteResult::Failure;
	}

	TArray<UObject*> NewDecorations;
	if (!FSequencerUtilities::PasteDecorations(TextToImport, FMovieScenePasteDecorationsParams(Containers), NewDecorations, PasteErrors))
	{
		return EPasteResult::Failure;
	}

	NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);

	return EPasteResult::Success;
}

bool FSequencer::CanPaste(const FString& TextToImport)
{
	return (FSequencerUtilities::CanPasteBindings(AsShared(), TextToImport) ||
			FSequencerUtilities::CanPasteTracks(TextToImport) ||
			FSequencerUtilities::CanPasteSections(TextToImport) ||
			FSequencerUtilities::CanPasteDecorations(TextToImport) ||
			FSequencerUtilities::CanPasteFolders(TextToImport));
}

void FSequencer::ObjectImplicitlyAdded(UObject* InObject) const
{
	for (int32 i = 0; i < TrackEditors.Num(); ++i)
	{
		TrackEditors[i]->ObjectImplicitlyAdded(InObject);
	}
}

void FSequencer::ObjectImplicitlyRemoved(UObject* InObject) const
{
	for (int32 i = 0; i < TrackEditors.Num(); ++i)
	{
		TrackEditors[i]->ObjectImplicitlyRemoved(InObject);
	}
}

void FSequencer::SetTrackFilterEnabled(const FText& InFilterName, bool bInEnabled)
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = GetFilterBar();
	if (const TSharedPtr<FSequencerTrackFilter> Filter = FilterBar->FindFilterByDisplayName(InFilterName.ToString()))
	{
		FilterBar->SetFilterEnabled(Filter.ToSharedRef(), bInEnabled, true);
	}
}

bool FSequencer::IsTrackFilterEnabled(const FText& InFilterName) const
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = GetFilterBar();
	if (const TSharedPtr<FSequencerTrackFilter> Filter = FilterBar->FindFilterByDisplayName(InFilterName.ToString()))
	{
		return FilterBar->IsFilterEnabled(Filter.ToSharedRef());
	}
	return false;
}

TArray<FText> FSequencer::GetTrackFilterNames() const
{
	const TSharedPtr<FSequencerFilterBar> FilterBar = GetFilterBar();
	return FilterBar->GetFilterDisplayNames();
}

bool FSequencer::TrackSupportsConditions(const UMovieSceneTrack* Track) const
{
	if (Track)
	{
		if (!Track->SupportsConditions())
		{
			return false;
		}

		// Non ECS tracks don't support conditions
		if (CompiledDataManager)
		{
			if (const FMovieSceneEvaluationTemplate* Template = CompiledDataManager->FindTrackTemplate(CompiledDataManager->FindDataID(Track->GetTypedOuter<UMovieSceneSequence>())))
			{
				if (Template->FindTrack(Track->GetSignature()))
				{
					return false;
				}
			}
		}

		return true;
	}
	return false;
}

void FSequencer::ToggleNodeLocked()
{
	using namespace UE::Sequencer;

	bool bIsLocked = !IsNodeLocked();

	const FScopedTransaction Transaction( NSLOCTEXT("Sequencer", "ToggleNodeLocked", "Toggle Node Locked") );

	for (FViewModelPtr OutlinerNode : ViewModel->GetSelection()->Outliner)
	{
		TSet<TWeakObjectPtr<UMovieSceneSection> > Sections;
		SequencerHelpers::GetAllSections(OutlinerNode, Sections);

		for (auto Section : Sections)
		{
			Section->Modify();
			Section->SetIsLocked(bIsLocked);
		}
	}
}


bool FSequencer::IsNodeLocked() const
{
	using namespace UE::Sequencer;

	// Locked only if all are locked
	int NumSections = 0;
	for (FViewModelPtr OutlinerNode : ViewModel->GetSelection()->Outliner)
	{
		TSet<TWeakObjectPtr<UMovieSceneSection> > Sections;
		SequencerHelpers::GetAllSections(OutlinerNode, Sections);

		for (auto Section : Sections)
		{
			if (!Section->IsLocked())
			{
				return false;
			}
			++NumSections;
		}
	}
	return NumSections > 0;
}

void FSequencer::GroupSelectedSections()
{
	UMovieScene* MovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();
	if (MovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("GroupSelectedSections", "Group Selected Sections"));

	TArray<UMovieSceneSection*> Sections;
	for (UMovieSceneSection* Section : ViewModel->GetSelection()->GetSelectedSections())
	{
		// We do not want to group sections that are infinite, as they should not be moveable
		if (Section && (Section->HasStartFrame() || Section->HasEndFrame()))
		{
			Sections.Add(Section);
		}
	}

	MovieScene->GroupSections(Sections);
}

bool FSequencer::CanGroupSelectedSections() const
{
	int32 GroupableSections = 0;
	for (UMovieSceneSection* Section : ViewModel->GetSelection()->GetSelectedSections())
	{
		// We do not want to group sections that are infinite, as they should not be moveable
		if (Section->HasStartFrame() || Section->HasEndFrame())
		{
			if (++GroupableSections >= 2)
			{
				return true;
			}
		}
	}
	return false;
}

void FSequencer::UngroupSelectedSections()
{
	UMovieScene* MovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();
	if (MovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("UngroupSelectedSections", "Ungroup Selected Sections"));

	for (UMovieSceneSection* Section : ViewModel->GetSelection()->GetSelectedSections())
	{
		MovieScene->UngroupSection(*Section);
	}
}

bool FSequencer::CanUngroupSelectedSections() const
{
	UMovieScene* MovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();

	for (UMovieSceneSection* Section : ViewModel->GetSelection()->GetSelectedSections())
	{
		if (MovieScene->IsSectionInGroup(*Section))
		{
			return true;
		}
	}
	return false;
}

void FSequencer::SaveSelectedNodesSpawnableState()
{
	using namespace UE::Sequencer;

	UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = Sequence->GetMovieScene();

	if (MovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	const FScopedTransaction Transaction( LOCTEXT("SaveSpawnableState", "Save spawnable state") );

	MovieScene->Modify();

	FScopedSlowTask SlowTask(ViewModel->GetSelection()->Outliner.Num(), LOCTEXT("SaveSpawnableStateProgress", "Saving selected spawnables"));
	SlowTask.MakeDialog(true);

	for (TViewModelPtr<IOutlinerExtension> OutlinerSelectionNode : ViewModel->GetSelection()->Outliner)
	{
		SlowTask.EnterProgressFrame();
		if (TViewModelPtr<IObjectBindingExtension> ObjectBindingNode = OutlinerSelectionNode.ImplicitCast())
		{
			if (const FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(ObjectBindingNode->GetObjectGuid()))
			{
				SpawnRegister->SaveDefaultSpawnableState(ObjectBindingNode->GetObjectGuid(), 0, ActiveTemplateIDs.Top(), GetSharedPlaybackState());
			}
			else if (const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
			{
				TArrayView<const FMovieSceneBindingReference> AllReferences = BindingReferences->GetReferences(ObjectBindingNode->GetObjectGuid());
				for (int32 Index = 0; Index < AllReferences.Num(); ++Index)
				{
					if (AllReferences[Index].CustomBinding && AllReferences[Index].CustomBinding->WillSpawnObject(GetSharedPlaybackState()))
					{
						SpawnRegister->SaveDefaultSpawnableState(ObjectBindingNode->GetObjectGuid(), Index, ActiveTemplateIDs.Top(), GetSharedPlaybackState());
					}
				}
			}

			if (GWarn->ReceivedUserCancel())
			{
				break;
			}
		}
	}
	NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

void FSequencer::ConvertToSpawnable(TSharedRef<UE::Sequencer::FObjectBindingModel> NodeToBeConverted)
{
	using namespace UE::Sequencer;

	if (GetFocusedMovieSceneSequence()->GetMovieScene()->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	const FScopedTransaction Transaction( LOCTEXT("ConvertSelectedNodeSpawnable", "Convert Node to Spawnables") );

	// Ensure we're in a non-possessed state
	TGuardValue<bool> Guard(bUpdatingExternalSelection, true);
	RestorePreAnimatedState();
	GetFocusedMovieSceneSequence()->GetMovieScene()->Modify();
	FMovieScenePossessable* Possessable = GetFocusedMovieSceneSequence()->GetMovieScene()->FindPossessable(NodeToBeConverted->GetObjectGuid());
	if (Possessable)
	{
		FSequencerUtilities::ConvertToSpawnable(AsShared(), Possessable->GetGuid());
		NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemsChanged );
	}
}
TArray<FGuid> FSequencer::ConvertToSpawnable(FGuid Guid)
{
	TArray< FMovieSceneSpawnable*> Spawnables = FSequencerUtilities::ConvertToSpawnable(AsShared(), Guid);
	TArray<FGuid> SpawnableGuids;
	if (Spawnables.Num() > 0)
	{
		for (FMovieSceneSpawnable* Spawnable: Spawnables)
		{
			FGuid NewGuid = Spawnable->GetGuid();
			SpawnableGuids.Add(NewGuid);
		}
	}
	NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	return SpawnableGuids;
}

void FSequencer::ConvertSelectedNodesToSpawnables()
{
	using namespace UE::Sequencer;

	UMovieScene* MovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();

	if (MovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	// @todo sequencer: Undo doesn't seem to be working at all
	const FScopedTransaction Transaction( LOCTEXT("ConvertSelectedNodesSpawnable", "Convert Selected Nodes to Spawnables") );

	// Ensure we're in a non-possessed state
	TGuardValue<bool> Guard(bUpdatingExternalSelection, true);
	RestorePreAnimatedState();
	MovieScene->Modify();

	TArray<IObjectBindingExtension*> ObjectBindingNodes;

	for (TViewModelPtr<IObjectBindingExtension> ObjectBindingNode : ViewModel->GetSelection()->Outliner.Filter<IObjectBindingExtension>())
	{
		// If we have a possessable for this node, and it has no parent, we can convert it to a spawnable
		FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectBindingNode->GetObjectGuid());
		if (Possessable && !Possessable->GetParent().IsValid())
		{
			ObjectBindingNodes.Add(ObjectBindingNode.Get());
		}
	}

	FScopedSlowTask SlowTask(ObjectBindingNodes.Num(), LOCTEXT("ConvertSpawnableProgress", "Converting Selected Possessable Nodes to Spawnables"));
	SlowTask.MakeDialog(true);

	TArray<AActor*> SpawnedActors;
	for (IObjectBindingExtension* ObjectBindingNode : ObjectBindingNodes)
	{
		SlowTask.EnterProgressFrame();
	
		FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectBindingNode->GetObjectGuid());
		if (Possessable)
		{
			TArray<FGuid> ConvertedGuids;

			if (IsLevelEditorSequencer())
			{
				if (FMovieScenePossessable* Spawnable = FSequencerUtilities::ConvertToCustomBinding(AsShared(), Possessable->GetGuid(), UMovieSceneSpawnableActorBinding::StaticClass()))
				{
					ConvertedGuids.Add(Spawnable->GetGuid());
				}
			}
			else
			{
				TArray<FMovieSceneSpawnable*> Spawnables = FSequencerUtilities::ConvertToSpawnable(AsShared(), Possessable->GetGuid());
				for (FMovieSceneSpawnable* Spawnable : Spawnables)
				{
					ConvertedGuids.Add(Spawnable->GetGuid());
				}
			}

			for (FGuid Guid : ConvertedGuids)
			{
				for (TWeakObjectPtr<> WeakObject : FindBoundObjects(Guid, GetFocusedTemplateID()))
				{
					if (AActor* SpawnedActor = Cast<AActor>(WeakObject.Get()))
					{
						SpawnedActors.Add(SpawnedActor);
					}
				}
			}
		}

		if (GWarn->ReceivedUserCancel())
		{
			break;
		}
	}

	if (SpawnedActors.Num())
	{
		const bool bNotifySelectionChanged = true;
		const bool bDeselectBSP = true;
		const bool bWarnAboutTooManyActors = false;
		const bool bSelectEvenIfHidden = false;

		GEditor->GetSelectedActors()->Modify();
		GEditor->GetSelectedActors()->BeginBatchSelectOperation();
		GEditor->SelectNone(bNotifySelectionChanged, bDeselectBSP, bWarnAboutTooManyActors);
		for (auto SpawnedActor : SpawnedActors)
		{
			GEditor->SelectActor(SpawnedActor, true, bNotifySelectionChanged, bSelectEvenIfHidden);
		}
		GEditor->GetSelectedActors()->EndBatchSelectOperation();
		GEditor->NoteSelectionChange();
	}

	NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemsChanged );
}

void FSequencer::ConvertToPossessable(TSharedRef<UE::Sequencer::FObjectBindingModel> NodeToBeConverted)
{
	using namespace UE::Sequencer;

	if (GetFocusedMovieSceneSequence()->GetMovieScene()->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	const FScopedTransaction Transaction( LOCTEXT("ConvertSelectedNodePossessable", "Convert Node to Possessables") );

	// Ensure we're in a non-possessed state
	TGuardValue<bool> Guard(bUpdatingExternalSelection, true);
	RestorePreAnimatedState();
	GetFocusedMovieSceneSequence()->GetMovieScene()->Modify();
	FMovieSceneSpawnable* Spawnable = GetFocusedMovieSceneSequence()->GetMovieScene()->FindSpawnable(NodeToBeConverted->GetObjectGuid());
	if (Spawnable)
	{
		FSequencerUtilities::ConvertToPossessable(AsShared(), Spawnable->GetGuid());
		NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	}
}

void FSequencer::ConvertSelectedNodesToPossessables()
{
	using namespace UE::Sequencer;

	UMovieScene* MovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();

	if (MovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	TArray<IObjectBindingExtension*> ObjectBindingNodes;

	for (TViewModelPtr<IObjectBindingExtension> ObjectBindingNode : ViewModel->GetSelection()->Outliner.Filter<IObjectBindingExtension>())
	{
		if (SpawnRegister->CanConvertToPossessable(ObjectBindingNode->GetObjectGuid(), GetFocusedTemplateID(), GetSharedPlaybackState()))
		{
			ObjectBindingNodes.Add(ObjectBindingNode.Get());
		}
	}

	if (ObjectBindingNodes.Num() > 0)
	{
		const FScopedTransaction Transaction(LOCTEXT("ConvertSelectedNodesPossessable", "Convert Selected Nodes to Possessables"));
		MovieScene->Modify();

		FScopedSlowTask SlowTask(ObjectBindingNodes.Num(), LOCTEXT("ConvertPossessablesProgress", "Converting Selected Spawnable Nodes to Possessables"));
		SlowTask.MakeDialog(true);

		TArray<AActor*> PossessedActors;
		for (IObjectBindingExtension* ObjectBindingNode : ObjectBindingNodes)
		{
			SlowTask.EnterProgressFrame();

			FMovieScenePossessable* Possessable = FSequencerUtilities::ConvertToPossessable(AsShared(), ObjectBindingNode->GetObjectGuid());

			ForceEvaluate();

			for (TWeakObjectPtr<> WeakObject : FindBoundObjects(Possessable->GetGuid(), ActiveTemplateIDs.Top()))
			{
				if (AActor* PossessedActor = Cast<AActor>(WeakObject.Get()))
				{
					PossessedActors.Add(PossessedActor);
				}
			}

			if (GWarn->ReceivedUserCancel())
			{
				break;
			}
		}

		if (PossessedActors.Num())
		{
			const bool bNotifySelectionChanged = true;
			const bool bDeselectBSP = true;
			const bool bWarnAboutTooManyActors = false;
			const bool bSelectEvenIfHidden = false;

			GEditor->GetSelectedActors()->Modify();
			GEditor->GetSelectedActors()->BeginBatchSelectOperation();
			GEditor->SelectNone(bNotifySelectionChanged, bDeselectBSP, bWarnAboutTooManyActors);
			for (auto PossessedActor : PossessedActors)
			{
				GEditor->SelectActor(PossessedActor, true, bNotifySelectionChanged, bSelectEvenIfHidden);
			}
			GEditor->GetSelectedActors()->EndBatchSelectOperation();
			GEditor->NoteSelectionChange();

			NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
		}
	}
}


void FSequencer::OnLoadRecordedData()
{
	UMovieSceneSequence* FocusedMovieSceneSequence = GetFocusedMovieSceneSequence();
	if (!FocusedMovieSceneSequence)
	{
		return;
	}
	UMovieScene* FocusedMovieScene = FocusedMovieSceneSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}
	if (FocusedMovieScene->IsReadOnly())
	{
		return;
	}
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpen = false;
	if (DesktopPlatform)
	{
		FString FileTypeDescription = TEXT("");
		FString DialogTitle = TEXT("Open Recorded Sequencer Data");
		FString InOpenDirectory = FPaths::ProjectSavedDir();
		bOpen = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			DialogTitle,
			InOpenDirectory,
			TEXT(""),
			FileTypeDescription,
			EFileDialogFlags::None,
			OpenFilenames
		);
	}

	if (!bOpen || !OpenFilenames.Num())
	{
		return;
	}
	IModularFeatures& ModularFeatures = IModularFeatures::Get();

	if (ModularFeatures.IsModularFeatureAvailable(ISerializedRecorder::ModularFeatureName))
	{
		ISerializedRecorder* Recorder = &IModularFeatures::Get().GetModularFeature<ISerializedRecorder>(ISerializedRecorder::ModularFeatureName);
		if (Recorder)
		{
			FScopedTransaction AddFolderTransaction(NSLOCTEXT("Sequencer", "LoadRecordedData_Transaction", "Load Recorded Data"));
			auto OnReadComplete = [this]()
			{
				NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);

			}; //callback
			UObject* PlaybackContext = GetPlaybackContext();
			UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
			for (const FString& FileName : OpenFilenames)
			{
				Recorder->LoadRecordedSequencerFile(FocusedMovieSceneSequence, World, FileName, OnReadComplete);
			}
		}
	}

}

void FSequencer::AddFolder()
{
	using namespace UE::Sequencer;

	UMovieScene* FocusedMovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	FScopedTransaction AddFolderTransaction( NSLOCTEXT("Sequencer", "AddFolder_Transaction", "Add Folder") );

	// Check if a folder, or child of a folder is currently selected.
	TArray<UMovieSceneFolder*> SelectedParentFolders;
	FString NewNodePath;
	CalculateSelectedFolderAndPath(SelectedParentFolders, NewNodePath);

	TArray<FName> ExistingFolderNames;
	
	// If there is a folder selected the existing folder names are the sibling folders.
	if ( SelectedParentFolders.Num() == 1 )
	{
		for ( UMovieSceneFolder* SiblingFolder : SelectedParentFolders[0]->GetChildFolders() )
		{
			ExistingFolderNames.Add( SiblingFolder->GetFolderName() );
		}
	}
	// Otherwise use the root folders.
	else
	{
		for ( UMovieSceneFolder* MovieSceneFolder : FocusedMovieScene->GetRootFolders() )
		{
			ExistingFolderNames.Add( MovieSceneFolder->GetFolderName() );
		}
	}

	FName UniqueName = FSequencerUtilities::GetUniqueName(FName("New Folder"), ExistingFolderNames);
	UMovieSceneFolder* NewFolder = NewObject<UMovieSceneFolder>( FocusedMovieScene, NAME_None, RF_Transactional );
	NewFolder->SetFolderName( UniqueName );

	// The folder's name is used as it's key in the path system.
	NewNodePath += UniqueName.ToString();

	if ( SelectedParentFolders.Num() == 1 )
	{
		SelectedParentFolders[0]->AddChildFolder( NewFolder );
	}
	else
	{
		FocusedMovieScene->Modify();
		FocusedMovieScene->AddRootFolder( NewFolder );
	}

	ViewModel->GetSelection()->Empty();

	// We can't add the newly created folder to the selection set as the nodes for it don't actually exist yet.
	// However, we can calculate the resulting path that the node will end up at and add that to the selection
	// set, which will cause the newly created node to be selected when the selection is restored post-refresh.
	SequencerWidget->AddAdditionalPathToSelectionSet(NewNodePath);

	SequencerWidget->RequestRenameNode(NewNodePath);

	NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::MovieSceneStructureItemAdded );
}

void FixSortingOrders(UMovieSceneTrack* InTrack, const TArray<UMovieSceneTrack*>& Tracks)
{
	// Fix the sorting orders if InTrack is going to conflict with any of the given Tracks (ie. same class and same name)
	TOptional<int32> MaxSortingOrder;
	for (UMovieSceneTrack* Track : Tracks)
	{
		if (Track != InTrack && Track->GetClass() == InTrack->GetClass() && Track->GetDisplayName().EqualTo(InTrack->GetDisplayName()))
		{
			if (MaxSortingOrder.IsSet())
			{
				MaxSortingOrder = FMath::Max(MaxSortingOrder.GetValue(), Track->GetSortingOrder());
			}
			else
			{
				MaxSortingOrder = Track->GetSortingOrder();
			}
		}
	}

	if (MaxSortingOrder.IsSet())
	{
		InTrack->Modify();
		InTrack->SetSortingOrder(MaxSortingOrder.GetValue() + 1);
	}
}

void FixSortingOrders(FMovieSceneBinding* InBinding, const UMovieScene* MovieScene)
{
	// Fix the sorting orders if InBinding is going to conflict with any of the existing bindings (ie. same class and same name)
	TOptional<int32> MaxSortingOrder;
	FText InBindingName = MovieScene->GetObjectDisplayName(InBinding->GetObjectGuid());
	for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
	{
		if (&Binding != InBinding && MovieScene->GetObjectDisplayName(Binding.GetObjectGuid()).EqualTo(InBindingName))
		{
			if (MaxSortingOrder.IsSet())
			{
				MaxSortingOrder = FMath::Max(MaxSortingOrder.GetValue(), Binding.GetSortingOrder());
			}
			else
			{
				MaxSortingOrder = Binding.GetSortingOrder();
			}
		}
	}

	if (MaxSortingOrder.IsSet())
	{
		InBinding->SetSortingOrder(MaxSortingOrder.GetValue() + 1);
	}
}

void FSequencer::OnAddBinding(const FGuid& ObjectBinding, UMovieScene* MovieScene)
{
	using namespace UE::Sequencer;

	FMovieSceneBinding* Binding = MovieScene->FindBinding(ObjectBinding);
	if (Binding)
	{
		FixSortingOrders(Binding, MovieScene);
	}

	NodeTree->SortAllNodesAndDescendants();

	const TSharedPtr<FSequencerFilterBar> FilterBar = GetFilterBar();
	if (FilterBar->HasIsolatedTracks())
	{
		for (const TViewModelPtr<IObjectBindingExtension>& ObjectBindingItem : NodeTree->GetRootNode()->GetDescendantsOfType<IObjectBindingExtension>())
		{
			if (ObjectBinding == ObjectBindingItem->GetObjectGuid())
			{
				const FString NewNodePath = IOutlinerExtension::GetPathName(ObjectBindingItem);
				SequencerWidget->AddNewNodePathsToIsolate({ NewNodePath });
				break;
			}
		}
	}
}

void FSequencer::OnAddTrack(const TWeakObjectPtr<UMovieSceneTrack>&InTrack, const FGuid & ObjectBinding)
{
	if (!ensureAlwaysMsgf(InTrack.IsValid(), TEXT("Attempted to add a null UMovieSceneTrack to Sequencer. This should never happen.")))
	{
		return;
	}

	UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = Sequence->GetMovieScene();

	FMovieSceneBinding* Binding = MovieScene->FindBinding(ObjectBinding);
	if (Binding)
	{
		FixSortingOrders(InTrack.Get(), Binding->GetTracks());
	}
	else if (MovieScene->ContainsTrack(*InTrack))
	{	
		FixSortingOrders(InTrack.Get(), MovieScene->GetTracks());
	}

	FString NewNodePath;

	// If they specified an object binding it's being added to, we don't add it to a folder since we can't have it existing
	// as a children of two places at once.
	if(!Binding)
	{
		TArray<UMovieSceneFolder*> SelectedParentFolders;
		CalculateSelectedFolderAndPath(SelectedParentFolders, NewNodePath);

		if (SelectedParentFolders.Num() == 1)
		{
			SelectedParentFolders[0]->Modify();
			SelectedParentFolders[0]->AddChildTrack(InTrack.Get());
		}
	}

	// We can't add the newly created folder to the selection set as the nodes for it don't actually exist yet.
	// However, we can calculate the resulting path that the node will end up at and add that to the selection
	// set, which will cause the newly created node to be selected when the selection is restored post-refresh.
	NewNodePath += InTrack->GetFName().ToString();
	SequencerWidget->AddAdditionalPathToSelectionSet(NewNodePath);

	NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	EmptySelection();
	if (InTrack->GetAllSections().Num() > 0)
	{
		SelectSection(InTrack->GetAllSections()[0]);
	}
	ThrobSectionSelection();

	NodeTree->SortAllNodesAndDescendants();

	const TSharedPtr<FSequencerFilterBar> FilterBar = GetFilterBar();
	if (FilterBar->HasIsolatedTracks())
	{
		SequencerWidget->AddNewNodePathsToIsolate({ NewNodePath });
	}
}


void FSequencer::CalculateSelectedFolderAndPath(TArray<UMovieSceneFolder*>& OutSelectedParentFolders, FString& OutNewNodePath)
{
	using namespace UE::Sequencer;

	// Check if a folder, or child of a folder is currently selected.
	for (FViewModelPtr CurrentNode : ViewModel->GetSelection()->Outliner)
	{
		while (CurrentNode && !CurrentNode->IsA<FFolderModel>())
		{
			CurrentNode = CurrentNode->GetParent();
		}
		if (CurrentNode)
		{
			UMovieSceneFolder* Folder = CurrentNode->CastThisChecked<FFolderModel>()->GetFolder();
			if (Folder && !OutSelectedParentFolders.Contains(Folder))
			{
				OutSelectedParentFolders.Add(Folder);
			}

			// The first valid folder we find will be used to put the new folder into, so it's the node that we
			// want to know the path from.
			if (OutNewNodePath.Len() == 0)
			{
				// Add an extra delimiter (".") as we know that the new folder will be appended onto the end of this.
				OutNewNodePath = FString::Printf(TEXT("%s."), *IOutlinerExtension::GetPathName(CurrentNode));

				// Make sure this folder is expanded too so that adding objects to hidden folders become visible.
				CurrentNode->CastThisChecked<IOutlinerExtension>()->SetExpansion(true);
			}
		}
	}
}

void FSequencer::TogglePlay()
{
	OnPlay(true);
}

void FSequencer::JumpToStart()
{
	OnJumpToStart();
}

void FSequencer::JumpToEnd()
{
	OnJumpToEnd();
}

void FSequencer::RestorePlaybackSpeed()
{
	TArray<float> PlaybackSpeeds = GetPlaybackSpeeds.Execute();

	CurrentSpeedIndex = PlaybackSpeeds.Find(1.f);
	check(CurrentSpeedIndex != INDEX_NONE);
	
	PlaybackSpeed = PlaybackSpeeds[CurrentSpeedIndex];
	if (PlaybackState != EMovieScenePlayerStatus::Playing)
	{
		OnPlayForward(false);
	}
}

void FSequencer::ShuttleForward()
{
	TArray<float> PlaybackSpeeds = GetPlaybackSpeeds.Execute();

	float CurrentSpeed = GetPlaybackSpeed();

	int32 Sign = 0;
	if(PlaybackState == EMovieScenePlayerStatus::Playing)
	{
		// if we are at positive speed, increase the positive speed
		if (CurrentSpeed > 0)
		{
			CurrentSpeedIndex = FMath::Min(PlaybackSpeeds.Num() - 1, ++CurrentSpeedIndex);
			Sign = 1;
		}
		else if (CurrentSpeed < 0)
		{
			// if we are at the negative slowest speed, turn to positive slowest speed
			if (CurrentSpeedIndex == 0)
			{
				Sign = 1;
			}
			// otherwise, just reduce negative speed
			else
			{
				CurrentSpeedIndex = FMath::Max(0, --CurrentSpeedIndex);
				Sign = -1;
			}
		}		
	}
	else
	{
		Sign = 1;
		CurrentSpeedIndex = PlaybackSpeeds.Find(1);
	}

	PlaybackSpeed = PlaybackSpeeds[CurrentSpeedIndex] * Sign;

	if (PlaybackState != EMovieScenePlayerStatus::Playing)
	{
		OnPlayForward(false);
	}
}

void FSequencer::ShuttleBackward()
{
	TArray<float> PlaybackSpeeds = GetPlaybackSpeeds.Execute();

	float CurrentSpeed = GetPlaybackSpeed();

	int32 Sign = 0;
	if(PlaybackState == EMovieScenePlayerStatus::Playing)
	{
		if (CurrentSpeed > 0)
		{
			// if we are at the positive slowest speed, turn to negative slowest speed
			if (CurrentSpeedIndex == 0)
			{
				Sign = -1;
			}
			// otherwise, just reduce positive speed
			else
			{
				CurrentSpeedIndex = FMath::Max(0, --CurrentSpeedIndex);
				Sign = 1;
			}
		}
		// if we are at negative speed, increase the negative speed
		else if (CurrentSpeed < 0)
		{
			CurrentSpeedIndex = FMath::Min(PlaybackSpeeds.Num() - 1, ++CurrentSpeedIndex);
			Sign = -1;
		}
	}
	else
	{
		Sign = -1;
		CurrentSpeedIndex = PlaybackSpeeds.Find(1);
	}

	PlaybackSpeed = PlaybackSpeeds[CurrentSpeedIndex] * Sign;

	if (PlaybackState != EMovieScenePlayerStatus::Playing)
	{
		OnPlayBackward(false);
	}
}

int32 FSequencer::FindClosestPlaybackSpeed(float InPlaybackSpeed, bool bExactOnly) const
{
	TArray<float> PlaybackSpeeds = GetPlaybackSpeeds.Execute();

	int32 NewSpeedIndex = INDEX_NONE;
	float Delta = TNumericLimits<float>::Max();

	for (int32 Idx = 0; Idx < PlaybackSpeeds.Num(); Idx++)
	{
		float NewDelta = FMath::Abs(InPlaybackSpeed - PlaybackSpeeds[Idx]);
		if (NewDelta < Delta)
		{
			Delta = NewDelta;
			NewSpeedIndex = Idx;
		}
	}

	if (bExactOnly && Delta > UE_KINDA_SMALL_NUMBER)
	{
		return INDEX_NONE;
	}

	return NewSpeedIndex;
}

void FSequencer::SnapToClosestPlaybackSpeed()
{
	float CurrentSpeed = GetPlaybackSpeed();
	int32 NewSpeedIndex = FindClosestPlaybackSpeed(CurrentSpeed);

	if (NewSpeedIndex != INDEX_NONE)
	{
		TArray<float> PlaybackSpeeds = GetPlaybackSpeeds.Execute();

		CurrentSpeedIndex = NewSpeedIndex;
		PlaybackSpeed = PlaybackSpeeds[NewSpeedIndex];
	}	
}

void FSequencer::Pause()
{
	SetPlaybackStatus(EMovieScenePlayerStatus::Stopped);

	// When stopping a sequence, we always evaluate a non-empty range if possible. This ensures accurate paused motion blur effects.
	if (Settings->GetForceWholeFrames())
	{
		FQualifiedFrameTime LocalTime          = GetUnwarpedLocalTime();
		FFrameRate          FocusedDisplayRate = GetFocusedDisplayRate();

		// Snap to the focused play rate
		TOptional<FFrameTime> RootPosition  = RootToUnwarpedLocalTransform.Inverse().TryTransformTime(
			FFrameRate::Snap(LocalTime.Time, LocalTime.Rate, FocusedDisplayRate), CurrentTimeBreadcrumbs);

		if (RootPosition)
		{
			// Convert the root position from tick resolution time base (the output rate), to the play position input rate
			FFrameTime InputPosition = ConvertFrameTime(RootPosition.GetValue(), PlayPosition.GetOutputRate(), PlayPosition.GetInputRate());
			EvaluateInternal(PlayPosition.PlayTo(InputPosition));
		}
	}
	else
	{
		// Update on stop (cleans up things like sounds that are playing)
		FMovieSceneEvaluationRange Range = PlayPosition.GetLastRange().Get(PlayPosition.GetCurrentPositionAsRange());
		EvaluateInternal(Range);
	}

	RestorePlaybackSpeedAfterPlay();

	OnStopDelegate.Broadcast();
}

void FSequencer::StepForward()
{
	OnStepForward();
}


void FSequencer::StepBackward()
{
	OnStepBackward();
}

void FSequencer::JumpForward()
{
	OnStepForward(Settings->GetJumpFrameIncrement());
}

void FSequencer::JumpBackward()
{
	OnStepBackward(Settings->GetJumpFrameIncrement());
}

void FSequencer::StepToNextShot()
{
	if (ActiveTemplateIDs.Num() < 2)
	{
		UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
		UMovieScene* MovieScene = Sequence->GetMovieScene();

		UMovieSceneTrack* CinematicShotTrack = MovieScene->FindTrack(UMovieSceneCinematicShotTrack::StaticClass());
		if (!CinematicShotTrack)
		{
			return;
		}

		UMovieSceneSection* TargetShotSection = MovieSceneHelpers::FindNextSection(CinematicShotTrack->GetAllSections(), GetLocalTime().Time.FloorToFrame());

		if (TargetShotSection)
		{
			SetLocalTime(TargetShotSection->GetRange().GetLowerBoundValue(), ESnapTimeMode::STM_None);
		}
		return;
	}

	FMovieSceneSequenceID OuterSequenceID = ActiveTemplateIDs[ActiveTemplateIDs.Num() - 2];
	UMovieSceneSequence* Sequence = RootTemplateInstance.GetSequence(OuterSequenceID);

	FMovieSceneCompiledDataID DataID = CompiledDataManager->Compile(RootSequence.Get());
	const FMovieSceneSequenceHierarchy& Hierarchy = CompiledDataManager->GetHierarchyChecked(DataID);

	const FMovieSceneSubSequenceData*  SubData = Hierarchy.FindSubData(GetFocusedTemplateID());
	if (!SubData)
	{
		return;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();

	UMovieSceneTrack* CinematicShotTrack = MovieScene->FindTrack(UMovieSceneCinematicShotTrack::StaticClass());
	if (!CinematicShotTrack)
	{
		return;
	}

	TOptional<FFrameTime> CurrentTime = SubData->OuterToInnerTransform.Inverse().TryTransformTime(SubSequenceRange.GetLowerBoundValue(), CurrentTimeBreadcrumbs);
	if (!CurrentTime)
	{
		return;
	}

	UMovieSceneSubSection* NextShot = Cast<UMovieSceneSubSection>(MovieSceneHelpers::FindNextSection(CinematicShotTrack->GetAllSections(), CurrentTime->FloorToFrame()));
	if (!NextShot)
	{
		return;
	}

	SequencerWidget->PopBreadcrumb();

	PopToSequenceInstance(ActiveTemplateIDs[ActiveTemplateIDs.Num()-2]);
	FocusSequenceInstance(*NextShot);

	SetLocalTime(SubSequenceRange.GetLowerBoundValue(), ESnapTimeMode::STM_None);
}


void FSequencer::StepToPreviousShot()
{
	if (ActiveTemplateIDs.Num() < 2)
	{
		UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
		UMovieScene* MovieScene = Sequence->GetMovieScene();

		UMovieSceneTrack* CinematicShotTrack = MovieScene->FindTrack(UMovieSceneCinematicShotTrack::StaticClass());
		if (!CinematicShotTrack)
		{
			return;
		}

		UMovieSceneSection* TargetShotSection = MovieSceneHelpers::FindPreviousSection(CinematicShotTrack->GetAllSections(), GetLocalTime().Time.FloorToFrame());

		if (TargetShotSection)
		{
			SetLocalTime(TargetShotSection->GetRange().GetLowerBoundValue(), ESnapTimeMode::STM_None);
		}
		return;
	}

	FMovieSceneSequenceID OuterSequenceID = ActiveTemplateIDs[ActiveTemplateIDs.Num() - 2];
	UMovieSceneSequence* Sequence = RootTemplateInstance.GetSequence(OuterSequenceID);

	FMovieSceneCompiledDataID DataID = CompiledDataManager->Compile(RootSequence.Get());
	const FMovieSceneSequenceHierarchy& Hierarchy = CompiledDataManager->GetHierarchyChecked(DataID);

	const FMovieSceneSubSequenceData*  SubData = Hierarchy.FindSubData(GetFocusedTemplateID());
	if (!SubData)
	{
		return;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();

	UMovieSceneTrack* CinematicShotTrack = MovieScene->FindTrack(UMovieSceneCinematicShotTrack::StaticClass());
	if (!CinematicShotTrack)
	{
		return;
	}

	TOptional<FFrameTime> CurrentTime = SubData->OuterToInnerTransform.Inverse().TryTransformTime(SubSequenceRange.GetLowerBoundValue(), CurrentTimeBreadcrumbs);
	if (!CurrentTime)
	{
		return;
	}

	UMovieSceneSubSection* PreviousShot = Cast<UMovieSceneSubSection>(MovieSceneHelpers::FindPreviousSection(CinematicShotTrack->GetAllSections(), CurrentTime->FloorToFrame()));
	if (!PreviousShot)
	{
		return;
	}

	SequencerWidget->PopBreadcrumb();

	PopToSequenceInstance(ActiveTemplateIDs[ActiveTemplateIDs.Num()-2]);
	FocusSequenceInstance(*PreviousShot);

	SetLocalTime(SubSequenceRange.GetLowerBoundValue(), ESnapTimeMode::STM_None);
}

FReply FSequencer::NavigateForward()
{
	TArray<FMovieSceneSequenceID> TemplateIDForwardStackCopy = TemplateIDForwardStack;
	TArray<FMovieSceneSequenceID> TemplateIDBackwardStackCopy = TemplateIDBackwardStack;

	TemplateIDBackwardStackCopy.Push(ActiveTemplateIDs.Top());

	FMovieSceneSequenceID SequenceID = TemplateIDForwardStackCopy.Pop();
	if (SequenceID == MovieSceneSequenceID::Root)
	{
		PopToSequenceInstance(SequenceID);
	}
	else if (UMovieSceneSubSection* SubSection = FindSubSection(SequenceID))
	{
		FocusSequenceInstance(*SubSection);
	}

	TemplateIDForwardStack = TemplateIDForwardStackCopy;
	TemplateIDBackwardStack = TemplateIDBackwardStackCopy;

	SequencerWidget->UpdateBreadcrumbs();

	return FReply::Handled();
}

FReply FSequencer::NavigateBackward()
{
	TArray<FMovieSceneSequenceID> TemplateIDForwardStackCopy = TemplateIDForwardStack;
	TArray<FMovieSceneSequenceID> TemplateIDBackwardStackCopy = TemplateIDBackwardStack;

	TemplateIDForwardStackCopy.Push(ActiveTemplateIDs.Top());

	FMovieSceneSequenceID SequenceID = TemplateIDBackwardStackCopy.Pop();
	if (SequenceID == MovieSceneSequenceID::Root)
	{
		PopToSequenceInstance(SequenceID);
	}
	else if (UMovieSceneSubSection* SubSection = FindSubSection(SequenceID))
	{
		FocusSequenceInstance(*SubSection);
	}

	TemplateIDForwardStack = TemplateIDForwardStackCopy;
	TemplateIDBackwardStack = TemplateIDBackwardStackCopy;

	SequencerWidget->UpdateBreadcrumbs();
	return FReply::Handled();
}

bool FSequencer::CanNavigateForward() const
{
	return TemplateIDForwardStack.Num() > 0;
}

bool FSequencer::CanNavigateBackward() const
{
	return TemplateIDBackwardStack.Num() > 0;
}

FText FSequencer::GetNavigateForwardTooltip() const
{
	if (TemplateIDForwardStack.Num() > 0)
	{
		FMovieSceneSequenceID SequenceID = TemplateIDForwardStack.Last();

		if (SequenceID == MovieSceneSequenceID::Root)
		{
			if (GetRootMovieSceneSequence())
			{
				return FText::Format(LOCTEXT("NavigateForwardTooltipFmt", "Forward to {0}"), GetRootMovieSceneSequence()->GetDisplayName());
			}
		}
		else if (UMovieSceneSubSection* SubSection = FindSubSection(SequenceID))
		{
			if (SubSection->GetSequence())
			{
				return FText::Format(LOCTEXT("NavigateForwardTooltipFmt", "Forward to {0}"), SubSection->GetSequence()->GetDisplayName());
			}
		}
	}
	return FText::GetEmpty();
}

FText FSequencer::GetNavigateBackwardTooltip() const
{
	if (TemplateIDBackwardStack.Num() > 0)
	{
		FMovieSceneSequenceID SequenceID = TemplateIDBackwardStack.Last();

		if (SequenceID == MovieSceneSequenceID::Root)
		{
			if (GetRootMovieSceneSequence())
			{
				return FText::Format( LOCTEXT("NavigateBackwardTooltipFmt", "Back to {0}"), GetRootMovieSceneSequence()->GetDisplayName());
			}
		}
		else if (UMovieSceneSubSection* SubSection = FindSubSection(SequenceID))
		{
			if (SubSection->GetSequence())
			{
				return FText::Format(LOCTEXT("NavigateBackwardTooltipFmt", "Back to {0}"), SubSection->GetSequence()->GetDisplayName());
			}
		}
	}
	return FText::GetEmpty();
}

void FSequencer::SortAllNodesAndDescendants()
{
	FScopedTransaction SortAllNodesTransaction(NSLOCTEXT("Sequencer", "SortAllNodes_Transaction", "Sort Tracks"));
	NodeTree->ClearCustomSortOrders();
	NodeTree->SortAllNodesAndDescendants();
}

void FSequencer::ToggleExpandCollapseNodes()
{
	using namespace UE::Sequencer;
	SequencerWidget->GetTreeView()->ToggleExpandCollapseNodes(ETreeRecursion::NonRecursive);
}


void FSequencer::ToggleExpandCollapseNodesAndDescendants()
{
	using namespace UE::Sequencer;
	SequencerWidget->GetTreeView()->ToggleExpandCollapseNodes(ETreeRecursion::Recursive);
}

void FSequencer::ExpandAllNodes()
{
	using namespace UE::Sequencer;
	const bool bExpandAll = true;
	const bool bCollapseAll = false;
	SequencerWidget->GetTreeView()->ToggleExpandCollapseNodes(ETreeRecursion::Recursive, bExpandAll, bCollapseAll);
}

void FSequencer::CollapseAllNodes()
{
	using namespace UE::Sequencer;
	const bool bExpandAll = false;
	const bool bCollapseAll = true;
	SequencerWidget->GetTreeView()->ToggleExpandCollapseNodes(ETreeRecursion::Recursive, bExpandAll, bCollapseAll);
}

void FSequencer::AddSelectedActors()
{
	USelection* ActorSelection = GEditor->GetSelectedActors();
	TArray<TWeakObjectPtr<AActor> > SelectedActors;
	for (FSelectionIterator Iter(*ActorSelection); Iter; ++Iter)
	{
		AActor* Actor = Cast<AActor>(*Iter);
		if (Actor)
		{
			SelectedActors.Add(Actor);
		}
	}

	AddActors(SelectedActors);
}

void FSequencer::SetKey()
{
	const TSharedPtr<UE::Sequencer::SimpleView::FSimpleViewTimeline> Timeline = GetSimpleViewTimeline();
	if (Timeline.IsValid() && Timeline->IsInSimpleView())
	{
		Timeline->ExecuteSetKeyCommand();
		return;
	}

	if (ViewModel->GetSelection()->Outliner.Num() > 0)
	{
		using namespace UE::Sequencer;

		FScopedTransaction SetKeyTransaction( NSLOCTEXT("Sequencer", "SetKey_Transaction", "Set Key") );

		const FFrameNumber KeyTime = GetLocalTime().Time.FrameNumber;

		FAddKeyOperation::FromNodes(ViewModel->GetSelection()->Outliner.GetSelected()).Commit(KeyTime, *this);
	}
}


bool FSequencer::CanSetKeyTime() const
{
	return ViewModel->GetSelection()->KeySelection.Num() > 0;
}


void FSequencer::SetKeyTime()
{
	using namespace UE::Sequencer;

	FFrameNumber KeyTime = 0;

	const FKeySelection& KeySelection = ViewModel->GetSelection()->KeySelection;
	for (FKeyHandle Key : KeySelection)
	{
		TSharedPtr<FChannelModel> Channel = KeySelection.GetModelForKey(Key);
		if (Channel)
		{
			KeyTime = Channel->GetKeyArea()->GetKeyTime(Key);
			break;
		}
	}

	// Create a popup showing the existing time value and let the user set a new one.
 	GenericTextEntryModeless(NSLOCTEXT("Sequencer.Popups", "SetKeyTimePopup", "New Time"), FText::FromString(GetNumericTypeInterface()->ToString(KeyTime.Value)),
 		FOnTextCommitted::CreateSP(this, &FSequencer::OnSetKeyTimeTextCommitted)
 	);
}


void FSequencer::OnSetKeyTimeTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	using namespace UE::Sequencer;

	bool bAnythingChanged = false;

	CloseEntryPopupMenu();
	if (CommitInfo == ETextCommit::OnEnter)
	{
		TOptional<double> NewFrameTime = GetNumericTypeInterface()->FromString(InText.ToString(), 0);
		if (!NewFrameTime.IsSet())
			return;

		FFrameNumber NewFrame = FFrameNumber((int32)NewFrameTime.GetValue());

		FScopedTransaction SetKeyTimeTransaction(NSLOCTEXT("Sequencer", "SetKeyTime_Transaction", "Set Key Time"));
		const FKeySelection& KeySelection = ViewModel->GetSelection()->KeySelection;
		for (FKeyHandle Key : KeySelection)
		{
			TSharedPtr<FChannelModel> Channel = KeySelection.GetModelForKey(Key);
			UMovieSceneSection*       Section = Channel ? Channel->GetSection() : nullptr;
			if (Channel && Section)
			{
	 			if (Section->TryModify())
	 			{
	 				Channel->GetKeyArea()->SetKeyTime(Key, NewFrame);
	 				bAnythingChanged = true;

					Section->ExpandToFrame(NewFrame);
	 			}
			}
		}
	}

	if (bAnythingChanged)
	{
		NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}
}


bool FSequencer::CanRekey() const
{
	return ViewModel->GetSelection()->KeySelection.Num() > 0;
}


void FSequencer::Rekey()
{
	using namespace UE::Sequencer;

	bool bAnythingChanged = false;

	FQualifiedFrameTime CurrentTime = GetLocalTime();

	FScopedTransaction RekeyTransaction(NSLOCTEXT("Sequencer", "Rekey_Transaction", "Rekey"));
	const FKeySelection& KeySelection = ViewModel->GetSelection()->KeySelection;

	for (FKeyHandle Key : KeySelection)
	{
		TSharedPtr<FChannelModel> Channel = KeySelection.GetModelForKey(Key);
		UMovieSceneSection*       Section = Channel ? Channel->GetSection() : nullptr;

		if (Channel && Section)
		{
	 		if (Section->TryModify())
	 		{
	 			Channel->GetKeyArea()->SetKeyTime(Key, CurrentTime.Time.FrameNumber);
	 			bAnythingChanged = true;

				Section->ExpandToFrame(CurrentTime.Time.FrameNumber);
	 		}
		}
	}

	if (bAnythingChanged)
	{
		NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}
}

TSet<FFrameNumber> FSequencer::GetVerticalFrames() const
{
	TSet<FFrameNumber> VerticalFrames;

	auto AddVerticalFrames = [](auto &InVerticalFrames, auto InTrack) 
	{
		for (UMovieSceneSection* Section : InTrack->GetAllSections())
		{
			if (Section->GetRange().HasLowerBound())
			{
				InVerticalFrames.Add(Section->GetRange().GetLowerBoundValue());
			}

			if (Section->GetRange().HasUpperBound())
			{
				InVerticalFrames.Add(Section->GetRange().GetUpperBoundValue());
			}
		}
	};

	UMovieSceneSequence* FocusedMovieSequence = GetFocusedMovieSceneSequence();
	if (FocusedMovieSequence != nullptr)
	{
		UMovieScene* FocusedMovieScene = FocusedMovieSequence->GetMovieScene();
		if (FocusedMovieScene != nullptr)
		{
			for (UMovieSceneTrack* Track : FocusedMovieScene->GetTracks())
			{
				if (Track && Track->DisplayOptions.bShowVerticalFrames)
				{
					AddVerticalFrames(VerticalFrames, Track);
				}
			}

			if (UMovieSceneTrack* CameraCutTrack = FocusedMovieScene->GetCameraCutTrack())
			{
				if (CameraCutTrack->DisplayOptions.bShowVerticalFrames)
				{
					AddVerticalFrames(VerticalFrames, CameraCutTrack);
				}
			}
		}
	}

	return VerticalFrames;
}

TArray<FMovieSceneMarkedFrame> FSequencer::GetMarkedFrames() const
{
	UMovieSceneSequence* FocusedMovieSequence = GetFocusedMovieSceneSequence();
	if (FocusedMovieSequence != nullptr)
	{
		UMovieScene* FocusedMovieScene = FocusedMovieSequence->GetMovieScene();
		if (FocusedMovieScene != nullptr)
		{
			return FocusedMovieScene->GetMarkedFrames();
		}
	}

	return TArray<FMovieSceneMarkedFrame>();
}

TArray<FMovieSceneMarkedFrame> FSequencer::GetGlobalMarkedFrames() const
{
	return GlobalMarkedFramesCache;
}

void FSequencer::UpdateGlobalMarkedFramesCache()
{
	GlobalMarkedFramesCache.Empty();
	FSequencerMarkedFrameHelper::FindGlobalMarkedFrames(*this, GlobalMarkedFramesCache);
	bGlobalMarkedFramesCached = true;
}

void FSequencer::ToggleShowMarkedFramesGlobally()
{
	UMovieSceneSequence* FocusedMovieSequence = GetFocusedMovieSceneSequence();
	if (!FocusedMovieSequence)
	{
		return;
	}

	UMovieScene* FocusedMovieScene = FocusedMovieSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("ToggleShowMarkedFramesGlobally", "Toggle Show Marked Frames Globally"));
	FocusedMovieScene->Modify();
	FocusedMovieScene->ToggleGloballyShowMarkedFrames();
	InvalidateGlobalMarkedFramesCache();
}

void FSequencer::ClearGlobalMarkedFrames()
{
	FSequencerMarkedFrameHelper::ClearGlobalMarkedFrames(*this);

	bGlobalMarkedFramesCached = false;
}

void FSequencer::ToggleMarkAtPlayPosition()
{
	UMovieSceneSequence* FocusedMovieSequence = GetFocusedMovieSceneSequence();
	if (!FocusedMovieSequence)
	{
		return;
	}
	
	UMovieScene* FocusedMovieScene = FocusedMovieSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	FFrameNumber TickFrameNumber = GetLocalTime().Time.FloorToFrame();
	int32 MarkedFrameIndex = FocusedMovieScene->FindMarkedFrameByFrameNumber(TickFrameNumber);
	if (MarkedFrameIndex != INDEX_NONE)
	{
		FScopedTransaction DeleteMarkedFrameTransaction(LOCTEXT("DeleteMarkedFrames_Transaction", "Delete Marked Frame"));

		FocusedMovieScene->Modify();
		FocusedMovieScene->DeleteMarkedFrame(MarkedFrameIndex);

		NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	}
	else
	{
		FScopedTransaction AddMarkedFrameTransaction(LOCTEXT("AddMarkedFrame_Transaction", "Add Marked Frame"));

		FocusedMovieScene->Modify();
		FocusedMovieScene->AddMarkedFrame(FMovieSceneMarkedFrame(TickFrameNumber));

		NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	}

	ViewModel->GetSelection()->MarkedFrames.Empty();
}

void FSequencer::SetMarkedFrame(int32 InMarkIndex, FFrameNumber InFrameNumber)
{
	UMovieSceneSequence* FocusedMovieSequence = GetFocusedMovieSceneSequence();
	if (!FocusedMovieSequence)
	{
		return;
	}
	
	UMovieScene* FocusedMovieScene = FocusedMovieSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}
			
	FocusedMovieScene->Modify();
	FocusedMovieScene->SetMarkedFrame(InMarkIndex, InFrameNumber);
}

void FSequencer::AddMarkedFrame(FFrameNumber FrameNumber)
{
	UMovieSceneSequence* FocusedMovieSequence = GetFocusedMovieSceneSequence();
	if (!FocusedMovieSequence)
	{
		return;
	}
	
	UMovieScene* FocusedMovieScene = FocusedMovieSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	FScopedTransaction AddMarkedFrameTransaction(LOCTEXT("AddMarkedFrame_Transaction", "Add Marked Frame"));

	FocusedMovieScene->Modify();
	FocusedMovieScene->AddMarkedFrame(FMovieSceneMarkedFrame(FrameNumber));

	NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

void FSequencer::DeleteAllMarkedFrames()
{
	UMovieSceneSequence* FocusedMovieSequence = GetFocusedMovieSceneSequence();
	if (!FocusedMovieSequence)
	{
		return;
	}
	
	UMovieScene* FocusedMovieScene = FocusedMovieSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}
			
	FScopedTransaction DeleteAllMarkedFramesTransaction(LOCTEXT("DeleteAllMarkedFrames_Transaction", "Delete All Marked Frames"));

	FocusedMovieScene->Modify();
	FocusedMovieScene->DeleteMarkedFrames();

	ViewModel->GetSelection()->MarkedFrames.Empty();

	NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

void FSequencer::StepToNextMark()
{
	UMovieSceneSequence* FocusedMovieSequence = GetFocusedMovieSceneSequence();
	if (!FocusedMovieSequence)
	{
		return;
	}
	
	UMovieScene* FocusedMovieScene = FocusedMovieSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	FFrameNumber CurrentTime = GetLocalTime().Time.RoundToFrame();
	TOptional<FFrameNumber> NearestTime;

	const bool bForwards = true;
	int32 MarkedIndex = FocusedMovieScene->FindNextMarkedFrame(CurrentTime, bForwards);
	if (MarkedIndex != INDEX_NONE)
	{
		NearestTime = FocusedMovieScene->GetMarkedFrames()[MarkedIndex].FrameNumber.Value;
	}

	TArray<FMovieSceneMarkedFrame> GlobalMarkedFrames = GetGlobalMarkedFrames();
	for (const FMovieSceneMarkedFrame& GlobalMarkedFrame : GlobalMarkedFrames)
	{
		if (GlobalMarkedFrame.FrameNumber > CurrentTime)
		{
			if (!NearestTime.IsSet())
			{
				NearestTime = GlobalMarkedFrame.FrameNumber;
			}
			else if (FMath::Abs(GlobalMarkedFrame.FrameNumber - CurrentTime) < FMath::Abs(NearestTime.GetValue() - CurrentTime))
			{
				NearestTime = GlobalMarkedFrame.FrameNumber;
			}
		}
	}

	if (NearestTime.IsSet())
	{
		SetLocalTimeDirectly(NearestTime.GetValue());
	}
}

void FSequencer::StepToPreviousMark()
{
	UMovieSceneSequence* FocusedMovieSequence = GetFocusedMovieSceneSequence();
	if (!FocusedMovieSequence)
	{
		return;
	}
	
	UMovieScene* FocusedMovieScene = FocusedMovieSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}
	
	FFrameNumber CurrentTime = GetLocalTime().Time.FloorToFrame();
	TOptional<FFrameNumber> NearestTime;

	const bool bForwards = false;
	int32 MarkedIndex = FocusedMovieScene->FindNextMarkedFrame(CurrentTime, bForwards);
	if (MarkedIndex != INDEX_NONE)
	{
		NearestTime = FocusedMovieScene->GetMarkedFrames()[MarkedIndex].FrameNumber.Value;
	}

	TArray<FMovieSceneMarkedFrame> GlobalMarkedFrames = GetGlobalMarkedFrames();
	for (const FMovieSceneMarkedFrame& GlobalMarkedFrame : GlobalMarkedFrames)
	{
		if (GlobalMarkedFrame.FrameNumber < CurrentTime)
		{
			if (!NearestTime.IsSet())
			{
				NearestTime = GlobalMarkedFrame.FrameNumber;
			}
			else if (FMath::Abs(GlobalMarkedFrame.FrameNumber - CurrentTime) < FMath::Abs(NearestTime.GetValue() - CurrentTime))
			{
				NearestTime = GlobalMarkedFrame.FrameNumber;
			}
		}
	}

	if (NearestTime.IsSet())
	{
		SetLocalTimeDirectly(NearestTime.GetValue());
	}
}

bool FSequencer::AreMarkedFramesLocked() const
{
	if (IsReadOnly())
	{
		return true;
	}
	
	UMovieSceneSequence* FocusedMovieSceneSequence = GetFocusedMovieSceneSequence();
	if (FocusedMovieSceneSequence != nullptr)
	{
		UMovieScene* MovieScene = FocusedMovieSceneSequence->GetMovieScene();

		if (MovieScene->IsReadOnly())
		{
			return true;
		}
	
		return MovieScene->AreMarkedFramesLocked();
	}

	return false;
}

void FSequencer::ToggleMarkedFramesLocked()
{
	UMovieSceneSequence* FocusedMovieSceneSequence = GetFocusedMovieSceneSequence();
	if ( FocusedMovieSceneSequence != nullptr )
	{
		UMovieScene* MovieScene = FocusedMovieSceneSequence->GetMovieScene();

		if (MovieScene->IsReadOnly())
		{
			FSequencerUtilities::ShowReadOnlyError();
			return;
		}

		FScopedTransaction ToggleMarkedFramesLockTransaction( NSLOCTEXT( "Sequencer", "ToggleMarkedFramesLocked", "Toggle marked frames lock" ) );
		MovieScene->Modify();
		MovieScene->SetMarkedFramesLocked( !MovieScene->AreMarkedFramesLocked() );
	}
}

void GatherTracksAndObjectsToCopy(TSharedRef<UE::Sequencer::FViewModel> Node, TArray<TSharedPtr<UE::Sequencer::FViewModel>>& TracksToCopy, TArray<TSharedPtr<UE::Sequencer::FObjectBindingModel>>& ObjectsToCopy, TArray<UMovieSceneFolder*>& FoldersToCopy, TArray<UObject*>& DecorationsToCopy)
{
	using namespace UE::Sequencer;

	// Decoration rows take precedence over track interfaces a model may also implement.
	if (FDecorationOutlinerModel* DecoNode = Node->CastThis<FDecorationOutlinerModel>())
	{
		if (UObject* Decoration = DecoNode->GetDecoration())
		{
			DecorationsToCopy.AddUnique(Decoration);
		}
	}
	else if (ITrackExtension* TrackNode = Node->CastThis<ITrackExtension>())
	{
		if (!TracksToCopy.Contains(Node))
		{
			TracksToCopy.Add(Node);
		}
	}
	else if (ITrackRowExtension* TrackRowNode = Node->CastThis<ITrackRowExtension>())
	{
		if (!TracksToCopy.Contains(Node))
		{
			TracksToCopy.Add(Node);
		}
	}
	else if (TSharedPtr<FObjectBindingModel> ObjectNode = Node->CastThisShared<FObjectBindingModel>())
	{
		if (!ObjectsToCopy.Contains(ObjectNode))
		{
			ObjectsToCopy.Add(ObjectNode);
		}
	}
	else if (FFolderModel* FolderNode = Node->CastThis<FFolderModel>())
	{
		FoldersToCopy.Add(FolderNode->GetFolder());
	}
}

void FSequencer::CopySelection()
{
	using namespace UE::Sequencer;

	if (ViewModel->GetSelection()->KeySelection.Num() != 0)
	{
		CopySelectedKeys();
	}
	else if (ViewModel->GetSelection()->GetSelectedSections().Num() != 0)
	{
		CopySelectedSections();
	}
	else
	{
		TArray<TSharedPtr<FViewModel>> TracksToCopy;
		TArray<TSharedPtr<FObjectBindingModel>> ObjectsToCopy;
		TArray<UMovieSceneFolder*> FoldersToCopy;
		TArray<UObject*> DecorationsToCopy;

		for (FViewModelPtr Node : ViewModel->GetSelection()->Outliner)
		{
			GatherTracksAndObjectsToCopy(Node.AsModel().ToSharedRef(), TracksToCopy, ObjectsToCopy, FoldersToCopy, DecorationsToCopy);
		}

		// Make a empty clipboard if the stack is empty
		if (GClipboardStack.Num() == 0)
		{
			TSharedRef<FMovieSceneClipboard> NullClipboard = MakeShareable(new FMovieSceneClipboard());
			GClipboardStack.Push(NullClipboard);
		}

		FString ObjectsExportedText;
		FString TracksExportedText;
		FString TracksInFoldersExportedText;
		FString FoldersExportedText;
		FString DecorationsExportedText;

		if (ObjectsToCopy.Num())
		{
			CopySelectedObjects(ObjectsToCopy, FoldersToCopy, ObjectsExportedText);
		}

		if (TracksToCopy.Num())
		{
			CopySelectedTracks(TracksToCopy, FoldersToCopy, TracksExportedText);
		}

		if (FoldersToCopy.Num())
		{
			CopySelectedFolders(FoldersToCopy, FoldersExportedText, ObjectsExportedText, TracksInFoldersExportedText);
		}

		if (DecorationsToCopy.Num())
		{
			FSequencerUtilities::CopyDecorations(DecorationsToCopy, DecorationsExportedText);
			GClipboardStack.Empty();
		}

		FString ExportedText;
		ExportedText += ObjectsExportedText;
		ExportedText += TracksExportedText;
		ExportedText += FoldersExportedText;
		ExportedText += TracksInFoldersExportedText;
		ExportedText += DecorationsExportedText;

		FPlatformApplicationMisc::ClipboardCopy(*ExportedText);
	}
}

void FSequencer::CutSelection()
{
	if (ViewModel->GetSelection()->KeySelection.Num() != 0)
	{
		CutSelectedKeys();
	}
	else if (ViewModel->GetSelection()->GetSelectedSections().Num() != 0)
	{
		CutSelectedSections();
	}
	else
	{
		FScopedTransaction CutSelectionTransaction(LOCTEXT("CutSelection_Transaction", "Cut Selection"));
		CopySelection();
		DeleteSelectedItems();
	}
}

void FSequencer::DuplicateSelection()
{
	using namespace UE::Sequencer;

	FScopedTransaction DuplicateSelectionTransaction(LOCTEXT("DuplicateSelection_Transaction", "Duplicate Selection"));

	const bool bClearSelection = true;

	const FKeySelection& KeySelection = ViewModel->GetSelection()->KeySelection;

	if (KeySelection.Num() != 0)
	{
		CopySelection();
		DoPaste(bClearSelection);

		// Shift duplicated keys by one display rate frame as an overlapping key isn't useful

		// Offset by a visible amount
		FFrameNumber FrameOffset = FFrameNumber((int32)GetDisplayRateDeltaFrameCount());
		TSet<UMovieSceneSection*> ModifiedSections;
		FKeySelection NewSelection;
		for (FKeyHandle KeyHandle : ViewModel->GetSelection()->KeySelection)
		{
			TSharedPtr<FChannelModel> Channel = KeySelection.GetModelForKey(KeyHandle);
			if (Channel)
			{
				TSharedPtr<IKeyArea> KeyArea = Channel->GetKeyArea();
				UMovieSceneSection* Section = KeyArea->GetOwningSection();

				bool bModified = ModifiedSections.Contains(Section);
				if (!bModified)
				{
					bModified = Section->TryModify();
				}

				if (bModified)
				{
					ModifiedSections.Add(Section);

					FKeyHandle NewKeyHandle = KeyArea->DuplicateKey(KeyHandle);
					KeyArea->SetKeyTime(NewKeyHandle, KeyArea->GetKeyTime(KeyHandle) + FrameOffset);

					NewSelection.Select(Channel, NewKeyHandle);
				}
			}
		}

		ViewModel->GetSelection()->KeySelection.ReplaceWith(MoveTemp(NewSelection));
		NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}
	else if (ViewModel->GetSelection()->GetSelectedSections().Num() != 0)
	{
		CopySelection();
		DoPaste(bClearSelection);
	}
	else
	{
		CopySelection();
		DoPaste(bClearSelection);

		SynchronizeSequencerSelectionWithExternalSelection();
	}
}

void FSequencer::CopySelectedKeys()
{
	using namespace UE::Sequencer;

	FMovieSceneClipboardBuilder Builder;

	// Map selected keys to their key areas
	TMap<TSharedPtr<FChannelModel>, TArray<FKeyHandle>> KeyAreaMap;
	for (FKeyHandle Key : ViewModel->GetSelection()->KeySelection)
	{
		TSharedPtr<FChannelModel> Channel = ViewModel->GetSelection()->KeySelection.GetModelForKey(Key);
		if (Channel)
		{
			KeyAreaMap.FindOrAdd(Channel).Add(Key);
		}
	}

	// Serialize each key area to the clipboard
	for (const TPair<TSharedPtr<FChannelModel>, TArray<FKeyHandle>>& Pair : KeyAreaMap)
	{
		Pair.Key->GetKeyArea()->CopyKeys(Builder, Pair.Value);
	}

	TSharedRef<FMovieSceneClipboard> Clipboard = MakeShareable( new FMovieSceneClipboard(Builder.Commit(TOptional<FFrameNumber>())) );
	
	Clipboard->GetEnvironment().TickResolution = GetFocusedTickResolution();
	Clipboard->GetEnvironment().TimeTransform = GetFocusedMovieSceneSequenceTransform().LinearTransform;

	if (Clipboard->GetKeyTrackGroups().Num())
	{
		GClipboardStack.Push(Clipboard);

		if (GClipboardStack.Num() > 10)
		{
			GClipboardStack.RemoveAt(0, 1);
		}
	}

	// Make sure to clear the clipboard for the sections/tracks/bindings
	FPlatformApplicationMisc::ClipboardCopy(TEXT(""));
}

void FSequencer::CutSelectedKeys()
{
	FScopedTransaction CutSelectedKeysTransaction(LOCTEXT("CutSelectedKeys_Transaction", "Cut Selected keys"));
	CopySelectedKeys();
	DeleteSelectedKeys();
}


void FSequencer::CopySelectedSections()
{
	TArray<UMovieSceneSection*> SelectedSections;
	for (TWeakObjectPtr<UMovieSceneSection> SelectedSectionPtr : ViewModel->GetSelection()->GetSelectedSections())
	{
		if (SelectedSectionPtr.IsValid())
		{
			SelectedSections.Add(SelectedSectionPtr.Get());
		}
	}

	FString ExportedText;
	FSequencerUtilities::CopySections(SelectedSections, ExportedText);

	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);

	// Make sure to clear the clipboard for the keys
	GClipboardStack.Empty();
}

void FSequencer::CutSelectedSections()
{
	FScopedTransaction CutSelectedSectionsTransaction(LOCTEXT("CutSelectedSections_Transaction", "Cut Selected sections"));
	CopySelectedSections();
	DeleteSections(ViewModel->GetSelection()->GetSelectedSections());
}


const TArray<TSharedPtr<FMovieSceneClipboard>>& FSequencer::GetClipboardStack() const
{
	return GClipboardStack;
}


void FSequencer::OnClipboardUsed(TSharedPtr<FMovieSceneClipboard> Clipboard)
{
	Clipboard->GetEnvironment().DateTime = FDateTime::UtcNow();

	// Last entry in the stack should be the most up-to-date
	GClipboardStack.Sort([](const TSharedPtr<FMovieSceneClipboard>& A, const TSharedPtr<FMovieSceneClipboard>& B){
		return A->GetEnvironment().DateTime < B->GetEnvironment().DateTime;
	});
}

void FSequencer::FixPossessableObjectClassInternal(UMovieSceneSequence* Sequence, FMovieSceneSequenceIDRef SequenceID)
{
	UMovieScene* MovieScene = Sequence->GetMovieScene();

	MovieScene->Modify();

	for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
	{
		FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);

		TOptional<UClass*> CommonBaseClass;

		for (TWeakObjectPtr<> WeakObject : FindBoundObjects(Possessable.GetGuid(), SequenceID))
		{
			if (WeakObject.IsValid())
			{
				if (!CommonBaseClass.IsSet())
				{
					CommonBaseClass = WeakObject->GetClass();
				}
				else
				{
					CommonBaseClass = UClass::FindCommonBase(WeakObject->GetClass(), CommonBaseClass.GetValue());
				}
			}
		}

		if (CommonBaseClass.IsSet() && CommonBaseClass.GetValue() != Possessable.GetPossessedObjectClass())
		{
			UE_LOGF(LogSequencer, Display, "Updated possessed object class in: %ls for: %ls, from: %ls, to: %ls", *GetNameSafe(Sequence), *Possessable.GetName(), *GetNameSafe(Possessable.GetPossessedObjectClass()), *GetNameSafe(CommonBaseClass.GetValue()));
			Possessable.SetPossessedObjectClass(CommonBaseClass.GetValue());
		}
	}
}

void FSequencer::FixPossessableObjectClass()
{
	FScopedTransaction FixPossessableObjectClassTransaction( NSLOCTEXT( "Sequencer", "FixPossessableObjectClass", "Fix Possessable Object Class" ) );

	FMovieSceneRootEvaluationTemplateInstance& RootTemplate = GetEvaluationTemplate();
		
	UMovieSceneSequence* Sequence = RootTemplate.GetSequence(MovieSceneSequenceID::Root);

	FixPossessableObjectClassInternal(Sequence, MovieSceneSequenceID::Root);

	const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(RootTemplateInstance.GetCompiledDataID());
	if (Hierarchy)
	{
		FMovieSceneEvaluationTreeRangeIterator Iter = Hierarchy->GetTree().IterateFromTime(PlayPosition.GetCurrentPosition().FrameNumber);

		for (const FMovieSceneSubSequenceTreeEntry& Entry : Hierarchy->GetTree().GetAllData(Iter.Node()))
		{
			UMovieSceneSequence* SubSequence = Hierarchy->FindSubSequence(Entry.SequenceID);
			if (SubSequence)
			{
				FixPossessableObjectClassInternal(SubSequence, Entry.SequenceID);
			}
		}
	}
}

void FSequencer::RebindPossessableReferences()
{
	UMovieSceneSequence* FocusedSequence = GetFocusedMovieSceneSequence();
	UMovieScene* FocusedMovieScene = FocusedSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("RebindAllPossessables", "Rebind Possessable References"));

	FocusedSequence->Modify();

	TMap<FGuid, TArray<UObject*, TInlineAllocator<1>>> AllObjects;

	UObject* PlaybackContext = PlaybackContextAttribute.Get(nullptr); 
	const FMovieSceneBindingReferences* Refs = FocusedSequence->GetBindingReferences();

	for (int32 Index = 0; Index < FocusedMovieScene->GetPossessableCount(); Index++)
	{
		const FMovieScenePossessable& Possessable = FocusedMovieScene->GetPossessable(Index);

		// Skip custom bindings here
		if (Refs && Refs->GetCustomBinding(Possessable.GetGuid(), 0))
		{
			continue;
		}

		TArray<UObject*, TInlineAllocator<1>>& References = AllObjects.FindOrAdd(Possessable.GetGuid());
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TArrayView<TWeakObjectPtr<>> BoundObjects = State.FindBoundObjects(Possessable.GetGuid(), GetFocusedTemplateID(), GetSharedPlaybackState());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		for (TWeakObjectPtr<> WeakObj : BoundObjects)
		{
			if (UObject* Obj = WeakObj.Get())
			{
				References.Add(Obj);
			}
		}
	}

	for (auto& Pair : AllObjects)
	{
		// Only rebind things if they exist
		if (Pair.Value.Num() > 0)
		{
			FocusedSequence->UnbindPossessableObjects(Pair.Key);
			for (UObject* Object : Pair.Value)
			{
				UObject* BindingContext = PlaybackContext;

				// Find this object's parent object, if it has one.
				UObject* ParentObject = FocusedSequence->GetParentObject(Object);
				if (ParentObject)
				{
					BindingContext = ParentObject;
				}

				FocusedSequence->BindPossessableObject(Pair.Key, *Object, BindingContext);
			}
		}
	}
}

void FSequencer::GenericTextEntryModeless(const FText& DialogText, const FText& DefaultText, FOnTextCommitted OnTextComitted)
{
	TSharedRef<STextEntryPopup> TextEntryPopup = 
		SNew(STextEntryPopup)
		.Label(DialogText)
		.DefaultText(DefaultText)
		.OnTextCommitted(OnTextComitted)
		.ClearKeyboardFocusOnCommit(false)
		.SelectAllTextWhenFocused(true)
		.MaxWidth(1024.0f);

	EntryPopupMenu = FSlateApplication::Get().PushMenu(
		ToolkitHost.Pin()->GetParentWidget(),
		FWidgetPath(),
		TextEntryPopup,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
	);
}


void FSequencer::CloseEntryPopupMenu()
{
	if (EntryPopupMenu.IsValid())
	{
		EntryPopupMenu.Pin()->Dismiss();
	}
}


void FSequencer::TrimSection(bool bTrimLeft)
{
	FScopedTransaction TrimSectionTransaction( NSLOCTEXT("Sequencer", "TrimSection_Transaction", "Trim Section") );
	MovieSceneToolHelpers::TrimSection(ViewModel->GetSelection()->GetSelectedSections(), GetLocalTime(), bTrimLeft, Settings->GetDeleteKeysWhenTrimming());
	NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::TrackValueChanged );
}


void FSequencer::TrimOrExtendSection(bool bTrimOrExtendLeft)
{
	using namespace UE::Sequencer;

	UMovieSceneSequence* FocusedMovieSceneSequence = GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = FocusedMovieSceneSequence ? FocusedMovieSceneSequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return;
	}

	FScopedTransaction TrimOrExtendSectionTransaction( NSLOCTEXT("Sequencer", "TrimOrExtendSection_Transaction", "Trim or Extend Section") );

	if (ViewModel->GetSelection()->Outliner.Num() > 0)
	{
		for (FViewModelPtr Node : ViewModel->GetSelection()->Outliner)
		{
			if (ITrackExtension* TrackNode = Node->CastThis<ITrackExtension>())
			{
				UMovieSceneTrack* Track = TrackNode->GetTrack();
				if (Track)
				{
					MovieSceneToolHelpers::TrimOrExtendSection(Track, TOptional<int32>(), GetLocalTime(), bTrimOrExtendLeft, Settings->GetDeleteKeysWhenTrimming());
				}
			}
			else if (ITrackRowExtension* TrackRowNode = Node->CastThis<ITrackRowExtension>())
			{
				UMovieSceneTrack* Track = TrackRowNode->GetParentTrack();
				if (Track)
				{
					MovieSceneToolHelpers::TrimOrExtendSection(Track, TrackRowNode->GetRowIndex(), GetLocalTime(), bTrimOrExtendLeft, Settings->GetDeleteKeysWhenTrimming());
				}
			}
			else if (IObjectBindingExtension* ObjectBindingNode = Node->CastThis<IObjectBindingExtension>())
			{
				const FMovieSceneBinding* Binding = MovieScene->FindBinding(ObjectBindingNode->GetObjectGuid());
				if (Binding)
				{
					for (UMovieSceneTrack* Track : Binding->GetTracks())
					{
						MovieSceneToolHelpers::TrimOrExtendSection(Track, TOptional<int32>(), GetLocalTime(), bTrimOrExtendLeft, Settings->GetDeleteKeysWhenTrimming());
					}
				}
			}
		}
	}
	else
	{
		for (UMovieSceneTrack* Track : MovieScene->GetTracks())
		{
			MovieSceneToolHelpers::TrimOrExtendSection(Track, TOptional<int32>(), GetLocalTime(), bTrimOrExtendLeft, Settings->GetDeleteKeysWhenTrimming());
		}
		for (const FMovieSceneBinding& Binding : ((const UMovieScene*)MovieScene)->GetBindings())
		{
			for (UMovieSceneTrack* Track : Binding.GetTracks())
			{
				MovieSceneToolHelpers::TrimOrExtendSection(Track, TOptional<int32>(), GetLocalTime(), bTrimOrExtendLeft, Settings->GetDeleteKeysWhenTrimming());
			}
		}
	}

	NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::TrackValueChanged );
}


void FSequencer::SplitSection()
{
	FScopedTransaction SplitSectionTransaction( NSLOCTEXT("Sequencer", "SplitSection_Transaction", "Split Section") );
	MovieSceneToolHelpers::SplitSection(ViewModel->GetSelection()->GetSelectedSections(), GetLocalTime(), Settings->GetDeleteKeysWhenTrimming());
	NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::RefreshAllImmediately );
}

void FSequencer::BindCommands()
{
	using namespace UE::Sequencer;

	const FSequencerCommands& Commands = FSequencerCommands::Get();

	SequencerCommandBindings->MapAction(
		Commands.TogglePlayViewport,
		FExecuteAction::CreateSP(this, &FSequencer::TogglePlay));

	SequencerCommandBindings->MapAction(
		Commands.JumpToStartViewport,
		FExecuteAction::CreateSP(this, &FSequencer::JumpToStart));

	SequencerCommandBindings->MapAction(
		Commands.JumpToEndViewport,
		FExecuteAction::CreateSP(this, &FSequencer::JumpToEnd));

	SequencerCommandBindings->MapAction(
		Commands.StepToNextKey,
		FExecuteAction::CreateLambda([this] { JumpToNextKey(); }));

	SequencerCommandBindings->MapAction(
		Commands.StepToPreviousKey,
		FExecuteAction::CreateLambda([this] { JumpToPreviousKey(); }));

	SequencerCommandBindings->MapAction(
		Commands.StepForwardViewport,
		FExecuteAction::CreateSP(this, &FSequencer::StepForward),
		EUIActionRepeatMode::RepeatEnabled);

	SequencerCommandBindings->MapAction(
		Commands.StepBackwardViewport,
		FExecuteAction::CreateSP(this, &FSequencer::StepBackward),
		EUIActionRepeatMode::RepeatEnabled);

	SequencerCommandBindings->MapAction(
		Commands.SortAllNodesAndDescendants,
		FExecuteAction::CreateSP(this, &FSequencer::SortAllNodesAndDescendants));

	SequencerCommandBindings->MapAction(
		Commands.ToggleShowMarkedFrames,
		FExecuteAction::CreateLambda([this] { Settings->SetShowMarkedFrames(!Settings->GetShowMarkedFrames()); }),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetShowMarkedFrames(); }));

	SequencerCommandBindings->MapAction(
		Commands.ToggleShowMarkedFramesGlobally,
		FExecuteAction::CreateSP(this, &FSequencer::ToggleShowMarkedFramesGlobally),
		FCanExecuteAction::CreateLambda([this] { return GetFocusedMovieSceneSequence() != nullptr; }),
		FIsActionChecked::CreateLambda([this] { return GetFocusedMovieSceneSequence()->GetMovieScene()->GetGloballyShowMarkedFrames(); }) );

	SequencerCommandBindings->MapAction(
		Commands.ClearGlobalMarkedFrames,
		FExecuteAction::CreateSP(this, &FSequencer::ClearGlobalMarkedFrames));

	SequencerCommandBindings->MapAction(
		Commands.ToggleAutoExpandNodesOnSelection,
		FExecuteAction::CreateLambda([this] { Settings->SetAutoExpandNodesOnSelection(!Settings->GetAutoExpandNodesOnSelection()); }),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetAutoExpandNodesOnSelection(); }) );
	
	SequencerCommandBindings->MapAction(
		Commands.ToggleAutoExpandCurveEditorOnSelection,
		FExecuteAction::CreateLambda([this] { Settings->SetAutoExpandCurveEditorOnSelection(!Settings->GetAutoExpandCurveEditorOnSelection()); }),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetAutoExpandCurveEditorOnSelection(); }) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleRestoreOriginalViewportOnCameraCutUnlock,
		FExecuteAction::CreateLambda([this] { Settings->SetRestoreOriginalViewportOnCameraCutUnlock(!Settings->GetRestoreOriginalViewportOnCameraCutUnlock()); }),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetRestoreOriginalViewportOnCameraCutUnlock(); }) );

	SequencerCommandBindings->MapAction(
		Commands.TogglePreviewCameraCutsInSimulate,
		FExecuteAction::CreateLambda([this] { TracksSettings->SetPreviewCameraCutsInSimulate(!TracksSettings->GetPreviewCameraCutsInSimulate()); }),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return TracksSettings->GetPreviewCameraCutsInSimulate(); }) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleExpandCollapseNodes,
		FExecuteAction::CreateSP(this, &FSequencer::ToggleExpandCollapseNodes));

	SequencerCommandBindings->MapAction(
		Commands.ToggleExpandCollapseNodesAndDescendants,
		FExecuteAction::CreateSP(this, &FSequencer::ToggleExpandCollapseNodesAndDescendants));

	SequencerCommandBindings->MapAction(
		Commands.ExpandAllNodes,
		FExecuteAction::CreateSP(this, &FSequencer::ExpandAllNodes));

	SequencerCommandBindings->MapAction(
		Commands.CollapseAllNodes,
		FExecuteAction::CreateSP(this, &FSequencer::CollapseAllNodes));

	SequencerCommandBindings->MapAction(
		Commands.AddActorsToSequencer,
		FExecuteAction::CreateSP( this, &FSequencer::AddSelectedActors));

	SequencerCommandBindings->MapAction(
		Commands.SetKey,
		FExecuteAction::CreateSP( this, &FSequencer::SetKey ) );

	SequencerCommandBindings->MapAction(
		Commands.TranslateLeft,
		FExecuteAction::CreateSP( this, &FSequencer::TranslateSelection, true) );

	SequencerCommandBindings->MapAction(
		Commands.TranslateRight,
		FExecuteAction::CreateSP( this, &FSequencer::TranslateSelection, false) );

	SequencerCommandBindings->MapAction(
		Commands.TrimSectionLeft,
		FExecuteAction::CreateSP( this, &FSequencer::TrimSection, true ),
		FCanExecuteAction::CreateLambda([this] { return MovieSceneToolHelpers::CanTrimSectionLeft(this->ViewModel->GetSelection()->GetSelectedSections(), GetLocalTime()); }));

	SequencerCommandBindings->MapAction(
		Commands.TrimSectionRight,
		FExecuteAction::CreateSP( this, &FSequencer::TrimSection, false ),
		FCanExecuteAction::CreateLambda([this] { return MovieSceneToolHelpers::CanTrimSectionRight(this->ViewModel->GetSelection()->GetSelectedSections(), GetLocalTime()); }));

	SequencerCommandBindings->MapAction(
		Commands.TrimOrExtendSectionLeft,
		FExecuteAction::CreateSP( this, &FSequencer::TrimOrExtendSection, true ) );

	SequencerCommandBindings->MapAction(
		Commands.TrimOrExtendSectionRight,
		FExecuteAction::CreateSP( this, &FSequencer::TrimOrExtendSection, false ) );

	SequencerCommandBindings->MapAction(
		Commands.SplitSection,
		FExecuteAction::CreateSP( this, &FSequencer::SplitSection ),
		FCanExecuteAction::CreateLambda([this] { return MovieSceneToolHelpers::CanSplitSection(this->ViewModel->GetSelection()->GetSelectedSections(), GetLocalTime()); }));

	// We can convert to spawnables if anything selected is a root-level possessable
	auto CanSelectedNodesBeConvertedToSpawnables = [this]{
		UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();
		if (!Sequence || !Sequence->AllowsSpawnableObjects())
		{
			return false;
		}

		UMovieScene* MovieScene = Sequence->GetMovieScene();

		for (FViewModelPtr Possessable : ViewModel->GetSelection()->Outliner.Filter<FPossessableModel>())
		{
			FViewModelPtr Parent = Possessable->GetParent();
			if (!Parent || !Parent->IsA<IObjectBindingExtension>())
			{
				return true;
			}
		}
		return false;
	};


	auto CanSelectedNodesBeConvertedToPossessables = [this] {
		UMovieScene* MovieScene = GetFocusedMovieSceneSequence()->GetMovieScene();

		for (TViewModelPtr<IObjectBindingExtension> ObjectBindingNode : ViewModel->GetSelection()->Outliner.Filter<IObjectBindingExtension>())
		{
			if (SpawnRegister->CanConvertToPossessable(ObjectBindingNode->GetObjectGuid(), GetFocusedTemplateID(), GetSharedPlaybackState()))
			{
				return true;
			}
		}
		return false;
	};

	auto CanBrowseToObject = [this] {
		for (TViewModelPtr<FSectionModel> SectionModel : ViewModel->GetSelection()->TrackArea.Filter<FSectionModel>())
		{
			if (UMovieSceneSection* Section = SectionModel->GetSection())
			{
				if (UObject* ObjectToFocus = Section->GetSourceObject())
				{
					return true;
				}
			}
		}
		return false;
	};

	SequencerCommandBindings->MapAction(
		FSequencerCommands::Get().ConvertToSpawnable,
		FExecuteAction::CreateSP(this, &FSequencer::ConvertSelectedNodesToSpawnables),
		FCanExecuteAction::CreateLambda(CanSelectedNodesBeConvertedToSpawnables)
	);

	SequencerCommandBindings->MapAction(
		FSequencerCommands::Get().ConvertToPossessable,
		FExecuteAction::CreateSP(this, &FSequencer::ConvertSelectedNodesToPossessables),
		FCanExecuteAction::CreateLambda(CanSelectedNodesBeConvertedToPossessables)
	);

	auto AreSpawnablesSelected = [this] {
		UMovieSceneSequence* MovieSceneSequence = GetFocusedMovieSceneSequence();

		for (TViewModelPtr<IObjectBindingExtension> ObjectBindingNode : ViewModel->GetSelection()->Outliner.Filter<IObjectBindingExtension>())
		{
			if (MovieSceneHelpers::IsBoundToAnySpawnable(MovieSceneSequence, ObjectBindingNode->GetObjectGuid(), GetSharedPlaybackState()))
			{
				return true;
			}
		}
		return false;
	};

	SequencerCommandBindings->MapAction(
		FSequencerCommands::Get().SaveCurrentSpawnableState,
		FExecuteAction::CreateSP(this, &FSequencer::SaveSelectedNodesSpawnableState),
		FCanExecuteAction::CreateLambda(AreSpawnablesSelected)
	);

	SequencerCommandBindings->MapAction(
		FSequencerCommands::Get().RestoreAnimatedState,
		FExecuteAction::CreateSP(this, &FSequencer::RestorePreAnimatedState)
	);

	SequencerCommandBindings->MapAction(
		Commands.SetAutoKey,
		FExecuteAction::CreateLambda( [this]{ Settings->SetAutoChangeMode( EAutoChangeMode::AutoKey ); } ),
		FCanExecuteAction::CreateLambda( [this]{ return Settings->GetAllowEditsMode() != EAllowEditsMode::AllowLevelEditsOnly; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetAutoChangeMode() == EAutoChangeMode::AutoKey; } ) );

	SequencerCommandBindings->MapAction(
		Commands.SetAutoTrack,
		FExecuteAction::CreateLambda([this] { Settings->SetAutoChangeMode(EAutoChangeMode::AutoTrack); } ),
		FCanExecuteAction::CreateLambda([this] { return Settings->GetAllowEditsMode() != EAllowEditsMode::AllowLevelEditsOnly; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetAutoChangeMode() == EAutoChangeMode::AutoTrack; } ) );

	SequencerCommandBindings->MapAction(
		Commands.SetAutoChangeAll,
		FExecuteAction::CreateLambda([this] { Settings->SetAutoChangeMode(EAutoChangeMode::All); } ),
		FCanExecuteAction::CreateLambda([this] { return Settings->GetAllowEditsMode() != EAllowEditsMode::AllowLevelEditsOnly; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetAutoChangeMode() == EAutoChangeMode::All; } ) );
	
	SequencerCommandBindings->MapAction(
		Commands.SetAutoChangeNone,
		FExecuteAction::CreateLambda([this] { Settings->SetAutoChangeMode(EAutoChangeMode::None); } ),
		FCanExecuteAction::CreateLambda([this] { return Settings->GetAllowEditsMode() != EAllowEditsMode::AllowLevelEditsOnly; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetAutoChangeMode() == EAutoChangeMode::None; } ) );

	SequencerCommandBindings->MapAction(
		Commands.AllowAllEdits,
		FExecuteAction::CreateLambda( [this]{ Settings->SetAllowEditsMode( EAllowEditsMode::AllEdits ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetAllowEditsMode() == EAllowEditsMode::AllEdits; } ) );

	SequencerCommandBindings->MapAction(
		Commands.AllowSequencerEditsOnly,
		FExecuteAction::CreateLambda([this] { Settings->SetAllowEditsMode(EAllowEditsMode::AllowSequencerEditsOnly); }),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetAllowEditsMode() == EAllowEditsMode::AllowSequencerEditsOnly; }));

	SequencerCommandBindings->MapAction(
		Commands.AllowLevelEditsOnly,
		FExecuteAction::CreateLambda([this] { Settings->SetAllowEditsMode(EAllowEditsMode::AllowLevelEditsOnly); }),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetAllowEditsMode() == EAllowEditsMode::AllowLevelEditsOnly; }));

	SequencerCommandBindings->MapAction(
		Commands.ToggleAutoKeyEnabled,
		FExecuteAction::CreateLambda( [this]{ Settings->SetAutoChangeMode(Settings->GetAutoChangeMode() == EAutoChangeMode::None ? EAutoChangeMode::AutoKey : EAutoChangeMode::None); } ),
		FCanExecuteAction::CreateLambda( [this]{ return Settings->GetAllowEditsMode() != EAllowEditsMode::AllowLevelEditsOnly; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetAutoChangeMode() == EAutoChangeMode::AutoKey; } ) );

	SequencerCommandBindings->MapAction(
		Commands.SetKeyChanged,
		FExecuteAction::CreateLambda([this] { Settings->SetKeyGroupMode(EKeyGroupMode::KeyChanged); }),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetKeyGroupMode() == EKeyGroupMode::KeyChanged; }));

	SequencerCommandBindings->MapAction(
		Commands.SetKeyGroup,
		FExecuteAction::CreateLambda([this] { Settings->SetKeyGroupMode(EKeyGroupMode::KeyGroup); }),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetKeyGroupMode() == EKeyGroupMode::KeyGroup; }));

	SequencerCommandBindings->MapAction(
		Commands.SetKeyAll,
		FExecuteAction::CreateLambda([this] { Settings->SetKeyGroupMode(EKeyGroupMode::KeyAll); }),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetKeyGroupMode() == EKeyGroupMode::KeyAll; }));

	SequencerCommandBindings->MapAction(
		Commands.ToggleMarkAtPlayPosition,
		FExecuteAction::CreateSP( this, &FSequencer::ToggleMarkAtPlayPosition));

	SequencerCommandBindings->MapAction(
		Commands.StepToNextMark,
		FExecuteAction::CreateSP( this, &FSequencer::StepToNextMark));

	SequencerCommandBindings->MapAction(
		Commands.StepToPreviousMark,
		FExecuteAction::CreateSP( this, &FSequencer::StepToPreviousMark));

	SequencerCommandBindings->MapAction(
		Commands.ToggleMarksLocked,
		FExecuteAction::CreateSP(this, &FSequencer::ToggleMarkedFramesLocked),
		FCanExecuteAction::CreateLambda( [this] { return GetFocusedMovieSceneSequence() != nullptr;	} ),
		FIsActionChecked::CreateSP(this, &FSequencer::AreMarkedFramesLocked));

	SequencerCommandBindings->MapAction(
		Commands.ToggleAutoScroll,
		FExecuteAction::CreateLambda( [this]{ Settings->SetAutoScrollEnabled( !Settings->GetAutoScrollEnabled() ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetAutoScrollEnabled(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.FindInContentBrowser,
		FExecuteAction::CreateSP( this, &FSequencer::FindInContentBrowser ) );

	SequencerCommandBindings->MapAction(
		Commands.BrowseToObject,
		FExecuteAction::CreateSP( this, &FSequencer::BrowseToObject),
		FCanExecuteAction::CreateLambda(CanBrowseToObject));

	SequencerCommandBindings->MapAction(
		Commands.ToggleLayerBars,
		FExecuteAction::CreateLambda( [this]{
			Settings->SetShowLayerBars( !Settings->GetShowLayerBars() );
			RefreshUI();
		} ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetShowLayerBars(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleKeyBars,
		FExecuteAction::CreateLambda( [this]{
			Settings->SetShowKeyBars( !Settings->GetShowKeyBars() );
			RefreshUI();
		} ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetShowKeyBars(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleChannelColors,
		FExecuteAction::CreateLambda( [this]{
			Settings->SetShowChannelColors( !Settings->GetShowChannelColors() );
		} ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetShowChannelColors(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleShowInfoButton,
		FExecuteAction::CreateLambda([this] {
			Settings->SetShowInfoButton(!Settings->GetShowInfoButton());
		}),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetShowInfoButton(); }));

	SequencerCommandBindings->MapAction(
		Commands.ChangeTimeDisplayFormat,
		FExecuteAction::CreateLambda( [this]{
			
			const UEnum* FrameNumberDisplayEnum = StaticEnum<EFrameNumberDisplayFormats>();
			check(FrameNumberDisplayEnum);

			uint8 NextIndex = (uint8)Settings->GetTimeDisplayFormat() + 1;
			if (NextIndex >= FrameNumberDisplayEnum->NumEnums() - 1)
			{
				NextIndex = 0;
			}

			while (FrameNumberDisplayEnum->HasMetaData(TEXT("Hidden"), NextIndex))
			{
				NextIndex++;

				if (NextIndex >= FrameNumberDisplayEnum->NumEnums() - 1)
				{
					NextIndex = 0;
				}
			}

			EFrameNumberDisplayFormats NextFormat = (EFrameNumberDisplayFormats)(NextIndex);

			// If the next framerate in the list is drop format timecode and we're not in a play rate that supports drop format timecode,
			// then we will skip over it.
			bool bCanShowDropFrameTimecode = FTimecode::UseDropFormatTimecode(GetFocusedDisplayRate());
			if (bCanShowDropFrameTimecode && NextFormat == EFrameNumberDisplayFormats::NonDropFrameTimecode)
			{
				NextFormat = EFrameNumberDisplayFormats::DropFrameTimecode;
			}
			else if (!bCanShowDropFrameTimecode && NextFormat == EFrameNumberDisplayFormats::DropFrameTimecode)
			{
				NextFormat = EFrameNumberDisplayFormats::Seconds;
			}
			Settings->SetTimeDisplayFormat( NextFormat );
		} ),
		FCanExecuteAction::CreateLambda([] { return true; }));

	SequencerCommandBindings->MapAction(
		Commands.ToggleShowTransportControl,
		FExecuteAction::CreateLambda( [this]{ SequencerWidget->ToggleTransportControlsVisible(); } ),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda( [this]{ return SequencerWidget->IsTransportControlsVisible(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleTransportControlsPoppedOut,
		FExecuteAction::CreateLambda( [this]{ SequencerWidget->ToggleTransportControlsPoppedOut(); } ),
		FCanExecuteAction::CreateLambda( [this]{ return Settings->ShouldShowTransportControls(); } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->ShouldRestoreTransportControlsPoppedOut(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleShowRangeSlider,
		FExecuteAction::CreateLambda( [this]{ Settings->SetShowRangeSlider( !Settings->GetShowRangeSlider() ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetShowRangeSlider(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleShowBottomTimeSlider,
		FExecuteAction::CreateLambda( [this]{ Settings->SetShowBottomTimeSlider( !Settings->GetShowBottomTimeSlider() ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetShowBottomTimeSlider(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleIsSnapEnabled,
		FExecuteAction::CreateLambda( [this]{ Settings->SetIsSnapEnabled( !Settings->GetIsSnapEnabled() ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetIsSnapEnabled(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleSnapKeyTimesToElements,
		FExecuteAction::CreateLambda( [this]{ Settings->SetSnapKeyTimesToElements( !Settings->GetSnapKeyTimesToElements() ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetSnapKeyTimesToElements(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleSnapSectionTimesToElements,
		FExecuteAction::CreateLambda( [this]{ Settings->SetSnapSectionTimesToElements( !Settings->GetSnapSectionTimesToElements() ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetSnapSectionTimesToElements(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleSnapPlayTimeToKeys,
		FExecuteAction::CreateLambda( [this]{ Settings->SetSnapPlayTimeToKeys( !Settings->GetSnapPlayTimeToKeys() ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetSnapPlayTimeToKeys(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleSnapPlayTimeToSections,
		FExecuteAction::CreateLambda( [this]{ Settings->SetSnapPlayTimeToSections( !Settings->GetSnapPlayTimeToSections() ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetSnapPlayTimeToSections(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleSnapPlayTimeToMarkers,
		FExecuteAction::CreateLambda( [this]{ Settings->SetSnapPlayTimeToMarkers( !Settings->GetSnapPlayTimeToMarkers() ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetSnapPlayTimeToMarkers(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleSnapPlayTimeToPressedKey,
		FExecuteAction::CreateLambda( [this]{ Settings->SetSnapPlayTimeToPressedKey( !Settings->GetSnapPlayTimeToPressedKey() ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetSnapPlayTimeToPressedKey(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleSnapPlayTimeToDraggedKey,
		FExecuteAction::CreateLambda( [this]{ Settings->SetSnapPlayTimeToDraggedKey( !Settings->GetSnapPlayTimeToDraggedKey() ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetSnapPlayTimeToDraggedKey(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleSnapCurveValueToInterval,
		FExecuteAction::CreateLambda( [this]{ Settings->SetSnapCurveValueToInterval( !Settings->GetSnapCurveValueToInterval() ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetSnapCurveValueToInterval(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleForceWholeFrames,
		FExecuteAction::CreateLambda([this] { Settings->SetForceWholeFrames(!Settings->GetForceWholeFrames()); }),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetForceWholeFrames(); }));

	SequencerCommandBindings->MapAction(
		Commands.ToggleShowCurveEditor,
		FExecuteAction::CreateLambda( [this]{ SetShowCurveEditor(!GetCurveEditorIsVisible()); } ),
		FCanExecuteAction::CreateLambda( [this]{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return GetCurveEditorIsVisible(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleLinkCurveEditorTimeRange,
		FExecuteAction::CreateLambda( [this]{ Settings->SetLinkCurveEditorTimeRange(!Settings->GetLinkCurveEditorTimeRange()); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetLinkCurveEditorTimeRange(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleShowPreAndPostRoll,
		FExecuteAction::CreateLambda( [this]{ Settings->SetShouldShowPrePostRoll(!Settings->ShouldShowPrePostRoll()); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->ShouldShowPrePostRoll(); } ) );

	auto CanCutOrCopy = [this]{
		// For copy tracks
		TSet<TWeakViewModelPtr<IOutlinerExtension>> SelectedNodes = ViewModel->GetSelection()->GetNodesWithSelectedKeysOrSections();
		// If this is empty then we are selecting display nodes
		if (SelectedNodes.Num() == 0)
		{
			SelectedNodes = ViewModel->GetSelection()->Outliner.GetSelected();
			for (TWeakViewModelPtr<IOutlinerExtension> WeakNode : SelectedNodes)
			{
				FViewModelPtr Node = WeakNode.Pin();
				if (Node->IsA<ITrackExtension>() || Node->IsA<ITrackRowExtension>() || Node->IsA<IObjectBindingExtension>() || Node->IsA<FFolderModel>())
				{
					// if contains one node that can be copied we allow the action
					// later on we will filter out the invalid nodes in CopySelection() or CutSelection()
					return true;
				}
				else if (Node->IsA<FDecorationOutlinerModel>())
				{
					return true;
				}
				else if (Node->GetParent().IsValid() && (Node->GetParent()->IsA<ITrackExtension>() || Node->GetParent()->IsA<ITrackRowExtension>()) && !Node->IsA<FCategoryGroupModel>())
				{
					return true;
				}
			}
			return false;
		}

		UMovieSceneTrack* Track = nullptr;
		for (FKeyHandle Key : ViewModel->GetSelection()->KeySelection)
		{
			TSharedPtr<FChannelModel> Channel = ViewModel->GetSelection()->KeySelection.GetModelForKey(Key);
			if (!Channel)
			{
				continue;
			}

			if (!Track)
			{
				Track = Channel->GetSection()->GetTypedOuter<UMovieSceneTrack>();
			}
			if (!Track || Track != Channel->GetSection()->GetTypedOuter<UMovieSceneTrack>())
			{
				return false;
			}
		}
		return true;
	};

	auto CanDelete = [this]{
		TSharedPtr<FSequencerSelection> Selection = this->ViewModel->GetSelection();
		return Selection->KeySelection.Num() || Selection->GetSelectedSections().Num() || Selection->Outliner.Num();
	};

	auto CanDuplicate = [this]{
		TSharedPtr<FSequencerSelection> Selection = this->ViewModel->GetSelection();
		if (Selection->KeySelection.Num() || Selection->GetSelectedSections().Num() || Selection->GetSelectedTracks().Num())
		{
			return true;
		}

		// For duplicate object tracks
		if (Selection->GetNodesWithSelectedKeysOrSections().Num() == 0)
		{
			// if contains one node that can be copied we allow the action
			return Selection->Outliner.Filter<IObjectBindingExtension>().SelectionSet->Num() > 0;
		}
		return false;
	};

	auto IsSelectionRangeNonEmpty = [this]{
		UMovieSceneSequence* EditedSequence = GetFocusedMovieSceneSequence();
		if (!EditedSequence || !EditedSequence->GetMovieScene())
		{
			return false;
		}

		return !EditedSequence->GetMovieScene()->GetSelectionRange().IsEmpty();
	};

	SequencerCommandBindings->MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateLambda([this]
		{
			TSharedPtr<FSequencerSelection> Selection = this->ViewModel->GetSelection();

			for (TViewModelPtr<IRenameableExtension> Renamable : Selection->Outliner.Filter<IRenameableExtension>())
			{
				if (Renamable->CanRename())
				{
					Renamable->OnRenameRequested().Broadcast();
				}
			}
		}),
		FCanExecuteAction::CreateLambda([this]
		{
			TSharedPtr<FSequencerSelection> Selection = this->ViewModel->GetSelection();

			for (TViewModelPtr<IRenameableExtension> Renamable : Selection->Outliner.Filter<IRenameableExtension>())
			{
				if (Renamable->CanRename())
				{
					return true;
				}
			}
			return false;
		})
	);

	SequencerCommandBindings->MapAction(
		Commands.TogglePlaybackRangeLocked,
		FExecuteAction::CreateSP( this, &FSequencer::TogglePlaybackRangeLocked ),
		FCanExecuteAction::CreateLambda( [this] { return GetFocusedMovieSceneSequence() != nullptr;	} ),
		FIsActionChecked::CreateSP( this, &FSequencer::IsPlaybackRangeLocked ));

	SequencerCommandBindings->MapAction(
		Commands.ToggleCleanPlaybackMode,
		FExecuteAction::CreateLambda( [this]{ Settings->SetCleanPlaybackMode( !Settings->GetCleanPlaybackMode() ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->GetCleanPlaybackMode(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleRerunConstructionScripts,
		FExecuteAction::CreateLambda( [this]{ Settings->SetRerunConstructionScripts( !Settings->ShouldRerunConstructionScripts() ); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->ShouldRerunConstructionScripts(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleAsyncEvaluation,
		FExecuteAction::CreateLambda( [this]{ this->ToggleAsyncEvaluation(); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return this->UsesAsyncEvaluation(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleDynamicWeighting,
		FExecuteAction::CreateLambda( [this]{ this->ToggleDynamicWeighting(); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return this->UsesDynamicWeighting(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.ToggleLoopCuts,
		FExecuteAction::CreateLambda( [this]{ this->ToggleLoopCuts(); } ),
		FCanExecuteAction::CreateLambda( []{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return this->UsesLoopCuts(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.SetPlayheadScrubbingLimitNone,
		FExecuteAction::CreateLambda([this] { Settings->SetPlayheadScrubbingLimit(ESequencerPlayheadScrubbingLimit::SPSL_None); }),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetPlayheadScrubbingLimit() == ESequencerPlayheadScrubbingLimit::SPSL_None; }));
	
	SequencerCommandBindings->MapAction(
		Commands.SetPlayheadScrubbingLimitPlaybackRange,
		FExecuteAction::CreateLambda([this] { Settings->SetPlayheadScrubbingLimit(ESequencerPlayheadScrubbingLimit::SPSL_PlaybackRange); }),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetPlayheadScrubbingLimit() == ESequencerPlayheadScrubbingLimit::SPSL_PlaybackRange; }));

	SequencerCommandBindings->MapAction(
		Commands.SetPlayheadScrubbingLimitSelectionRange,
		FExecuteAction::CreateLambda([this] { Settings->SetPlayheadScrubbingLimit(ESequencerPlayheadScrubbingLimit::SPSL_SelectionRange); }),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetPlayheadScrubbingLimit() == ESequencerPlayheadScrubbingLimit::SPSL_SelectionRange; }));

	SequencerCommandBindings->MapAction(
		Commands.ToggleResetPlayheadWhenNavigating,
		FExecuteAction::CreateLambda([this] { Settings->SetResetPlayheadWhenNavigating(!Settings->ShouldResetPlayheadWhenNavigating()); }),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return Settings->ShouldResetPlayheadWhenNavigating(); }));

	SequencerCommandBindings->MapAction(
		Commands.ToggleKeepPlaybackRangeInSectionBounds,
		FExecuteAction::CreateLambda([this] { Settings->SetKeepPlayRangeInSectionBounds(!Settings->ShouldKeepPlayRangeInSectionBounds()); NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged); }),
		FCanExecuteAction::CreateLambda([] { return true; }),
		FIsActionChecked::CreateLambda([this] { return Settings->ShouldKeepPlayRangeInSectionBounds(); }));

	SequencerCommandBindings->MapAction(
		Commands.ToggleEvaluateSubSequencesInIsolation,
		FExecuteAction::CreateLambda( [this]{
			const bool bNewValue = !Settings->ShouldEvaluateSubSequencesInIsolation();
			Settings->SetEvaluateSubSequencesInIsolation( bNewValue );
		} ),
		FCanExecuteAction::CreateLambda( [this]{ return true; } ),
		FIsActionChecked::CreateLambda( [this]{ return Settings->ShouldEvaluateSubSequencesInIsolation(); } ) );

	SequencerCommandBindings->MapAction(
		Commands.RenderMovie,
		FExecuteAction::CreateLambda([this]{ RenderMovieInternal(GetPlaybackRange()); })
	);

	SequencerCommandBindings->MapAction(
		Commands.CreateCamera,
		FExecuteAction::CreateLambda([this]{ 
			const bool bSpawnable = Settings->GetCreateSpawnableCameras() && GetFocusedMovieSceneSequence()->AllowsSpawnableObjects();
			ACineCameraActor* OutActor;
			FSequencerUtilities::CreateCamera(AsShared(), bSpawnable, OutActor); 
		} ),
		FCanExecuteAction(),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateLambda([this] { return Cast<ULevelSequence>(GetFocusedMovieSceneSequence()) != nullptr; })
	);

	SequencerCommandBindings->MapAction(
		Commands.FixPossessableObjectClass,
		FExecuteAction::CreateSP( this, &FSequencer::FixPossessableObjectClass ),
		FCanExecuteAction::CreateLambda( []{ return true; } ) );

	SequencerCommandBindings->MapAction(
		Commands.RebindPossessableReferences,
		FExecuteAction::CreateSP( this, &FSequencer::RebindPossessableReferences ),
		FCanExecuteAction::CreateLambda( []{ return true; } ) );

	SequencerCommandBindings->MapAction(
		Commands.MoveToNewFolder,
		FExecuteAction::CreateSP( this, &FSequencer::MoveSelectedNodesToNewFolder ),
		FCanExecuteAction::CreateLambda( [this]{ return (GetSelectedNodesToMove().Num() > 0); } ) );

	SequencerCommandBindings->MapAction(
		Commands.RemoveFromFolder,
		FExecuteAction::CreateSP( this, &FSequencer::RemoveSelectedNodesFromFolders ),
		FCanExecuteAction::CreateLambda( [this]{ return (GetSelectedNodesInFolders().Num() > 0); } ) );

	for (int32 i = 0; i < TrackEditors.Num(); ++i)
	{
		TrackEditors[i]->BindCommands(SequencerCommandBindings);
	}


	SequencerCommandBindings->MapAction(
		Commands.AddTransformKey,
		FExecuteAction::CreateSP(this, &FSequencer::OnAddTransformKeysForSelectedObjects, EMovieSceneTransformChannel::All),
		FCanExecuteAction::CreateSP(this, &FSequencer::CanAddTransformKeysForSelectedObjects, EMovieSceneTransformChannel::All));
	SequencerCommandBindings->MapAction(
		Commands.AddTranslationKey,
		FExecuteAction::CreateSP(this, &FSequencer::OnAddTransformKeysForSelectedObjects, EMovieSceneTransformChannel::Translation),
		FCanExecuteAction::CreateSP(this, &FSequencer::CanAddTransformKeysForSelectedObjects, EMovieSceneTransformChannel::Translation));
	SequencerCommandBindings->MapAction(
		Commands.AddRotationKey,
		FExecuteAction::CreateSP(this, &FSequencer::OnAddTransformKeysForSelectedObjects, EMovieSceneTransformChannel::Rotation),
		FCanExecuteAction::CreateSP(this, &FSequencer::CanAddTransformKeysForSelectedObjects, EMovieSceneTransformChannel::Rotation));
	SequencerCommandBindings->MapAction(
		Commands.AddScaleKey,
		FExecuteAction::CreateSP(this, &FSequencer::OnAddTransformKeysForSelectedObjects, EMovieSceneTransformChannel::Scale),
		FCanExecuteAction::CreateSP(this, &FSequencer::CanAddTransformKeysForSelectedObjects, EMovieSceneTransformChannel::Scale));
	SequencerCommandBindings->MapAction(
		Commands.DeleteTransformKey,
		FExecuteAction::CreateSP(this, &FSequencer::OnDeleteTransformKeysForSelectedObjects, EMovieSceneTransformChannel::All),
		FCanExecuteAction::CreateSP(this, &FSequencer::CanDeleteTransformKeysForSelectedObjects, EMovieSceneTransformChannel::All));
	SequencerCommandBindings->MapAction(
		Commands.DeleteTranslationKey,
		FExecuteAction::CreateSP(this, &FSequencer::OnDeleteTransformKeysForSelectedObjects, EMovieSceneTransformChannel::Translation),
		FCanExecuteAction::CreateSP(this, &FSequencer::CanDeleteTransformKeysForSelectedObjects, EMovieSceneTransformChannel::Translation));
	SequencerCommandBindings->MapAction(
		Commands.DeleteRotationKey,
		FExecuteAction::CreateSP(this, &FSequencer::OnDeleteTransformKeysForSelectedObjects, EMovieSceneTransformChannel::Rotation),
		FCanExecuteAction::CreateSP(this, &FSequencer::CanDeleteTransformKeysForSelectedObjects, EMovieSceneTransformChannel::Rotation));
	SequencerCommandBindings->MapAction(
		Commands.DeleteScaleKey,
		FExecuteAction::CreateSP(this, &FSequencer::OnDeleteTransformKeysForSelectedObjects, EMovieSceneTransformChannel::Scale),
		FCanExecuteAction::CreateSP(this, &FSequencer::CanDeleteTransformKeysForSelectedObjects, EMovieSceneTransformChannel::Scale));

	SequencerCommandBindings->MapAction(
		Commands.SetKeyTime,
		FExecuteAction::CreateSP(this, &FSequencer::SetKeyTime),
		FCanExecuteAction::CreateSP(this, &FSequencer::CanSetKeyTime));
	SequencerCommandBindings->MapAction(
		Commands.Rekey,
		FExecuteAction::CreateSP(this, &FSequencer::Rekey),
		FCanExecuteAction::CreateSP(this, &FSequencer::CanRekey));
	SequencerCommandBindings->MapAction(
		Commands.SnapToFrame,
		FExecuteAction::CreateSP(this, &FSequencer::SnapToFrame),
		FCanExecuteAction::CreateSP(this, &FSequencer::CanSnapToFrame));

	SequencerCommandBindings->MapAction(
		Commands.TogglePilotCamera,
		FExecuteAction::CreateSP(this, &FSequencer::OnTogglePilotCamera),
		FCanExecuteAction::CreateLambda( [] { return true; } ),
		FIsActionChecked::CreateSP(this, &FSequencer::IsPilotCamera));

	SequencerCommandBindings->MapAction(
		Commands.ToggleSimpleView,
		FExecuteAction::CreateSP(this, &FSequencer::ToggleSimpleView));

	// copy subset of sequencer commands to shared commands
	*SequencerSharedBindings = *SequencerCommandBindings;

	// Sequencer-only bindings
	SequencerCommandBindings->MapAction(
		FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &FSequencer::CutSelection),
		FCanExecuteAction::CreateLambda(CanCutOrCopy)
	);

	SequencerCommandBindings->MapAction(
		FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &FSequencer::CopySelection),
		FCanExecuteAction::CreateLambda(CanCutOrCopy)
	);

	SequencerCommandBindings->MapAction(
		FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &FSequencer::DuplicateSelection),
		FCanExecuteAction::CreateLambda(CanDuplicate)
	);

	SequencerCommandBindings->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP( this, &FSequencer::DeleteSelectedItems ),
		FCanExecuteAction::CreateLambda(CanDelete));

	SequencerCommandBindings->MapAction(
		Commands.TogglePlay,
		FExecuteAction::CreateSP(this, &FSequencer::TogglePlay));

	SequencerCommandBindings->MapAction(
		Commands.PlayForward,
		FExecuteAction::CreateLambda([this] { OnPlayForward(false); }));

	SequencerCommandBindings->MapAction(
		Commands.JumpToStart,
		FExecuteAction::CreateSP(this, &FSequencer::JumpToStart));

	SequencerCommandBindings->MapAction(
		Commands.JumpToEnd,
		FExecuteAction::CreateSP(this, &FSequencer::JumpToEnd));

	SequencerCommandBindings->MapAction(
		Commands.StepForward,
		FExecuteAction::CreateSP(this, &FSequencer::StepForward),
		EUIActionRepeatMode::RepeatEnabled);

	SequencerCommandBindings->MapAction(
		Commands.StepBackward,
		FExecuteAction::CreateSP(this, &FSequencer::StepBackward),
		EUIActionRepeatMode::RepeatEnabled);

	SequencerCommandBindings->MapAction(
		Commands.JumpForward,
		FExecuteAction::CreateSP(this, &FSequencer::JumpForward),
		EUIActionRepeatMode::RepeatEnabled);

	SequencerCommandBindings->MapAction(
		Commands.JumpBackward,
		FExecuteAction::CreateSP(this, &FSequencer::JumpBackward),
		EUIActionRepeatMode::RepeatEnabled);

	SequencerCommandBindings->MapAction(
		Commands.SetInterpolationCubicSmartAuto,
		FExecuteAction::CreateSP(this, &FSequencer::SetInterpTangentMode, ERichCurveInterpMode::RCIM_Cubic, ERichCurveTangentMode::RCTM_SmartAuto));

	SequencerCommandBindings->MapAction(
		Commands.SetInterpolationCubicAuto,
		FExecuteAction::CreateSP(this, &FSequencer::SetInterpTangentMode, ERichCurveInterpMode::RCIM_Cubic, ERichCurveTangentMode::RCTM_Auto));

	SequencerCommandBindings->MapAction(
		Commands.SetInterpolationCubicUser,
		FExecuteAction::CreateSP(this, &FSequencer::SetInterpTangentMode, ERichCurveInterpMode::RCIM_Cubic, ERichCurveTangentMode::RCTM_User));

	SequencerCommandBindings->MapAction(
		Commands.SetInterpolationCubicBreak,
		FExecuteAction::CreateSP(this, &FSequencer::SetInterpTangentMode, ERichCurveInterpMode::RCIM_Cubic, ERichCurveTangentMode::RCTM_Break));

	SequencerCommandBindings->MapAction(
		Commands.ToggleWeightedTangents,
		FExecuteAction::CreateSP(this, &FSequencer::ToggleInterpTangentWeightMode));

	SequencerCommandBindings->MapAction(
		Commands.SetInterpolationLinear,
		FExecuteAction::CreateSP(this, &FSequencer::SetInterpTangentMode, ERichCurveInterpMode::RCIM_Linear, ERichCurveTangentMode::RCTM_Auto));

	SequencerCommandBindings->MapAction(
		Commands.SetInterpolationConstant,
		FExecuteAction::CreateSP(this, &FSequencer::SetInterpTangentMode, ERichCurveInterpMode::RCIM_Constant, ERichCurveTangentMode::RCTM_Auto));

	SequencerCommandBindings->MapAction(
		Commands.ShuttleForward,
		FExecuteAction::CreateSP( this, &FSequencer::ShuttleForward ));

	SequencerCommandBindings->MapAction(
		Commands.RestorePlaybackSpeed,
		FExecuteAction::CreateSP(this, &FSequencer::RestorePlaybackSpeed));

	SequencerCommandBindings->MapAction(
		Commands.ShuttleBackward,
		FExecuteAction::CreateSP( this, &FSequencer::ShuttleBackward ));

	SequencerCommandBindings->MapAction(
		Commands.Pause,
		FExecuteAction::CreateSP( this, &FSequencer::Pause ));

	SequencerCommandBindings->MapAction(
		Commands.SetSelectionRangeEnd,
		FExecuteAction::CreateLambda([this]{ SetSelectionRangeEnd(GetLocalTime().Time); }));

	SequencerCommandBindings->MapAction(
		Commands.SetSelectionRangeStart,
		FExecuteAction::CreateLambda([this]{ SetSelectionRangeStart(GetLocalTime().Time); }));

	SequencerCommandBindings->MapAction(
		Commands.ClearSelectionRange,
		FExecuteAction::CreateLambda([this]{ ClearSelectionRange(); }),
		FCanExecuteAction::CreateLambda(IsSelectionRangeNonEmpty));

	SequencerCommandBindings->MapAction(
		Commands.SelectKeysInSelectionRange,
		FExecuteAction::CreateSP(this, &FSequencer::SelectInSelectionRange, true, false, false),
		FCanExecuteAction::CreateLambda(IsSelectionRangeNonEmpty));

	SequencerCommandBindings->MapAction(
		Commands.SelectSectionsInSelectionRange,
		FExecuteAction::CreateSP(this, &FSequencer::SelectInSelectionRange, false, true, false),
		FCanExecuteAction::CreateLambda(IsSelectionRangeNonEmpty));

	SequencerCommandBindings->MapAction(
		Commands.SelectMarkersInSelectionRange,
		FExecuteAction::CreateSP(this, &FSequencer::SelectInSelectionRange, false, false, true),
		FCanExecuteAction::CreateLambda(IsSelectionRangeNonEmpty));

	SequencerCommandBindings->MapAction(
		Commands.SelectAllInSelectionRange,
		FExecuteAction::CreateSP(this, &FSequencer::SelectInSelectionRange, true, true, true),
		FCanExecuteAction::CreateLambda(IsSelectionRangeNonEmpty));

	SequencerCommandBindings->MapAction(
		Commands.SelectForward,
		FExecuteAction::CreateSP(this, &FSequencer::SelectForward));

	SequencerCommandBindings->MapAction(
		Commands.SelectBackward,
		FExecuteAction::CreateSP(this, &FSequencer::SelectBackward));

	SequencerCommandBindings->MapAction(
		Commands.SelectNone,
		FExecuteAction::CreateSP(this, &FSequencer::EmptySelection));

	SequencerCommandBindings->MapAction(
		Commands.StepToNextShot,
		FExecuteAction::CreateSP( this, &FSequencer::StepToNextShot ) );

	SequencerCommandBindings->MapAction(
		Commands.StepToPreviousShot,
		FExecuteAction::CreateSP( this, &FSequencer::StepToPreviousShot ) );

	SequencerCommandBindings->MapAction(
		Commands.NavigateForward,
		FExecuteAction::CreateLambda([this] { NavigateForward(); }),
		FCanExecuteAction::CreateLambda([this] { return CanNavigateForward(); }));

	SequencerCommandBindings->MapAction(
		Commands.NavigateBackward,
		FExecuteAction::CreateLambda([this] { NavigateBackward(); }),
		FCanExecuteAction::CreateLambda([this] { return CanNavigateBackward(); }));

	SequencerCommandBindings->MapAction(
		Commands.SetStartPlaybackRange,
		FExecuteAction::CreateLambda([this] { SetPlaybackStart(); }) );

	SequencerCommandBindings->MapAction(
		Commands.FocusPlaybackTime,
		FExecuteAction::CreateSP(this, &FSequencer::FocusPlaybackTime));

	SequencerCommandBindings->MapAction(
		Commands.ResetViewRange,
		FExecuteAction::CreateSP( this, &FSequencer::ResetViewRange ) );

	SequencerCommandBindings->MapAction(
		Commands.ZoomToFit,
		FExecuteAction::CreateSP( this, &FSequencer::ZoomToFit ) );

	SequencerCommandBindings->MapAction(
		Commands.ZoomToFitSelectionRange,
		FExecuteAction::CreateSP( this, &FSequencer::ZoomToFitSelectionRange) );

	SequencerCommandBindings->MapAction(
		Commands.ZoomInViewRange,
		FExecuteAction::CreateSP( this, &FSequencer::ZoomInViewRange ),
		FCanExecuteAction(),
		EUIActionRepeatMode::RepeatEnabled );

	SequencerCommandBindings->MapAction(
		Commands.ZoomOutViewRange,
		FExecuteAction::CreateSP( this, &FSequencer::ZoomOutViewRange ),		
		FCanExecuteAction(),
		EUIActionRepeatMode::RepeatEnabled );

	SequencerCommandBindings->MapAction(
		Commands.SetEndPlaybackRange,
		FExecuteAction::CreateLambda([this] { SetPlaybackEnd(); }) );

	SequencerCommandBindings->MapAction(
		Commands.SetSelectionRangeToNextShot,
		FExecuteAction::CreateSP( this, &FSequencer::SetSelectionRangeToShot, true ),
		FCanExecuteAction::CreateSP( this, &FSequencer::IsViewingRootSequence ) );

	SequencerCommandBindings->MapAction(
		Commands.SetSelectionRangeToPreviousShot,
		FExecuteAction::CreateSP( this, &FSequencer::SetSelectionRangeToShot, false ),
		FCanExecuteAction::CreateSP( this, &FSequencer::IsViewingRootSequence ) );

	SequencerCommandBindings->MapAction(
		Commands.SetPlaybackRangeToAllShots,
		FExecuteAction::CreateSP( this, &FSequencer::SetPlaybackRangeToAllShots ),
		FCanExecuteAction::CreateSP( this, &FSequencer::IsViewingRootSequence ) );

	SequencerCommandBindings->MapAction(
		Commands.RefreshUI,
		FExecuteAction::CreateSP( this, &FSequencer::RefreshUI));

	SequencerCommandBindings->MapAction(
		Commands.ToggleLimitViewportSelection,
		FExecuteAction::CreateSP(this, &FSequencer::ToggleLimitViewportSelection),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FSequencer::IsViewportSelectionLimited));

	if (HostCapabilities.bSupportsSidebar)
	{
		SequencerCommandBindings->MapAction(
			Commands.ToggleSidebarVisible,
			FExecuteAction::CreateSP(this, &FSequencer::ToggleSidebar),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FSequencer::IsSidebarVisible));

		SequencerCommandBindings->MapAction(
			Commands.ToggleSelectionDetailsPoppedOut,
			FExecuteAction::CreateSP(this, &FSequencer::ToggleSelectionDetailsPoppedOut),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FSequencer::IsSelectionDetailsPoppedOut));
	}

	SequencerCommandBindings->MapAction(
		Commands.ToggleSidebarSelectionDrawerOpen,
		FExecuteAction::CreateSP(this, &FSequencer::ToggleSidebarSelectionDrawer));

	SequencerCommandBindings->MapAction(
		Commands.ToggleSidebarDrawerDock,
		FExecuteAction::CreateSP(this, &FSequencer::ToggleSidebarDrawerDocked));

	SequencerCommandBindings->MapAction(
		Commands.AlignSelectionToPlayhead,
		FExecuteAction::CreateSP(this, &FSequencer::AlignSelectionToPlayhead),
		FCanExecuteAction::CreateSP(this, &FSequencer::CanAlignSelectionToPlayhead));

	SequencerCommandBindings->MapAction(
		Commands.ToggleTrackSelectionPin,
		FExecuteAction::CreateSP(this, &FSequencer::ToggleTrackSelectionPin));

	SequencerCommandBindings->MapAction(
		Commands.ToggleTrackSelectionLock,
		FExecuteAction::CreateSP(this, &FSequencer::ToggleTrackSelectionLock));

	SequencerCommandBindings->MapAction(
		Commands.ToggleTrackSelectionDeactive,
		FExecuteAction::CreateSP(this, &FSequencer::ToggleTrackSelectionDeactive));

	SequencerCommandBindings->MapAction(
		Commands.ToggleTrackSelectionMute,
		FExecuteAction::CreateSP(this, &FSequencer::ToggleTrackSelectionMute));

	SequencerCommandBindings->MapAction(
		Commands.ToggleTrackSelectionSolo,
		FExecuteAction::CreateSP(this, &FSequencer::ToggleTrackSelectionSolo));

	// bind widget specific commands
	SequencerWidget->BindCommands(SequencerCommandBindings);
	
	ViewModel->GetOutlinerFilterArea()->BindCommands(SequencerCommandBindings);
	
	// If this sequencer supports a curve editor, let's add bindings for it. 
	// As FOutlinerFilterAreaManager::BindCommands binds the shared filter bar commands, which the curve editor also binds to, this must happen after.
	if (FSequencerCurveEditorApp* CurveEditorIntegration = FSequencerCurveEditorApp::Get(*this))
	{
		// We want a subset of the commands to work in the Curve Editor too, but bound to our functions. This minimizes code duplication
		// while also freeing us up from issues that result from Sequencer already using two lists (for which our commands might be spread
		// across both lists which makes a direct copy like it already uses difficult).
		CurveEditorIntegration->BindCommands();
	}
}

void FSequencer::BuildAddTrackMenu(class FMenuBuilder& MenuBuilder)
{
	if (IsLevelEditorSequencer())
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("LoadRecording", "Load Recorded Data"),
			LOCTEXT("LoadRecordingDataTooltip", "Load in saved data from a previous recording."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("ContentBrowser.AssetTreeFolderOpen")),
			FUIAction(FExecuteAction::CreateRaw(this, &FSequencer::OnLoadRecordedData)));
	}

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AddFolder", "Add Folder"),
		LOCTEXT("AddFolderToolTip", "Adds a new folder track."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("ContentBrowser.AssetTreeFolderOpen")),
		FUIAction(FExecuteAction::CreateRaw(this, &FSequencer::AddFolder)));

	TArray<TSharedPtr<ISequencerTrackEditor>> SortedAndFilteredTrackEditors = TrackEditors;

	// Filter unsupported
	for (int32 Index = SortedAndFilteredTrackEditors.Num()-1; Index >= 0; --Index)
	{
		if (!SortedAndFilteredTrackEditors[Index]->SupportsSequence(GetFocusedMovieSceneSequence()))
		{
			SortedAndFilteredTrackEditors.RemoveAtSwap(Index, 1, EAllowShrinking::No);
		}
	}

	// Sort by name
	SortedAndFilteredTrackEditors.Sort([](const TSharedPtr<ISequencerTrackEditor>& InA, const TSharedPtr<ISequencerTrackEditor>& InB)
		{
			return InA->GetDisplayName().CompareTo(InB->GetDisplayName()) < 0;
		});

	for (TSharedPtr<ISequencerTrackEditor> TrackEditor : SortedAndFilteredTrackEditors)
	{
		TrackEditor->BuildPinnedAddTrackMenu(MenuBuilder);
	}

	MenuBuilder.AddMenuSeparator(NAME_None);

	for (const TSharedPtr<ISequencerTrackEditor>& TrackEditor : SortedAndFilteredTrackEditors)
	{
		TrackEditor->BuildAddTrackMenu(MenuBuilder);
	}
}

void FSequencer::BuildAddObjectBindingsMenu(FMenuBuilder& MenuBuilder)
{
	TArray<TSharedPtr<ISequencerEditorObjectBinding>> SortedObjectBindings = ObjectBindings;
	SortedObjectBindings.Sort([](const TSharedPtr<ISequencerEditorObjectBinding>& InA, const TSharedPtr<ISequencerEditorObjectBinding>& InB) -> bool
		{
			return InA->GetDisplayName().CompareTo(InB->GetDisplayName()) < 0;
		});

	for (const TSharedPtr<ISequencerEditorObjectBinding>& ObjectBinding : SortedObjectBindings)
	{
		if (ObjectBinding->SupportsSequence(GetFocusedMovieSceneSequence()))
		{
			ObjectBinding->BuildSequencerAddMenu(MenuBuilder);
		}
	}
}

void FSequencer::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& InObjectBindings, const UClass* ObjectClass)
{
	for (int32 i = 0; i < TrackEditors.Num(); ++i)
	{
		TrackEditors[i]->BuildObjectBindingTrackMenu(MenuBuilder, InObjectBindings, ObjectClass);
	}
}

void FSequencer::BuildAddSelectedToFolderMenu(FMenuBuilder& MenuBuilder)
{
	using namespace UE::Sequencer;

	MenuBuilder.AddMenuEntry(
		LOCTEXT("MoveNodesToNewFolder", "New Folder"),
		LOCTEXT("MoveNodesToNewFolderTooltip", "Create a new folder and adds the selected nodes"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetTreeFolderOpen"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FSequencer::MoveSelectedNodesToNewFolder),
			FCanExecuteAction::CreateLambda( [this]{ return (GetSelectedNodesToMove().Num() > 0); })));
		
	UMovieSceneSequence* FocusedMovieSceneSequence = GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = FocusedMovieSceneSequence ? FocusedMovieSceneSequence->GetMovieScene() : nullptr;
	if (MovieScene)
	{
		TSharedRef<TArray<UMovieSceneFolder*>> ExcludedFolders = MakeShared<TArray<UMovieSceneFolder*> >();
		for (TViewModelPtr<FFolderModel> FolderNode : ViewModel->GetSelection()->Outliner.Filter<FFolderModel>())
		{
			if (FolderNode->CanDrag())
			{
				ExcludedFolders->Add(FolderNode->GetFolder());
			}
		}

		// Copy the list of root folders and remove any currently dragged folders. The rest represents the
		// list of possible folders to move the selection into.
		TArray<UMovieSceneFolder*> ChildFolders;
		MovieScene->GetRootFolders(ChildFolders);
		for (int32 Index = 0; Index < ChildFolders.Num(); ++Index)
		{
			if (ExcludedFolders->Contains(ChildFolders[Index]))
			{
				ChildFolders.RemoveAt(Index);
				--Index;
			}
		}

		if (ChildFolders.Num() > 0)
		{
			MenuBuilder.AddMenuSeparator();
		}

		for (UMovieSceneFolder* Folder : ChildFolders)
		{
			BuildAddSelectedToFolderMenuEntry(MenuBuilder, ExcludedFolders, Folder);
		}
	}
}

void FSequencer::BuildAddSelectedToFolderSubMenu(FMenuBuilder& InMenuBuilder, TSharedRef<TArray<UMovieSceneFolder*> >InExcludedFolders, UMovieSceneFolder* InFolder, TArray<UMovieSceneFolder*> InChildFolders)
{
	InMenuBuilder.AddMenuEntry(
		LOCTEXT("MoveNodesHere", "Move Here"),
		LOCTEXT("MoveNodesHereTooltip", "Move the selected nodes to this existing folder"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &FSequencer::MoveSelectedNodesToFolder, InFolder)));

	if (InChildFolders.Num() > 0)
	{
		InMenuBuilder.AddSeparator();

		for (UMovieSceneFolder* Folder : InChildFolders)
		{
			BuildAddSelectedToFolderMenuEntry(InMenuBuilder, InExcludedFolders, Folder);
		}
	}
}

void FSequencer::BuildAddSelectedToFolderMenuEntry(FMenuBuilder& InMenuBuilder, TSharedRef<TArray<UMovieSceneFolder*> > InExcludedFolders, UMovieSceneFolder* InFolder)
{
	TArray<UMovieSceneFolder*> ChildFolders;

	for (UMovieSceneFolder* Folder : InFolder->GetChildFolders())
	{
		if (!InExcludedFolders->Contains(Folder))
		{
			ChildFolders.Add(Folder);
		}
	}

	if (ChildFolders.Num() > 0)
	{
		InMenuBuilder.AddSubMenu(
			FText::FromName(InFolder->GetFolderName()),
			LOCTEXT("MoveNodesToFolderTooltip2", "Move the selected nodes to an existing folder"),
			FNewMenuDelegate::CreateSP(this, &FSequencer::BuildAddSelectedToFolderSubMenu, InExcludedFolders, InFolder, ChildFolders));
	}
	else
	{
		InMenuBuilder.AddMenuEntry(
			FText::FromName(InFolder->GetFolderName()),
			LOCTEXT("MoveNodesToFolderTooltip1", "Move the selected nodes to this existing folder"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FSequencer::MoveSelectedNodesToFolder, InFolder)));
	}
}

void FSequencer::BuildAddSelectedToNodeGroupMenu(FMenuBuilder& MenuBuilder)
{
	UMovieSceneSequence* FocusedMovieSceneSequence = GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = FocusedMovieSceneSequence ? FocusedMovieSceneSequence->GetMovieScene() : nullptr;
	if (MovieScene)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("NewNodeGroup", "New Group"),
			LOCTEXT("AddNodesToNewNodeGroupTooltip", "Creates a new group and adds the selected nodes"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FSequencer::AddSelectedNodesToNewNodeGroup)));

		if (MovieScene->GetNodeGroups().Num() > 0)
		{
			MenuBuilder.AddMenuSeparator();

			for (UMovieSceneNodeGroup* NodeGroup : MovieScene->GetNodeGroups())
			{
				MenuBuilder.AddMenuEntry(
					FText::FromName(NodeGroup->GetName()),
					LOCTEXT("AddNodesToNodeGroupFormatTooltip", "Adds the selected nodes to this existing group"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &FSequencer::AddSelectedNodesToExistingNodeGroup, NodeGroup)));
			}
		}
	}
}

void FSequencer::BuildSortMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("SortByStartTimeAscending", "Start Time Ascending"),
		LOCTEXT("SortByStartTimeAscendingTooltip", "Sort the selected tracks by start time of the first layer bar ascending"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Profiler.Misc.SortAscending")),
		FUIAction(FExecuteAction::CreateSP(this, &FSequencer::SortSelectedOutlinerItems, false, false)));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SortByStartTimeDescending", "Start Time Descending"),
		LOCTEXT("SortByStartTimeDescendingTooltip", "Sort the selected tracks by start time of the first layer bar descending"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Profiler.Misc.SortDescending")),
		FUIAction(FExecuteAction::CreateSP(this, &FSequencer::SortSelectedOutlinerItems, false, true)));

	MenuBuilder.AddMenuSeparator();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SortBySelectionAscending", "Selection Order Ascending"),
		LOCTEXT("SortBySelectionAscendingTooltip", "Sort the selected tracks by the order selected ascending"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Profiler.Misc.SortAscending")),
		FUIAction(FExecuteAction::CreateSP(this, &FSequencer::SortSelectedOutlinerItems, true, false)));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SortBySelectionDescending", "Selection Order Descending"),
		LOCTEXT("SortBySelectionDescendingTooltip", "Sort the selected tracks by the order selected descending"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Profiler.Misc.SortDescending")),
		FUIAction(FExecuteAction::CreateSP(this, &FSequencer::SortSelectedOutlinerItems, true, true)));
}

void FSequencer::UpdateTimeBases()
{
	UMovieSceneSequence* RootSequencePtr = GetRootMovieSceneSequence();
	UMovieScene*         RootMovieScene  = RootSequencePtr ? RootSequencePtr->GetMovieScene() : nullptr;

	if (RootMovieScene)
	{
		EMovieSceneEvaluationType EvaluationType  = RootMovieScene->GetEvaluationType();
		FFrameRate                TickResolution  = RootMovieScene->GetTickResolution();
		FFrameRate                DisplayRate     = EvaluationType == EMovieSceneEvaluationType::FrameLocked ? RootMovieScene->GetDisplayRate() : TickResolution;

		if (DisplayRate != PlayPosition.GetInputRate())
		{
			bNeedsEvaluate = true;
		}

		// We set the play position in terms of the display rate,
		// but want evaluation ranges in the moviescene's tick resolution
		PlayPosition.SetTimeBase(DisplayRate, TickResolution, EvaluationType);
	}
}

void FSequencer::UnbindFromSequencerSettingsDelegates(USequencerSettings* InSettings)
{
	if (InSettings)
	{
		InSettings->GetOnEvaluateSubSequencesInIsolationChanged().RemoveAll(this);
		InSettings->GetOnShowSelectedNodesOnlyChanged().RemoveAll(this);
		InSettings->GetOnTimeDisplayFormatChanged().RemoveAll(this);
	}
}

void FSequencer::BindToSequencerSettingsDelegates(USequencerSettings* InSettings)
{
	if (InSettings)
	{
		InSettings->GetOnEvaluateSubSequencesInIsolationChanged().AddSP(this, &FSequencer::OnEvaluateSubSequencesInIsolationChanged);
		InSettings->GetOnShowSelectedNodesOnlyChanged().AddSP(this, &FSequencer::OnSelectedNodesOnlyChanged);
		InSettings->GetOnTimeDisplayFormatChanged().AddSP(this, &FSequencer::OnTimeDisplayFormatChanged);
	}
}

void FSequencer::ResetTimeController()
{
	UMovieScene* MovieScene = GetRootMovieSceneSequence()->GetMovieScene();
	switch (MovieScene->GetClockSource())
	{
	case EUpdateClockSource::Audio:    TimeController = MakeShared<FMovieSceneTimeController_AudioClock>();         break;
	case EUpdateClockSource::Platform: TimeController = MakeShared<FMovieSceneTimeController_PlatformClock>();      break;
	case EUpdateClockSource::RelativeTimecode: TimeController = MakeShared<FMovieSceneTimeController_RelativeTimecodeClock>();      break;
	case EUpdateClockSource::Timecode: TimeController = MakeShared<FMovieSceneTimeController_TimecodeClock>();      break;
	case EUpdateClockSource::PlayEveryFrame: TimeController = MakeShared<FMovieSceneTimeController_PlayEveryFrame>();      break;
	case EUpdateClockSource::Custom:   TimeController = MovieScene->MakeCustomTimeController(GetPlaybackContext()); break;
	default:                           TimeController = MakeShared<FMovieSceneTimeController_Tick>();               break;
	}

	if (!TimeController)
	{
		TimeController = MakeShared<FMovieSceneTimeController_Tick>();
	}

	TimeController->PlayerStatusChanged(PlaybackState, GetGlobalTime());
}

void FSequencer::BuildCustomContextMenuForGuid(FMenuBuilder& MenuBuilder, FGuid ObjectBinding)
{
	using namespace UE::Sequencer;

	FSequencerOutlinerViewModel* OutlinerViewModel = ViewModel->GetOutliner()->CastThisChecked<FSequencerOutlinerViewModel>();
	OutlinerViewModel->BuildCustomContextMenuForGuid(MenuBuilder, ObjectBinding);
}

void FSequencer::SetSectionColorTint(TArray<UMovieSceneSection*> Sections, FColor ColorTint)
{
	FScopedTransaction Transaction(LOCTEXT("SetSectionColorTint", "Set Section Color Tint"));

	for (UMovieSceneSection* Section : Sections)
	{
		Section->Modify();
		Section->SetColorTint(ColorTint);
	}
}

bool FSequencer::GetGridMetrics(const float PhysicalWidth, const double InViewStart, const double InViewEnd, double& OutMajorInterval, int32& OutMinorDivisions) const
{
	FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 8);
	TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	// Use the end of the view as the longest number
	FString TickString = GetNumericTypeInterface()->ToString((InViewEnd * GetFocusedDisplayRate()).FrameNumber.Value);
	FVector2D MaxTextSize = FontMeasureService->Measure(TickString, SmallLayoutFont);

	static float MajorTickMultiplier = 2.f;

	float MinTickPx = MaxTextSize.X + 5.f;
	float DesiredMajorTickPx = MaxTextSize.X * MajorTickMultiplier;

	if (PhysicalWidth > 0)
	{
		return GetFocusedDisplayRate().ComputeGridSpacing(
			PhysicalWidth / (InViewEnd - InViewStart),
			OutMajorInterval,
			OutMinorDivisions,
			MinTickPx,
			DesiredMajorTickPx);
	}

	return false;
}

double FSequencer::GetDisplayRateDeltaFrameCount() const
{
	return GetFocusedTickResolution().AsDecimal() * GetFocusedDisplayRate().AsInterval();
}

void FSequencer::RecompileDirtyDirectors()
{
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");

	TSet<UMovieSceneSequence*> AllSequences;

	// Gather all sequences in the hierarchy
	if (UMovieSceneSequence* Sequence = RootSequence.Get())
	{
		AllSequences.Add(Sequence);
	}

	const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(RootTemplateInstance.GetCompiledDataID());
	if (Hierarchy)
	{
		for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
		{
			if (UMovieSceneSequence* Sequence = Pair.Value.GetSequence())
			{
				AllSequences.Add(Sequence);
			}
		}
	}

	// Recompile them all if they are dirty
	for (UMovieSceneSequence* Sequence : AllSequences)
	{
		FMovieSceneSequenceEditor* SequenceEditor = SequencerModule.FindSequenceEditor(Sequence->GetClass());
		UBlueprint*                DirectorBP     = SequenceEditor ? SequenceEditor->FindDirectorBlueprint(Sequence) : nullptr;

		if (DirectorBP && (DirectorBP->Status == BS_Unknown || DirectorBP->Status == BS_Dirty))
		{
			FKismetEditorUtilities::CompileBlueprint(DirectorBP);
		}
	}
}

void FSequencer::SetDisplayName(FGuid Binding, const FText& InDisplayName)
{
	using namespace UE::Sequencer;

	for (TViewModelPtr<FObjectBindingModel> ObjectBindingNode : ViewModel->GetSelection()->Outliner.Filter<FObjectBindingModel>())
	{
		FGuid Guid = ObjectBindingNode->GetObjectGuid();
		if (Guid == Binding)
		{
			ObjectBindingNode->Rename(InDisplayName);
			break;
		}
	}
}

FText FSequencer::GetDisplayName(FGuid Binding)
{
	using namespace UE::Sequencer;

	for (TViewModelPtr<FObjectBindingModel> ObjectBindingNode : ViewModel->GetSelection()->Outliner.Filter<FObjectBindingModel>())
	{
		FGuid Guid = ObjectBindingNode->GetObjectGuid();
		if (Guid == Binding)
		{
			return ObjectBindingNode->GetLabel();
		}
	}
	return FText();
}

void FSequencer::OnCurveModelDisplayChanged(FCurveModel *InCurveModel, bool bDisplayed, const FCurveEditor* InCurveEditor)
{
	OnCurveDisplayChanged.Broadcast(InCurveModel, bDisplayed, InCurveEditor);
}

void FSequencer::ToggleAsyncEvaluation()
{
	UMovieSceneSequence* Sequence = GetRootMovieSceneSequence();

	EMovieSceneSequenceFlags NewFlags = Sequence->GetFlags();
	NewFlags ^= EMovieSceneSequenceFlags::BlockingEvaluation;

	FScopedTransaction Transaction(EnumHasAnyFlags(NewFlags, EMovieSceneSequenceFlags::BlockingEvaluation) ? LOCTEXT("DisableAsyncEvaluation", "Disable Async Evaluation") : LOCTEXT("EnableAsyncEvaluation", "Enable Async Evaluation"));

	Sequence->Modify();
	Sequence->SetSequenceFlags(NewFlags);
}

bool FSequencer::UsesAsyncEvaluation()
{
	return !EnumHasAnyFlags(GetRootMovieSceneSequence()->GetFlags(), EMovieSceneSequenceFlags::BlockingEvaluation);
}

void FSequencer::ToggleDynamicWeighting()
{
	UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();

	EMovieSceneSequenceFlags NewFlags = Sequence->GetFlags();
	NewFlags ^= EMovieSceneSequenceFlags::DynamicWeighting;

	FScopedTransaction Transaction(EnumHasAnyFlags(NewFlags, EMovieSceneSequenceFlags::DynamicWeighting) ? LOCTEXT("EnableDynamicWeighting", "Enable Dynamic Weighting") : LOCTEXT("DisableDynamicWeighting", "Disable Dynamic Weighting"));

	Sequence->Modify();
	Sequence->SetSequenceFlags(NewFlags);
}

bool FSequencer::UsesDynamicWeighting()
{
	return EnumHasAnyFlags(GetFocusedMovieSceneSequence()->GetFlags(), EMovieSceneSequenceFlags::DynamicWeighting);
}

void FSequencer::ToggleLoopCuts()
{
	UMovieSceneSequence* Sequence = GetFocusedMovieSceneSequence();

	EMovieSceneSequenceFlags NewFlags = Sequence->GetFlags();
	NewFlags ^= EMovieSceneSequenceFlags::LoopCuts;

	FScopedTransaction Transaction(EnumHasAnyFlags(NewFlags, EMovieSceneSequenceFlags::LoopCuts) ? LOCTEXT("EnableLoopCuts", "Enable Loop Cuts") : LOCTEXT("DisableLoopCuts", "Disable Loop Cuts"));

	Sequence->Modify();
	Sequence->SetSequenceFlags(NewFlags);
}

bool FSequencer::UsesLoopCuts()
{
	return EnumHasAnyFlags(GetFocusedMovieSceneSequence()->GetFlags(), EMovieSceneSequenceFlags::LoopCuts);
}

UE::Sequencer::FSequencerSelection& FSequencer::GetSelection()
{
	return *ViewModel->GetSelection();
}

bool FSequencer::ShouldRestoreEditorViewports()
{
	return Settings->GetRestoreOriginalViewportOnCameraCutUnlock();
}

float FSequencer::GetCameraBlendPlayRate()
{
	return PlaybackSpeed;
}

void FSequencer::OnCameraCutUpdated(const UE::MovieScene::FOnCameraCutUpdatedParams& Params)
{
	LastViewTargetCamera = Params.ViewTargetCamera;
	OnCameraCutEvent.Broadcast(Params.ViewTarget, Params.bIsJumpCut);
}

void FSequencer::ToggleLimitViewportSelection()
{
	bSelectionLimited = !bSelectionLimited;
	
	SetViewportSelectionLimited(bSelectionLimited);
}

bool FSequencer::IsViewportSelectionLimited() const
{
	return bSelectionLimited;
}

void FSequencer::SetViewportSelectionLimited(const bool bInSelectionLimited)
{
	bSelectionLimited = bInSelectionLimited;

	if (FSequencerEdMode* const SequencerEdMode = (FSequencerEdMode*)GLevelEditorModeTools().GetActiveMode(FSequencerEdMode::EM_SequencerMode))
	{
		SequencerEdMode->EnableSelectabilityTool(bSelectionLimited);
	}

	OnSelectionLimitedChangedDelegate.Broadcast(bSelectionLimited);
}

bool FSequencer::IsObjectSelectableInViewport(UObject* const InObject)
{
	if (!bSelectionLimited)
	{
		return true;
	}

	if (const IViewportSelectableObject* const SelectableObject = Cast<IViewportSelectableObject>(InObject))
	{
		return SelectableObject->IsSelectable();
	}

	UMovieSceneSequence* const FocusedSequence = GetFocusedMovieSceneSequence();
	if (!IsValid(FocusedSequence))
	{
		return true;
	}

	const TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = GetSharedPlaybackState();

	FMovieSceneEvaluationState* const EvaluationState = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>();
	if (!EvaluationState)
	{
		return true;
	}

	const UMovieSceneSequence* OutSequence = nullptr;

	// Early out on first sequence the object is found in
	ForEachSubSequenceRecursively(FocusedSequence,
		[this, InObject, &SharedPlaybackState, EvaluationState, &OutSequence](UMovieSceneSequence* const InCurrentSequence)
		{
			const FMovieSceneSequenceID SequenceID = EvaluationState->FindSequenceId(InCurrentSequence);
			const FGuid ObjectGuid = EvaluationState->FindCachedObjectId(*InObject, SequenceID, SharedPlaybackState);
			if (ObjectGuid.IsValid())
			{
				OutSequence = InCurrentSequence;
				return false; // Stop looping recursively
			}
			return true; // Continue loop recursively
		});

	return IsValid(OutSequence);
}

void FSequencer::ForEachSubSequenceRecursively(UMovieSceneSequence* const InSequence, const TFunctionRef<bool(UMovieSceneSequence* const InCurrentSequence)>& InFunction)
{
	if (!IsValid(InSequence) || !InFunction(InSequence))
	{
		return;
	}

	UMovieScene* const MovieScene = InSequence->GetMovieScene();
	if (!IsValid(MovieScene))
	{
		return;
	}

	// Converting to TSet as GetAllSections() seems to return multiples of the same object
	const TSet<UMovieSceneSection*> AllSections = TSet<UMovieSceneSection*>(MovieScene->GetAllSections());

	for (UMovieSceneSection* const Section : AllSections)
	{
		UMovieSceneSubSection* const SubSection = Cast<UMovieSceneSubSection>(Section);
		if (!IsValid(SubSection))
		{
			continue;
		}

		UMovieSceneSequence* const Sequence = SubSection->GetSequence();
		if (!IsValid(Sequence) || !InFunction(Sequence))
		{
			continue;
		}

		ForEachSubSequenceRecursively(Sequence, InFunction);
	}
}

ISequencer::FOnViewportSelectionLimitedChanged& FSequencer::OnViewportSelectionLimitedChanged()
{
	return OnSelectionLimitedChangedDelegate;
}

bool FSequencer::RegisterDrawer(FSidebarDrawerConfig&& InDrawerConfig)
{
	if (SequencerWidget.IsValid())
	{
		return SequencerWidget->RegisterDrawer(MoveTemp(InDrawerConfig));
	}
	return false;
}

bool FSequencer::UnregisterDrawer(const FName InDrawerId)
{
	if (SequencerWidget.IsValid())
	{
		return SequencerWidget->UnregisterDrawer(InDrawerId);
	}
	return false;
}

bool FSequencer::RegisterDrawerSection(const FName InDrawerId, const TSharedPtr<ISidebarDrawerContent>& InSection)
{
	if (SequencerWidget.IsValid())
	{
		return SequencerWidget->RegisterDrawerSection(InDrawerId, InSection);
	}
	return false;
}

bool FSequencer::UnregisterDrawerSection(const FName InDrawerId, const FName InSectionId)
{
	if (SequencerWidget.IsValid())
	{
		return SequencerWidget->UnregisterDrawerSection(InDrawerId, InSectionId);
	}
	return false;
}

bool FSequencer::IsSimpleView() const
{
	return SequencerWidget.IsValid() ? SequencerWidget->IsSimpleView() : false;
}

void FSequencer::EnableSimpleView(const bool bInEnable)
{
	if (SequencerWidget.IsValid())
	{
		SequencerWidget->EnableSimpleView(bInEnable);
	}
}

void FSequencer::ToggleSimpleView()
{
	if (SequencerWidget.IsValid())
	{
		SequencerWidget->ToggleSimpleView();
	}
}

void FSequencer::ToggleSidebar()
{
	if (SequencerWidget.IsValid())
	{
		SequencerWidget->ToggleSidebarVisible();
	}
}

bool FSequencer::IsSidebarVisible() const
{
	return SequencerWidget.IsValid() ? SequencerWidget->IsSidebarVisible() : false;
}

void FSequencer::ToggleSelectionDetailsPoppedOut()
{
	if (SequencerWidget.IsValid())
	{
		SequencerWidget->ToggleSelectionDetailsPoppedOut();
	}
}

void FSequencer::ToggleSidebarSelectionDrawer()
{
	if (SequencerWidget.IsValid())
	{
		SequencerWidget->ToggleSidebarSelectionDrawerOpen();
	}
}

bool FSequencer::IsSelectionDetailsPoppedOut() const
{
	return SequencerWidget.IsValid() ? SequencerWidget->IsSelectionDetailsPoppedOut() : false;
}

void FSequencer::SetSelectionDetailsPoppedOut(const bool bInPoppedOut)
{
	if (SequencerWidget.IsValid())
	{
		SequencerWidget->SetSelectionDetailsPoppedOut(bInPoppedOut);
	}
}

void FSequencer::ToggleSidebarDrawerDocked()
{
	if (SequencerWidget.IsValid())
	{
		SequencerWidget->ToggleSidebarDrawerDock();
	}
}

FText FSequencer::GetSidebarSelectionDrawerToolTipText() const
{
	const TSharedRef<const FInputChord> DrawerOpenActiveChord = FSequencerCommands::Get().ToggleSidebarSelectionDrawerOpen->GetFirstValidChord();
	const TSharedRef<const FInputChord> DrawerDockActiveChord = FSequencerCommands::Get().ToggleSidebarDrawerDock->GetFirstValidChord();

	FText ToolTipText = LOCTEXT("SelectionDetailsPanelTooltip", "Open Sequencer selection details panel.");

	if (DrawerOpenActiveChord->IsValidChord() || DrawerDockActiveChord->IsValidChord())
	{
		ToolTipText = FText::Format(LOCTEXT("ExtendedSelectionDetailsPanelTooltip", "{0}\n"), ToolTipText);
	}

	if (DrawerOpenActiveChord->IsValidChord())
	{
		ToolTipText = FText::Format(LOCTEXT("ExtendedSelectionDrawerOpenDetailsPanelTooltip", "{0}\n"
			"{1} to toggle the drawer open or closed")
			, ToolTipText
			, DrawerOpenActiveChord->GetInputText(true));
	}

	if (DrawerDockActiveChord->IsValidChord())
	{
		ToolTipText = FText::Format(LOCTEXT("ExtendedSelectionDrawerDockDetailsPanelTooltip", "{0}\n"
			"{1} to toggle the drawer docked or undocked")
			, ToolTipText
			, DrawerDockActiveChord->GetInputText(true));
	}

	return ToolTipText;
}

TSharedRef<ISequencerTrackFilters> FSequencer::GetFilterInterface() const
{
	return GetFilterBar();
}

TSharedRef<FSequencerFilterBar> FSequencer::GetFilterBar() const
{
	return ViewModel->GetLinkedFilterViewModelImpl()->GetActiveFilterBarImpl();
}

EVisibility FSequencer::GetPlayRateComboVisibility() const
{
	using namespace UE::Sequencer;

	TViewModelPtr<IClockExtension> ClockExtension = GetViewModel()->GetRootSequenceModel().ImplicitCast();
	if (!ClockExtension || ClockExtension->ShouldShowPlayRateCombo(AsShared()))
	{
		return EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

void FSequencer::SortSelectedOutlinerItems(const bool bInSortBySelectOrder, const bool bInDescending)
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencerSelection> SequencerSelection = ViewModel->GetSelection();
	if (!SequencerSelection.IsValid())
	{
		return;
	}

	TArray<TViewModelPtr<IOutlinerExtension>> OutlinerItems;
	for (const TViewModelPtr<IOutlinerExtension> Item : SequencerSelection->Outliner)
	{
		OutlinerItems.Add(Item);
	}

	SequencerHelpers::SortOutlinerItems(*this, OutlinerItems, bInSortBySelectOrder, bInDescending);
}

bool FSequencer::CanAlignSelectionToPlayhead() const
{
	return FSequencerSelectionAlignmentUtils::CanAlignSelection(*this);
}

void FSequencer::AlignSelectionToPlayhead() const
{
	FSequencerSelectionAlignmentUtils::AlignSelectionToPlayhead(*this);
}

void FSequencer::ToggleTrackSelectionPin()
{
	using namespace UE::Sequencer;

	FOutlinerSelection& OutlinerSelection = GetSelection().Outliner;

	const bool bAllSame = Algo::AllOf(OutlinerSelection, [](const FViewModelPtr& InSelection)
		{
			for (const TViewModelPtr<IPinnableExtension>& Extension : InSelection->GetDescendantsOfType<IPinnableExtension>(true))
			{
				if (Extension.IsValid() && Extension->IsPinned())
				{
					return true;
				}
			}
			return false;
		});

	const bool bNewState = !bAllSame;

	bool bAnyChanged = false;

	for (const TViewModelPtr<IOutlinerExtension>& OutlinerItem : OutlinerSelection)
	{
		for (const TViewModelPtr<IPinnableExtension>& Extension : OutlinerItem.AsModel()->GetDescendantsOfType<IPinnableExtension>(true))
		{
			if (Extension->IsPinned() != bNewState)
			{
				Extension->SetPinned(bNewState);
				NodeTree->SavePinnedState(*Extension.AsModel(), bNewState);
				bAnyChanged = true;
			}
		}
	}

	if (bAnyChanged)
	{
		RefreshTree();
	}
}

void FSequencer::ToggleTrackSelectionLock()
{
	using namespace UE::Sequencer;

	FOutlinerSelection& OutlinerSelection = GetSelection().Outliner;

	const bool bAllSame = Algo::AllOf(OutlinerSelection, [](const FViewModelPtr& InSelection)
		{
			for (const TViewModelPtr<ILockableExtension>& Extension : InSelection->GetDescendantsOfType<ILockableExtension>(true))
			{
				if (Extension.IsValid() && Extension->GetLockState() != ELockableLockState::None)
				{
					return true;
				}
			}
			return false;
		});

	const bool bNewState = !bAllSame;

	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "ToggleLock", "Toggle Lock"));

	bool bAnyChanged = false;

	for (const TViewModelPtr<IOutlinerExtension>& OutlinerItem : OutlinerSelection)
	{
		for (const TViewModelPtr<ILockableExtension>& Extension : OutlinerItem.AsModel()->GetDescendantsOfType<ILockableExtension>(true))
        {
			const bool bThisState = Extension->GetLockState() == ELockableLockState::Locked;
			if (bThisState != bNewState)
			{
				Extension->SetIsLocked(bNewState);
				bAnyChanged = true;
			}
        }
	}

	if (bAnyChanged)
	{
		RefreshTree();
	}
}

void FSequencer::ToggleTrackSelectionDeactive()
{
	using namespace UE::Sequencer;

	FOutlinerSelection& OutlinerSelection = GetSelection().Outliner;

	const bool bAllSame = Algo::AllOf(OutlinerSelection, [](const FViewModelPtr& InSelection)
		{
			for (const TViewModelPtr<IDeactivatableExtension>& Extension : InSelection->GetDescendantsOfType<IDeactivatableExtension>(true))
			{
				if (Extension.IsValid() && Extension->IsDeactivated())
				{
					return true;
				}
			}
			return false;
		});

	const bool bNewState = !bAllSame;

	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "ToggleDeactivate", "Toggle Deactivate"));

	bool bAnyChanged = false;

	for (const TViewModelPtr<IOutlinerExtension>& OutlinerItem : OutlinerSelection)
	{
		for (const TViewModelPtr<IDeactivatableExtension>& Extension : OutlinerItem.AsModel()->GetDescendantsOfType<IDeactivatableExtension>(true))
		{
			if (Extension->IsDeactivated() != bNewState)
			{
				Extension->SetIsDeactivated(bNewState);
				bAnyChanged = true;
			}
		}
	}

	if (bAnyChanged)
	{
		RefreshTree();
	}
}

void FSequencer::ToggleTrackSelectionMute()
{
	using namespace UE::Sequencer;

	FOutlinerSelection& OutlinerSelection = GetSelection().Outliner;

	const bool bAllSame = Algo::AllOf(OutlinerSelection, [](const FViewModelPtr& InSelection)
		{
			for (const TViewModelPtr<IMutableExtension>& Extension : InSelection->GetDescendantsOfType<IMutableExtension>(true))
			{
				if (Extension.IsValid() && Extension->IsMuted())
				{
					return true;
				}
			}
			return false;
		});

	const bool bNewState = !bAllSame;

	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "ToggleMute", "Toggle Mute"));

	bool bAnyChanged = false;

	for (const TViewModelPtr<IOutlinerExtension>& OutlinerItem : OutlinerSelection)
	{
		for (const TViewModelPtr<IMutableExtension>& Extension : OutlinerItem.AsModel()->GetDescendantsOfType<IMutableExtension>(true))
		{
			if (Extension->IsMuted() != bNewState)
			{
				Extension->SetIsMuted(bNewState);
				bAnyChanged = true;
			}
		}
	}

	if (bAnyChanged)
	{
		RefreshTree();
	}
}

void FSequencer::ToggleTrackSelectionSolo()
{
	using namespace UE::Sequencer;

	FOutlinerSelection& OutlinerSelection = GetSelection().Outliner;

	const bool bAllSame = Algo::AllOf(OutlinerSelection, [](const FViewModelPtr& InSelection)
		{
			for (const TViewModelPtr<ISoloableExtension>& Extension : InSelection->GetDescendantsOfType<ISoloableExtension>(true))
			{
				if (Extension.IsValid() && Extension->IsSolo())
				{
					return true;
				}
			}
			return false;
		});

	const bool bNewState = !bAllSame;

	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "ToggleSolo", "Toggle Solo"));

	bool bAnyChanged = false;

	for (const TViewModelPtr<IOutlinerExtension>& OutlinerItem : OutlinerSelection)
	{
		for (const TViewModelPtr<ISoloableExtension>& Extension : OutlinerItem.AsModel()->GetDescendantsOfType<ISoloableExtension>(true))
		{
			if (Extension->IsSolo() != bNewState)
			{
				Extension->SetIsSoloed(bNewState);
				bAnyChanged = true;
			}
		}
	}

	if (bAnyChanged)
	{
		RefreshTree();
	}
}

TSharedPtr<UE::Sequencer::SOutlinerView> FSequencer::GetOutlinerViewWidget() const
{
	return SequencerWidget->GetTreeView();
}

#undef LOCTEXT_NAMESPACE
