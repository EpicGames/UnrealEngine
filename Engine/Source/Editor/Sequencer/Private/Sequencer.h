// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Misc/Guid.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "UObject/WeakInterfacePtr.h"
#include "Widgets/SWidget.h"
#include "SequencerNodeTree.h"
#include "SequencerTimeDomainOverride.h"
#include "UObject/GCObject.h"
#include "MovieSceneMarkedFrame.h"
#include "MovieSceneSequenceID.h"
#include "IMovieScenePlayer.h"
#include "ITimeSlider.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Animation/CurveHandle.h"
#include "Animation/CurveSequence.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "TickableEditorObject.h"
#include "EditorUndoClient.h"
#include "KeyPropertyParams.h"
#include "ISequencer.h"
#include "ISequencerModule.h"
#include "ISequencerObjectChangeListener.h"
#include "SequencerSelectionPreview.h"
#include "SequencerCustomizationManager.h"
#include "ITransportControl.h"
#include "Evaluation/CameraCutPlaybackCapability.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "LevelEditor.h"
#include "AcquiredResources.h"
#include "SequencerSettings.h"
#include "Curves/RichCurve.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "SequencerTimeChangeUndoRedoProxy.h"
#include "Capabilities/CameraCutViewTargetCacheCapability.h"
#include "Filters/ISequencerTrackFilters.h"

class AActor;
class ACameraActor;
class APlayerController;
class FLevelEditorViewportClient;
class FMenuBuilder;
class FMovieSceneClipboard;
class FSequencerPropertyKeyedStatusHandler;
class FViewportClient;
class IDetailKeyframeHandler;
class IAssetViewport;
class IMenu;
class FCurveEditor;
class ISequencerEditTool;
class FSequencerKeyCollection;
class FObjectBindingTagCache;
class ISequencerDecorationEditor;
class ISequencerTrackEditor;
class ISequencerEditorObjectBinding;
class SSequencer;
class ISidebarDrawerContent;
class ULevel;
class UMovieSceneSequence;
class UMovieSceneSubSection;
class USequencerSettings;
class UMovieSceneCopyableBinding;
class UMovieSceneCompiledDataManager;
class UMovieSceneCopyableTrack;
class UMovieSceneNodeGroup;
class UMovieSceneTracksSettings;
template <class TClass> class TSubclassOf;
class UMovieSceneCustomBinding;

struct FMovieSceneTimeController;
struct FMovieSceneSequencePlaybackParams;
struct FMovieScenePossessable;
struct FTransformData;
struct FKeyAttributes;
struct FNotificationInfo;
struct FEditorViewportViewModifierParams;
struct FMovieSceneMarkedFrame;
struct FSidebarDrawerConfig;
template<typename T> struct STemporarilyFocusedSpinBox;

enum class EMapChangeType : uint8;
enum class ENearestKeyOption : uint8;

namespace UE
{
	namespace MovieScene
	{
		struct FInitialValueCache;
	} // namespace MovieScene

	namespace Sequencer
	{
		class FSequenceModel;
		class FSequencerEditorViewModel;
		class FSequencerEditorViewModelImpl;
		class FSequencerSelection;
		class FViewModel;
		class SSequencerWrapperWidget;
		struct FViewModelTypeID;

		namespace ToolableTimeline
		{
			class FToolableTimeline;
		} // namespace ToolableTimeline

		namespace SimpleView
		{
			class FSimpleViewTimeline;
		} // namespace SimpleView

	} // namespace Sequencer
} // namespace UE


/**
 * Sequencer is the editing tool for MovieScene assets.
 */
class FSequencer final
	: public ISequencer
	, public FGCObject
	, public FEditorUndoClient
	, public FTickableEditorObject
	, public UE::MovieScene::FCameraCutPlaybackCapability
	, private UE::MovieScene::FCameraCutViewTargetCacheCapability
{
	using FViewModel = UE::Sequencer::FViewModel;

	friend class UE::Sequencer::ToolableTimeline::FToolableTimeline;

public:
	SEQUENCER_API static const FName SelectionDrawerId;
	SEQUENCER_API static const FName SelectionDetailsTabId;

	/** Constructor */
	FSequencer();

	/** Virtual destructor */
	virtual ~FSequencer();

public:

	/**
	 * Initializes sequencer
	 *
	 * @param InitParams Initialization parameters.
	 * @param InObjectChangeListener The object change listener to use.
	 * @param TrackEditorDelegates Delegates to call to create auto-key handlers for this sequencer.
	 * @param EditorObjectBindingDelegates Delegates to call to create object bindings for this sequencer.
	 */
	void InitSequencer(const FSequencerInitParams& InitParams, const TSharedRef<ISequencerObjectChangeListener>& InObjectChangeListener, const TArray<FOnCreateTrackEditor>& TrackEditorDelegates, const TArray<FOnCreateDecorationEditor>& DecorationEditorDelegates, const TArray<FOnCreateEditorObjectBinding>& EditorObjectBindingDelegatess, const TArray<FOnCreateOutlinerColumn>& OutlinerColumnDelegates, const TArray<FOnCreateOutlinerIndicator>& OutlinerIndicatorDelegates);

	/**
	 * Reinitializes sequencer after the playback context has changed
	 *
	 * The playback context changes when the user attaches the sequencer to a different world either
	 * explicitly via the world dropdown picker, or indirectly when e.g. starting/stopping PIE.
	 */
	void OnPlaybackContextChanged();

protected:

	void InitRootSequenceInstance();

public:

	/** @return The current view range */
	virtual FAnimatedRange GetViewRange() const override;
	virtual void SetViewRange(TRange<double> NewViewRange, EViewRangeInterpolation Interpolation = EViewRangeInterpolation::Animated) override;

	/** @return The current clamp range */
	FAnimatedRange GetClampRange() const;
	virtual void SetClampRange(TRange<double> InNewClampRange) override;

public:

	virtual TRange<FFrameNumber> GetSelectionRange() const override;

	/**
	 * Set the selection selection range.
	 *
	 * @param Range The new range to set.
	 * @see GetSelectionRange, SetSelectionRangeEnd, SetSelectionRangeStart
	 */
	void SetSelectionRange(TRange<FFrameNumber> Range);
	
	virtual void SetSelectionRangeEnd(FFrameTime EndFrame) override;

	virtual void SetSelectionRangeStart(FFrameTime StartFrame) override;

	/** Clear and reset the selection range. */
	void ClearSelectionRange();

	/** Select all that fall into the current selection range. */
	void SelectInSelectionRange(bool bSelectKeys, bool bSelectSections, bool bSelectMarkers);
	void SelectMarkersInRange(const TRange<FFrameNumber>& InRange);

	/** Select all keys, sections and markers forward from the current time */
	void SelectForward();

	/** Select all keys, sections and markers backward from the current time */
	void SelectBackward();

	/**
	 * Compute a major grid interval and number of minor divisions to display
	 */
	bool GetGridMetrics(const float PhysicalWidth, const double InViewStart, const double InViewEnd, double& OutMajorInterval, int32& OutMinorDivisions) const;

public:

	/**
	 * Get the playback range.
	 *
	 * @return Playback range.
	 * @see SetPlaybackRange, SetPlaybackRangeEnd, SetPlaybackRangeStart
	 */
	virtual TRange<FFrameNumber> GetPlaybackRange() const override;

	/**
	 * Set the playback range.
	 *
	 * @param Range The new range to set.
	 * @see GetPlaybackRange, SetPlaybackRangeEnd, SetPlaybackRangeStart
	 */
	void SetPlaybackRange(TRange<FFrameNumber> Range);
	
	/**
	 * Set the selection range to the next or previous shot's range.
	 *
	 */	
	void SetSelectionRangeToShot(const bool bNextShot);

	/**
	 * Set the playback range to all the shot's playback ranges.
	 *
	 */	
	void SetPlaybackRangeToAllShots();

public:

	bool IsPlaybackRangeLocked() const;
	void TogglePlaybackRangeLocked();
	void FocusPlaybackTime();
	void ResetViewRange();
	void ZoomViewRange(float InZoomDelta);
	void ZoomInViewRange();
	void ZoomOutViewRange();

public:

	/** Gets the tree of nodes which is used to populate the animation outliner. */
	TSharedRef<FSequencerNodeTree> GetNodeTree()
	{
		return NodeTree;
	}
	
	TSharedRef<UE::Sequencer::FSequencerEditorViewModelImpl> GetViewModelImpl() const
	{
		return ViewModel.ToSharedRef();
	}

	FObjectBindingTagCache* GetObjectBindingTagCache()
	{
		return ObjectBindingTagCache.Get();
	}

	bool IsPerspectiveViewportPossessionEnabled() const override
	{
		return bPerspectiveViewportPossessionEnabled;
	}

	bool IsPerspectiveViewportCameraCutEnabled() const override
	{
		return bPerspectiveViewportCameraCutEnabled;
	}

	/** Gets the list of bindings for camera objects. */
	virtual void GetCameraObjectBindings(TArray<FGuid>& OutBindingIDs) override;

	/**
	 * Pops the current focused movie scene from the stack.  The parent of this movie scene will be come the focused one
	 */
	virtual void PopToSequenceInstance( FMovieSceneSequenceIDRef SequenceID ) override;

	/** Deletes the passed in sections. */
	void DeleteSections(const TSet<UMovieSceneSection*> & Sections);

	/** Deletes the currently selected in keys. */
	void DeleteSelectedKeys();

	/** Set interpolation modes. */
	void SetInterpTangentMode(ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode);

	/** Toggle tangent weight mode. */
	void ToggleInterpTangentWeightMode();

	/** Snap the currently selected keys to frame. */
	void SnapToFrame();

	/** Are there keys to snap? */
	bool CanSnapToFrame() const;

	/** Transform the selection */
	void TransformSelection(FFrameTime InDeltaTime, float InScale);

	/** Translate the selection by the time snap interval */
	void TranslateSelection(bool bTranslateLeft);

	/** Stretch time*/
	void StretchTime(FFrameTime InDeltaTime);

	/** Shrink time*/
	void ShrinkTime(FFrameTime InDeltaTime);

	/**
	 * @return Movie scene tools used by the sequencer
	 */
	const TArray<TSharedPtr<ISequencerTrackEditor>>& GetTrackEditors() const
	{
		return TrackEditors;
	}

	/** @return Decoration editors created for this sequencer */
	const TArray<TSharedPtr<ISequencerDecorationEditor>>& GetDecorationEditors() const
	{
		return DecorationEditors;
	}

	virtual ISequencerDecorationEditor* FindDecorationEditor(UClass* DecorationClass) const override;
	virtual TTuple<ISequencerDecorationEditor*, UObject*> FindDecorationOwner(
		const FMovieSceneChannelHandle& ChannelHandle,
		UMovieSceneSection* Section) const override;

private:
	/** Returns a shared no-op decoration editor used as the keying fallback for decorations without a registered editor. Lazily created on first use. */
	ISequencerDecorationEditor* GetOrCreateDefaultDecorationEditor() const;

public:

	/**
	* @return Outliner Columns registered to the sequencer by column name
	*/
	const TMap<FName, TSharedPtr<UE::Sequencer::IOutlinerColumn>>& GetOutlinerColumns() const
	{
		return OutlinerColumns;
	}

	/**
	* @return Outliner Indicator Items registered to the sequencer by column name
	*/
	const TMap<FName, TSharedPtr<UE::Sequencer::IOutlinerIndicatorBuilder>>& GetOutlinerIndicators() const
	{
		return OutlinerIndicators;
	}

public:

	/** @return The set of vertical frames */
	TSet<FFrameNumber> GetVerticalFrames() const;

	/** @return The set of marked frames */
	TArray<FMovieSceneMarkedFrame> GetMarkedFrames() const override;

	TArray<FMovieSceneMarkedFrame> GetGlobalMarkedFrames() const;
	virtual void InvalidateGlobalMarkedFramesCache() override { bGlobalMarkedFramesCached = false; }
	void UpdateGlobalMarkedFramesCache();

	/** Toggle whether to show marked frames globally */
	void ToggleShowMarkedFramesGlobally();

	/** Disables all global marked frames from all sub-sequences */
	void ClearGlobalMarkedFrames();

protected:

	/** Set/Clear a Mark at the current time */
	void ToggleMarkAtPlayPosition();
	void StepToNextMark();
	void StepToPreviousMark();
	bool AreMarkedFramesLocked() const;
	void ToggleMarkedFramesLocked();

	/**
	 * @param InMarkIndex The marked frame index to set
	 * @param InFrameNumber The FrameNumber in Ticks
	 */
	void SetMarkedFrame(int32 InMarkIndex, FFrameNumber InFrameNumber);

	/**
	 * @param	FrameNumber The FrameNumber in Ticks
	 */
	void AddMarkedFrame(FFrameNumber FrameNumber);

	void DeleteAllMarkedFrames();

public:

	/**
	 * Converts the specified possessable GUID to a spawnable
	 *
	 * @param	PossessableGuid		The guid of the possessable to convert
	 */
	void ConvertToSpawnable(TSharedRef<UE::Sequencer::FObjectBindingModel> NodeToBeConverted);

	/**
	 * Converts the specified spawnable GUID to a possessable
	 *
	 * @param	SpawnableGuid		The guid of the spawnable to convert
	 */
	void ConvertToPossessable(TSharedRef<UE::Sequencer::FObjectBindingModel> NodeToBeConverted);

	/**
	 * Converts all the currently selected nodes to be spawnables, if possible
	 */
	void ConvertSelectedNodesToSpawnables();

	/**
	 * Converts all the currently selected nodes to be possessables, if possible
	 */
	void ConvertSelectedNodesToPossessables();

protected:

	/**
	 * Save default spawnable state for the currently selected objects
	 */
	void SaveSelectedNodesSpawnableState();

public:

	/** Called when new actors are dropped in the viewport. */
	void OnNewActorsDropped(const TArray<UObject*>& DroppedObjects, const TArray<AActor*>& DroppedActors);

	/**
	 * Call when an asset is dropped into the sequencer. Will proprogate this
	 * to all tracks, and potentially consume this asset
	 * so it won't be added as a spawnable
	 *
	 * @param DroppedAsset		The asset that is dropped in
	 * @param TargetObjectGuid	Object to be targeted on dropping
	 * @return					If true, this asset should be consumed
	 */
	bool OnHandleAssetDropped( UObject* DroppedAsset, const FGuid& TargetObjectGuid );
	
	/**
	 * Called to delete all moviescene data from a node
	 *
	 * @param NodeToBeDeleted	Node with data that should be deleted
	 * @return true if anything was deleted, otherwise false.
	 */
	bool OnRequestNodeDeleted( TSharedRef<FViewModel> NodeToBeDeleted, const bool bKeepState );

	/** Zooms to the edges of all currently selected sections and keys. */
	void ZoomToFit();

	/** Zooms to the movie scene selection range, or falls back to time bounds if empty. */
	void ZoomToFitSelectionRange();

	/** Gets the overlay fading animation curve lerp. */
	float GetOverlayFadeCurve() const;

	/** Gets the command bindings for the sequencer */
	virtual TSharedPtr<FUICommandList> GetCommandBindings(ESequencerCommandBindings Type = ESequencerCommandBindings::Sequencer) const override
	{
		if (Type == ESequencerCommandBindings::Sequencer)
		{
			return SequencerCommandBindings;
		}
		else if (Type == ESequencerCommandBindings::CurveEditor)
		{
			return CurveEditorSharedBindings;
		}

		return SequencerSharedBindings;
	}

	virtual void SetDisplayName(FGuid InBinding,const FText& InDisplayName) override;
	virtual FText GetDisplayName(FGuid InBinding) override;

	/**
	 * Builds up the sequencer's "Add Track" menu.
	 *
	 * @param MenuBuilder The menu builder to add things to.
	 */
	void BuildAddTrackMenu(FMenuBuilder& MenuBuilder);

	/**
	 * Builds up the object bindings in sequencer's "Add Track" menu.
	 *
	 * @param MenuBuilder The menu builder to add things to.
	 */
	void BuildAddObjectBindingsMenu(FMenuBuilder& MenuBuilder);

	/**
	 * Builds up the track menu for object binding nodes in the outliner
	 * 
	 * @param MenuBuilder	The track menu builder to add things to
	 * @param ObjectBindings The array of object bindings to add tracks to (if there are more than 1 selected)
	 * @param ObjectClass	The class of the selected object
	 */
	void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass);

	/**
	 * Builds up the menu of folders to add selected nodes to
	 *
	 * @param MenuBuilder The menu builder to add things to.
	 */
	void BuildAddSelectedToFolderMenu(FMenuBuilder& MenuBuilder);

protected:
	void BuildAddSelectedToFolderSubMenu(FMenuBuilder& InMenuBuilder, TSharedRef<TArray<UMovieSceneFolder*> > InExcludedFolders, UMovieSceneFolder* InFolder, TArray<UMovieSceneFolder*> InChildFolders);
	void BuildAddSelectedToFolderMenuEntry(FMenuBuilder& InMenuBuilder, TSharedRef<TArray<UMovieSceneFolder*> > InExcludedFolders, UMovieSceneFolder* InFolder);

public:

	/**
	 * Builds up the menu of node groups to add selected nodes to
	 *
	 * @param MenuBuilder The menu builder to add things to.
	 */
	void BuildAddSelectedToNodeGroupMenu(FMenuBuilder& MenuBuilder);

	void BuildSortMenu(FMenuBuilder& MenuBuilder);

	/** Called when an actor is dropped into Sequencer */
	void OnActorsDropped( const TArray<TWeakObjectPtr<AActor> >& Actors );

	/** Functions to push on to the transport controls we use */
	FReply OnRecord();
	DLLEXPORT FReply OnPlayForward(bool bTogglePlay);
	DLLEXPORT FReply OnPlayBackward(bool bTogglePlay);
	FReply OnStepForward(FFrameNumber Increment = FFrameNumber(1));
	FReply OnStepBackward(FFrameNumber Increment = FFrameNumber(1));
	FReply OnJumpToStart();
	FReply OnJumpToEnd();
	FReply OnCycleLoopMode();
	FReply SetPlaybackEnd();
	FReply SetPlaybackStart();
	FReply JumpToPreviousKey();
	FReply JumpToNextKey();

	bool CanAddTransformKeysForSelectedObjects(const EMovieSceneTransformChannel InChannel = EMovieSceneTransformChannel::All) const;
	bool CanDeleteTransformKeysForSelectedObjects(const EMovieSceneTransformChannel InChannel = EMovieSceneTransformChannel::All) const;
	void OnAddTransformKeysForSelectedObjects(const EMovieSceneTransformChannel InChannel);
	void OnDeleteTransformKeysForSelectedObjects(const EMovieSceneTransformChannel InChannel);

	void OnTogglePilotCamera();
	bool IsPilotCamera() const;

	/** Sets the new global time calculated from local time and the given warp counter, accounting for looping options */
	void SetLocalTimeLooped(FFrameTime InTime, const FMovieSceneTransformBreadcrumbs& Breadcrumbs=FMovieSceneTransformBreadcrumbs(), bool bForwards=true);

	ESequencerLoopMode GetLoopMode() const;

	DLLEXPORT EPlaybackMode::Type GetPlaybackMode() const;

	/** @return The toolkit that this sequencer is hosted in (if any) */
	virtual TSharedPtr<IToolkitHost> GetToolkitHost() const override { return ToolkitHost.Pin(); }

	/** @return The capabilities of this sequencer host */
	virtual const FSequencerHostCapabilities& GetHostCapabilities() const override { return HostCapabilities; }

	/** @return Whether or not this sequencer is used in the level editor */
	virtual bool IsLevelEditorSequencer() const override { return bIsEditingWithinLevelEditor; }

	/** @return Whether to show the curve editor or not */
	virtual void SetShowCurveEditor(bool bInShowCurveEditor) override;
	/** @return If the curve editor is currently visible. */
	virtual bool GetCurveEditorIsVisible() const override;

	/** Called to save the current movie scene */
	void SaveCurrentMovieScene();

	/** Called to save the current movie scene under a new name */
	void SaveCurrentMovieSceneAs();

	FReply NavigateForward();
	FReply NavigateBackward();
	bool CanNavigateForward() const;
	bool CanNavigateBackward() const;
	FText GetNavigateForwardTooltip() const;
	FText GetNavigateBackwardTooltip() const;

	/** Called when a user executes the delete node menu item */
	void DeleteNode(TSharedRef<FViewModel> NodeToBeDeleted, const bool bKeepState);
	void DeleteSelectedNodes(const bool bKeepState);

	/** @return The list of nodes which must be moved to move the current selected nodes */
	TArray<TSharedRef<FViewModel>> GetSelectedNodesToMove();
	TArray<TSharedRef<FViewModel>> GetSelectedNodesInFolders();

	/** Called when a user executes the move to new folder menu item */
	void MoveSelectedNodesToNewFolder();
	void RemoveSelectedNodesFromFolders();
	void MoveNodeToFolder(TSharedRef<FViewModel> NodeToMove, UMovieSceneFolder* DestinationFolder);
	void MoveSelectedNodesToFolder(UMovieSceneFolder* DestinationFolder);

	/** Called when a user executes the copy track menu item */
	void CopySelectedObjects(TArray<TSharedPtr<UE::Sequencer::FObjectBindingModel>>& ObjectNodes, const TArray<UMovieSceneFolder*>& Folders, /*out*/ FString& ExportedText);
	void CopySelectedTracks(TArray<TSharedPtr<UE::Sequencer::FViewModel>>& TrackNodes, const TArray<UMovieSceneFolder*>& Folders, /*out*/ FString& ExportedText);
	void CopySelectedFolders(const TArray<UMovieSceneFolder*>& Folders, /*out*/ FString& FoldersExportedText, FString& TracksExportedText, FString& ObjectsExportedText);

	/** Called when a user executes the paste track menu item */
	bool CanPaste(const FString& TextToImport);

	/**
	 * Attempts to paste from the clipboard
	 * @return Whether the paste event was handled
	 */
	enum class EPasteResult : uint8
	{
		Success,
		Failure,
		Cancelled
	};

	bool DoPaste(bool bClearSelection = false);
	EPasteResult PasteTracks(const FString& TextToImport, UMovieSceneFolder* ParentFolder, const TArray<UMovieSceneFolder*>& InFolders, TArray<FNotificationInfo>& PasteErrors, bool bClearSelection = false);
	EPasteResult PasteSections(const FString& TextToImport, TArray<FNotificationInfo>& PasteErrors);
	EPasteResult PasteDecorations(const FString& TextToImport, TArray<FNotificationInfo>& PasteErrors);
	EPasteResult PasteObjectBindings(const FString& TextToImport, UMovieSceneFolder* ParentFolder, const TArray<UMovieSceneFolder*>& InFolders, TArray<FNotificationInfo>& PasteErrors, bool bClearSelection = false);

	/** Called when a user executes the locked node menu item */
	void ToggleNodeLocked();
	bool IsNodeLocked() const;

	/** Called when a user executes the Group menu item */
	void GroupSelectedSections();
	bool CanGroupSelectedSections() const;

	/** Called when a user executes the Ungroup menu item */
	void UngroupSelectedSections();
	bool CanUngroupSelectedSections() const;

	/** Called when a user executes the set key time for selected keys */
	bool CanSetKeyTime() const;
	void SetKeyTime();
	void OnSetKeyTimeTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

	/** Called when a user executes the rekey for selected keys */
	bool CanRekey() const;
	void Rekey();

	void SelectKey(UMovieSceneSection* InSection, TSharedPtr<UE::Sequencer::FChannelModel> InChannel, FKeyHandle KeyHandle, bool bToggle);

	/** Updates the external selection to match the current sequencer selection. */
	void SynchronizeExternalSelectionWithSequencerSelection();

	/** Updates the sequencer selection to match the current external selection. */
	void SynchronizeSequencerSelectionWithExternalSelection();
		
	/** Updates the sequencer selection to match the list of node paths. */
	void SelectNodesByPath(const TSet<FString>& NodePaths);

	/** Whether the binding is visible in the tree view */
	bool IsBindingVisible(const FMovieSceneBinding& InBinding);

	/** Whether the track is visible in the tree view */
	bool IsTrackVisible(const UMovieSceneTrack* InTrack);

	void OnSelectedNodesOnlyChanged();

	void OnTimeDisplayFormatChanged();

	/** Will create a custom menu if FSequencerViewParams::OnBuildCustomContextMenuForGuid is specified. */
	void BuildCustomContextMenuForGuid(FMenuBuilder& MenuBuilder, FGuid ObjectBinding);

	/** Set the color tint for the requested sections */
	void SetSectionColorTint(TArray<UMovieSceneSection*> Sections, FColor ColorTint);

public:

	/** Copy the selection, whether it's keys or tracks */
	void CopySelection();

	/** Cut the selection, whether it's keys or tracks */
	void CutSelection();

	/** Duplicate the selection */
	void DuplicateSelection();

	/** Copy the selected keys to the clipboard */
	void CopySelectedKeys();

	/** Copy the selected keys to the clipboard, then delete them as part of an undoable transaction */
	void CutSelectedKeys();

	/** Copy the selected sections to the clipboard */
	void CopySelectedSections();

	/** Copy the selected sections to the clipboard, then delete them as part of an undoable transaction */
	void CutSelectedSections();

	/** Get the in-memory clipboard stack */
	const TArray<TSharedPtr<FMovieSceneClipboard>>& GetClipboardStack() const;

	/** Promote a clipboard to the top of the clipboard stack, and update its timestamp */
	void OnClipboardUsed(TSharedPtr<FMovieSceneClipboard> Clipboard);

	/** Attempts to automatically fix up possessables whose object class don't match the object class of their currently bound objects */
	void FixPossessableObjectClass();
	void FixPossessableObjectClassInternal(UMovieSceneSequence* Sequence, FMovieSceneSequenceIDRef SequenceID);

	/** Rebinds all possessable references in the current sequence to update them to the latest referencing mechanism. */
	void RebindPossessableReferences();

public:

	/** Put the sequencer in a horizontally auto-scrolling state with the given rate */
	void StartAutoscroll(float UnitsPerS);

	/** Stop the sequencer from auto-scrolling */
	void StopAutoscroll();

	/**
	 * Update auto-scroll mechanics as a result of a new time position
	 */
	void UpdateAutoScroll(double NewTime, float ThresholdPercentage = 0.025f);

	/** Autoscrub to destination time */
	void AutoScrubToTime(FFrameTime DestinationTime);

public:

	//~ FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

private:
	//object for undo redo support for changing time
	FSequencerTimeChangedHandler TimeUndoRedoHandler;

public:

	//~ FTickableEditorObject Interface

	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FSequencer, STATGROUP_Tickables); }

public:
	
	TSharedRef<SSequencer> GetUnderlyingSequencerWidget() const { return SequencerWidget.ToSharedRef(); }

	//~ ISequencer Interface

	virtual void Close() override;
	virtual FOnCloseEvent& OnCloseEvent() override { return OnCloseEventDelegate; }
	virtual TSharedRef<SWidget> GetSequencerWidget() const override;
	virtual UE::Sequencer::ISequencerWidget& GetSequencerWidgetInterface() const override;
	virtual FMovieSceneSequenceIDRef GetRootTemplateID() const override { return ActiveTemplateIDs[0]; }
	virtual FMovieSceneSequenceIDRef GetFocusedTemplateID() const override { return ActiveTemplateIDs.Top(); }
	virtual const TArray<FMovieSceneSequenceID>& GetSubSequenceHierarchy() const override { return ActiveTemplateIDs; };
	virtual UMovieSceneSubSection* FindSubSection(FMovieSceneSequenceID SequenceID) const override;
	virtual UMovieSceneSequence* GetRootMovieSceneSequence() const override;
	virtual UMovieSceneSequence* GetFocusedMovieSceneSequence() const override;
	virtual FMovieSceneSequenceTransform GetFocusedMovieSceneSequenceTransform() const override;
	virtual FMovieSceneSequenceTransform GetFocusedMovieSceneSequenceTransformUnwarped() const override;
	virtual FMovieSceneSequenceTransform GetLocalTimeWarpTransform() const override;
	virtual FMovieSceneSequenceTransform GetGlobalPlaybackWarpTransform() const override;
	virtual FMovieSceneRootEvaluationTemplateInstance& GetEvaluationTemplate() override { return RootTemplateInstance; }
	virtual void ResetToNewRootSequence(UMovieSceneSequence& NewSequence) override;
	virtual void FocusSequenceInstance(UMovieSceneSubSection& InSubSection) override;
	virtual TOptional<TRange<FFrameNumber>> GetSubSequenceRange() const override;
	virtual TSharedPtr<UE::Sequencer::FSequencerEditorViewModel> GetViewModel() const override;
	virtual void SuppressAutoEvaluation(UMovieSceneSequence* Sequence, const FGuid& InSequenceSignature) override;
	virtual EAutoChangeMode GetAutoChangeMode() const override;
	virtual void SetAutoChangeMode(EAutoChangeMode AutoChangeMode) override;
	virtual EAllowEditsMode GetAllowEditsMode() const override;
	virtual void SetAllowEditsMode(EAllowEditsMode AllowEditsMode) override;
	virtual EKeyGroupMode GetKeyGroupMode() const override;
	virtual void SetKeyGroupMode(EKeyGroupMode) override;
	virtual EMovieSceneKeyInterpolation GetKeyInterpolation() const override;
	virtual void SetKeyInterpolation(EMovieSceneKeyInterpolation) override;
	virtual bool GetInfiniteKeyAreas() const override;
	virtual void SetInfiniteKeyAreas(bool bInfiniteKeyAreas) override;
	virtual bool GetAutoSetTrackDefaults() const override;
	virtual FQualifiedFrameTime GetLocalTime() const override;
	virtual FQualifiedFrameTime GetGlobalTime() const override;
	virtual FQualifiedFrameTime GetUnwarpedLocalTime() const override;
	virtual TOptional<int32> GetLocalLoopIndex() const override;
	virtual UE::Sequencer::FTimeDomainOverride OverrideTimeDomain(UE::Sequencer::ETimeDomain NewDomain) override;
	virtual void SetLocalTime(FFrameTime Time, ESnapTimeMode SnapTimeMode = ESnapTimeMode::STM_None, bool bEvaluate = true) override;
	virtual void SetLocalTimeDirectly(FFrameTime NewTime, bool bEvaluate = true) override;
	virtual FFrameTime GetLastEvaluatedLocalTime() const override;
	virtual void SetGlobalTime(FFrameTime Time, bool bEvaluate = true) override;
	virtual void PlayTo(FMovieSceneSequencePlaybackParams PlaybackParams) override;
	virtual void SnapSequencerTime(FFrameTime& InOutScrubTime) override;
	virtual void RestorePlaybackSpeed() override;
	virtual void SnapToClosestPlaybackSpeed() override;
	virtual void RequestInvalidateCachedData() override { bNeedsInvalidateCachedData = true; }
	virtual void RequestEvaluate() override { bNeedsEvaluate = true; }
	virtual void ForceEvaluate() override;
	virtual void SetPerspectiveViewportPossessionEnabled(bool bEnabled) override;
	virtual void SetPerspectiveViewportCameraCutEnabled(bool bEnabled) override;
	virtual void RenderMovie(const TArray<UMovieSceneCinematicShotSection*>& InSections) const override;
	virtual void RecreateCurveEditor() override;
	virtual void EnterSilentMode() override;
	virtual void ExitSilentMode() override;
	virtual bool IsInSilentMode() const override { return SilentModeCount != 0; }
	virtual void Save() override { SaveCurrentMovieScene(); }
	virtual FGuid GetHandleToObject(UObject* Object, bool bCreateHandleIfMissing = true, const FName& CreatedFolderName = NAME_None) override;
	virtual ISequencerObjectChangeListener& GetObjectChangeListener() override;
	virtual ISequencerPropertyKeyedStatusHandler& GetPropertyKeyedStatusHandler() override;

protected:
	virtual void NotifyMovieSceneDataChangedInternal() override;
public:
	virtual void NotifyMovieSceneDataChanged( EMovieSceneDataChangeType DataChangeType ) override;
	virtual void RefreshTree() override;
	virtual void UpdatePlaybackRange() override;
	virtual void SetPlaybackSpeed(float InPlaybackSpeed) override;
	virtual float GetPlaybackSpeed() const override { return PlaybackSpeed; }
	virtual TArray<FGuid> AddActors(const TArray<TWeakObjectPtr<AActor> >& InActors, bool bSelectActors = true) override;
	virtual FGuid AddEmptyBinding() override;
	virtual TArray<FGuid> ConvertToSpawnable(FGuid Guid) override;
	virtual void AddSubSequence(UMovieSceneSequence* Sequence) override;
	virtual bool CanKeyProperty(FCanKeyPropertyParams CanKeyPropertyParams) const override;
	virtual void KeyProperty(FKeyPropertyParams KeyPropertyParams) override;
	EPropertyKeyedStatus GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const override;
	virtual void GetSelectedTracks(TArray<UMovieSceneTrack*>& OutSelectedTracks) override;
	virtual void GetSelectedTrackRows(TArray<TPair<UMovieSceneTrack*, int32>>& OutSelectedTrackRows) override;
	virtual void GetSelectedSections(TArray<UMovieSceneSection*>& OutSelectedSections) override;
	virtual void GetSelectedFolders(TArray<UMovieSceneFolder*>& OutSelectedFolders) override;
	virtual void GetSelectedKeyAreas(TArray<const IKeyArea*>& OutSelectedKeyAreas, bool bIncludeSelectedKeys = true)  override;
	virtual void GetSelectedObjects(TArray<FGuid>& OutObjects) override;
	virtual void SelectObject(FGuid ObjectBinding) override;
	virtual void DeselectObject(FGuid ObjectBinding) override;
	virtual void SelectTrack(UMovieSceneTrack* Track) override;
	virtual void SelectSection(UMovieSceneSection* Section) override;
	virtual void SelectFolder(UMovieSceneFolder* Folder) override;
	virtual void SelectByPropertyPaths(const TArray<FString>& InPropertyPaths) override;
	virtual void SelectByChannels(UMovieSceneSection* Section, TArrayView<const FMovieSceneChannelHandle> InChannels, bool bSelectParentInstead, bool bSelect) override;
	virtual void SelectByChannels(UMovieSceneSection* Section, const TArray<FName>& InChannelNames, bool bSelectParentInstead, bool bSelect) override;
	virtual void SelectByNthCategoryNode(UMovieSceneSection* Section, int Index, bool bSelect) override;
	virtual void EmptySelection() override;
	virtual void ThrobKeySelection() override;
	virtual void ThrobSectionSelection() override;
	virtual FOnGlobalTimeChanged& OnGlobalTimeChanged() override { return OnGlobalTimeChangedDelegate; }
	virtual FOnPlayEvent& OnPlayEvent() override { return OnPlayDelegate; }
	virtual FOnStopEvent& OnStopEvent() override { return OnStopDelegate; }
	virtual FOnRecordEvent& OnRecordEvent() override { return OnRecordDelegate; }
	virtual FOnBeginScrubbingEvent& OnBeginScrubbingEvent() override { return OnBeginScrubbingDelegate; }
	virtual FOnEndScrubbingEvent& OnEndScrubbingEvent() override { return OnEndScrubbingDelegate; }
	virtual FOnMovieSceneDataChanged& OnMovieSceneDataChanged() override { return OnMovieSceneDataChangedDelegate; }

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "FOnChannelChanged OnChannelChanged() is now deprecated. Use FMovieSceneOnChannelChangedWithTime OnChannelChangedWithTime()")
	FOnChannelChanged& OnChannelChanged() { return OnChannelChangedDelegate_DEPRACRATED; }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual FMovieSceneOnChannelChangedWithTime& OnChannelChangedWithTime() override { return OnChannelChangedDelegate; }
	virtual FOnMovieSceneBindingsChanged& OnMovieSceneBindingsChanged() override { return OnMovieSceneBindingsChangedDelegate; }
	virtual FOnMovieSceneBindingsPasted& OnMovieSceneBindingsPasted() override { return OnMovieSceneBindingsPastedDelegate; }
	virtual FOnSelectionChangedObjectGuids& GetSelectionChangedObjectGuids() override { return OnSelectionChangedObjectGuidsDelegate; }
	virtual FOnSelectionChangedTracks& GetSelectionChangedTracks() override { return OnSelectionChangedTracksDelegate; }
	virtual FOnCurveDisplayChanged& GetCurveDisplayChanged() override { return OnCurveDisplayChanged; }
	virtual FOnSelectionChangedSections& GetSelectionChangedSections() override { return OnSelectionChangedSectionsDelegate; }
	virtual FOnTreeViewChanged& OnTreeViewChanged() override { return OnTreeViewChangedDelegate; }
	virtual FGuid CreateBinding(UObject& InObject, const UE::Sequencer::FCreateBindingParams& InParams) override;
	virtual UObject* GetPlaybackContext() const override;
	virtual IMovieScenePlaybackClient* GetPlaybackClient() override;
	virtual TArray<UObject*> GetEventContexts() const override; 
	virtual FOnActorAddedToSequencer& OnActorAddedToSequencer() override;
	virtual FOnPreSave& OnPreSave() override;
	virtual FOnPostSave& OnPostSave() override;
	virtual FOnPreNavigateToSequence& OnPreNavigateToSequence() override;
	virtual FOnActivateSequence& OnActivateSequence() override;
	virtual FOnCameraCut& OnCameraCut() override;
	virtual void AddNumericTypeInterface(TSharedRef<UE::Sequencer::FSequencerNumericTypeInterface> InNumericTypeInterface);
	virtual void RemoveNumericTypeInterface(TSharedRef<UE::Sequencer::FSequencerNumericTypeInterface> InNumericTypeInterface);
	virtual TArrayView<const TSharedRef<UE::Sequencer::FSequencerNumericTypeInterface>> GetNumericTypeInterfaces() const override;
	virtual TSharedPtr<INumericTypeInterface<double>> GetNumericTypeInterface(UE::Sequencer::ENumericIntent Intent = UE::Sequencer::ENumericIntent::Position) const override;
	virtual TSharedRef<IPropertyTypeCustomization> MakeFrameNumberDetailsCustomization() override;
	virtual TSharedRef<SWidget> MakeTransportControls(bool bExtended) override;
	virtual FReply OnPlay(bool bTogglePlay = true) override;
	virtual void Pause() override;
	virtual TSharedRef<SWidget> MakeTimeRange(const TSharedRef<SWidget>& InnerContent, bool bShowWorkingRange, bool bShowViewRange, bool bShowPlaybackRange) override;
	virtual UObject* FindSpawnedObjectOrTemplate(const FGuid& BindingId) override;
	virtual FGuid MakeNewSpawnable(UObject& SourceObject, UActorFactory* ActorFactory = nullptr, bool bSetupDefaults = true) override;
	virtual bool IsReadOnly() const override;
	virtual void ExternalSelectionHasChanged() override;
	virtual TSharedPtr<ISequencerTrackEditor> GetTrackEditor(UMovieSceneTrack* InTrack) override;
	virtual void ObjectImplicitlyAdded(UObject* InObject) const override;
	virtual void ObjectImplicitlyRemoved(UObject* InObject) const override;

	/** Access the user-supplied settings object */
	virtual USequencerSettings* GetSequencerSettings() const override { return Settings; }
	virtual void SetSequencerSettings(USequencerSettings* InSettings) override;
	virtual TSharedPtr<class ITimeSlider> GetTopTimeSliderWidget() const override;
	virtual void ResetTimeController() override;
	virtual void SetTrackFilterEnabled(const FText& InFilterName, bool bInEnabled) override;
	virtual bool IsTrackFilterEnabled(const FText& InFilterName) const override;
	virtual TArray<FText> GetTrackFilterNames() const override;

	virtual bool TrackSupportsConditions(const UMovieSceneTrack* Track) const override;

public:

	// IMovieScenePlayer interface

	virtual void NotifyBindingsChanged() override;
	virtual void SetViewportSettings(const TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) override;
	virtual void GetViewportSettings(TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) const override;
	DLLEXPORT virtual EMovieScenePlayerStatus::Type GetPlaybackStatus() const override;
	virtual void SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus) override;
	virtual FMovieSceneSpawnRegister& GetSpawnRegister() override { return *SpawnRegister; }
	virtual bool IsPreview() const override { return SilentModeCount != 0; }
	virtual UMovieSceneEntitySystemLinker* ConstructEntitySystemLinker() override;

	/** Shortcut for GetEditorModel()->GetSelection() for backwards compat with existing code */
	UE::Sequencer::FSequencerSelection& GetSelection();
	FSequencerSelectionPreview& GetSelectionPreview();

	/**
	 * Gets the far time boundaries of the currently edited movie scene
	 * If the scene has shots, it only takes the shot section boundaries
	 * Otherwise, it finds the furthest boundaries of all sections
	 */
	TRange<FFrameNumber> GetTimeBounds() const;


	/**
	 * Gets the time boundaries of the root movie scene.
	 */
	TRange<FFrameNumber> GetRootTimeBounds() const;

	virtual TArrayView<const TSubclassOf<UMovieSceneCustomBinding>> GetSupportedCustomBindingTypes() const override { return SupportedCustomBindingTypes; }
	
	virtual void RefreshSupportedCustomBindingTypes() override;

protected:

	// FCameraCutPlaybackCapability interface

	virtual bool ShouldUpdateCameraCut() override { return IsPerspectiveViewportCameraCutEnabled(); }
	virtual bool ShouldRestoreEditorViewports() override;
	virtual float GetCameraBlendPlayRate() override;
	virtual void OnCameraCutUpdated(const UE::MovieScene::FOnCameraCutUpdatedParams& Params) override;

protected:

	/** Reevaluate the sequence at the current time */
	void EvaluateInternal(FMovieSceneEvaluationRange InRange, bool bHasJumped = false);

	/** Reset data about a movie scene when pushing or popping a movie scene. */
	void ResetPerMovieSceneData();

	/** Forcibly refresh the UI */
	void RefreshUI();

	/** Update the time bounds to the focused movie scene */
	void UpdateTimeBoundsToFocusedMovieScene();
	
	/**
	 * Gets the time boundaries of the currently filtering shot sections.
	 * If there are no shot filters, an empty range is returned.
	 */
	TRange<float> GetFilteringShotsTimeBounds() const;

	/**
	 * Called when the clamp range is changed by the user
	 *
	 * @param	NewClampRange The new clamp range
	 */
	void OnClampRangeChanged( TRange<double> NewClampRange );

	/*
	 * Called to get the nearest key
	 *
	 * @param InTime The time to get the nearest key to
	 * @param NearestKeyOption (ie. search keys/markers/all tracks)
	 * @return NearestKey
	 */
	virtual FFrameNumber OnGetNearestKey(FFrameTime InTime, ENearestKeyOption NearestKeyOption) override;

	/**
	 * Called when the scrub position is changed by the user
	 * This will stop any playback from happening
	 *
	 * @param NewScrubPosition	The new scrub position
	 * @param bScrubbing If scrubbing
	 * @param bEvaluate  Do evaluate sequencer after changing time
	 */
	void OnScrubPositionChanged( FFrameTime NewScrubPosition, bool bScrubbing, bool bEvaluate) override;

	void OnBeginScrubbing() override;

	/** Called when the user has finished scrubbing */
	void OnEndScrubbing() override;

	/** Called when the user has begun dragging the playback range */
	void OnPlaybackRangeBeginDrag();

	/** Called when the user has finished dragging the playback range */
	void OnPlaybackRangeEndDrag();

	/** Called when the user has begun dragging the selection range */
	void OnSelectionRangeBeginDrag();

	/** Called when the user has finished dragging the selection range */
	void OnSelectionRangeEndDrag();

	/** Called when the user has begun dragging a mark */
	void OnMarkBeginDrag();

	/** Called when the user has finished dragging a mark */
	void OnMarkEndDrag();

	/** Get the current scrub position based on settings */
	FFrameTime GetScrubPosition() const;

	/** Get the frame time text */
	FString GetFrameTimeText() const;

	/** The parent sequence that the scrub position display text is relative to */
	FMovieSceneSequenceID GetScrubPositionParent() const;
	
	/** The parent sequence chain of the current sequence */
	TArray<FMovieSceneSequenceID> GetScrubPositionParentChain() const;
	
	/** Called when the scrub position parent sequence is changed */
	void OnScrubPositionParentChanged(FMovieSceneSequenceID InScrubPositionParent);

	void OnEvaluateSubSequencesInIsolationChanged();

protected:

	/**
	 * Ensure that the specified local time is in the view
	 */
	void ScrollIntoView(float InLocalTime);

	/**
	 * Calculates the amount of encroachment the specified time has into the autoscroll region, if any
	 */
	TOptional<float> CalculateAutoscrollEncroachment(double NewTime, float ThresholdPercentage = 0.1f) const;

	/** Called to toggle auto-scroll on and off */
	void OnToggleAutoScroll();

	/**
	 * Whether auto-scroll is enabled.
	 *
	 * @return true if auto-scroll is enabled, false otherwise.
	 */
	bool IsAutoScrollEnabled() const;

	/** Find the viewed sequence asset in the content browser. */
	void FindInContentBrowser();

	/** Browse to the selected section's object. */
	void BrowseToObject();

	/** Get the asset we're currently editing, if applicable. */
	UObject* GetCurrentAsset() const;

protected:

	/** Get all the keys for the current sequencer selection */
	virtual void GetKeysFromSelection(TUniquePtr<FSequencerKeyCollection>& KeyCollection, float DuplicateThresholdSeconds) override;
	virtual FSequencerKeyCollection* GetKeyCollection() override;

	void GetAllKeys(TUniquePtr<FSequencerKeyCollection>& KeyCollection, float DuplicateThresoldSeconds) const;

protected:
	
	/** Called when a user executes the delete command to delete sections or keys */
	void DeleteSelectedItems();
	
	/** Transport controls */
	void TogglePlay();
	void JumpToStart();
	void JumpToEnd();
	/** Increases positive speed or decreases negative speed until positive speed is reached */
	void ShuttleForward();
	/** Increases negative speed or decreases positive speed until negative speed is reached */
	void ShuttleBackward();
	void StepForward();
	void StepBackward();
	void JumpForward();
	void JumpBackward();
	void StepToNextShot();
	void StepToPreviousShot();

	/** Expand or collapse selected nodes */
	void ToggleExpandCollapseNodes();

	/** Expand or collapse selected nodes and descendants*/
	void ToggleExpandCollapseNodesAndDescendants();

public:

	/** Expand or collapse all nodes and descendants*/
	void ExpandAllNodes();
	void CollapseAllNodes();

protected:

	/** Sort all nodes and their descendants by category then alphabetically */
	void SortAllNodesAndDescendants();

	/** Add selected actors to sequencer */
	void AddSelectedActors();

	/** Manually sets a key for the selected objects at the current time */
	void SetKey();

	/** Modeless Version of the String Entry Box */
	void GenericTextEntryModeless(const FText& DialogText, const FText& DefaultText, FOnTextCommitted OnTextComitted);
	
	/** Closes the popup created by GenericTextEntryModeless*/
	void CloseEntryPopupMenu();

	/** Trim a section to the left or right */
	void TrimSection(bool bTrimLeft);

	/** Trim or extend section to the current time */
	void TrimOrExtendSection(bool bTrimOrExtendLeft);

	/** Split a section */
	void SplitSection();

	/** Generates command bindings for UI commands */
	void BindCommands();

	//~ Begin FEditorUndoClient Interface
	virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjects) const override;
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

	void OnSelectionChanged();
	// Called on Tick after OnSelectionChanged has been called
	void HandleSelectedOutlinerNodesChanged();

	void AddNodeGroupsCollectionChangedDelegate();
	void RemoveNodeGroupsCollectionChangedDelegate();

	void OnNodeGroupsCollectionChanged();

public:
	void AddSelectedNodesToNewNodeGroup();
	void AddSelectedNodesToExistingNodeGroup(UMovieSceneNodeGroup* NodeGroup);
	void AddNodesToExistingNodeGroup(TArrayView<const UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>> InItems, UMovieSceneNodeGroup* NodeGroup);

	void ClearFilters();

	/** Handles adding a new folder to the outliner tree. */
	void AddFolder();

private:

	bool CanAddTransformKeysForSelectedObjectsInternal(const EMovieSceneTransformChannel InChannel = EMovieSceneTransformChannel::All) const;
	void AddTransformKeysForSelectedObjectsInternal(const EMovieSceneTransformChannel InChannel);

	/** Updates viewport clients' actor locks if they relate to sequencer cameras */
	void UpdateLevelViewportClientsActorLocks();

	/** Internal function to render movie for a given start/end time */
	void RenderMovieInternal(TRange<FFrameNumber> Range, bool bSetFrameOverrides = false) const;

	/** Handles loading in previously recorded data. */
	void OnLoadRecordedData();
	
	/** Adds the binding and selects the binding, throbs it. */
	void OnAddBinding(const FGuid& ObjectBinding, UMovieScene* MovieScene) override;

	/** Adds the track to the selected folder (if FGuid is invalid) and selects the track, throbs it, and notifies the sequence to rebuild any necessary data. */
	void OnAddTrack(const TWeakObjectPtr<UMovieSceneTrack>& InTrack, const FGuid& ObjectBinding) override;

	/** Determines the selected parent folders and returns the node path to the first folder. Also expands the first folder. */
	void CalculateSelectedFolderAndPath(TArray<UMovieSceneFolder*>& OutSelectedParentFolders, FString& OutNewNodePath);

	/** Returns the tail folder from the given Folder Path, creating each folder if needed. */
	UMovieSceneFolder* CreateFoldersRecursively(const TArray<FName>& FolderPaths, int32 FolderPathIndex, UMovieScene* OwningMovieScene, UMovieSceneFolder* ParentFolder, TArrayView<UMovieSceneFolder* const> FoldersToSearch);

	/** Create set playback start transport control */
	TSharedRef<SWidget> OnCreateTransportSetPlaybackStart();

	/** Create jump to previous key transport control */
	TSharedRef<SWidget> OnCreateTransportJumpToPreviousKey();

	/** Create jump to next key transport control */
	TSharedRef<SWidget> OnCreateTransportJumpToNextKey();

	/** Create set playback end transport control */
	TSharedRef<SWidget> OnCreateTransportSetPlaybackEnd();

	/** Select keys and/or sections in a display node that fall into the current selection range. */
	void SelectInSelectionRange(const TSharedPtr<UE::Sequencer::FViewModel>& Item, const TRange<FFrameNumber>& SelectionRange, bool bSelectKeys, bool bSelectSections);
	
	/** Create loop mode transport control */
	TSharedRef<SWidget> OnCreateTransportLoopMode();

	/** Create record transport control */
	TSharedRef<SWidget> OnCreateTransportRecord();

	/** Update the locked subsequence range (displayed as playback range for subsequences), and root to local transform */
	void UpdateSubSequenceData();

	/** Adjust sequencer customizations based on the currently focused sequence */
	void UpdateSequencerCustomizations(const UMovieSceneSequence* PreviousFocusedSequence);

	/** Rerun construction scripts on bound actors */
	void RerunConstructionScripts();

	/** Get actors that want to rerun construction scripts */
	void GetConstructionScriptActors(UMovieScene*, FMovieSceneSequenceIDRef SequenceID, TSet<TWeakObjectPtr<AActor> >& BoundActors, TArray < TPair<FMovieSceneSequenceID, FGuid> >& BoundGuids);

	/** Check whether we're viewing the root sequence or not */
	bool IsViewingRootSequence() const { return ActiveTemplateIDs.Num() == 1; }

	/** Recompile any dirty director blueprints in the sequence hierarchy */
	void RecompileDirtyDirectors();

	void ToggleAsyncEvaluation();
	bool UsesAsyncEvaluation();

	void ToggleDynamicWeighting();
	bool UsesDynamicWeighting();

	void ToggleLoopCuts();
	bool UsesLoopCuts();

	void UpdateCachedPlaybackContextAndClient();

	void UpdateCachedCameraActors();

	int32 FindClosestPlaybackSpeed(float InPlaybackSpeed, bool bExactOnly = false) const;
	void RestorePlaybackSpeedAfterPlay();

	FGuid FindUnspawnedObjectGuid(UObject& InObject);
	
	void DestroySequencerWidget();

public:

	/** Helper function which returns how many frames (in tick resolution) one display rate frame represents. */
	double GetDisplayRateDeltaFrameCount() const;

	/** Retrieve the desired scrubber style for this instance. */
	ESequencerScrubberStyle GetScrubStyle() const
	{
		return ScrubStyle;
	}

	/** Get the name of the movie renderer to use, defaults to the first available if the setting is empty */
	FString GetMovieRendererName() const;

	TSharedRef<STemporarilyFocusedSpinBox<double>> MakePlayTimeDisplay();

	/** @return True if the Sequencer is currently limiting selection to only Sequencer key-able objects */
	bool IsViewportSelectionLimited() const override;

	/** Turns on or off Sequencer selection limiting. */
	void SetViewportSelectionLimited(const bool bInSelectionLimited) override;

	/** @return True if the object specified is selectable in the viewport */
	bool IsObjectSelectableInViewport(UObject* const InObject) override;

	ISequencer::FOnViewportSelectionLimitedChanged& OnViewportSelectionLimitedChanged() override;

	/**
	 * Registers and displays a new drawer in the sidebar.
	 * 
	 * @param InDrawerConfig	Configuration info for the new drawer
	 * 
	 * @return True if the new drawer registration was successful.
	 */
	virtual bool RegisterDrawer(FSidebarDrawerConfig&& InDrawerConfig) override;

	/**
	 * Unregisters and removes a drawer from the sidebar.
	 *
	 * @param InDrawerId	Unique drawer Id to unregister
	 * 
	 * @return True if the drawer removal was successful.
	 */
	virtual bool UnregisterDrawer(const FName InDrawerId) override;
	
	/**
	 * Registers and displays a new drawer section in the sidebar.
	 * 
	 * @param InDrawerId	Unique drawer Id to register
	 * @param InSection		Drawer content interface for the section
	 * 
	 * @return True if the new drawer section registration was successful.
	 */
	virtual bool RegisterDrawerSection(const FName InDrawerId, const TSharedPtr<ISidebarDrawerContent>& InSection) override;
	
	/**
	 * Unregisters and removes a drawer section from the sidebar.
	 * 
	 * @param InDrawerId	Unique drawer Id that contains the section to unregister
	 * @param InSectionId	Unique drawer section Id to unregister
	 * 
	 * @return True if the drawer removal was successful.
	 */
	virtual bool UnregisterDrawerSection(const FName InDrawerId, const FName InSectionId) override;

	/** @return True if the sequencer is in the simplified view */
	virtual bool IsSimpleView() const override;

	/** Enables or disables the Sequencer simplified view */
	virtual void EnableSimpleView(const bool bInEnable) override;

	/** Toggles the enabled state of the Sequencer simplified view */
	void ToggleSimpleView();

	/** Toggles the visibility of the Sequencer sidebar. */
	void ToggleSidebar();

	/** @return True if the Sequencer sidebar is currently visible. */
	bool IsSidebarVisible() const;

	/** Toggles the state of the Sequencer selection details between docked in the Sequencer tab and being popped out to its own tab window. */
	void ToggleSelectionDetailsPoppedOut();

	/** @return True if the Sequencer selection details can be popped out to its own tab window. */
	bool CanSelectionDetailsPopOut() const;

	/** @return True if the Sequencer selection details are currently popped out to its own tab window. */
	bool IsSelectionDetailsPoppedOut() const;

	/** Set the popped out state of the selection details in the Sequencer tab. */
	void SetSelectionDetailsPoppedOut(const bool bInPoppedOut);

	/** Toggles the sidebar "Selection" drawer open or closed. */
	void ToggleSidebarSelectionDrawer();

	/** Undocks the docked sidebar drawer if docked or docks the sidebar drawer if there is one open and no currently docked drawer. */
	void ToggleSidebarDrawerDocked();

	virtual TSharedRef<ISequencerTrackFilters> GetFilterInterface() const override;

	TSharedRef<FSequencerFilterBar> GetFilterBar() const;

	/** Checks to see if a clock extension wants the standard PlayRate combo menu hidden */
	EVisibility GetPlayRateComboVisibility() const;

	virtual TSharedPtr<UE::Sequencer::SOutlinerView> GetOutlinerViewWidget() const override;
	
	/** @return Whether the sequencer selection is being updated from changes to the external selection. */
	bool IsUpdatingSequencerSelection() const { return bUpdatingSequencerSelection; }

private:
	/** Unbind from the delegates of the given settings object. */
	void UnbindFromSequencerSettingsDelegates(USequencerSettings* InSettings);
	/** Setup delegate bindings to the given settings object. */
	void BindToSequencerSettingsDelegates(USequencerSettings* InSettings);

	/** Update the time bases for the current movie scene */
	void UpdateTimeBases();

	/** Toggles Sequencer selection limiting. */
	void ToggleLimitViewportSelection();

	void ForEachSubSequenceRecursively(UMovieSceneSequence* const InSequence, const TFunctionRef<bool(UMovieSceneSequence* const InCurrentSequence)>& InFunction);

	FText GetSidebarSelectionDrawerToolTipText() const;

	void SortSelectedOutlinerItems(const bool bInSortBySelectOrder, const bool bInDescending);

	bool CanAlignSelectionToPlayhead() const;
	void AlignSelectionToPlayhead() const;

	void ToggleTrackSelectionPin();
	void ToggleTrackSelectionLock();
	void ToggleTrackSelectionDeactive();
	void ToggleTrackSelectionMute();
	void ToggleTrackSelectionSolo();

	TSharedPtr<UE::Sequencer::SimpleView::FSimpleViewTimeline> GetSimpleViewTimeline() const;

	/** User-supplied settings object for this sequencer */
	TObjectPtr<USequencerSettings> Settings;

	/** User-supplied settings object for this sequencer */
	TObjectPtr<UMovieSceneTracksSettings> TracksSettings;

	/** Command list for sequencer commands (Sequencer widgets only). */
	TSharedRef<FUICommandList> SequencerCommandBindings;

	/** Command list for sequencer commands (shared by non-Sequencer). */
	TSharedRef<FUICommandList> SequencerSharedBindings;

	/** Command list privately shared with the Curve Editor to allow a subset of keybinds to have matching behavior there. */
	TSharedRef<FUICommandList> CurveEditorSharedBindings;

	/** List of tools we own */
	TArray<TSharedPtr<ISequencerTrackEditor>> TrackEditors;
	TMap<FObjectKey, TSharedPtr<ISequencerTrackEditor>> TrackEditorsByType;

	/** Decoration editors owned by this sequencer */
	TArray<TSharedPtr<ISequencerDecorationEditor>> DecorationEditors;

	/** Lazily created keying fallback used for decoration channels whose owning class has no registered editor. */
	mutable TSharedPtr<ISequencerDecorationEditor> DefaultDecorationEditor;

	/** List of Outliner column creators that are supported by the Sequencer. */
	TMap<FName, TSharedPtr<UE::Sequencer::IOutlinerColumn>> OutlinerColumns;

	/** List of Outliner indicator item creators that are supported by the Sequencer. */
	TMap<FName, TSharedPtr<UE::Sequencer::IOutlinerIndicatorBuilder>> OutlinerIndicators;

	/** List of object bindings we can use */
	TArray<TSharedPtr<ISequencerEditorObjectBinding>> ObjectBindings;

	/** Listener for object changes being made while this sequencer is open*/
	TSharedPtr<ISequencerObjectChangeListener> ObjectChangeListener;

	/** Responsible for getting the keyed status of a property (whether it's keyed in current frame, other frame, etc)  */
	TSharedPtr<FSequencerPropertyKeyedStatusHandler> PropertyKeyedStatusHandler;

	/** Main sequencer widget */
	TSharedPtr<SSequencer> SequencerWidget;
	/** Returned by the public API GetSequencerWidget. Allows for the underlying SSequencer widget to be destroyed by ~FSequencer. */
	TSharedPtr<UE::Sequencer::SSequencerWrapperWidget> SequencerWrapperWidget;

	/** Spawn register for keeping track of what is spawned */
	TSharedPtr<FMovieSceneSpawnRegister> SpawnRegister;

	/** The asset editor that created this Sequencer if any */
	TWeakPtr<IToolkitHost> ToolkitHost;

	/** A copy of the supported features/capabilities we were initialized with. */
	FSequencerHostCapabilities HostCapabilities;
	
	/** Active customization callbacks */
	TArray<FOnSequencerPaste> OnPaste;

	TWeakObjectPtr<UMovieSceneSequence> RootSequence;
	FMovieSceneRootEvaluationTemplateInstance RootTemplateInstance;

	/** A stack of the current sequence hierarchy for keeping track of nestled sequences. */
	TArray<FMovieSceneSequenceID> ActiveTemplateIDs;

	/** A stack of sequences that have been navigated to. */
	TArray<FMovieSceneSequenceID> TemplateIDForwardStack;
	TArray<FMovieSceneSequenceID> TemplateIDBackwardStack;

	/**
	* The active state of each sequence. A sequence can be in another sequence multiple times
	* and the owning subsection contains the active state of the sequence, so this stack is kept 
	* in sync with the ActiveTemplateIDs as you enter a sequence via the specific subsection node.
	 */
	TArray<bool> ActiveTemplateStates;

	/** Time transformation from the root sequence to the currently edited sequence. */
	FMovieSceneSequenceTransform GlobalPlaybackWarpTransform;
	FMovieSceneSequenceTransform RootToUnwarpedLocalTransform;
	FMovieSceneSequenceTransform RootToWarpedLocalTransform;
	FMovieSceneSequenceTransform LocalToWarpedLocalTransform;

	/** Breadcrumbs to the current local time from the root sequence. */
	FMovieSceneTransformBreadcrumbs CurrentTimeBreadcrumbs;
	/** Breadcrumbs to the local time that scrubbing was started at. */
	FMovieSceneTransformBreadcrumbs ScrubStartBreadcrumbs;
	/** A linear offset to apply to time operations when scrubbing. Used for traversing loop boundaries etc. */
	FFrameTime ScrubLinearOffset;

	/** The time range target to be viewed */
	TRange<double> TargetViewRange;

	/** The last time range that was viewed */
	TRange<double> LastViewRange;

	/** The index of the playback speed in the supported playback speeds */
	int32 CurrentSpeedIndex;

	/** The index of the playback speed before we started playing */
	int32 SpeedIndexBeforePlay;
	
	/** The view range before zooming */
	TRange<double> ViewRangeBeforeZoom;

	/** The amount of autoscroll pan offset that is currently being applied */
	TOptional<float> AutoscrollOffset;

	/** The amount of autoscrub offset that is currently being applied */
	TOptional<float> AutoscrubOffset;

	struct FAutoScrubTarget
	{
		FAutoScrubTarget(FFrameTime InDestinationTime, FFrameTime InSourceTime, double InStartTime)
			: DestinationTime(InDestinationTime)
			, SourceTime(InSourceTime)
			, StartTime(InStartTime) {}
		FFrameTime DestinationTime;
		FFrameTime SourceTime;
		double StartTime;
	};

	TOptional<FAutoScrubTarget> AutoScrubTarget;

	/** The latest scrub position request, to be applied in the next tick */
	TOptional<TTuple<FFrameTime, UE::Sequencer::ETimeDomain>> PendingScrubPosition;

	/** Zoom smoothing curves */
	FCurveSequence ZoomAnimation;
	FCurveHandle ZoomCurve;

	/** Overlay fading curves */
	FCurveSequence OverlayAnimation;
	FCurveHandle OverlayCurve;

	FCurveSequence RecordingAnimation;

	/** Whether we are playing, recording, etc. */
	EMovieScenePlayerStatus::Type PlaybackState;

	/** Current play position */
	FMovieScenePlaybackPosition PlayPosition;

	/** The playback speed */
	float PlaybackSpeed;

	/** The playback speed before we started playing */
	float PlaybackSpeedBeforePlay;

	/** The shuttle multiplier */
	float ShuttleMultiplier;

	bool bPerspectiveViewportPossessionEnabled;
	bool bPerspectiveViewportCameraCutEnabled;

	/** True if this sequencer is being edited within the level editor */
	bool bIsEditingWithinLevelEditor;

	/** Whether the sequence should be editable or read only */
	bool bReadOnly;

	/** Scrub style provided on construction */
	ESequencerScrubberStyle ScrubStyle;

	/** Generic Popup Entry */
	TWeakPtr<IMenu> EntryPopupMenu;

	TArray<TSharedRef<UE::Sequencer::FSequencerNumericTypeInterface>> NumericTypeInterfaces;

	/** Stores a dirty bit for whether the sequencer tree (and other UI bits) may need to be refreshed.  We
	    do this simply to avoid refreshing the UI more than once per frame. (e.g. during live recording where
		the MovieScene data can change many times per frame.) */
	bool bNeedTreeRefresh;

	//FSequencerSelection Selection;
	FSequencerSelectionPreview SelectionPreview;

	/** Represents the tree of nodes to display in the animation outliner. */
	TSharedRef<FSequencerNodeTree> NodeTree;

	/** A delegate which is called when the sequencer closes. */
	FOnCloseEvent OnCloseEventDelegate;

	/** A delegate which is called any time the global time changes. */
	FOnGlobalTimeChanged OnGlobalTimeChangedDelegate;

	/** A delegate which is called whenever the user begins playing the sequence. */
	FOnPlayEvent OnPlayDelegate;

	/** A delegate which is called whenever the user stops playing the sequence. */
	FOnStopEvent OnStopDelegate;

	/** A delegate which is called whenever the user toggles recording. */
	FOnRecordEvent OnRecordDelegate;

	/** A delegate which is called whenever the treeview changes. */
	FOnTreeViewChanged OnTreeViewChangedDelegate;

	/** A delegate which is called whenever the user begins scrubbing. */
	FOnBeginScrubbingEvent OnBeginScrubbingDelegate;

	/** A delegate which is called whenever the user stops scrubbing. */
	FOnEndScrubbingEvent OnEndScrubbingDelegate;

	/** A delegate which is called any time the movie scene data is changed. */
	FOnMovieSceneDataChanged OnMovieSceneDataChangedDelegate;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FOnChannelChanged OnChannelChangedDelegate_DEPRACRATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** A delegate which is called when the channel is changed by Sequencer. */
	FMovieSceneOnChannelChangedWithTime OnChannelChangedDelegate;

	/** A delegate which is called any time the movie scene bindings are changed. */
	FOnMovieSceneBindingsChanged OnMovieSceneBindingsChangedDelegate;

	/** A delegate which is called any time a binding is pasted. */
	FOnMovieSceneBindingsPasted OnMovieSceneBindingsPastedDelegate;

	/** A delegate which is called any time the sequencer selection changes. */
	FOnSelectionChangedObjectGuids OnSelectionChangedObjectGuidsDelegate;

	/** A delegate which is called any time the sequencer selection changes. */
	FOnSelectionChangedTracks OnSelectionChangedTracksDelegate;

	/** A delegate which is called any time the sequencers curve eidtor selection changes. */
	FOnCurveDisplayChanged OnCurveDisplayChanged;

	/** A delegate which is called any time the sequencer selection changes. */
	FOnSelectionChangedSections OnSelectionChangedSectionsDelegate;

	FOnActorAddedToSequencer OnActorAddedToSequencerEvent;
	FOnCameraCut OnCameraCutEvent;
	FOnPreSave OnPreSaveEvent;
	FOnPostSave OnPostSaveEvent;
	FOnPreNavigateToSequence OnPreNavigateToSequenceEvent;
	FOnActivateSequence OnActivateSequenceEvent;
	
	/** Delegate for Curve Display Changed Event from the Curve Editor, which we than pass to the FOnCurveDisplayChanged delegate */
	void OnCurveModelDisplayChanged(FCurveModel *InCurveModel, bool bDisplayed, const FCurveEditor* InCurveEditor);

	int32 SilentModeCount;

	/** When true the sequencer selection is being updated from changes to the external selection. */
	bool bUpdatingSequencerSelection;

	/** When true the external selection is being updated from changes to the sequencer selection. */
	bool bUpdatingExternalSelection;

	/** The maximum tick rate prior to playing (used for overriding delta time during playback). */
	TOptional<double> OldMaxTickRate;

	/** Timing manager that can adjust playback times */
	TSharedPtr<FMovieSceneTimeController> TimeController;

	struct FCachedViewTarget
	{
		/** The player controller we're possessing */
		TWeakObjectPtr<APlayerController> PlayerController;
		/** The view target it was pointing at before we took over */
		TWeakObjectPtr<AActor> ViewTarget;
	};

	/** Cached array of view targets that were set before we possessed the player controller with a camera from sequencer */
	TArray<FCachedViewTarget> PrePossessionViewTargets;

	/** Attribute used to retrieve the playback context for this frame */
	TAttribute<UObject*> PlaybackContextAttribute;

	/** Attribute used to retrieve the playback client for this frame */
	TAttribute<IMovieScenePlaybackClient*> PlaybackClientAttribute;

	/** Cached playback context for this frame */
	TWeakObjectPtr<UObject> CachedPlaybackContext;

	/** Cached playback context for this frame */
	TWeakInterfacePtr<IMovieScenePlaybackClient> CachedPlaybackClient;

	/** Attribute used to retrieve event contexts */
	TAttribute<TArray<UObject*>> EventContextsAttribute;

	/** Event contexts retrieved from the above attribute once per frame */
	TArray<TWeakObjectPtr<UObject>> CachedEventContexts;

	UE::Sequencer::ETimeDomain TimeOperationDomain;

	/** When true, sequence will be forcefully evaluated on the next tick */
	bool bNeedsEvaluate;

	bool bIsDraggingPlaybackRange = false;
	bool bPlaybackRangeChangedDuringDrag = false;

	/** When true, cached data will be invalidated on the next tick */
	bool bNeedsInvalidateCachedData;

	FAcquiredResources AcquiredResources;

	bool bDraggingMarks;
	bool bGlobalMarkedFramesCached;
	TArray<FMovieSceneMarkedFrame> GlobalMarkedFramesCache;

	/** If set, pause playback on this frame */
	TOptional<FFrameTime> PauseOnFrame;

	/** The range of the currently displayed sub sequence in relation to its parent section, in the resolution of the current sub sequence */
	TRange<FFrameNumber> SubSequenceRange;

	TObjectPtr<UMovieSceneCompiledDataManager> CompiledDataManager;
	TSharedPtr<FMovieSceneEntitySystemRunner> Runner;

	TSharedPtr<UE::MovieScene::IDeferredSignedObjectChangeHandler> DeferredSignedObjectChangeHandler;

	TMap<FName, TFunction<void()>> CleanupFunctions;

	/** Transient collection of keys that is used for jumping between keys contained within the current selection */
	TUniquePtr<FSequencerKeyCollection> SelectedKeyCollection;

	TSharedPtr<UE::MovieScene::FInitialValueCache> InitialValueCache;

	/** A signature that will suppress auto evaluation when it is the only change dirtying the template. */
	TOptional<TTuple<TWeakObjectPtr<UMovieSceneSequence>, FGuid>> SuppressAutoEvalSignature;

	TUniquePtr<FObjectBindingTagCache> ObjectBindingTagCache;

	TSharedPtr<UE::Sequencer::FSequencerEditorViewModelImpl> ViewModel;

	struct FCachedViewState
	{
		FCachedViewState()
			: bValid(false) {}

	public:
		void StoreViewState();
		void RestoreViewState();

	private:
		bool bValid;
		TArray<TPair<int32, bool> > GameViewStates;
	};

	FCachedViewState CachedViewState;
	
	/** Frame time last evaluated at, needed to make sure we evaluate at the right time when interrogating values*/
	FFrameTime LastEvaluatedLocalTime;

	TOptional<FMovieSceneSequenceID> ScrubPositionParent;
	/** Cache of all bound cameras in the sequence hierarchy */
	TMap<AActor*, FGuid> CachedCameraActors;
	uint32 LastKnownStateSerial = 0;

	/** Cached list of supported custom binding types */
	TArray<const TSubclassOf<UMovieSceneCustomBinding>> SupportedCustomBindingTypes;

	/** True if limiting selection to only Sequencer objects */
	static bool bSelectionLimited;

	/** Delegate that is called when selection limiting has been toggled on or off */
	FOnViewportSelectionLimitedChanged OnSelectionLimitedChangedDelegate;

	enum class ETimeControllerState
	{
		ReadyToPlay,
		PreparingToPlay
	};
	ETimeControllerState TimeControllerState;
};
