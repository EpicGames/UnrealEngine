// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerAddKeyOperation.h"

#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Containers/Array.h"
#include "HAL/PlatformCrt.h"
#include "IKeyArea.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Extensions/ISectionOwnerExtension.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "SequencerKeyParams.h"
#include "ISequencerDecorationEditor.h"
#include "Templates/Tuple.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"

namespace UE::Sequencer { class FSequenceModel; }


namespace UE
{
namespace Sequencer
{

FAddKeyOperation FAddKeyOperation::FromNodes(const TSet<TWeakViewModelPtr<IOutlinerExtension>>& InNodes)
{
	FAddKeyOperation Operation;

	TArray<TWeakPtr<FViewModel>> FilteredNodes;

	// Remove any child nodes that have a parent also included in the set
	for (const TWeakViewModelPtr<IOutlinerExtension>& ProspectiveNode : InNodes)
	{
		TSharedPtr<FViewModel> Parent = ProspectiveNode.Pin().AsModel()->GetParent();
		while (Parent)
		{
			if (InNodes.Contains(CastViewModel<IOutlinerExtension>(Parent)))
			{
				goto Continue;
			}
			Parent = Parent->GetParent();
		}

		FilteredNodes.Add(ProspectiveNode);

	Continue:
		continue;
	}

	Operation.AddPreFilteredNodes(FilteredNodes);
	return Operation;
}

FAddKeyOperation FAddKeyOperation::FromNode(TWeakPtr<FViewModel> InNode)
{
	FAddKeyOperation Operation;
	Operation.AddPreFilteredNodes(MakeArrayView(&InNode, 1));
	return Operation;
}

FAddKeyOperation FAddKeyOperation::FromKeyAreas(ISequencerTrackEditor* TrackEditor, const TArrayView<TSharedRef<IKeyArea>> InKeyAreas)
{
	FAddKeyOperation Operation;
	if (ensure(TrackEditor))
	{
		for (const TSharedRef<IKeyArea>& KeyArea : InKeyAreas)
		{
			Operation.ProcessKeyArea(TrackEditor, KeyArea.ToSharedPtr());
		}
	}
	return Operation;
}

FAddKeyOperation FAddKeyOperation::FromChannelModels(const TSet<TWeakViewModelPtr<FChannelModel>>& InChannelModels)
{
	FAddKeyOperation Operation;

	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : InChannelModels)
	{
		const TSharedPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
		if (!ChannelModel.IsValid())
		{
			continue;
		}

		const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea();
		if (!KeyArea.IsValid())
		{
			continue;
		}

		const TViewModelPtr<ITrackExtension> TrackModel = ChannelModel->FindAncestorOfType<ITrackExtension>(/*bIncludeThis=*/true);
		if (!TrackModel.IsValid())
		{
			continue;
		}

		const TSharedPtr<ISequencerTrackEditor> TrackEditor = TrackModel->GetTrackEditor();
		if (!TrackEditor.IsValid())
		{
			continue;
		}

		Operation.ProcessKeyArea(TrackEditor.Get(), KeyArea);
	}

	return Operation;
}

void FAddKeyOperation::AddPreFilteredNodes(TArrayView<const TWeakPtr<FViewModel>> FilteredNodes)
{
	TSharedPtr<FSequenceModel> SequenceModel;
	const TArray<FViewModelTypeID> TypeIDs({ ISectionOwnerExtension::ID, IOutlinerExtension::ID });
	for (const TWeakPtr<FViewModel>& FilteredNode : FilteredNodes)
	{
		TSharedPtr<FViewModel> Node = FilteredNode.Pin();
		// A self-contained track owns its sections directly; an outer SectionOwner
		// may belong to a different track with a different track editor.
		TSharedPtr<FViewModel> ParentModel;
		if (Node->IsA<ITrackExtension>() && Node->IsA<ISectionOwnerExtension>() && Node->IsA<IOutlinerExtension>())
		{
			ParentModel = Node;
		}
		else
		{
			ParentModel = Node->FindAncestorOfTypes(MakeArrayView(TypeIDs));
		}
		if (ParentModel)
		{
			ConsiderKeyableAreas(ParentModel->CastThisShared<ISectionOwnerExtension>(), Node);
		}
		else
		{
			constexpr bool bIncludeThis = true;
			for (FParentFirstChildIterator Child(FilteredNode.Pin(), bIncludeThis); Child; ++Child)
			{
				if (Child->IsA<ISectionOwnerExtension>() && Child->IsA<IOutlinerExtension>())
				{
					TViewModelPtr<ISectionOwnerExtension> SectionOwner = Child->CastThisShared<ISectionOwnerExtension>();
					ConsiderKeyableAreas(SectionOwner, *Child);

					// Stop recursively iterating everything within this Child again since we just considered all keyable areas for this and its children
					Child.IgnoreCurrentChildren();
				}
			}
		}
	}
}

bool FAddKeyOperation::ConsiderKeyableAreas(TViewModelPtr<ISectionOwnerExtension> InSectionOwner, FViewModelPtr KeyAnythingBeneath)
{
	bool bKeyedAnything = false;

	constexpr bool bIncludeThis = true;
	
	// Prefer the section that is marked SectionToKey
	TViewModelPtr<ITrackExtension> TrackModel = InSectionOwner.AsModel()->FindAncestorOfType<ITrackExtension>(bIncludeThis);
	TSharedPtr<ISequencerTrackEditor> TrackEditor = TrackModel ? TrackModel->GetTrackEditor() : nullptr;
	for (const TViewModelPtr<FSectionModel>& SectionModel : KeyAnythingBeneath->GetDescendantsOfType<FSectionModel>(bIncludeThis))
	{
		UMovieSceneSection* Section = SectionModel->GetSection();
		UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>();
		if (Track && Track->GetSectionToKey() == Section)
		{
			for (TParentFirstChildIterator<FChannelModel> ChannelModelIt(SectionModel->AsShared(), bIncludeThis); ChannelModelIt; ++ChannelModelIt)
			{	
				if (!ChannelModelIt->GetLinkedOutlinerItem() || ChannelModelIt->GetLinkedOutlinerItem()->IsFilteredOut() == false)
				{
					bKeyedAnything |= ProcessKeyArea(TrackEditor.Get(), ChannelModelIt->GetKeyArea());
				}
			}
		}
	}

	// Otherwise if nothing was found, key all
	if (!bKeyedAnything)
	{
		for (TParentFirstChildIterator<FChannelGroupModel> ChannelGroupModelIt(KeyAnythingBeneath->AsShared(), bIncludeThis); ChannelGroupModelIt; ++ChannelGroupModelIt)
		{
			bKeyedAnything |= ProcessKeyArea(TrackEditor.Get(), *ChannelGroupModelIt);
		}
	}

	return bKeyedAnything;
}

bool FAddKeyOperation::ProcessKeyArea(ISequencerTrackEditor* InTrackEditor, TViewModelPtr<FChannelGroupModel> InChannelGroupModel)
{
	bool bKeyedAnything = false;

	IOutlinerExtension* OutlinerExtension = InChannelGroupModel->CastThis<IOutlinerExtension>();
	if (!OutlinerExtension || OutlinerExtension->IsFilteredOut() == false)
	{
		constexpr bool bIncludeThis = true;
		for (const TWeakViewModelPtr<FChannelModel>& WeakChannel : InChannelGroupModel->GetChannels())
		{
			if (TSharedPtr<FChannelModel> Channel = WeakChannel.Pin())
			{
				bKeyedAnything |= ProcessKeyArea(InTrackEditor, Channel->GetKeyArea());
			}
		}
	}

	return bKeyedAnything;
}

bool FAddKeyOperation::ProcessKeyArea(ISequencerTrackEditor* InTrackEditor, TSharedPtr<IKeyArea> InKeyArea)
{
	// Decoration-owned channels are keyed through the decoration editor at Commit time
	// (the per-Sequencer editor lookup needs the ISequencer that's only available there).
	if (const FMovieSceneChannelMetaData* MetaData = InKeyArea->GetChannel().GetMetaData())
	{
		UObject* OwningObject = MetaData->WeakOwningObject.Get();
		UMovieSceneSection* OwningSection = InKeyArea->GetOwningSection();
		if (OwningObject && OwningObject != static_cast<UObject*>(OwningSection))
		{
			DecorationKeyOperations.Add({OwningSection, InKeyArea->GetChannel()});
			return true;
		}
	}

	TSharedPtr<ISequencerSection> Section       = InKeyArea->GetSectionInterface();
	UMovieSceneSection*           SectionObject = Section       ? Section->GetSectionObject()                      : nullptr;
	UMovieSceneTrack*             TrackObject   = SectionObject ? SectionObject->GetTypedOuter<UMovieSceneTrack>() : nullptr;

	if (TrackObject)
	{
		GetTrackOperation(InTrackEditor).Populate(TrackObject, Section, InKeyArea);
		return true;
	}

	return false;
}

void FAddKeyOperation::Commit(FFrameNumber KeyTime, ISequencer& InSequencer, TArray<FAddKeyResult>* OutResults)
{
	for (TTuple<ISequencerTrackEditor*, FKeyOperation>& Pair : OperationsByTrackEditor)
	{
		Pair.Value.InitializeOperation(KeyTime);
		Pair.Key->ProcessKeyOperation(KeyTime, Pair.Value, InSequencer, OutResults);
	}

	// Process decoration key operations collected during ProcessKeyArea.
	for (const FDecorationKeyOp& Op : DecorationKeyOperations)
	{
		if (!Op.Section)
		{
			continue;
		}
		auto [DecorationEditor, Decoration] = InSequencer.FindDecorationOwner(Op.ChannelHandle, Op.Section);
		if (DecorationEditor && Decoration)
		{
			DecorationEditor->AddOrUpdateKey(*Decoration, *Op.Section, Op.ChannelHandle, KeyTime, FGuid(), InSequencer);
		}
	}

	InSequencer.UpdatePlaybackRange();
	InSequencer.NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
}

FKeyOperation& FAddKeyOperation::GetTrackOperation(ISequencerTrackEditor* TrackEditor)
{
	return OperationsByTrackEditor.FindOrAdd(TrackEditor);
}

} // namespace Sequencer
} // namespace UE

