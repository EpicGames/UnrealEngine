// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerBakingSetupRestore.h"
#include "ISequencer.h"
#include "SequencerSettings.h"

namespace UE::Sequencer
{
	const TCHAR* CVarName = TEXT("Sequencer.SubsequenceAnimBakeInIsolation");

	static bool bSubSequenceBakingInIsolation = true; //default to false for now will turn it on after more testing
	static FAutoConsoleVariableRef CVarSetIsSequencerBakingInIsolation(
		CVarName,
		bSubSequenceBakingInIsolation,
		TEXT("Automatically Force Evaluate Subsequence In Isolation when Baking")
	);

	FSequencerBakingSetupRestore::FSequencerBakingSetupRestore(TSharedPtr<ISequencer>& SequencerPtr)
	{
		if (!SequencerPtr || bSubSequenceBakingInIsolation == false)
		{
			return;
		}
		WeakSequencer = SequencerPtr;
		UMovieSceneSequence* MovieSceneSequence = SequencerPtr->GetFocusedMovieSceneSequence();
		UMovieSceneSequence* RootMovieSceneSequence = SequencerPtr->GetRootMovieSceneSequence();
		if (MovieSceneSequence != RootMovieSceneSequence &&
			SequencerPtr->GetSequencerSettings()->ShouldEvaluateSubSequencesInIsolation() == false)
		{
			bRestoreShouldEvaluateSubSequencesInIsolation = false;
			SequencerPtr->GetSequencerSettings()->SetEvaluateSubSequencesInIsolation(true);
		}
	}
	FSequencerBakingSetupRestore::~FSequencerBakingSetupRestore()
	{
		if (WeakSequencer.IsValid() && bRestoreShouldEvaluateSubSequencesInIsolation.IsSet())
		{
			TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
			SequencerPtr->GetSequencerSettings()->SetEvaluateSubSequencesInIsolation(false);
		}
	}

} // namespace UE::Sequencer