// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerChannelTraits.h"

#include "CurveModel.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"

class FCurveModel;
class FSequencerSectionPainter;
struct FGeometry;
struct FMovieSceneChannel;

namespace Sequencer
{


void DrawKeys(FMovieSceneChannel* Channel, TArrayView<const FKeyHandle> InHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams)
{
	// By default just render diamonds for keys
	FKeyDrawParams DefaultParams;
	DefaultParams.BorderBrush = DefaultParams.FillBrush = FAppStyle::Get().GetBrush("Sequencer.KeyDiamond");
	DefaultParams.ConnectionStyle = EKeyConnectionStyle::Solid;

	for (FKeyDrawParams& Param : OutKeyDrawParams)
	{
		Param = DefaultParams;
	}
}
/* Most channels do nothing*/
int32 DrawExtra(FMovieSceneChannel* InChannel, const UMovieSceneSection* Owner, const FSequencerChannelPaintArgs& PaintArgs, int32 LayerId)
{
	return LayerId;
}


bool SupportsCurveEditorModels(const FMovieSceneChannelHandle& ChannelHandle)
{
	return false;
}

TUniquePtr<FCurveModel> CreateCurveEditorModel(const FMovieSceneChannelHandle& ChannelHandle, const UE::Sequencer::FCreateCurveEditorModelParams& Params)
{
	return nullptr;
}

}	// namespace Sequencer