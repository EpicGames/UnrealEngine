// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerClipboardReconciler.h"

#include "HAL/PlatformCrt.h"
#include "IKeyArea.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

class UMovieSceneSection;

TMap<FName, TArray<FName>> FSequencerClipboardReconciler::KeyAreaAliases;

FSequencerClipboardReconciler::FSequencerClipboardReconciler(TSharedRef<const FMovieSceneClipboard> InClipboard)
	: Clipboard(MoveTemp(InClipboard))
	, bCanAutoPaste(false)
{
}

bool FSequencerClipboardReconciler::Paste(const FSequencerPasteEnvironment& PasteEnvironment)
{
	if (PasteDestination.Num() == 0)
	{
		return false;
	}

	if (Reconcile())
	{
		return PasteImpl(PasteEnvironment);
	}

	return false;
}

bool FSequencerClipboardReconciler::Reconcile()
{
	if (!bReconcileResult.IsSet())
	{
		if (PasteDestination.Num() == 0)
		{
			bReconcileResult = false;
		}
		else if (Clipboard->GetKeyTrackGroups().Num() == 1)
		{
			bReconcileResult = ReconcileOneToMany();
		}
		else
		{
			bReconcileResult = false;
		}
	}

	return bReconcileResult.GetValue();
}

FSequencerClipboardPasteGroup FSequencerClipboardReconciler::AddDestinationGroup()
{
	return FSequencerClipboardPasteGroup(PasteDestination);
}

void FSequencerClipboardReconciler::AddTrackAlias(FName TargetName, FName Alias)
{
	KeyAreaAliases.FindOrAdd(TargetName).Add(Alias);
	KeyAreaAliases.FindOrAdd(Alias).Add(TargetName);
}

bool FSequencerClipboardReconciler::CanAutoPaste() const
{
	return bCanAutoPaste;
}

bool FSequencerClipboardReconciler::PasteImpl(const FSequencerPasteEnvironment& PasteEnvironment)
{
	bool bAnythingPasted = false;

	for (auto& MetaDataPair : MetaData)
	{
		const TArray<FMovieSceneClipboardKeyTrack>& SrcArray = Clipboard->GetKeyTrackGroups()[MetaDataPair.Value.SourceGroup];
		TArray<TSharedPtr<UE::Sequencer::FChannelModel>>& DstArray = PasteDestination[MetaDataPair.Key];

		int32 SrcIndex = 0;
		int32 DstIndex = 0;

		FPasteMetaData::EMethod Method = MetaDataPair.Value.Method;

		if (Method == FPasteMetaData::Custom)
		{
			for (auto& IndexPair : MetaDataPair.Value.DestToSrcMap)
			{
				if (UMovieSceneSection* Section = DstArray[DstIndex]->GetSection())
				{
					TArray<FKeyHandle> PastedKeys =  DstArray[IndexPair.Key]->GetKeyArea()->PasteKeys(SrcArray[IndexPair.Value], Clipboard->GetEnvironment(), PasteEnvironment);

					for (FKeyHandle KeyHandle : PastedKeys)
					{
						PasteEnvironment.ReportPastedKey(KeyHandle, DstArray[IndexPair.Key]);
					}
					bAnythingPasted = true;
				}
			}
		}
		else for (; SrcIndex < SrcArray.Num() && DstIndex < DstArray.Num();)
		{
			if (UMovieSceneSection* Section = DstArray[DstIndex]->GetSection())
			{
				TArray<FKeyHandle> PastedKeys = DstArray[DstIndex]->GetKeyArea()->PasteKeys(SrcArray[SrcIndex], Clipboard->GetEnvironment(), PasteEnvironment);
				for (FKeyHandle KeyHandle : PastedKeys)
				{
					PasteEnvironment.ReportPastedKey(KeyHandle, DstArray[DstIndex]);
				}

				bAnythingPasted = true;
			}

			switch (Method)
			{
			case FPasteMetaData::Compress:
				++SrcIndex;
				break;

			case FPasteMetaData::Expand:
				++DstIndex;
				break;

			case FPasteMetaData::Apply:
				++SrcIndex;
				++DstIndex;
				break;

			case FPasteMetaData::ApplyRepeating:
				++SrcIndex;
				++DstIndex;
				if (SrcIndex >= SrcArray.Num())
				{
					// Wrap around source index
					SrcIndex = 0;
				}
				break;
			}
		}
	}

	return bAnythingPasted;
}

bool FSequencerClipboardReconciler::FindMatchingGroup(const TArray<TSharedPtr<UE::Sequencer::FChannelModel>>& Destination, const TArray<FMovieSceneClipboardKeyTrack>& Source, TMap<int32, int32>& Map, bool bAllowAliases)
{
	// If we find an exact match, we use custom
	bool bFoundMatch = false;
	for (int32 DstIndex = 0; DstIndex < Destination.Num(); ++DstIndex)
	{
		FName DstName = Destination[DstIndex]->GetKeyArea()->GetName();

		const int32 SourceIndex = Source.IndexOfByPredicate([&](const FMovieSceneClipboardKeyTrack& Track){

			FName SrcName = Track.GetName();

			if (SrcName == DstName)
			{
				return true;
			}

			if (const TArray<FName>* CustomRules = KeyAreaAliases.Find(DstName))
			{
				return CustomRules->ContainsByPredicate([&](const FName& Match){
					return SrcName == Match;
				});
			}

			return false;
		});

		if (SourceIndex != INDEX_NONE)
		{
			Map.Add(DstIndex, SourceIndex);
			bFoundMatch = true;
		}
	}

	return bFoundMatch;
}

bool FSequencerClipboardReconciler::ReconcileOneToMany()
{
	const TArray<FMovieSceneClipboardKeyTrack>& Source = Clipboard->GetKeyTrackGroups()[0];
	const int32 NumSourceTracks = Source.Num();

	// We have one group of tracks, and are pasting into one or more groups of tracks
	for (int32 Index = 0; Index < PasteDestination.Num(); ++Index)
	{
		FPasteMetaData& ThisMetaData = MetaData.Add(Index, FPasteMetaData(0, FPasteMetaData::Apply));

		const int32 NumDestTracks = PasteDestination[Index].Num();

		// Precedence list:
		//	1. Find an exact name match in any destination
		//	2. Expand single source tracks to multiple destination tracks
		//	3. Compress multiple source tracks to a single destination track
		//	4. Find ay alias for the source tracks by name
		//	5. Blindly copy the source tracks by order if they are numerically equal to the destination
		//	6. Blindly copy the source tracks in a repeating way across the destination tracks, if dest is a multiple of src
		//	7. Bail - we can't make any more reasonable assumptions about what the user expects
		if (FindMatchingGroup(PasteDestination[Index], Source, ThisMetaData.DestToSrcMap, false /*bAllowAliases*/))
		{
			bCanAutoPaste = true;
			ThisMetaData.Method = FPasteMetaData::Custom;
		}
		else if (NumSourceTracks == 1 && NumDestTracks != 1)
		{
			// If we're pasting a single track, paste it into all destination areas
			ThisMetaData.Method = FPasteMetaData::Expand;
		}
		else if (NumDestTracks == 1 && NumSourceTracks != 1)
		{
			// If we're pasting multiple into a single track, compress them toghether
			ThisMetaData.Method = FPasteMetaData::Compress;
		}
		else if (FindMatchingGroup(PasteDestination[Index], Source, ThisMetaData.DestToSrcMap, true /*bAllowAliases*/))
		{
			ThisMetaData.Method = FPasteMetaData::Custom;
		}
		else if (NumSourceTracks == NumDestTracks)
		{
			// If they have the same number of tracks, just apply directly
			ThisMetaData.Method = FPasteMetaData::Apply;
			bCanAutoPaste = true;
		}
		else if (NumDestTracks % NumSourceTracks == 0)
		{
			// If we're into a multiple of the source tracks, apply the selection multiple times
			ThisMetaData.Method = FPasteMetaData::ApplyRepeating;
		}
		else
		{
			// Incompatible
			MetaData.Remove(Index);
		}
	}

	return MetaData.Num() != 0;
}
