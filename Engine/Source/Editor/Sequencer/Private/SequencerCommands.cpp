// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "SequencerCommands"

UE_DEFINE_TCOMMANDS(FSequencerCommands)

void FSequencerCommands::RegisterCommands()
{
	UI_COMMAND( TogglePlay, "Toggle Play", "Toggle the timeline playing", EUserInterfaceActionType::Button, FInputChord(EKeys::SpaceBar) );
	UI_COMMAND( TogglePlayViewport, "Toggle Play (Viewport)", "Toggle the timeline playing in all viewports and sequencer", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::SpaceBar));
	UI_COMMAND( ScrubTimeViewport, "Scrub Time (Viewport)", "Scrub mouse left and right to change time", EUserInterfaceActionType::Button, FInputChord(EKeys::B) );
	UI_COMMAND( PlayForward, "Play Forward", "Play the timeline forward", EUserInterfaceActionType::Button, FInputChord(EKeys::Down) );
	UI_COMMAND( JumpToStart, "Jump to Start", "Jump to the start of the playback range", EUserInterfaceActionType::Button, FInputChord(EKeys::Up) );
	UI_COMMAND( JumpToEnd, "Jump to End", "Jump to the end of the playback range", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Up) );
	UI_COMMAND( JumpToStartViewport, "Jump to Start (Viewport)", "Jump to the start of the playback range in all viewports and sequencer", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( JumpToEndViewport, "Jump to End (Viewport)", "Jump to the end of the playback range in all viewports and sequencer", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ShuttleBackward, "Shuttle Backward", "Shuttle backward", EUserInterfaceActionType::Button, FInputChord(EKeys::J) );
	UI_COMMAND( ShuttleForward, "Shuttle Forward", "Shuttle forward", EUserInterfaceActionType::Button, FInputChord(EKeys::L) );
	UI_COMMAND( Pause, "Pause", "Pause playback", EUserInterfaceActionType::Button, FInputChord(EKeys::K) );
	UI_COMMAND( RestorePlaybackSpeed, "Restore Speed", "Restores the playback speed to 1.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( StepForward, "Step Forward", "Step the timeline forward", EUserInterfaceActionType::Button, FInputChord(EKeys::Right) );
	UI_COMMAND( StepBackward, "Step Backward", "Step the timeline backward", EUserInterfaceActionType::Button, FInputChord(EKeys::Left) );
	UI_COMMAND( StepForwardViewport, "Step Forward (Viewport)", "Step the timeline forward in all viewports and sequencer", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::Period) );
	UI_COMMAND( StepBackwardViewport, "Step Backward (Viewport)", "Step the timeline backward in all viewports and sequencer", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::Comma) );
	UI_COMMAND( JumpForward, "Jump Forward", "Jump the timeline forward a user defined number of frames/times", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::Right) );
	UI_COMMAND( JumpBackward, "Jump Backward", "Jump the timeline backward a user defined number of frames/times", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::Left) );
	UI_COMMAND( StepToNextKey, "Step to Next Key (Viewport)", "Step to the next key in all viewports and sequencer", EUserInterfaceActionType::Button, FInputChord(EKeys::Period) );
	UI_COMMAND( StepToPreviousKey, "Step to Previous Key (Viewport)", "Step to the previous key in all viewports and sequencer", EUserInterfaceActionType::Button, FInputChord(EKeys::Comma) );
	UI_COMMAND( StepToNextShot, "Step to Next Shot", "Step to the next shot", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::Period) );
	UI_COMMAND( StepToPreviousShot, "Step to Previous Shot", "Step to the previous shot", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::Comma) );
	UI_COMMAND( SetStartPlaybackRange, "Set Start Playback Range", "Set the start playback range", EUserInterfaceActionType::Button, FInputChord(EKeys::LeftBracket) );
	UI_COMMAND( SetEndPlaybackRange, "Set End Playback Range", "Set the end playback range", EUserInterfaceActionType::Button, FInputChord(EKeys::RightBracket) );
	UI_COMMAND( FocusPlaybackTime, "Focus Playback Time", "Focus the view range on the current playback time without changing zoom level", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( ResetViewRange, "Reset View Range", "Reset view range to the playback range", EUserInterfaceActionType::Button, FInputChord(EKeys::A) );
	UI_COMMAND( ZoomToFit, "Zoom to Fit", "Zoom to Fit", EUserInterfaceActionType::Button, FInputChord(EKeys::F) );
	UI_COMMAND( ZoomToFitSelectionRange, "Focus Selection Range", "Zoom the view to fit the selection range, or the full time bounds if no selection range is set.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ZoomInViewRange, "Zoom into the View Range", "Zoom into the view range", EUserInterfaceActionType::Button, FInputChord(EKeys::Equals) );
	UI_COMMAND( ZoomOutViewRange, "Zoom out of the View Range", "Zoom out of the view range", EUserInterfaceActionType::Button, FInputChord(EKeys::Hyphen) );
	UI_COMMAND( NavigateBackward, "Navigate Backward", "Go backward to the previously viewed shot/subsequence", EUserInterfaceActionType::Button, FInputChord(EKeys::ThumbMouseButton), FInputChord(EModifierKey::Alt, EKeys::Left));
	UI_COMMAND( NavigateForward, "Navigate Forward", "Go forward to the previously viewed shot/subsequence", EUserInterfaceActionType::Button, FInputChord(EKeys::ThumbMouseButton2), FInputChord(EModifierKey::Alt, EKeys::Right));

	UI_COMMAND( SetSelectionRangeToNextShot, "Set Selection Range to Next Shot", "Set the selection range to the next shot", EUserInterfaceActionType::Button, FInputChord(EKeys::PageUp) );
	UI_COMMAND( SetSelectionRangeToPreviousShot, "Set Selection Range to Previous Shot", "Set the selection range to the previous shot", EUserInterfaceActionType::Button, FInputChord(EKeys::PageDown) );
	UI_COMMAND( SetPlaybackRangeToAllShots, "Set Playback Range to All Shots", "Set the playback range to all the shots", EUserInterfaceActionType::Button, FInputChord(EKeys::End) );

	UI_COMMAND( TogglePlaybackRangeLocked, "Playback Range Locked", "Prevent editing the start and end times for the sequence.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleCleanPlaybackMode, "Game View (Clean Playback Mode)", "Enable game view and hide viewport buttons while playing.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleRerunConstructionScripts, "Allow Construction Scripts to Rerun Every Frame", "Blueprint Actors need to have Run Construction Script in Sequencer active in their Class Settings.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleAsyncEvaluation, "Async Evaluation", "When enabled, enables a single asynchronous evaluation once per-frame. When disabled, forces a full blocking evaluation every time this sequence is evaluated (should be avoided for real-time content).", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleDynamicWeighting, "Dynamic Weighting", "When enabled, all blendable tracks will cache their initial values to ensure that they are able to correctly blend in/out when dynamic weights are being used.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleLoopCuts, "Loop Cuts", "When enabled, the sequence considers looping as a jump/cut. When disabled, the sequence considers looping to be seamless.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND( ToggleResetPlayheadWhenNavigating, "Reset Playhead When Navigating", "When checked, if the playhead is outside the playback range, it will be reset to the beginning when navigating in and out of subsequences", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( ToggleKeepPlaybackRangeInSectionBounds, "Keep Playback Range in Section Bounds", "When checked, the playback range will be synchronized to the section bounds", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( SetPlayheadScrubbingLimitNone, "None", "While scrubbing, don't restrict the playhead to any range", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( SetPlayheadScrubbingLimitPlaybackRange, "Playback Range", "While scrubbing, restrict the playhead to the playback range", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( SetPlayheadScrubbingLimitSelectionRange, "Selection Range", "While scrubbing, restrict the playhead to the selection range", EUserInterfaceActionType::RadioButton, FInputChord() );

	UI_COMMAND( ToggleAutoExpandNodesOnSelection, "Auto Expand Nodes on Selection", "Toggle auto expanding the outliner tree on child selection", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( ToggleAutoExpandCurveEditorOnSelection, "Auto Expand Curve Editor on Selection", "Toggles whether selecting items in Sequencer's Outliner will expand Curve Editor's tree view to show the item.", EUserInterfaceActionType::ToggleButton, FInputChord());
	
	UI_COMMAND( ToggleRestoreOriginalViewportOnCameraCutUnlock, "Restore Cinematic Viewports", "Toggle whether unlocking a camera cut track should return the viewport to its original location, or keep it where the camera cut was", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( TogglePreviewCameraCutsInSimulate, "Preview Camera Cuts in Simulate", "Toggle whether camera cuts are evaluated in SIE, or after ejecting from the player controller in PIE", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleExpandCollapseNodes, "Expand/Collapse Nodes", "Toggle expand or collapse selected nodes", EUserInterfaceActionType::Button, FInputChord(EKeys::V) );
	UI_COMMAND( ToggleExpandCollapseNodesAndDescendants, "Expand/Collapse Nodes and Descendants", "Toggle expand or collapse selected nodes and descendants", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::V) );
	UI_COMMAND( ExpandAllNodes, "Expand All Nodes", "Expand all nodes and descendants", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( CollapseAllNodes, "Collapse All Nodes", "Collapse all nodes and descendants", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SortAllNodesAndDescendants, "Sort All Nodes", "Sorts all nodes by type and then alphabetically.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND( SetSelectionRangeEnd, "Set Selection End", "Sets the end of the selection range", EUserInterfaceActionType::Button, FInputChord(EKeys::O) );
	UI_COMMAND( SetSelectionRangeStart, "Set Selection Start", "Sets the start of the selection range", EUserInterfaceActionType::Button, FInputChord(EKeys::I) );
	UI_COMMAND( ClearSelectionRange, "Clear Selection Range", "Clear the selection range", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control|EModifierKey::Shift, EKeys::X) );
	UI_COMMAND( SelectKeysInSelectionRange, "Select Keys in Selection Range", "Select all keys that fall into the selection range", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SelectSectionsInSelectionRange, "Select Sections in Selection Range", "Select all sections that fall into the selection range", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SelectMarkersInSelectionRange, "Select Markers in Selection Range", "Select all markers that fall into the selection range", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SelectAllInSelectionRange, "Select All in Selection Range", "Select all keys and section that fall into the selection range", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( SelectForward, "Select All Keys, Sections and Markers Forward", "Select all keys, sections and markers forward from the current time", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::RightBracket) );
	UI_COMMAND( SelectBackward, "Select All Keys, Sections and Markers Backward", "Select all keys, sections and markers backward from the current time", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::LeftBracket) );
	UI_COMMAND( SelectNone, "Select None", "Select none", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));

	UI_COMMAND( AddActorsToSequencer, "Add Actors", "Add actors to sequencer", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::A) );
	UI_COMMAND( AddSelectionFromContentBrowser, "Selection from Content Browser", "Add selected content from the content browser", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND( SetKey, "Set Key", "Sets a key on the selected tracks", EUserInterfaceActionType::Button, FInputChord(EKeys::Enter) );
	UI_COMMAND(SetInterpolationCubicSmartAuto, "Set Key Smart Auto", "Cubic interpolation - Smart Automatic tangents", EUserInterfaceActionType::Button, FInputChord(EKeys::Zero));
	UI_COMMAND( SetInterpolationCubicAuto, "Set Key Auto", "Cubic interpolation - Automatic tangents", EUserInterfaceActionType::Button, FInputChord(EKeys::One));
	UI_COMMAND( SetInterpolationCubicUser, "Set Key User", "Cubic interpolation - User flat tangents", EUserInterfaceActionType::Button, FInputChord(EKeys::Two));
	UI_COMMAND( SetInterpolationCubicBreak, "Set Key Break", "Cubic interpolation - User broken tangents", EUserInterfaceActionType::Button, FInputChord(EKeys::Three));

	UI_COMMAND( SetInterpolationLinear, "Set Key Linear", "Linear interpolation", EUserInterfaceActionType::Button, FInputChord(EKeys::Four));
	UI_COMMAND( SetInterpolationConstant, "Set Key Constant", "Constant interpolation", EUserInterfaceActionType::Button, FInputChord(EKeys::Five));

	UI_COMMAND( ToggleWeightedTangents, "Toggle Weighted Tangents", "Toggles cubic tangents to be weighted/non-weighted", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND( TrimSectionLeft, "Trim Section Left", "Trim section at current time to the left (keeps the right)", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Comma) );
	UI_COMMAND( TrimSectionRight, "Trim Section Right", "Trim section at current time to the right (keeps the left)", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Period) );
	UI_COMMAND( TrimOrExtendSectionLeft, "Trim or Extend Section Left", "Trim or extend closest sections to the left for the selected tracks (or all tracks if none selected) to the current time", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::LeftBracket) );
	UI_COMMAND( TrimOrExtendSectionRight, "Trim or Extend Section Right", "Trim or extend closest sections to the right for the selected tracks (or all tracks if none selected) to the current time", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::RightBracket) );
	UI_COMMAND( SplitSection, "Split Section", "Split section at current time", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Slash) );

	UI_COMMAND( TranslateLeft, "Translate Left", "Translate selected keys and sections to the left", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Left) );
	UI_COMMAND( TranslateRight, "Translate Right", "Translate selected keys and sections to the right", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Right) );

	UI_COMMAND( SetAutoKey, "Auto-key", "Create a key when channels/properties change. Only automatically adds a key when there's already a track and at least one key.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetAutoTrack, "Auto-track", "Create a track when channels/properties change.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( SetAutoChangeAll, "All", "Create a key and a track if it doesn't exist when channels/properties change.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( SetAutoChangeNone, "None", "Disable auto-keying and auto-tracking.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND( AllowAllEdits, "Allow All Edits", "Allow any edits to occur, some of which may produce tracks/keys or modify default properties.", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( AllowSequencerEditsOnly, "Allow Sequencer Edits Only", "All edits will produce either a track or a key.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( AllowLevelEditsOnly, "Allow Level Edits Only", "Properties in the details panel will be disabled if they have a track.", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(ToggleAutoKeyEnabled, "Auto-key", "Create a key when channels/properties change. Only automatically adds a key when there's already a track and at least one key.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND( SetKeyChanged, "Key Changed", "Key just the changed channel when it changes.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( SetKeyGroup, "Key Group", "Key the groups channels/properties when only one of them changes. ie. Keys all three translation channels when only translation Y changes", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( SetKeyAll, "Key All", "Key all channels/properties when only one of them changes. ie. Keys all translation, rotation, scale channels when only translation Y changes", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND( ToggleShowMarkedFrames, "Marked Frames", "Show marked frames.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( ToggleMarkAtPlayPosition, "Toggle Mark", "Sets or clears a mark at the current play position.", EUserInterfaceActionType::Button, FInputChord(EKeys::M) );
	UI_COMMAND( StepToNextMark, "Step to Next Marked Frame", "Step to the next marked frame", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::Period) );
	UI_COMMAND( StepToPreviousMark, "Step to Previous Marked Frame", "Step to the previous marked frame", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::Comma) );
	UI_COMMAND( ToggleMarksLocked, "Marked Frames Locked", "Prevent editing the marked frames.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleShowMarkedFramesGlobally, "Marked Frames Globally", "Makes marked frames in this sub-sequence visible in parent/sibling sequences for the current editor session.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ClearGlobalMarkedFrames, "Clear Global Marked Frames", "Set all marked frames in all sub-sequences to not be globally displayed.", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( ToggleAutoScroll, "Auto Scroll", "Toggle auto-scroll: When enabled, automatically scrolls the sequencer view to keep the current time visible", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Shift, EKeys::S) );

	UI_COMMAND( ChangeTimeDisplayFormat, "Change Time Display Format", "Rotates through supported display formats for time", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control|EModifierKey::Shift, EKeys::T) );
	UI_COMMAND( ToggleShowGotoBox, "Go to Time...", "Go to a particular point on the timeline", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::T) );
	UI_COMMAND( ToggleShowTransformBox, "Transform Selection...", "Transform the selected keys and sections by a given amount", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::M));
	UI_COMMAND( ToggleShowStretchBox, "Stretch/Shrink...", "Stretch or shrink a given amount, moving keys forwards/backwards as necessary", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( OpenDirectorBlueprint, "Open Director Blueprint", "Opens the director blueprint for this sequence.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( OpenTaggedBindingManager, "Open Binding Tag Manager", "Specifies options for tagging bindings within this sequence for external systems to reference as a persistent name.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( OpenNodeGroupsManager, "Open Sequencer Group Manager", "Manage groups within this sequence.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( ToggleShowTransportControl, "Transport Controls", "Enables and disables showing the transport controls panel", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( ToggleTransportControlsPoppedOut, "Popout Transport Controls", "Toggles the popped out state of the transport controls panel", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( ToggleShowRangeSlider, "Range Slider", "Enables and disables showing the time range slider", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( ToggleShowBottomTimeSlider, "Bottom Time Slider", "Enables and disables showing the bottom time slider", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( ToggleShowInfoButton, "Info Button", "Enables and disables showing the information button in the playback controls", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( ToggleIsSnapEnabled, "Enable Snapping", "Enables and disables snapping", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( ToggleSnapKeyTimesToElements, "Snap Keys to Elements", "Snap keys to other keys, sections, markers, playback range, selection range, playhead, and previous position", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleSnapSectionTimesToElements, "Snap Sections to Elements", "Snap sections to other sections, keys, markers, playback range, selection range, and playhead", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( ToggleSnapPlayTimeToKeys, "Snap to Keys While Scrubbing", "Snap the playhead to keys of the selected track while scrubbing", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleSnapPlayTimeToSections, "Snap to Sections While Scrubbing", "Snap the playhead to section bounds while scrubbing", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleSnapPlayTimeToMarkers, "Snap to Markers While Scrubbing", "Snap the playhead to markers while scrubbing", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleSnapPlayTimeToPressedKey, "Snap to the Pressed Key", "Snap the playhead to the pressed key", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleSnapPlayTimeToDraggedKey, "Snap to the Dragged Key", "Snap the playhead to the dragged key", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( ToggleSnapCurveValueToInterval, "Snap Curve Key Values", "Snap curve keys to the value snapping interval", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( ToggleForceWholeFrames, "Force Whole Frames", "Enables or disables keys/sections/playhead on whole frames", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( FindInContentBrowser, "Find in Content Browser", "Find the viewed sequence asset in the content browser", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( BrowseToObject, "Browse to Object", "Browse to the selected section's object", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::B) );
	UI_COMMAND( ToggleLayerBars, "Layer Bars", "Show/hide the layer bars to edit keyframes in bulk", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleKeyBars, "Key Bars", "Show/hide key bar connectors for quickly retiming pairs of keys", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleChannelColors, "Channel Colors", "Show/hide the channel colors for the key bars", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( ToggleShowCurveEditor, "Curve Editor", "Show the animation keys in a curve editor", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleLinkCurveEditorTimeRange, "Link Curve Editor Time Range", "Link the curve editor time range to the sequence", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleLinkFiltersWithCurveEditor, "Link Filters with Curve Editor", "Link the curve editor tree view with the sequencer filters", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND( ToggleShowPreAndPostRoll, "Pre/Post Roll", "Toggles visualization of pre and post roll", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( RenderMovie, "Render Movie", "Render this movie to a video, or image frame sequence", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( CreateCamera, "Create Camera", "Create a new camera and set it as the current camera cut", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( PasteFromHistory, "Paste From History", "Paste from the sequencer clipboard history", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::V) );

	UI_COMMAND( ConvertToSpawnable, "Convert to Spawnable", "Make the specified possessed objects spawnable from sequencer. This will allow sequencer to have control over the lifetime of the object.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( ConvertToPossessable, "Convert to Possessable", "Make the specified object bindings into level objects possessed by sequencer.", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( SaveCurrentSpawnableState, "Save Default State", "Save the current state of this spawnable object as its default properties.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( RestoreAnimatedState, "Restore Pre-Animated State", "Restore any objects that have been animated by sequencer back to their original state.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::R) );

	UI_COMMAND( FixPossessableObjectClass, "Fix Possessable Object Class", "Try to automatically fix up possessables whose object class don't match the object class of their currently bound objects.", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( RebindPossessableReferences, "Rebind Possessable References", "Rebinds all possessables in the current sequence to ensure they're using the most robust referencing mechanism.", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( ToggleEvaluateSubSequencesInIsolation, "Evaluate Sub Sequences In Isolation", "When enabled, will only evaluate the currently focused sequence; otherwise evaluate from the root sequence.", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( QuickTreeSearch, "Quick Tree Search", "Jumps keyboard focus to the tree searchbox to allow searching for tracks in the current Sequence.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::F));
	
	UI_COMMAND( MoveToNewFolder, "Move to New Folder", "Move selected nodes to new folder", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::G));
	UI_COMMAND( RemoveFromFolder, "Remove from Folder", "Remove selected nodes from their folders", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::G));

	UI_COMMAND(AddTransformKey, "Add Transform Key", "Add a transform key at the current time for the selected actor.", EUserInterfaceActionType::Button, FInputChord(EKeys::S));
	UI_COMMAND(AddTranslationKey, "Add Translation Key", "Add a translation key at the current time for the selected actor.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::W));
	UI_COMMAND(AddRotationKey, "Add Rotation Key", "Add a rotation key at the current time for the selected actor.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::E));
	UI_COMMAND(AddScaleKey, "Add Scale Key", "Add a scale key at the current time for the selected actor.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::R));

	UI_COMMAND(DeleteTransformKey, "Delete Transform Key", "Delete the transform key at the current time for the selected actor.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteTranslationKey, "Delete Translation Key", "Delete the translation key at the current time for the selected actor.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteRotationKey, "Delete Rotation Key", "Delete the rotation key at the current time for the selected actor.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteScaleKey, "Delete Scale Key", "Delete the scale key at the current time for the selected actor.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND( SetKeyTime, "Set Key Time", "Set the key to a specified time", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( Rekey, "Rekey", "Set the selected key's time to the current time", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( SnapToFrame, "Snap To Frame", "Snap selected keys to frame", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND(TogglePilotCamera, "Pilot Camera", "Toggle piloting the last camera or the camera cut camera.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Shift, EKeys::P));

	UI_COMMAND(RefreshUI, "Refresh UI", "Forcibly refresh the UI from source data.", EUserInterfaceActionType::Button, FInputChord(EKeys::F5));

	UI_COMMAND(ToggleLimitViewportSelection, "Limit Viewport Selection", "Toggles viewport selectability between Sequencer only objects and all objects.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(ToggleSidebarVisible, "Sidebar Details", "Toggles the visibility of the Sequencer sidebar.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleSelectionDetailsPoppedOut, "Popout Selection Details", "Toggles the state of the Sequencer selection details between docked in the Sequencer tab and being popped out to its own tab window", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(ToggleSidebarSelectionDrawerOpen, "Sidebar Selection Drawer Open", "Toggles the visible state of the sidebar selection drawer. Undocks and closes the sidebar drawer if docked.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::E));
	UI_COMMAND(ToggleSidebarDrawerDock, "Sidebar Drawer Docked", "Toggles the docked state of the sidebar current drawer. Undocks the docked sidebar drawer if docked or docks the sidebar drawer if there is one open and no currently docked drawer.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::D));

	UI_COMMAND(AlignSelectionToPlayhead, "Align Selection To Playhead", "Aligns layer bars, sections, or key selections to the current playhead location", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ToggleTrackSelectionPin, "Toggle Track Pin", "Toggle the pinned state of Sequencer track selection", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleTrackSelectionLock, "Toggle Track Lock", "Toggle the locked state of Sequencer track selection", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleTrackSelectionDeactive, "Toggle Track Deactive", "Toggle the deactive state of Sequencer track selection", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleTrackSelectionMute, "Toggle Track Mute", "Toggle the muted state of Sequencer track selection", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleTrackSelectionSolo, "Toggle Track Solo", "Toggle the soloed state of Sequencer track selection", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ToggleToolBarVisible, "Toggle Tool Bar Visible", "Toggle between the toolbar panel being visible and hidden", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleTransportPanelVisible, "Toggle Transport Panel Visible", "Toggle between the transport controls panel being visible and hidden", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleTransportPanelDocked, "Toggle Transport Panel Docked", "Toggle between the transport controls panel being docked within its inline content or undocked in its own tab window", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ToggleSimpleView, "Toggle Simple/Full View", "Toggle between Sequencer simplified view and full views", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::BackSpace))
}

#undef LOCTEXT_NAMESPACE
