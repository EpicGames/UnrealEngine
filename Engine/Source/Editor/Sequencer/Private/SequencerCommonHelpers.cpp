// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerCommonHelpers.h"

#include "Algo/Reverse.h"
#include "Decorations/IMovieSceneChannelDecoration.h"
#include "Decorations/IMovieSceneSectionProviderDecoration.h"
#include "Decorations/MovieSceneDecorationContainer.h"
#include "Decorations/MovieSceneDecorationContainerCustomization.h"
#include "IMovieSceneModule.h"
#include "ISequencer.h"
#include "ISequencerDecorationEditor.h"
#include "EntitySystem/IMovieSceneBlenderSystemSupport.h"
#include "EntitySystem/MovieSceneBlenderSystem.h"
#include "FrameNumberDetailsCustomization.h"
#include "IDetailsView.h"
#include "ISequencerSection.h"
#include "IStructureDetailsView.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/FolderModel.h"
#include "MVVM/ViewModels/LayerBarModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/VirtualTrackArea.h"
#include "MVVM/Views/ITrackAreaHotspot.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/Selection/SequencerCoreSelectionTypes.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneSectionDetailsCustomization.h"
#include "MovieSceneSequence.h"
#include "Sections/MovieSceneSubSection.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "PropertyPermissionList.h"
#include "SSequencer.h"
#include "Sequencer.h"
#include "SequencerContextMenus.h"
#include "SequencerSelectedKey.h"
#include "SequencerUtilities.h"
#include "Styling/CoreStyle.h"
#include "Conditions/MovieSceneDirectorBlueprintConditionCustomization.h"
#include "Conditions/MovieSceneConditionCustomization.h"
#include "Decorations/IMovieSceneDecorationMenuProvider.h"
#include "Features/IModularFeatures.h"

void SequencerHelpers::GetAllChannels(TSharedPtr<FViewModel> DataModel, TSet<TSharedPtr<UE::Sequencer::FChannelModel>>& Channels)
{
	using namespace UE::Sequencer;

	if (DataModel)
	{
		constexpr bool bIncludeThis = true;
		for (const FViewModelPtr& Child : DataModel->GetDescendants(bIncludeThis))
		{
			if (TSharedPtr<ITrackAreaExtension> TrackArea = Child.ImplicitCast())
			{
				for (const FViewModelPtr& TrackAreaModel : TrackArea->GetTrackAreaModelList())
				{
					if (TViewModelPtr<FChannelModel> Channel = TrackAreaModel.ImplicitCast())
					{
						Channels.Add(Channel);
					}
				}
			}
			else if (TSharedPtr<FChannelModel> Channel = Child.ImplicitCast())
			{
				Channels.Add(Channel);
			}
		}
	}
}

void SequencerHelpers::GetAllKeyAreas(TSharedPtr<FViewModel> DataModel, TSet<TSharedPtr<IKeyArea>>& Channels)
{
	using namespace UE::Sequencer;

	if (DataModel)
	{
		constexpr bool bIncludeThis = true;
		for (const FViewModelPtr& Child : DataModel->GetDescendants(bIncludeThis))
		{
			if (TSharedPtr<ITrackAreaExtension> TrackArea = Child.ImplicitCast())
			{
				for (const FViewModelPtr& TrackAreaModel : TrackArea->GetTrackAreaModelList())
				{
					if (TViewModelPtr<FChannelModel> Channel = TrackAreaModel.ImplicitCast())
					{
						Channels.Add(Channel->GetKeyArea());
					}
				}
			}
			else if (TSharedPtr<FChannelModel> Channel = Child.ImplicitCast())
			{
				Channels.Add(Channel->GetKeyArea());
			}
		}
	}
}

void SequencerHelpers::GetAllSections(TSharedPtr<FViewModel> DataModel, TSet<TWeakObjectPtr<UMovieSceneSection>>& Sections)
{
	using namespace UE::Sequencer;

	if (DataModel)
	{
		constexpr bool bIncludeThis = true;
		for (TSharedPtr<FSectionModel> Section : TParentFirstChildIterator<FSectionModel>(DataModel, bIncludeThis))
		{
			Sections.Add(Section->GetSection());
		}
	}
}

int32 SequencerHelpers::GetSectionFromTime(TArrayView<UMovieSceneSection* const> InSections, FFrameNumber Time)
{
	FFrameNumber ClosestLowerBound = TNumericLimits<int32>::Max();
	TOptional<int32> MaxOverlapPriority, MaxProximalPriority;

	int32 MostRelevantIndex = INDEX_NONE;

	for (int32 Index = 0; Index < InSections.Num(); ++Index)
	{
		const UMovieSceneSection* Section = InSections[Index];
		if (Section)
		{
			const int32 ThisSectionPriority = Section->GetOverlapPriority();
			TRange<FFrameNumber> SectionRange = Section->GetRange();

			// If the specified time is within the section bounds
			if (SectionRange.Contains(Time))
			{
				if (ThisSectionPriority >= MaxOverlapPriority.Get(ThisSectionPriority))
				{
					MaxOverlapPriority = ThisSectionPriority;
					MostRelevantIndex = Index;
				}
			}
			// Check for nearby sections if there is nothing overlapping
			else if (!MaxOverlapPriority.IsSet() && SectionRange.HasLowerBound())
			{
				const FFrameNumber LowerBoundValue = SectionRange.GetLowerBoundValue();
				// If this section exists beyond the current time, we can choose it if its closest to the time
				if (LowerBoundValue >= Time)
				{
					if (
						(LowerBoundValue < ClosestLowerBound) ||
						(LowerBoundValue == ClosestLowerBound && ThisSectionPriority >= MaxProximalPriority.Get(ThisSectionPriority))
						)
					{
						MostRelevantIndex = Index;
						ClosestLowerBound = LowerBoundValue;
						MaxProximalPriority = ThisSectionPriority;
					}
				}
			}
		}
	}

	// If we didn't find one, use the last one (or return -1)
	if (MostRelevantIndex == -1)
	{
		MostRelevantIndex = InSections.Num() - 1;
	}

	return MostRelevantIndex;
}

void SequencerHelpers::GetDescendantNodes(TSharedRef<FViewModel> DataModel, TSet<TSharedRef<FViewModel>>& Nodes)
{
	using namespace UE::Sequencer;

	for (TSharedPtr<FViewModel> ChildNode : DataModel->GetChildren())
	{
		if (ChildNode->IsA<IOutlinerExtension>())
		{
			Nodes.Add(ChildNode.ToSharedRef());
		}

		GetDescendantNodes(ChildNode.ToSharedRef(), Nodes);
	}
}

bool IsSectionSelectedInNode(FSequencer& Sequencer, TSharedPtr<UE::Sequencer::FViewModel> InModel)
{
	using namespace UE::Sequencer;

	const FTrackAreaSelection& Selection = Sequencer.GetViewModel()->GetSelection()->TrackArea;

	if (ITrackAreaExtension* TrackArea = InModel->CastThis<ITrackAreaExtension>())
	{
		for (TSharedPtr<FViewModel> TrackAreaModel : TrackArea->GetTrackAreaModelList())
		{
			constexpr bool bIncludeThis = true;
			for (TSharedPtr<FSectionModel> Section : TParentFirstChildIterator<FSectionModel>(TrackAreaModel, bIncludeThis))
			{
				if (Selection.IsSelected(Section))
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool AreKeysSelectedInNode(FSequencer& Sequencer, TSharedPtr<UE::Sequencer::FViewModel> InModel)
{
	using namespace UE::Sequencer;

	TSet<TSharedPtr<FChannelModel>> Channels;
	SequencerHelpers::GetAllChannels(InModel, Channels);

	const FKeySelection& Selection = Sequencer.GetViewModel()->GetSelection()->KeySelection;

	for (FKeyHandle Key : Selection)
	{
		TSharedPtr<FChannelModel> Channel = Selection.GetModelForKey(Key);
		if (Channels.Contains(Channel))
		{
			return true;
		}
	}

	return false;
}


void SequencerHelpers::PerformDefaultSelection(FSequencer& Sequencer, const FPointerEvent& MouseEvent)
{
	using namespace UE::Sequencer;

	// @todo: selection in transactions
	FHotspotSelectionManager SelectionManager(&MouseEvent, &Sequencer);
	TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = Sequencer.GetViewModel()->CastThisShared<FSequencerEditorViewModel>();
	TSharedPtr<ITrackAreaHotspot> Hotspot = SequencerViewModel->GetHotspot();
	if (TSharedPtr<IMouseHandlerHotspot> MouseHandler = HotspotCast<IMouseHandlerHotspot>(Hotspot))
	{
		MouseHandler->HandleMouseSelection(SelectionManager);
	}
	else
	{
		// No hotspot so clear the selection if we're not adding to it
		SelectionManager.ConditionallyClearSelection();
	}
}

void SequencerHelpers::RemoveDuplicateKeys(const UE::Sequencer::FKeySelection& KeySelection, TArrayView<FKeyHandle> KeyHandles)
{
	using namespace UE::Sequencer;

	TMap<TViewModelPtr<FChannelModel>, TArray<FKeyHandle> > ChannelToKeyHandles;
	TMap<TViewModelPtr<FChannelModel>, bool> ChannelsToCheck;
	auto ShouldCheckChannel = [&ChannelsToCheck](const TViewModelPtr<FChannelModel>& ChannelModel)
	{
		if (ChannelModel)
		{
			if (bool* Found = ChannelsToCheck.Find(ChannelModel))
			{
				return *Found;
			}
			
			bool bCheck = false;
			
			if (FMovieSceneChannel* Channel = ChannelModel->GetChannel())
			{
				TArray<FFrameNumber> AllTimes;
				TArray<FKeyHandle> AllHandles;
				Channel->GetKeys(TRange<FFrameNumber>::All(), &AllTimes, &AllHandles);
			
				if (AllTimes.Num() != AllHandles.Num())
				{
					// should never happen btw...
					bCheck = true;
				}
				else if (AllTimes.Num() > 1) // no need to check if there's only one key
				{
					// if there is more than one key, check is only needed if the times are not increasing 
					for (int32 Index = 1; Index < AllTimes.Num(); Index++)
					{
						if (AllTimes[Index] <= AllTimes[Index-1])
						{
							bCheck = true;
							break;
						}
					}
				}
			}
   
			ChannelsToCheck.Add(ChannelModel, bCheck);
			return bCheck;
		}
		return false;
	};
	
	for (const FKeyHandle& KeyHandle : KeyHandles)
	{
		TViewModelPtr<FChannelModel> ChannelModel = KeySelection.GetModelForKey(KeyHandle);
		if (ChannelModel)
		{
			if (!ShouldCheckChannel(ChannelModel))
			{
				continue;
			}
			
			if (FMovieSceneChannel* Channel = ChannelModel->GetChannel())
			{	
				if (Channel->GetIndex(KeyHandle) == INDEX_NONE)
				{
					continue;
				}

				FFrameNumber KeyTime;
				Channel->GetKeyTime(KeyHandle, KeyTime);

				TArray<FFrameNumber> OutKeyTimes;
				TArray<FKeyHandle> OutKeyHandles;
				Channel->GetKeys(TRange<FFrameNumber>(KeyTime, KeyTime), &OutKeyTimes, &OutKeyHandles);
				
				// nothing to do if there's only one handle
				if (OutKeyHandles.Num() > 1)
				{	
					OutKeyHandles.Remove(KeyHandle);
					ChannelToKeyHandles.FindOrAdd(ChannelModel).Append(OutKeyHandles);
				}
			}
		}
	}

	for (TPair<TViewModelPtr<FChannelModel>, TArray<FKeyHandle> >& ChannelToKey : ChannelToKeyHandles)
	{
		TViewModelPtr<FChannelModel> ChannelModel = ChannelToKey.Key;
		ChannelModel->GetSection()->Modify();
		ChannelModel->GetChannel()->DeleteKeys(ChannelToKey.Value);
	}
}

TSharedPtr<SWidget> SequencerHelpers::SummonContextMenu(FSequencer& Sequencer, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	using namespace UE::Sequencer;

	// @todo sequencer replace with UI Commands instead of faking it

	// Attempt to paste into either the current node selection, or the clicked on track
	TSharedRef<SSequencer> SequencerWidget = Sequencer.GetUnderlyingSequencerWidget();
	const FFrameNumber PasteAtTime = Sequencer.GetLocalTime().Time.FrameNumber;

	// The menu are generated through reflection and sometime the API exposes some recursivity (think about a Widget returning it parent which is also a Widget). Just by reflection
	// it is not possible to determine when the root object is reached. It needs a kind of simulation which is not implemented. Also, even if the recursivity was correctly handled, the possible
	// permutations tend to grow exponentially. Until a clever solution is found, the simple approach is to disable recursively searching those menus. User can still search the current one though.
	// See UE-131257
	const bool bInRecursivelySearchable = false;

	const bool bShouldCloseWindowAfterMenuSelection = true;

	TSharedPtr<FExtender> MenuExtender = MakeShared<FExtender>();

	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, Sequencer.GetCommandBindings(), MenuExtender, false, &FCoreStyle::Get(), true, NAME_None, bInRecursivelySearchable);

	TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = Sequencer.GetViewModel()->CastThisShared<FSequencerEditorViewModel>();
	TSharedPtr<ITrackAreaHotspot> Hotspot = SequencerViewModel->GetHotspot();

	if (Hotspot.IsValid() && Hotspot->PopulateContextMenu(MenuBuilder, MenuExtender, PasteAtTime))
	{
		return MenuBuilder.MakeWidget();
	}
	else if (Sequencer.GetClipboardStack().Num() != 0)
	{
		const TWeakPtr<FSequencer> WeakSequencer = StaticCastWeakPtr<FSequencer>(Sequencer.AsWeak());
		TSharedPtr<FPasteContextMenu> PasteMenu = FPasteContextMenu::CreateMenu(WeakSequencer, SequencerWidget->GeneratePasteArgs(PasteAtTime));
		if (PasteMenu.IsValid() && PasteMenu->IsValidPaste())
		{
			PasteMenu->PopulateMenu(MenuBuilder, MenuExtender);

			return MenuBuilder.MakeWidget();
		}
	}

	return nullptr;
}

/** A widget which wraps the section details view which is an FNotifyHook which is used to forward
	changes to the section to sequencer. */
class SSectionDetailsNotifyHookWrapper : public SCompoundWidget, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SSectionDetailsNotifyHookWrapper) {}
	SLATE_END_ARGS();

	void Construct(FArguments InArgs) { }

	void SetDetailsAndSequencer(TSharedRef<SWidget> InDetailsPanel, TWeakPtr<ISequencer> InWeakSequencer)
	{
		ChildSlot
		[
			InDetailsPanel
		];
		WeakSequencer = InWeakSequencer;
	}

	//~ FNotifyHook interface
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override
	{
		if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
		{
			if (PropertyThatChanged && PropertyThatChanged->GetName() == TEXT("Condition"))
			{
				// Rebuild hierarchy on changing a condition so the indicators have a chance to refresh
				Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
			}
			else
			{
				Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
			}
		}
	}

private:
	TWeakPtr<ISequencer> WeakSequencer;
};

void SequencerHelpers::BuildNewSectionMenu(const TWeakPtr<FSequencer>& InWeakSequencer
	, const int32 InRowIndex
	, const TWeakObjectPtr<UMovieSceneTrack>& InTrackWeak
	, FMenuBuilder& MenuBuilder)
{
	using namespace UE::Sequencer;

	MenuBuilder.AddSubMenu(
		NSLOCTEXT("Sequencer", "AddSection", "Add Section"),
		FText(),
		FNewMenuDelegate::CreateLambda([InWeakSequencer, InRowIndex, InTrackWeak](FMenuBuilder& SubMenuBuilder)
		{
			if (const TSharedPtr<ISequencer> Sequencer = InWeakSequencer.Pin())
			{
				FSequencerUtilities::PopulateMenu_CreateNewSection(SubMenuBuilder, InRowIndex, InTrackWeak.Get(), Sequencer);
			}
		}));
}

void SequencerHelpers::BuildEditSectionMenu(const TWeakPtr<FSequencer>& InWeakSequencer
	, const TArray<TWeakObjectPtr<>>& InWeakSections
	, FMenuBuilder& MenuBuilder
	, const bool bInSubMenu)
{
	using namespace UE::Sequencer;

	if (InWeakSections.Num() == 0)
	{
		return;
	}

	auto BuildSection = [InWeakSequencer, InWeakSections](FMenuBuilder& LambdaMenuBuilder)
	{
		const TSharedPtr<FSequencer> Sequencer = InWeakSequencer.Pin();
		if (!Sequencer)
		{
			return;
		}

		UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
		const TWeakObjectPtr<UMovieScene> CurrentScene = Sequence->GetMovieScene();

		TSharedRef<SSectionDetailsNotifyHookWrapper> DetailsNotifyWrapper = SNew(SSectionDetailsNotifyHookWrapper);
		FDetailsViewArgs DetailsViewArgs;
		{
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.bCustomFilterAreaLocation = true;
			DetailsViewArgs.bCustomNameAreaLocation = true;
			DetailsViewArgs.bHideSelectionTip = true;
			DetailsViewArgs.bLockable = false;
			DetailsViewArgs.bSearchInitialKeyFocus = true;
			DetailsViewArgs.bUpdatesFromSelection = false;
			DetailsViewArgs.bShowOptions = false;
			DetailsViewArgs.bShowModifiedPropertiesOption = false;
			DetailsViewArgs.bShowScrollBar = false;
			DetailsViewArgs.NotifyHook = &DetailsNotifyWrapper.Get();
			DetailsViewArgs.ColumnWidth = 0.45f;
		}

		// We pass the current scene to the UMovieSceneSection customization so we can get the overall bounds of the section when we change a section from infinite->bounded.

		TSharedRef<IDetailsView> DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
		DetailsView->RegisterInstancedCustomPropertyTypeLayout("FrameNumber",
			FOnGetPropertyTypeCustomizationInstance::CreateSP(Sequencer.ToSharedRef(), &FSequencer::MakeFrameNumberDetailsCustomization));

		DetailsView->RegisterInstancedCustomPropertyLayout(UMovieSceneSection::StaticClass(),
			FOnGetDetailCustomizationInstance::CreateLambda([TypeInterface = Sequencer->GetNumericTypeInterface(), CurrentScene]()
			{
				return MakeShared<FMovieSceneSectionDetailsCustomization>(TypeInterface, CurrentScene.Get());
			}));

		DetailsView->RegisterInstancedCustomPropertyTypeLayout("MovieSceneConditionContainer", FOnGetPropertyTypeCustomizationInstance::CreateLambda([WeakSequence = TWeakObjectPtr(Sequence), InWeakSequencer]() 
		{
			return FMovieSceneConditionCustomization::MakeInstance(WeakSequence, InWeakSequencer); 
		}));

		DetailsView->RegisterInstancedCustomPropertyTypeLayout("MovieSceneDirectorBlueprintConditionData", FOnGetPropertyTypeCustomizationInstance::CreateLambda([=]() {
			return FMovieSceneDirectorBlueprintConditionCustomization::MakeInstance(CurrentScene.Get());}));
			
		DetailsView->RegisterInstancedCustomPropertyTypeLayout("MovieSceneDecorationContainer", FOnGetPropertyTypeCustomizationInstance::CreateLambda([]() {
			return FMovieSceneDecorationContainerCustomization::MakeInstance(); }));
	
		DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateLambda([](const FPropertyAndParent& PropertyAndParent)
			{			
				return FPropertyEditorPermissionList::Get().DoesPropertyPassFilter(PropertyAndParent.Property.GetOwnerStruct(), PropertyAndParent.Property.GetFName());
			})
		);

		// Let section interfaces further customize the properties details view.
		TSharedRef<FSequencerNodeTree> SequencerNodeTree = Sequencer->GetNodeTree();
		for (TWeakObjectPtr<> Section : InWeakSections)
		{
			if (Section.IsValid())
			{
				TSharedPtr<FSectionModel> SectionHandle = SequencerNodeTree->GetSectionModel(Cast<UMovieSceneSection>(Section));
				if (SectionHandle)
				{
					TSharedPtr<ISequencerSection> SectionInterface = SectionHandle->GetSectionInterface();
					FSequencerSectionPropertyDetailsViewCustomizationParams CustomizationDetails(
						SectionInterface.ToSharedRef(), InWeakSequencer, *SectionHandle->GetParentTrackExtension()->GetTrackEditor().Get());
					TSharedPtr<FObjectBindingModel> ParentObjectBindingNode = SectionHandle->FindAncestorOfType<FObjectBindingModel>();
					if (ParentObjectBindingNode.IsValid())
					{
						CustomizationDetails.ParentObjectBindingGuid = ParentObjectBindingNode->GetObjectGuid();
					}
					SectionInterface->CustomizePropertiesDetailsView(DetailsView, CustomizationDetails);
				}
			}
		}

		Sequencer->OnInitializeDetailsPanel().Broadcast(DetailsView, Sequencer.ToSharedRef());
		DetailsView->SetObjects(InWeakSections);

		DetailsNotifyWrapper->SetDetailsAndSequencer(DetailsView, InWeakSequencer);
		DetailsNotifyWrapper->SetEnabled(!Sequencer->IsReadOnly());

		LambdaMenuBuilder.BeginSection(TEXT("TrackSection"));
        {
			LambdaMenuBuilder.AddWidget(DetailsNotifyWrapper, FText::GetEmpty(), true);
        }
        LambdaMenuBuilder.EndSection();
	};

	if (bInSubMenu)
	{
		const FText MenuLabel = InWeakSections.Num() > 1
			? NSLOCTEXT("Sequencer", "BatchEditSections", "Batch Edit Sections")
			: NSLOCTEXT("Sequencer", "EditSection", "Edit Section");

		MenuBuilder.AddSubMenu(
			MenuLabel,
			FText(),
			FNewMenuDelegate::CreateLambda([BuildSection](FMenuBuilder& SubMenuBuilder)
			{
				BuildSection(SubMenuBuilder);
			}));
	}
	else
	{
		BuildSection(MenuBuilder);
	}
}

void SequencerHelpers::BuildEditTrackMenu(const TWeakPtr<FSequencer>& InWeakSequencer
	, const TArray<TWeakObjectPtr<>>& InWeakTracks
	, FMenuBuilder& MenuBuilder
	, const bool bInSubMenu)
{
	using namespace UE::Sequencer;

	if (InWeakTracks.Num() == 0)
	{
		return;
	}

	auto BuildTrack = [InWeakSequencer, InWeakTracks](FMenuBuilder& LambdaMenuBuilder)
	{
		const TSharedPtr<FSequencer> Sequencer = InWeakSequencer.Pin();
		if (!Sequencer)
		{
			return;
		}

		TSharedRef<SSectionDetailsNotifyHookWrapper> DetailsNotifyWrapper = SNew(SSectionDetailsNotifyHookWrapper);
		FDetailsViewArgs DetailsViewArgs;
		{
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.bCustomFilterAreaLocation = true;
			DetailsViewArgs.bCustomNameAreaLocation = true;
			DetailsViewArgs.bHideSelectionTip = true;
			DetailsViewArgs.bLockable = false;
			DetailsViewArgs.bSearchInitialKeyFocus = true;
			DetailsViewArgs.bUpdatesFromSelection = false;
			DetailsViewArgs.bShowOptions = false;
			DetailsViewArgs.bShowModifiedPropertiesOption = false;
			DetailsViewArgs.bShowScrollBar = false;
			DetailsViewArgs.NotifyHook = &DetailsNotifyWrapper.Get();
			DetailsViewArgs.ColumnWidth = 0.45f;
		}

		// We pass the current scene to the UMovieSceneSection customization so we can get the overall bounds of the section when we change a section from infinite->bounded.
		UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
		UMovieScene* CurrentScene = Sequence->GetMovieScene();

		TSharedRef<IDetailsView> DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);

		DetailsView->RegisterInstancedCustomPropertyTypeLayout("MovieSceneConditionContainer", FOnGetPropertyTypeCustomizationInstance::CreateLambda([Sequence, InWeakSequencer]() {
			return FMovieSceneConditionCustomization::MakeInstance(Sequence, InWeakSequencer); }));

		DetailsView->RegisterInstancedCustomPropertyTypeLayout("MovieSceneDirectorBlueprintConditionData", FOnGetPropertyTypeCustomizationInstance::CreateLambda([=]() {
			return FMovieSceneDirectorBlueprintConditionCustomization::MakeInstance(CurrentScene); }));

		DetailsView->RegisterInstancedCustomPropertyTypeLayout("MovieSceneDecorationContainer", FOnGetPropertyTypeCustomizationInstance::CreateLambda([]() {
			return FMovieSceneDecorationContainerCustomization::MakeInstance(); }));


		DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateLambda([](const FPropertyAndParent& PropertyAndParent)
			{
				return FPropertyEditorPermissionList::Get().DoesPropertyPassFilter(PropertyAndParent.Property.GetOwnerStruct(), PropertyAndParent.Property.GetFName());
			})
		);

		Sequencer->OnInitializeDetailsPanel().Broadcast(DetailsView, Sequencer.ToSharedRef());
		DetailsView->SetObjects(InWeakTracks);

		DetailsNotifyWrapper->SetDetailsAndSequencer(DetailsView, Sequencer);
		DetailsNotifyWrapper->SetEnabled(!Sequencer->IsReadOnly());

		LambdaMenuBuilder.BeginSection(TEXT("Track"));
		{
			LambdaMenuBuilder.AddWidget(DetailsNotifyWrapper, FText::GetEmpty(), true);
		}
		LambdaMenuBuilder.EndSection();
	};

	if (bInSubMenu)
	{
		const FText MenuLabel = InWeakTracks.Num() > 1
			? NSLOCTEXT("Sequencer", "BatchEditTracks", "Batch Edit Tracks")
			: NSLOCTEXT("Sequencer", "EditTrack", "Edit Track");

		MenuBuilder.AddSubMenu(
			MenuLabel,
			FText(),
			FNewMenuDelegate::CreateLambda([BuildTrack](FMenuBuilder& SubMenuBuilder)
				{
					BuildTrack(SubMenuBuilder);
				}));
	}
	else
	{
		BuildTrack(MenuBuilder);
	}
}

void SequencerHelpers::BuildEditDecorationMenu(const TWeakPtr<FSequencer>& InWeakSequencer
	, const TArray<TWeakObjectPtr<>>& InWeakDecorations
	, FMenuBuilder& MenuBuilder)
{
	using namespace UE::Sequencer;

	if (InWeakDecorations.Num() == 0)
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = InWeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	// Group decorations by class so each class gets its own details view
	TMap<UClass*, TArray<TWeakObjectPtr<>>> DecorationsByClass;
	for (const TWeakObjectPtr<>& WeakDecoration : InWeakDecorations)
	{
		if (UObject* Decoration = WeakDecoration.Get())
		{
			DecorationsByClass.FindOrAdd(Decoration->GetClass()).Add(WeakDecoration);
		}
	}

	for (const auto& Pair : DecorationsByClass)
	{
		UClass* DecorationClass = Pair.Key;
		const TArray<TWeakObjectPtr<>>& ClassDecorations = Pair.Value;

		// Check if this class has any editable properties - skip if empty
		TArray<TWeakObjectPtr<>> ObjectsToDisplay;
		bool bIsSectionProvider = false;
		{
			bool bHasEditableProperties = false;
			for (TFieldIterator<FProperty> PropIt(DecorationClass); PropIt; ++PropIt)
			{
				if (PropIt->HasAnyPropertyFlags(CPF_Edit))
				{
					bHasEditableProperties = true;
					break;
				}
			}
			if (!bHasEditableProperties)
			{
				continue;
			}

			ObjectsToDisplay = ClassDecorations;

			for (const TWeakObjectPtr<>& WeakDecoration : ClassDecorations)
			{
				if (Cast<IMovieSceneSectionProviderDecoration>(WeakDecoration.Get()))
				{
					bIsSectionProvider = true;
					break;
				}
			}
		}

		if (ObjectsToDisplay.IsEmpty())
		{
			continue;
		}

		TSharedRef<SSectionDetailsNotifyHookWrapper> DetailsNotifyWrapper = SNew(SSectionDetailsNotifyHookWrapper);
		FDetailsViewArgs DetailsViewArgs;
		{
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.bCustomFilterAreaLocation = true;
			DetailsViewArgs.bCustomNameAreaLocation = true;
			DetailsViewArgs.bHideSelectionTip = true;
			DetailsViewArgs.bLockable = false;
			DetailsViewArgs.bSearchInitialKeyFocus = true;
			DetailsViewArgs.bUpdatesFromSelection = false;
			DetailsViewArgs.bShowOptions = false;
			DetailsViewArgs.bShowModifiedPropertiesOption = false;
			DetailsViewArgs.bShowScrollBar = false;
			DetailsViewArgs.NotifyHook = &DetailsNotifyWrapper.Get();
			DetailsViewArgs.ColumnWidth = 0.45f;
		}

		TSharedRef<IDetailsView> DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);

		DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateLambda([bIsSectionProvider](const FPropertyAndParent& PropertyAndParent)
			{
				// Hide the Decorations property when showing sections from section-providing decorations
				// to prevent the "Modifiers" category with decoration management UI from appearing
				if (bIsSectionProvider && PropertyAndParent.Property.GetFName() == TEXT("Decorations"))
				{
					return false;
				}
				return FPropertyEditorPermissionList::Get().DoesPropertyPassFilter(PropertyAndParent.Property.GetOwnerStruct(), PropertyAndParent.Property.GetFName());
			})
		);

		Sequencer->OnInitializeDetailsPanel().Broadcast(DetailsView, Sequencer.ToSharedRef());
		DetailsView->SetObjects(ObjectsToDisplay);

		DetailsNotifyWrapper->SetDetailsAndSequencer(DetailsView, InWeakSequencer);
		DetailsNotifyWrapper->SetEnabled(!Sequencer->IsReadOnly());

		// Use the decoration class display name as the section header
		FText SectionLabel = DecorationClass->GetDisplayNameText();
		MenuBuilder.BeginSection(*DecorationClass->GetName(), SectionLabel);
		{
			MenuBuilder.AddWidget(DetailsNotifyWrapper, FText::GetEmpty(), true);
		}
		MenuBuilder.EndSection();
	}
}

void SequencerHelpers::BuildBlendingMenu(const TWeakPtr<FSequencer>& InWeakSequencer
	, const TWeakObjectPtr<UMovieSceneTrack>& InTrackWeak
	, FMenuBuilder& MenuBuilder)
{
	if (!InTrackWeak.IsValid())
	{
		return;
	}

	IMovieSceneBlenderSystemSupport* const BlenderSystemSupport = Cast<IMovieSceneBlenderSystemSupport>(InTrackWeak.Get());
	if (!BlenderSystemSupport)
	{
		return;
	}

	TArray<TSubclassOf<UMovieSceneBlenderSystem>> BlenderTypes;
	BlenderSystemSupport->GetSupportedBlenderSystems(BlenderTypes);
	if (BlenderTypes.Num() < 2)
	{
		return;
	}

	MenuBuilder.AddSubMenu(
		NSLOCTEXT("Sequencer", "BlendingAlgorithmSubMenu", "Blending Algorithm"),
		FText(),
		FNewMenuDelegate::CreateLambda([InWeakSequencer, InTrackWeak](FMenuBuilder& SubMenuBuilder)
		{
			if (const TSharedPtr<ISequencer> Sequencer = InWeakSequencer.Pin())
			{
				FSequencerUtilities::PopulateMenu_BlenderSubMenu(SubMenuBuilder, InTrackWeak.Get(), Sequencer);
			}
		}));
}

void SequencerHelpers::BuildObjectBindingMenu(const TWeakPtr<FSequencer>& InWeakSequencer
	, const UE::Sequencer::TViewModelPtr<UE::Sequencer::FObjectBindingModel>& InObjectBindingModel
	, FMenuBuilder& MenuBuilder)
{
	TSharedPtr<FSequencer> Sequencer = InWeakSequencer.Pin();
	if (!Sequencer || !InObjectBindingModel)
	{
		return;
	}

	const TSharedPtr<UE::Sequencer::FSequencerEditorViewModel> EditorViewModel = InObjectBindingModel->GetEditor();
	if (!EditorViewModel)
	{
		return;
	}

	ISequencerModule& SequencerModule = FModuleManager::GetModuleChecked<ISequencerModule>(TEXT("Sequencer"));
	
	TArray<UObject*> ContextObjects;
	for (TWeakObjectPtr<UObject> RuntimeObject : Sequencer->FindObjectsInCurrentSequence(InObjectBindingModel->GetObjectGuid()))
	{
		UObject* BoundObject = RuntimeObject.Get();
		if (BoundObject)
		{
			ContextObjects.Add(BoundObject);
		}
	}

	const TSharedPtr<FExtender> Extender = EditorViewModel->GetSequencerMenuExtender(
		SequencerModule.GetSidebarExtensibilityManager(), 
		ContextObjects,
		&FSequencerCustomizationInfo::OnBuildSidebarMenu, 
		InObjectBindingModel);

	if (Extender.IsValid())
	{
		MenuBuilder.PushExtender(Extender.ToSharedRef());
	}

	MenuBuilder.BeginSection(TEXT("ObjectBindingActions"));
	MenuBuilder.EndSection();
}

TArray<TWeakObjectPtr<>> SequencerHelpers::GetSectionObjectsFromTrackAreaModels(const UE::Sequencer::FViewModelVariantIterator& InTrackAreaModels)
{
	using namespace UE::Sequencer;

	TArray<TWeakObjectPtr<>> OutWeakSectionObjects;

	for (const TViewModelPtr<FViewModel>& TrackAreaModel : InTrackAreaModels)
	{
		constexpr bool bIncludeThis = true;
		for (const TSharedPtr<FSectionModel> SectionModel : TParentFirstChildIterator<FSectionModel>(TrackAreaModel, bIncludeThis))
		{
			if (UMovieSceneSection* const SectionObject = SectionModel->GetSection())
			{
				OutWeakSectionObjects.AddUnique(SectionObject);
			}
		}
	}

	return OutWeakSectionObjects;
}

void SequencerHelpers::SortOutlinerItems(FSequencer& Sequencer
	, const TArray<UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>>& InItems
	, const bool bInSortByItemOrder
	, const bool bInDescending
	, const bool bInTransact)
{
	using namespace UE::Sequencer;

	if (InItems.IsEmpty())
	{
		return;
	}

	UMovieSceneSequence* const Sequence = Sequencer.GetFocusedMovieSceneSequence();
	if (!IsValid(Sequence))
	{
		return;
	}

	UMovieScene* const MovieScene = Sequence->GetMovieScene();
	if (!IsValid(Sequence))
	{
		return;
	}

	/** Represents a layer bar. Holds cached pointers for element operations. */
	struct FSortBarElement
	{
		static FSortBarElement FromTrack(FSequencer& InSequencer, const TViewModelPtr<ITrackAreaExtension>& InTrackAreaExtension)
		{
			for (const FViewModelPtr& TrackAreaModel : InTrackAreaExtension->GetTopLevelChildTrackAreaModels())
			{
				if (const TViewModelPtr<FLayerBarModel> BarModel = TrackAreaModel.ImplicitCast())
				{
					return FSortBarElement(InSequencer, BarModel);
				}
			}
			for (const TViewModelPtr<ILayerBarExtension>& TrackAreaModel : InTrackAreaExtension->GetTrackAreaModelListAs<ILayerBarExtension>())
			{
				if (const TViewModelPtr<ILayerBarExtension> BarModel = TrackAreaModel.ImplicitCast())
				{
					return FSortBarElement(InSequencer, BarModel);
				}
			}
			return FSortBarElement(InSequencer);
		}
	
		FSortBarElement(FSequencer& InSequencer)
			: Sequencer(InSequencer)
		{}
		FSortBarElement(FSequencer& InSequencer, const TViewModelPtr<FLayerBarModel>& InBarModel)
			: Sequencer(InSequencer)
		{
			if (const TViewModelPtr<IOutlinerExtension> LinkedOutlinerItem = InBarModel->GetLinkedOutlinerItem())
			{
				if (const TViewModelPtr<ISortableExtension> SortableExtension = LinkedOutlinerItem.ImplicitCast())
				{
					const TViewModelPtr<ISortableExtension> ParentSortableItem = LinkedOutlinerItem.AsModel()->FindAncestorOfType<ISortableExtension>();

					BarModel.Emplace<TViewModelPtr<FLayerBarModel>>(InBarModel);
					SortableItem = SortableExtension;
					ParentItem = ParentSortableItem.IsValid()? ParentSortableItem : CastViewModelChecked<ISortableExtension>(Sequencer.GetViewModel()->GetRootSequenceModel());
					Range = InBarModel->ComputeRange();
				}
			}
		}
		FSortBarElement(FSequencer& InSequencer, const TViewModelPtr<ILayerBarExtension>& InBarModel)
			: Sequencer(InSequencer)
		{
			if (const TViewModelPtr<FLinkedOutlinerExtension> LinkedOutlinerExtension = InBarModel.ImplicitCast())
			{
				if (const TViewModelPtr<IOutlinerExtension> LinkedOutlinerItem = LinkedOutlinerExtension->GetLinkedOutlinerItem())
				{
					if (const TViewModelPtr<ISortableExtension> SortableExtension = LinkedOutlinerItem.ImplicitCast())
					{
						const TViewModelPtr<ISortableExtension> ParentSortableItem = LinkedOutlinerItem.AsModel()->FindAncestorOfType<ISortableExtension>();

						BarModel.Emplace<TViewModelPtr<ILayerBarExtension>>(InBarModel);
						SortableItem = SortableExtension;
						ParentItem = ParentSortableItem.IsValid() ? ParentSortableItem : CastViewModelChecked<ISortableExtension>(Sequencer.GetViewModel()->GetRootSequenceModel());
						Range = InBarModel->GetLayerBarRange();
					}
				}
			}
		}

		bool IsValid() const
		{
			if (const TViewModelPtr<FLayerBarModel>* LayerBarModel = BarModel.TryGet<TViewModelPtr<FLayerBarModel>>())
			{
				return LayerBarModel->IsValid();
			}
			if (const TViewModelPtr<ILayerBarExtension>* LayerBarExtension = BarModel.TryGet<TViewModelPtr<ILayerBarExtension>>())
			{
				return LayerBarExtension->IsValid();
			}
			return false;
		}

		FSequencer& Sequencer;

		TVariant<TViewModelPtr<FLayerBarModel>, TViewModelPtr<ILayerBarExtension>> BarModel;

		TViewModelPtr<ISortableExtension> SortableItem;
		TViewModelPtr<ISortableExtension> ParentItem;
		TRange<FFrameNumber> Range;
	};

	// Gather all the selected folder and layer bar models from InItems so we can sort them
	TMap<TViewModelPtr<FFolderModel>, TArray<FSortBarElement>> FolderPairs;
	TMap<TViewModelPtr<ISortableExtension>, TArray<FSortBarElement>> SortItems;

	for (const TViewModelPtr<IOutlinerExtension>& TrackItem : InItems)
	{
		// Handle folders
		if (!bInSortByItemOrder)
		{
			if (const TViewModelPtr<FFolderModel> FolderModel = TrackItem.ImplicitCast())
			{
				TArray<FSortBarElement> FolderElements;

				for (const TViewModelPtr<IOutlinerExtension>& ChildTrack : FolderModel->GetChildrenOfType<IOutlinerExtension>())
				{
					if (const TViewModelPtr<ITrackAreaExtension> ChildTrackArea = ChildTrack.ImplicitCast())
					{
						FSortBarElement NewElement = FSortBarElement::FromTrack(Sequencer, ChildTrackArea);
						if (NewElement.IsValid())
						{
							FolderElements.Add(MoveTemp(NewElement));
						}
					}
				}

				if (!FolderElements.IsEmpty())
				{
					FolderPairs.Add(FolderModel, FolderElements);
				}
			}
		}

		// Handle track areas
		if (const TViewModelPtr<ITrackAreaExtension> TrackArea = TrackItem.ImplicitCast())
		{
			FSortBarElement NewElement = FSortBarElement::FromTrack(Sequencer, TrackArea);
			if (NewElement.IsValid())
			{
				SortItems.FindOrAdd(NewElement.ParentItem).Add(NewElement);
			}
		}
	}

	// Begin the transaction operation
	const FText TransactionText = bInDescending
		? NSLOCTEXT("Sequencer", "SortByAscending_Transaction", "Sort Tracks (Ascending)")
		: NSLOCTEXT("Sequencer", "SortByDescending_Transaction", "Sort Tracks (Descending)");
	FScopedTransaction SortNodesTransaction(TransactionText, bInTransact);

	if (bInTransact)
	{
		MovieScene->Modify();
	}

	// Set the sort order for all selected folder children
	if (!bInSortByItemOrder)
	{
		int32 SortOrder = 0;
		for (TPair<TViewModelPtr<FFolderModel>, TArray<FSortBarElement>>& FolderModel : FolderPairs)
		{
			FolderModel.Value.Sort([bInDescending](const FSortBarElement& InLayerBarA, const FSortBarElement& InLayerBarB) -> bool
				{
					if (bInDescending)
					{
						return InLayerBarA.Range.GetLowerBoundValue() > InLayerBarB.Range.GetLowerBoundValue();
					}
					return InLayerBarA.Range.GetLowerBoundValue() < InLayerBarB.Range.GetLowerBoundValue();
				});

			SortOrder = 0;
			for (const FSortBarElement& Element : FolderModel.Value)
			{
				Element.SortableItem->SetCustomOrder(SortOrder);
				++SortOrder;
			}
		}
	}

	// Set the sort order for all items
	for (TPair<TViewModelPtr<ISortableExtension>, TArray<FSortBarElement>>& LayerBarSortItemPair : SortItems)
	{
		TViewModelPtr<ISortableExtension>& ParentSortableItem = LayerBarSortItemPair.Key;
		TArray<FSortBarElement>& Elements = LayerBarSortItemPair.Value;

		// Remove all layer models that have a descendant that is in the InItems list
		Elements.RemoveAll([&InItems](const FSortBarElement& InElement) -> bool
			{
				for (const TViewModelPtr<IOutlinerExtension>& ChildOutlinerExtension : InElement.SortableItem.AsModel()->GetDescendantsOfType<IOutlinerExtension>())
				{
					if (InItems.Contains(ChildOutlinerExtension))
					{
						return true;
					}
				}
				return false;
			});

		if (!bInSortByItemOrder)
		{
			Elements.Sort([bInDescending](const FSortBarElement& InSortItemA, const FSortBarElement& InSortItemB) -> bool
				{
					if (bInDescending)
					{
						return InSortItemA.Range.GetLowerBoundValue() > InSortItemB.Range.GetLowerBoundValue();
					}
					return InSortItemA.Range.GetLowerBoundValue() < InSortItemB.Range.GetLowerBoundValue();
				});
		}

		// Save the current sortable children and begin the re-ordering operation.
		// We will attempt to maintain any current custom ordering that exists.
		TArray<TViewModelPtr<ISortableExtension>> SortableChildren;
		ParentSortableItem.AsModel()->GetChildrenOfType<ISortableExtension>().ToArray(SortableChildren);

		// Sort to make sure we are in the correct order since the array can be out of order from what is actually displayed
		SortableChildren.Sort([](const TViewModelPtr<ISortableExtension>& InA, const TViewModelPtr<ISortableExtension>& InB) -> bool
			{
				return FSortingKey::CompareCustomOrderFirst(InA->GetSortingKey(), InB->GetSortingKey());
			});

		// Loop backwards and remove items to re-insert while also re-ordering items
		TArray<int32> ItemIndicesToReinsert;

		for (int32 Index = SortableChildren.Num() - 1; Index >= 0; --Index)
		{
			const TViewModelPtr<ISortableExtension>& SortableChild = SortableChildren[Index];
			const bool bHasItemToSort = Elements.ContainsByPredicate([&SortableChild](const FSortBarElement& InItemCache)
				{
					return InItemCache.SortableItem == SortableChild;
				});
			if (bHasItemToSort)
			{
				SortableChildren.RemoveAt(Index);
				ItemIndicesToReinsert.Add(Index);
			}
		}

		// Reverse the array since we added while looping backwards through the array
		Algo::Reverse(ItemIndicesToReinsert);

		// Insert the sorted (selected) items to the array
		int32 CurrentIndex = 0;
		for (const int32 ItemIndex : ItemIndicesToReinsert)
		{
			if (Elements.IsValidIndex(CurrentIndex))
			{
				SortableChildren.Insert(Elements[CurrentIndex].SortableItem, ItemIndex);
				++CurrentIndex;
			}
		}

		// Use the SortableChildren index to set the custom order
		int32 CustomOrder = 0;
		for (const TViewModelPtr<ISortableExtension>& SortableChild : SortableChildren)
		{
			SortableChild->SetCustomOrder(CustomOrder++);
		}
	}

	Sequencer.RefreshTree();
}

bool SequencerHelpers::HasCompatibleDecorations(const UMovieSceneDecorationContainerObject* DecorationContainer)
{
	if (!DecorationContainer)
	{
		return false;
	}

	TSet<UClass*> CompatibleDecorations;
	IMovieSceneModule::Get().GetCompatibleDecorationsForContainer(DecorationContainer, CompatibleDecorations);

	// Check if any decoration is not already added
	for (UClass* DecorationClass : CompatibleDecorations)
	{
		bool bAlreadyExists = DecorationContainer->GetDecorations().ContainsByPredicate(
			[DecorationClass](const TObjectPtr<UObject>& Existing)
			{
				return Existing && Existing->GetClass() == DecorationClass;
			});
		if (!bAlreadyExists)
		{
			return true;
		}
	}
	return false;
}

bool SequencerHelpers::HasOutlinerVisibleDecorations(const UMovieSceneDecorationContainerObject* Container)
{
	if (!Container)
	{
		return false;
	}

	for (const TObjectPtr<UObject>& Decoration : Container->GetDecorations())
	{
		if (!Decoration)
		{
			continue;
		}

		if (Cast<IMovieSceneChannelDecoration>(Decoration.Get()) ||
			Cast<IMovieSceneSectionProviderDecoration>(Decoration.Get()))
		{
			return true;
		}
	}

	return false;
}

void SequencerHelpers::BuildDecorationMenu(
	FMenuBuilder& MenuBuilder,
	UMovieSceneDecorationContainerObject* DecorationContainer,
	const FGuid& ObjectBindingID,
	TWeakPtr<ISequencer> InSequencer,
	const FText& SectionLabel)
{
	if (!DecorationContainer)
	{
		return;
	}

	TSet<UClass*> CompatibleDecorations;
	IMovieSceneModule::Get().GetCompatibleDecorationsForContainer(DecorationContainer, CompatibleDecorations);

	if (CompatibleDecorations.Num() == 0)
	{
		return;
	}

	UObject* BoundObject = nullptr;

	if (const TSharedPtr<ISequencer> PinnedSequencer = InSequencer.Pin())
	{
		const TArrayView<TWeakObjectPtr<>> Objects = PinnedSequencer->FindObjectsInCurrentSequence(ObjectBindingID);
		if (Objects.Num())
		{
			BoundObject = Objects[0].Get();
		}
	}

	// Get menu providers
	TArray<IMovieSceneDecorationMenuProvider*> MenuProviders = IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneDecorationMenuProvider>(IMovieSceneDecorationMenuProvider::GetModularFeatureName());

	// Sort the menu providers so their entries are added in the order specified by their priority.
	MenuProviders.Sort([](const IMovieSceneDecorationMenuProvider& A, const IMovieSceneDecorationMenuProvider& B)
	{
		return A.GetDecorationMenuPriority() < B.GetDecorationMenuPriority();
	});

	const FText& Label = SectionLabel.IsEmpty() ? NSLOCTEXT("SequencerHelpers", "Modifiers_Label", "Modifiers") : SectionLabel;
	MenuBuilder.BeginSection(NAME_None, Label);
	{

		// First, allow menu providers to handle decorations.
		for (IMovieSceneDecorationMenuProvider* MenuProvider : MenuProviders)
		{
			if (!MenuProvider)
			{
				continue;
			}
			const UClass* HandledClass = MenuProvider->GetHandledDecorationClass();
			if (CompatibleDecorations.Contains(HandledClass))
			{
				// The custom menu provider should handle the case when the decoration already exists, so PopulateAddDecorationMenu is always called.
				MenuProvider->PopulateAddDecorationMenu(MenuBuilder, BoundObject, DecorationContainer, InSequencer);
				// Remove class, so its entry isn't duplicated. 
				CompatibleDecorations.Remove(HandledClass);
			}
		}

		for (UClass* DecorationClass : CompatibleDecorations)
		{
			const TObjectPtr<UObject>* ExistingDecoration = DecorationContainer->GetDecorations().FindByPredicate(
				[DecorationClass](const TObjectPtr<UObject>& Existing)
				{
					return Existing && Existing->GetClass() == DecorationClass;
				});

			IMovieSceneSectionProviderDecoration* SectionProvider = ExistingDecoration ? Cast<IMovieSceneSectionProviderDecoration>(ExistingDecoration->Get()) : nullptr;
			const bool bSupportsMultipleSections = SectionProvider ? SectionProvider->SupportsMultipleSections() : false;

			// Let a registered decoration editor supply an icon for the menu entry.
			FSlateIcon MenuIcon;
			if (TSharedPtr<ISequencer> PinnedSequencer = InSequencer.Pin())
			{
				if (ISequencerDecorationEditor* DecorationEditor = PinnedSequencer->FindDecorationEditor(DecorationClass))
				{
					MenuIcon = DecorationEditor->GetMenuIcon();
				}
			}

			if (!ExistingDecoration)
			{
				MenuBuilder.AddMenuEntry(
					DecorationClass->GetDisplayNameText(),
					DecorationClass->GetToolTipText(),
					MenuIcon,
					FUIAction(FExecuteAction::CreateLambda([DecorationContainer, DecorationClass]()
					{
						FScopedTransaction Transaction(
							FText::Format(NSLOCTEXT("SequencerHelpers", "AddDecoration", "Add {0}"),
							DecorationClass->GetDisplayNameText()));

						DecorationContainer->Modify();
						UObject* NewDecoration = NewObject<UObject>(
							DecorationContainer, DecorationClass, NAME_None, RF_Transactional);
						DecorationContainer->AddDecoration(NewDecoration);
					}))
				);
			}
			else if (bSupportsMultipleSections)
			{
				MenuBuilder.AddMenuEntry(
					DecorationClass->GetDisplayNameText(),
					DecorationClass->GetToolTipText(),
					MenuIcon,
					FUIAction(FExecuteAction::CreateLambda([DecorationContainer, DecorationClass, SectionProvider, InSequencer]()
					{
						FScopedTransaction Transaction(
							FText::Format(NSLOCTEXT("SequencerHelpers", "AddDecorationSection", "Add {0}"),
							DecorationClass->GetDisplayNameText()));

						TSharedPtr<ISequencer> Sequencer = InSequencer.IsValid() ? InSequencer.Pin() : nullptr;
						if (!Sequencer)
						{
							return;
						}

						FQualifiedFrameTime CurrentTime = Sequencer->GetLocalTime();
						FFrameNumber PlaybackEnd = UE::MovieScene::DiscreteExclusiveUpper(Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange());

						DecorationContainer->Modify();
						UMovieSceneSection* NewSection = SectionProvider->CreateNewSection();
						if (Sequencer->GetInfiniteKeyAreas() && NewSection->GetSupportsInfiniteRange())
						{
							NewSection->SetRange(TRange<FFrameNumber>::All());
						}
						else
						{
							FFrameNumber NewSectionRangeEnd = PlaybackEnd;
							if (PlaybackEnd <= CurrentTime.Time.FrameNumber)
							{
								const FAnimatedRange ViewRange = Sequencer->GetViewRange();
								const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
								NewSectionRangeEnd = (ViewRange.GetUpperBoundValue() * TickResolution).FloorToFrame();
							}

							NewSection->SetRange(TRange<FFrameNumber>(CurrentTime.Time.FrameNumber, NewSectionRangeEnd));
						}
						SectionProvider->AddSection(NewSection);
						DecorationContainer->MarkAsChanged();
					}))
				);
			}
		}
	}
	MenuBuilder.EndSection();
}

FPropertyPath SequencerHelpers::PropertyHandleToPropertyPath(const IPropertyHandle& InPropertyHandle)
{
	TSharedRef<const IPropertyHandle> CurrentHandle = InPropertyHandle.AsShared();
	TSharedPtr<const IPropertyHandle> ParentHandle = CurrentHandle->GetParentHandle();

	TArray<FPropertyInfo> PropertyInfos;
	if (CurrentHandle->GetProperty())
	{
		PropertyInfos.Emplace(CurrentHandle->GetProperty(), CurrentHandle->GetArrayIndex());
	}

	while (ParentHandle.IsValid() && ParentHandle->GetProperty())
	{
		PropertyInfos.Emplace(ParentHandle->GetProperty(), ParentHandle->GetArrayIndex());
		CurrentHandle = ParentHandle.ToSharedRef();
		ParentHandle = ParentHandle->GetParentHandle();
	}

	FPropertyPath PropertyPath;
	for (const FPropertyInfo& PropertyInfo : ReverseIterate(PropertyInfos))
	{
		PropertyPath.AddProperty(PropertyInfo);
	}

	return PropertyPath;
}