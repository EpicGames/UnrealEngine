// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerContextMenus.h"
#include "CurveEditor.h"
#include "CurveEditorCommands.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Decorations/MovieSceneSectionAnchorsDecoration.h"
#include "SequencerCommonHelpers.h"
#include "SequencerCommands.h"
#include "SSequencer.h"
#include "IKeyArea.h"
#include "SSequencerSection.h"
#include "SequencerSettings.h"
#include "MVVM/Views/ITrackAreaHotspot.h"
#include "SequencerHotspots.h"
#include "ScopedTransaction.h"
#include "MovieSceneToolHelpers.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneKeyStruct.h"
#include "Framework/Commands/GenericCommands.h"
#include "IDetailsView.h"
#include "IStructureDetailsView.h"
#include "PropertyEditorModule.h"
#include "Sections/MovieSceneSubSection.h"
#include "Curves/IntegralCurve.h"
#include "Editor.h"
#include "SequencerUtilities.h"
#include "ClassViewerModule.h"
#include "Generators/MovieSceneEasingFunction.h"
#include "ClassViewerFilter.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "ISequencerChannelInterface.h"
#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "SKeyEditInterface.h"
#include "MovieSceneTimeHelpers.h"
#include "FrameNumberDetailsCustomization.h"
#include "MovieSceneSectionDetailsCustomization.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "Channels/MovieSceneChannel.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/Selection/Selection.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "Algo/AnyOf.h"
#include "IKeyArea.h"
#include "SequencerToolMenuContext.h"
#include "ToolMenus.h"
#include "ToolableTimeline/ToolableTimelineUtils.h"

#define LOCTEXT_NAMESPACE "SequencerContextMenus"

static void CreateKeyStructForSelection(TWeakPtr<ISequencer> InWeakSequencer, TSharedPtr<FStructOnScope>& OutKeyStruct, TWeakObjectPtr<UMovieSceneSection>& OutKeyStructSection, TWeakObjectPtr<UObject>& OutKeyStructOwningObject)
{
	using namespace UE::Sequencer;

	const TSharedPtr<ISequencer> Sequencer = InWeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const FKeySelection& SelectedKeys = Sequencer->GetViewModel()->GetSelection()->KeySelection;

	if (SelectedKeys.Num() == 1)
	{
		for (FKeyHandle Key : SelectedKeys)
		{
			TSharedPtr<FChannelModel> Channel = SelectedKeys.GetModelForKey(Key);
			if (Channel)
			{
				OutKeyStruct = Channel->GetKeyArea()->GetKeyStruct(Key);
				OutKeyStructSection = Channel->GetSection();
				if (const FMovieSceneChannelMetaData* MetaData = Channel->GetKeyArea()->GetChannel().GetMetaData())
				{
					OutKeyStructOwningObject = MetaData->WeakOwningObject;
				}
				return;
			}
		}
	}
	else
	{
		TArray<FKeyHandle> KeyHandles;
		UMovieSceneSection* CommonSection = nullptr;
		for (FKeyHandle Key : SelectedKeys)
		{
			TSharedPtr<FChannelModel> Channel = SelectedKeys.GetModelForKey(Key);
			if (Channel)
			{
				KeyHandles.Add(Key);

				if (!CommonSection)
				{
					CommonSection = Channel->GetSection();
				}
				else if (CommonSection != Channel->GetSection())
				{
					CommonSection = nullptr;
					return;
				}
			}
		}

		if (CommonSection)
		{
			OutKeyStruct = CommonSection->GetKeyStruct(KeyHandles);
			OutKeyStructSection = CommonSection;
		}
	}
}

namespace UE::Sequencer::Private
{
	
TSet<TSharedPtr<FChannelModel>> GetChannelModels(const TWeakPtr<FSequencer>& InWeakSequencer)
{
	TSet<TSharedPtr<FChannelModel>> Channels;
	
	const TSharedPtr<FSequencer> Sequencer = InWeakSequencer.IsValid() ? InWeakSequencer.Pin() : nullptr;
	if (Sequencer.IsValid())
	{
		const FSequencerSelection& Selection = *Sequencer->GetViewModel()->GetSelection();
		for (const TViewModelPtr<IOutlinerExtension>& Item : Selection.Outliner)
		{
			SequencerHelpers::GetAllChannels(Item, Channels);
		}

		if (Channels.IsEmpty())
		{
			for (const TWeakViewModelPtr<IOutlinerExtension>& DisplayNode : Selection.GetNodesWithSelectedKeysOrSections())
			{
				SequencerHelpers::GetAllChannels(DisplayNode.Pin(), Channels);
			}
		}
	}
	
	return MoveTemp(Channels);
}

}

void FKeyContextMenu::BuildMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, TWeakPtr<FSequencer> InWeakSequencer)
{
	TSharedRef<FKeyContextMenu> Menu = MakeShareable(new FKeyContextMenu(InWeakSequencer));
	Menu->PopulateMenu(MenuBuilder, MenuExtender);
}

void FKeyContextMenu::PopulateMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender)
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	TSharedRef<FKeyContextMenu> Shared = AsShared();

	TWeakObjectPtr<UObject> UnusedKeyStructOwningObject;
	CreateKeyStructForSelection(WeakSequencer, KeyStruct, KeyStructSection, UnusedKeyStructOwningObject);

	{
		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");

		FSelectedKeysByChannel SelectedKeysByChannel(Sequencer->GetViewModel()->GetSelection()->KeySelection);

		TMap<FName, TArray<FExtendKeyMenuParams>> ChannelAndHandlesByType;
		for (FSelectedChannelInfo& ChannelInfo : SelectedKeysByChannel.SelectedChannels)
		{
			FExtendKeyMenuParams ExtendKeyMenuParams;
			ExtendKeyMenuParams.Section   = ChannelInfo.OwningSection;
			ExtendKeyMenuParams.WeakOwner = ChannelInfo.OwningObject;
			ExtendKeyMenuParams.Channel   = ChannelInfo.Channel;
			ExtendKeyMenuParams.Handles   = MoveTemp(ChannelInfo.KeyHandles);

			ChannelAndHandlesByType.FindOrAdd(ChannelInfo.Channel.GetChannelTypeName()).Add(MoveTemp(ExtendKeyMenuParams));
		}

		for (auto& Pair : ChannelAndHandlesByType)
		{
			ISequencerChannelInterface* ChannelInterface = SequencerModule.FindChannelEditorInterface(Pair.Key);
			if (ChannelInterface)
			{
				ChannelInterface->ExtendKeyMenu_Raw(MenuBuilder, MenuExtender, MoveTemp(Pair.Value), Sequencer);
			}
		}
	}

	if (KeyStruct.IsValid())
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("KeyProperties", "Properties"),
			LOCTEXT("KeyPropertiesTooltip", "Modify the key properties"),
			FNewMenuDelegate::CreateLambda([Shared](FMenuBuilder& SubMenuBuilder){ Shared->AddPropertiesMenu(SubMenuBuilder); }),
			FUIAction(
				FExecuteAction(),
				// @todo sequencer: only one struct per structure view supported right now :/
				FCanExecuteAction::CreateLambda([this]{ return KeyStruct.IsValid(); })
			),
			NAME_None,
			EUserInterfaceActionType::Button,
			/*bInOpenSubMenuOnClick=*/false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Details"))
		);
	}

	if (!UToolMenus::Get()->IsMenuRegistered(TEXT("Sequencer.KeyContextMenu")))
	{
		UToolMenu* const KeyContextMenu = UToolMenus::Get()->RegisterMenu(TEXT("Sequencer.KeyContextMenu"), NAME_None, EMultiBoxType::Menu);
		KeyContextMenu->bSearchable = false;

		KeyContextMenu->AddDynamicSection(TEXT("Edit"), FNewToolMenuDelegate::CreateStatic([](UToolMenu* InMenu)
		{
			USequencerToolMenuContext* const ContextObject = InMenu->FindContext<USequencerToolMenuContext>();
			const TSharedPtr<ISequencer>     Sequencer     = ContextObject ? ContextObject->WeakSequencer.Pin() : nullptr;
			if (!Sequencer.IsValid())
			{
				return;
			}

			if (HotspotCast<FKeyHotspot>(Sequencer->GetViewModel()->GetHotspot()))
			{
				FToolMenuSection& Section = InMenu->AddSection(TEXT("SequencerKeyEdit"), LOCTEXT("EditMenu", "Edit"));

				Section.AddMenuEntry(FGenericCommands::Get().Cut);
				Section.AddMenuEntry(FGenericCommands::Get().Copy);
				Section.AddMenuEntry(FGenericCommands::Get().Duplicate);
			}

			FToolMenuSection& KeysRemovalSection = InMenu->AddSection(TEXT("KeyRemoval")
				, FText::GetEmpty(), FToolMenuInsert(NAME_None, EToolMenuInsertType::Last));

			KeysRemovalSection.AddSeparator(NAME_None);

			KeysRemovalSection.AddMenuEntry(
				NAME_None,
				LOCTEXT("DeleteKeys.Label", "Delete Keys"),
				LOCTEXT("DeleteKeys.Description", "Deletes the selected keys"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
				FUIAction(
					FExecuteAction::CreateLambda([WeakSequencer = TWeakPtr<ISequencer>(Sequencer)]
					{
						if (const TSharedPtr<FSequencer> Sequencer = StaticCastSharedPtr<FSequencer>(WeakSequencer.Pin()))
						{
							Sequencer->DeleteSelectedKeys();
						}
					}))
			);
		}));

		KeyContextMenu->AddDynamicSection(TEXT("SequencerKeys"), FNewToolMenuDelegate::CreateStatic([](UToolMenu* InMenu)
		{
			USequencerToolMenuContext* const ContextObject = InMenu->FindContext<USequencerToolMenuContext>();
			const TSharedPtr<ISequencer> Sequencer = ContextObject ? ContextObject->WeakSequencer.Pin() : nullptr;
			if (!Sequencer.IsValid())
			{
				return;
			}

			const FSequencerCommands& SequencerCommands = FSequencerCommands::Get();

			FToolMenuSection& KeysSection = InMenu->FindOrAddSection(TEXT("SequencerKeys"), LOCTEXT("KeysMenu", "Keys"));

			KeysSection.AddMenuEntry(SequencerCommands.SetKeyTime
				, SequencerCommands.SetKeyTime->GetLabel()
				, SequencerCommands.SetKeyTime->GetDescription()
				, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("TimecodeProvider.TabIcon")));

			KeysSection.AddMenuEntry(SequencerCommands.Rekey
				, SequencerCommands.Rekey->GetLabel()
				, SequencerCommands.Rekey->GetDescription()
				, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.KeyDiamond")));

			KeysSection.AddMenuEntry(SequencerCommands.SnapToFrame
				, SequencerCommands.SnapToFrame->GetLabel()
				, SequencerCommands.SnapToFrame->GetDescription()
				, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Snap")));

			// @TODO: Simple view will sync selection to curve editor and use its commands to perform the operations
			// on the selected keys, so we can't use this in Sequencer full view. We would need more abstraction so
			// Curve Editor and Sequencer can have their own versions.
			if (Sequencer->IsSimpleView())
			{
				if (const TSharedPtr<FCurveEditor> CurveEditor = ToolableTimeline::Utils::GetSequencerCurveEditor(*Sequencer))
				{
					if (const TSharedPtr<FUICommandList> CurveEditorCommandBindings = CurveEditor->GetCommands())
					{
						const FCurveEditorCommands& CurveEditorCommands = FCurveEditorCommands::Get();

						KeysSection.AddMenuEntryWithCommandList(CurveEditorCommands.FlattenTangents, CurveEditorCommandBindings);
						KeysSection.AddMenuEntryWithCommandList(CurveEditorCommands.StraightenTangents, CurveEditorCommandBindings);
						KeysSection.AddMenuEntryWithCommandList(CurveEditorCommands.SmartSnapKeys, CurveEditorCommandBindings);
					}
				}
			}
		}));
	}

	USequencerToolMenuContext* ContextObject = NewObject<USequencerToolMenuContext>();
	ContextObject->WeakSequencer = Sequencer;

	FToolMenuContext MenuContext(ContextObject);
	MenuContext.AppendCommandList(Sequencer->GetCommandBindings(ESequencerCommandBindings::Sequencer));
	MenuContext.AddExtender(MenuExtender);

	const TSharedRef<SWidget> Widget = UToolMenus::Get()->GenerateWidget(TEXT("Sequencer.KeyContextMenu"), MenuContext);
	MenuBuilder.AddWidget(Widget, FText(), true, false);
}

void FKeyContextMenu::AddPropertiesMenu(FMenuBuilder& MenuBuilder)
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	auto UpdateAndRetrieveEditData = [this]
	{
		FKeyEditData EditData;
		CreateKeyStructForSelection(WeakSequencer, EditData.KeyStruct, EditData.OwningSection, EditData.OwningObject);
		return EditData;
	};

	MenuBuilder.AddWidget(
		SNew(SKeyEditInterface, Sequencer.ToSharedRef())
		.EditData_Lambda(UpdateAndRetrieveEditData)
		, FText::GetEmpty(), true);
}


FSectionContextMenu::FSectionContextMenu(TWeakPtr<FSequencer> InWeakSequencer, FFrameTime InMouseDownTime)
	: WeakSequencer(InWeakSequencer)
	, MouseDownTime(InMouseDownTime)
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	for (UMovieSceneSection* Section : Sequencer->GetViewModel()->GetSelection()->GetSelectedSections())
	{
		FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
		for (const FMovieSceneChannelEntry& Entry : ChannelProxy.GetAllEntries())
		{
			FName ChannelTypeName = Entry.GetChannelTypeName();

			TArray<UMovieSceneSection*>& SectionArray = SectionsByType.FindOrAdd(ChannelTypeName);
			SectionArray.Add(Section);

			TArray<FMovieSceneChannelHandle>& ChannelHandles = ChannelsByType.FindOrAdd(ChannelTypeName);

			const int32 NumChannels = Entry.GetChannels().Num();
			for (int32 Index = 0; Index < NumChannels; ++Index)
			{
				ChannelHandles.Add(ChannelProxy.MakeHandle(ChannelTypeName, Index));
			}
		}
	}
}

void FSectionContextMenu::BuildMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, TWeakPtr<FSequencer> InWeakSequencer, FFrameTime InMouseDownTime)
{
	TSharedRef<FSectionContextMenu> Menu = MakeShareable(new FSectionContextMenu(InWeakSequencer, InMouseDownTime));
	Menu->PopulateMenu(MenuBuilder, MenuExtender);
}

void FSectionContextMenu::BuildKeyEditMenu(FMenuBuilder& MenuBuilder, TWeakPtr<FSequencer> InWeakSequencer, FFrameTime InMouseDownTime)
{
	TSharedRef<FSectionContextMenu> Menu = MakeShareable(new FSectionContextMenu(InWeakSequencer, InMouseDownTime));
	Menu->AddKeyInterpolationMenu(MenuBuilder);
	Menu->AddKeyEditMenu(MenuBuilder);
}


void FSectionContextMenu::PopulateMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender)
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	// Copy a reference to the context menu by value into each lambda handler to ensure the type stays alive until the menu is closed
	TSharedRef<FSectionContextMenu> Shared = AsShared();

	// Clean SectionGroups to prevent any potential stale references from affecting the context menu entries
	Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->CleanSectionGroups();
	
	// These are potentially expensive checks in large sequences, and won't change while context menu is open
	const bool bCanGroup = Sequencer->CanGroupSelectedSections();
	const bool bCanUngroup = Sequencer->CanUngroupSelectedSections();

	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");

	for (auto& Pair : ChannelsByType)
	{
		const TArray<UMovieSceneSection*>& Sections = SectionsByType.FindChecked(Pair.Key);

		ISequencerChannelInterface* ChannelInterface = SequencerModule.FindChannelEditorInterface(Pair.Key);
		if (ChannelInterface)
		{
			TArray<TWeakObjectPtr<UMovieSceneSection>> WeakSections;
			Algo::Transform(Sections, WeakSections, [](UMovieSceneSection* const InSection)
				{
					return InSection;
				});
			ChannelInterface->ExtendSectionMenu_Raw(MenuBuilder, MenuExtender, Pair.Value, WeakSections, WeakSequencer);
		}
	}

	MenuBuilder.AddSubMenu(
		LOCTEXT("SectionProperties", "Properties"),
		LOCTEXT("SectionPropertiesTooltip", "Modify the section properties"),
		FNewMenuDelegate::CreateLambda([this](FMenuBuilder& SubMenuBuilder)
		{
			const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
			if (!Sequencer.IsValid())
			{
				return;
			}

			TArray<TWeakObjectPtr<UObject>> Sections;
			for (TViewModelPtr<FSectionModel> SectionModel : Sequencer->GetViewModel()->GetSelection()->TrackArea.Filter<FSectionModel>())
			{
				if (UMovieSceneSection* Section = SectionModel->GetSection())
				{
					Sections.Add(Section);
				}
			}

			SequencerHelpers::BuildEditSectionMenu(Sequencer, Sections, SubMenuBuilder, false);
		})
	);

	MenuBuilder.BeginSection("SequencerKeyEdit", LOCTEXT("EditMenu", "Edit"));
	{
		TSharedPtr<FPasteFromHistoryContextMenu> PasteFromHistoryMenu;
		TSharedPtr<FPasteContextMenu> PasteMenu;

		if (Sequencer->GetClipboardStack().Num() != 0)
		{
			FPasteContextMenuArgs PasteArgs = FPasteContextMenuArgs::PasteAt(MouseDownTime.FrameNumber);
			PasteMenu = FPasteContextMenu::CreateMenu(Sequencer, PasteArgs);
			PasteFromHistoryMenu = FPasteFromHistoryContextMenu::CreateMenu(Sequencer, PasteArgs);
		}

		MenuBuilder.AddSubMenu(
			LOCTEXT("Paste", "Paste"),
			FText(),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder){ if (PasteMenu.IsValid()) { PasteMenu->PopulateMenu(SubMenuBuilder, MenuExtender); } }),
			FUIAction (
				FExecuteAction(),
				FCanExecuteAction::CreateLambda([=]{ return PasteMenu.IsValid() && PasteMenu->IsValidPaste(); })
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("PasteFromHistory", "Paste From History"),
			FText(),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder){ if (PasteFromHistoryMenu.IsValid()) { PasteFromHistoryMenu->PopulateMenu(SubMenuBuilder, MenuExtender); } }),
			FUIAction (
				FExecuteAction(),
				FCanExecuteAction::CreateLambda([=]{ return PasteFromHistoryMenu.IsValid(); })
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection(); // SequencerKeyEdit

	MenuBuilder.BeginSection("SequencerChannels", LOCTEXT("ChannelsMenu", "Channels"));
	{
	}
	MenuBuilder.EndSection(); // SequencerChannels

	MenuBuilder.BeginSection("SequencerSections", LOCTEXT("SectionsMenu", "Sections"));
	{
		if (CanSelectAllKeys())
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("SelectAllKeys", "Select All Keys"),
				LOCTEXT("SelectAllKeysTooltip", "Select all keys in section"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([=] { return Shared->SelectAllKeys(); }))
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("CopyAllKeys", "Copy All Keys"),
				LOCTEXT("CopyAllKeysTooltip", "Copy all keys in section"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([=] { return Shared->CopyAllKeys(); }))
			);
		}

		if (SelectionSupportsScaling())
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("ScalingSection", "Scaling"),
				LOCTEXT("ScalingSectionTooltip", "Options for scaling this section"),
				FNewMenuDelegate::CreateLambda([=](FMenuBuilder& InMenuBuilder) { Shared->AddScalingMenu(InMenuBuilder); }));
		}

		MenuBuilder.AddSubMenu(
			LOCTEXT("EditSection", "Edit"),
			LOCTEXT("EditSectionTooltip", "Edit section"),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& InMenuBuilder) { Shared->AddEditMenu(InMenuBuilder); }));

		MenuBuilder.AddSubMenu(
			LOCTEXT("OrderSection", "Order"),
			LOCTEXT("OrderSectionTooltip", "Order section"),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder) { Shared->AddOrderMenu(SubMenuBuilder); }));

		if (GetSupportedBlendTypes().Num() > 1)
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("BlendTypeSection", "Blend Type"),
				LOCTEXT("BlendTypeSectionTooltip", "Change the way in which this section blends with other sections of the same type"),
				FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder) { Shared->AddBlendTypeMenu(SubMenuBuilder); }));
		}

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleSectionActive", "Active"),
			LOCTEXT("ToggleSectionActiveTooltip", "Toggle section active/inactive"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([=] { Shared->ToggleSectionActive(); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([=] { return Shared->IsSectionActive(); })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("Sequencer", "ToggleSectionLocked", "Locked"),
			NSLOCTEXT("Sequencer", "ToggleSectionLockedTooltip", "Toggle section locked/unlocked"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([=] { Shared->ToggleSectionLocked(); }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([=] { return Shared->IsSectionLocked(); })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("GroupSections", "Group"),
			LOCTEXT("GroupSectionsTooltip", "Group selected sections together so that when any section is moved, all sections in that group move together."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(Sequencer.ToSharedRef(), &FSequencer::GroupSelectedSections),
				FCanExecuteAction::CreateLambda([bCanGroup] { return bCanGroup; })
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("UngroupSections", "Ungroup"),
			LOCTEXT("UngroupSectionsTooltip", "Ungroup selected sections"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(Sequencer.ToSharedRef(), &FSequencer::UngroupSelectedSections),
				FCanExecuteAction::CreateLambda([bCanUngroup] { return bCanUngroup; })
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		// @todo Sequencer this should delete all selected sections
		// delete/selection needs to be rethought in general
		MenuBuilder.AddMenuEntry(
			LOCTEXT("DeleteSection", "Delete"),
			LOCTEXT("DeleteSectionToolTip", "Deletes this section"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([=] { return Shared->DeleteSection(); }),
				FCanExecuteAction::CreateLambda([=] { return Shared->CanDeleteSection(); })
			)
		);


		if (CanSetSectionToKey())
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("KeySection", "Key This Section"),
				LOCTEXT("KeySection_ToolTip", "This section will get changed when we modify the property externally"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([=] { return Shared->SetSectionToKey(); }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([=] { return Shared->IsSectionToKey(); })
				),
				NAME_None,
				EUserInterfaceActionType::Check
			);
		}
	}
	MenuBuilder.EndSection(); // SequencerSections
}

bool FSectionContextMenu::SelectionSupportsScaling() const
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return false;
	}

	TSet<UClass*> CompatibleDecorations;
	for (UMovieSceneSection* Section : Sequencer->GetViewModel()->GetSelection()->GetSelectedSections())
	{
		if (Section)
		{
			if (Cast<IMovieSceneScalingDriver>(Section))
			{
				return true;
			}

			Section->GetCompatibleUserDecorations(CompatibleDecorations);
		}
	}

	return CompatibleDecorations.Contains(UMovieSceneSectionAnchorsDecoration::StaticClass());
}

void FSectionContextMenu::AddScalingMenu(FMenuBuilder& MenuBuilder)
{
	// Copy a reference to the context menu by value into each lambda handler to ensure the type stays alive until the menu is closed
	TSharedRef<FSectionContextMenu> Shared = AsShared();

	auto GetScalingDriverCheckState = [Shared]
	{
		TOptional<ECheckBoxState> CheckState;
		TSharedPtr<FSequencer> Sequencer = Shared->WeakSequencer.Pin();
		if (Sequencer)
		{
			for (UMovieSceneSection* Section : Sequencer->GetViewModel()->GetSelection()->GetSelectedSections())
			{
				ECheckBoxState ThisCheckState = (Section && Section->FindDecoration<UMovieSceneSectionAnchorsDecoration>() != nullptr)
					? ECheckBoxState::Checked
					: ECheckBoxState::Unchecked;

				if (!CheckState.IsSet())
				{
					CheckState = ThisCheckState;
				}
				else if (CheckState.GetValue() != ThisCheckState)
				{
					return ECheckBoxState::Undetermined;
				}
			}
		}
		return CheckState.Get(ECheckBoxState::Unchecked);
	};

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ScalingDriver", "Scaling Driver"),
		LOCTEXT("ScalingDriverTooltip", "Defines whether this section will rescale the sequence based on its start/end times when being played back."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([Shared, GetScalingDriverCheckState]
			{
				TSharedPtr<FSequencer> Sequencer = Shared->WeakSequencer.Pin();
				if (Sequencer)
				{
					ECheckBoxState State = GetScalingDriverCheckState();
					FScopedTransaction Transaction(LOCTEXT("ToggleScaling", "Toggle Scaling"));

					for (UMovieSceneSection* Section : Sequencer->GetViewModel()->GetSelection()->GetSelectedSections())
					{
						if (Section)
						{
							Section->Modify();
							if (State == ECheckBoxState::Checked)
							{
								Section->RemoveDecoration<UMovieSceneSectionAnchorsDecoration>();
							}
							else
							{
								Section->GetOrCreateDecoration<UMovieSceneSectionAnchorsDecoration>();
							}
						}
					}
				}
			}),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda(GetScalingDriverCheckState)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
}

void FSectionContextMenu::AddEditMenu(FMenuBuilder& MenuBuilder)
{
	// Copy a reference to the context menu by value into each lambda handler to ensure the type stays alive until the menu is closed
	TSharedRef<FSectionContextMenu> Shared = AsShared();

	MenuBuilder.BeginSection("Trimming", LOCTEXT("TrimmingSectionMenu", "Trimming"));

	MenuBuilder.AddMenuEntry(FSequencerCommands::Get().TrimSectionLeft);

	MenuBuilder.AddMenuEntry(FSequencerCommands::Get().TrimSectionRight);

	MenuBuilder.AddMenuEntry(FSequencerCommands::Get().SplitSection);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("DeleteKeysWhenTrimming", "Delete Keys"),
		LOCTEXT("DeleteKeysWhenTrimmingTooltip", "Delete keys outside of the trimmed range"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]
			{
				const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
				if (!Sequencer.IsValid())
				{
					return;
				}
				Sequencer->GetSequencerSettings()->SetDeleteKeysWhenTrimming(!Sequencer->GetSequencerSettings()->GetDeleteKeysWhenTrimming());
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]
			{
				const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
				if (!Sequencer.IsValid())
				{
					return false;
				}
				return Sequencer->GetSequencerSettings()->GetDeleteKeysWhenTrimming();
			})),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.EndSection();
		
	MenuBuilder.AddMenuSeparator();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AutoSizeSection", "Auto Size"),
		LOCTEXT("AutoSizeSectionTooltip", "Auto size the section length to the duration of the source of this section (ie. audio, animation or shot length)"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([=]{ Shared->AutoSizeSection(); }),
			FCanExecuteAction::CreateLambda([=]{ return Shared->CanAutoSize(); }))
	);

	AddKeyInterpolationMenu(MenuBuilder);
	AddKeyEditMenu(MenuBuilder);
}

void FSectionContextMenu::AddKeyInterpolationMenu(FMenuBuilder& MenuBuilder)
{
	// Copy a reference to the context menu by value into each lambda handler to ensure the type stays alive until the menu is closed
	TSharedRef<FSectionContextMenu> Shared = AsShared();

	MenuBuilder.BeginSection(TEXT("SequencerInterpolation"), LOCTEXT("KeyInterpolationMenu", "Key Interpolation"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SetKeyInterpolationSmartAuto", "Cubic (Smart Auto)"),
		LOCTEXT("SetKeyInterpolationSmartAutoTooltip", "Set key interpolation to smart auto"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.IconKeySmartAuto")),
		FUIAction(
			FExecuteAction::CreateLambda([=] { Shared->SetInterpTangentMode(RCIM_Cubic, RCTM_SmartAuto); }),
			FCanExecuteAction::CreateLambda([=] { return Shared->CanSetInterpTangentMode(); }))
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SetKeyInterpolationAuto", "Cubic (Auto)"),
		LOCTEXT("SetKeyInterpolationAutoTooltip", "Set key interpolation to auto"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.IconKeyAuto")),
		FUIAction(
			FExecuteAction::CreateLambda([=]{ Shared->SetInterpTangentMode(RCIM_Cubic, RCTM_Auto); }),
			FCanExecuteAction::CreateLambda([=]{ return Shared->CanSetInterpTangentMode(); }) )
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SetKeyInterpolationUser", "Cubic (User)"),
		LOCTEXT("SetKeyInterpolationUserTooltip", "Set key interpolation to user"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.IconKeyUser")),
		FUIAction(
			FExecuteAction::CreateLambda([=]{ Shared->SetInterpTangentMode(RCIM_Cubic, RCTM_User); }),
			FCanExecuteAction::CreateLambda([=]{ return Shared->CanSetInterpTangentMode(); }) )
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SetKeyInterpolationBreak", "Cubic (Break)"),
		LOCTEXT("SetKeyInterpolationBreakTooltip", "Set key interpolation to break"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.IconKeyBreak")),
		FUIAction(
			FExecuteAction::CreateLambda([=]{ Shared->SetInterpTangentMode(RCIM_Cubic, RCTM_Break); }),
			FCanExecuteAction::CreateLambda([=]{ return Shared->CanSetInterpTangentMode(); }) )
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SetKeyInterpolationLinear", "Linear"),
		LOCTEXT("SetKeyInterpolationLinearTooltip", "Set key interpolation to linear"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.IconKeyLinear")),
		FUIAction(
			FExecuteAction::CreateLambda([=]{ Shared->SetInterpTangentMode(RCIM_Linear, RCTM_Auto); }),
			FCanExecuteAction::CreateLambda([=]{ return Shared->CanSetInterpTangentMode(); }) )
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SetKeyInterpolationConstant", "Constant"),
		LOCTEXT("SetKeyInterpolationConstantTooltip", "Set key interpolation to constant"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.IconKeyConstant")),
		FUIAction(
			FExecuteAction::CreateLambda([=]{ Shared->SetInterpTangentMode(RCIM_Constant, RCTM_Auto); }),
			FCanExecuteAction::CreateLambda([=]{ return Shared->CanSetInterpTangentMode(); }) )
	);

	MenuBuilder.EndSection();
}

void FSectionContextMenu::AddKeyEditMenu(FMenuBuilder& MenuBuilder)
{
	// Copy a reference to the context menu by value into each lambda handler to ensure the type stays alive until the menu is closed
	TSharedRef<FSectionContextMenu> Shared = AsShared();

	MenuBuilder.BeginSection(TEXT("Key Editing"), LOCTEXT("KeyEditingSectionMenus", "Key Editing"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ReduceKeysSection", "Reduce Keys"),
		LOCTEXT("ReduceKeysTooltip", "Reduce keys in this section"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([=]{ Shared->ReduceKeys(); }),
			FCanExecuteAction::CreateLambda([=]{ return Shared->CanReduceKeys(); }))
	);

	auto OnReduceKeysToleranceChanged = [this](float InNewValue)
	{
		const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
		if (!Sequencer.IsValid())
		{
			return;
		}
		Sequencer->GetSequencerSettings()->SetReduceKeysTolerance(InNewValue);
	};

	MenuBuilder.AddWidget(
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SSpacer)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSpinBox<float>)
			.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>(TEXT("Sequencer.HyperlinkSpinBox")))
			.OnValueCommitted_Lambda([OnReduceKeysToleranceChanged](float Value, ETextCommit::Type)
			{
				OnReduceKeysToleranceChanged(Value);
			})
			.OnValueChanged_Lambda(OnReduceKeysToleranceChanged)
			.MinValue(0)
			.MaxValue(TOptional<float>())
			.Value_Lambda([this]() -> float
			{
				const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
				if (!Sequencer.IsValid())
				{
					return 0;
				}
				return Sequencer->GetSequencerSettings()->GetReduceKeysTolerance();
			})
		],
		LOCTEXT("ReduceKeysTolerance", "Tolerance"), true);

	MenuBuilder.EndSection();
}

FMovieSceneBlendTypeField FSectionContextMenu::GetSupportedBlendTypes() const
{
	FMovieSceneBlendTypeField BlendTypes = FMovieSceneBlendTypeField::All();

	if (const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin())
	{
		for (UMovieSceneSection* Section : Sequencer->GetViewModel()->GetSelection()->GetSelectedSections())
		{
			// Remove unsupported blend types
			BlendTypes.Remove(Section->GetSupportedBlendTypes().Invert());
		}
	}

	return BlendTypes;
}

void FSectionContextMenu::AddOrderMenu(FMenuBuilder& MenuBuilder)
{
	// Copy a reference to the context menu by value into each lambda handler to ensure the type stays alive until the menu is closed
	TSharedRef<FSectionContextMenu> Shared = AsShared();

	MenuBuilder.AddMenuEntry(LOCTEXT("BringToFront", "Bring To Front"), FText(), FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([=]{ return Shared->BringToFront(); })));

	MenuBuilder.AddMenuEntry(LOCTEXT("SendToBack", "Send To Back"), FText(), FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([=]{ return Shared->SendToBack(); })));

	MenuBuilder.AddMenuEntry(LOCTEXT("BringForward", "Bring Forward"), FText(), FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([=]{ return Shared->BringForward(); })));

	MenuBuilder.AddMenuEntry(LOCTEXT("SendBackward", "Send Backward"), FText(), FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([=]{ return Shared->SendBackward(); })));
}

void FSectionContextMenu::AddBlendTypeMenu(FMenuBuilder& MenuBuilder)
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	TArray<TWeakObjectPtr<UMovieSceneSection>> Sections;
	for (TViewModelPtr<FSectionModel> SectionModel : Sequencer->GetViewModel()->GetSelection()->TrackArea.Filter<FSectionModel>())
	{
		if (UMovieSceneSection* Section = SectionModel->GetSection())
		{
			Sections.Add(Section);
		}
	}

	FSequencerUtilities::PopulateMenu_SetBlendType(MenuBuilder, Sections, WeakSequencer);
}

void FSectionContextMenu::SelectAllKeys()
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSet<TSharedPtr<FChannelModel>> Channels = Private::GetChannelModels(WeakSequencer);
	if (!Channels.IsEmpty())
	{
		FSequencerSelection& Selection = *Sequencer->GetViewModel()->GetSelection();
		FSelectionEventSuppressor EventSuppressor = Selection.SuppressEvents();
		
		TArray<FKeyHandle> HandlesScratch;
		for (const TSharedPtr<FChannelModel>& Channel: Channels)
		{
			if (Channel->GetLinkedOutlinerItem() && !Channel->GetLinkedOutlinerItem()->IsFilteredOut())
			{
				HandlesScratch.Reset();
				Channel->GetKeyArea()->GetKeyHandles(HandlesScratch);
				Selection.KeySelection.SelectRange(Channel, HandlesScratch);
			}
		}
	}
}

void FSectionContextMenu::CopyAllKeys()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	SelectAllKeys();
	Sequencer->CopySelectedKeys();
}

void FSectionContextMenu::SetSectionToKey()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	TSet<UMovieSceneSection*> SelectedSections = Sequencer->GetViewModel()->GetSelection()->GetSelectedSections();
	if (SelectedSections.Num() != 1)
	{
		return;
	}

	const bool bToggle = IsSectionToKey();
	UMovieSceneSection* Section = *SelectedSections.CreateConstIterator();
	UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>();
	if (Track)
	{
		FScopedTransaction Transaction(LOCTEXT("SetSectionToKey", "Set Section To Key"));
		Track->Modify();
		Track->SetSectionToKey(bToggle ? nullptr : Section);
	}
}

bool FSectionContextMenu::IsSectionToKey() const
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	for (UMovieSceneSection* Section : Sequencer->GetViewModel()->GetSelection()->GetSelectedSections())
	{
		UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>();
		if (Track && Track->GetSectionToKey() != Section)
		{
			return false;
		}
	}

	return true;
}

bool FSectionContextMenu::CanSetSectionToKey() const
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	TSet<UMovieSceneSection*> SelectedSections = Sequencer->GetViewModel()->GetSelection()->GetSelectedSections();
	if (SelectedSections.Num() != 1)
	{
		return false;
	}

	UMovieSceneSection* Section = *SelectedSections.CreateConstIterator();
	UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>();
	if (Track && Section->GetBlendType().IsValid() && Section->GetBlendType().Get() != EMovieSceneBlendType::Invalid)
	{
		return true;
	}
	return false;
}

bool FSectionContextMenu::CanSelectAllKeys() const
{
	for (const TTuple<FName, TArray<FMovieSceneChannelHandle>>& Pair : ChannelsByType)
	{
		for (const FMovieSceneChannelHandle& Handle : Pair.Value)
		{
			const FMovieSceneChannel* Channel = Handle.Get();
			if (Channel && Channel->GetNumKeys() != 0)
			{
				return true;
			}
		}
	}
	return false;
}

void FSectionContextMenu::AutoSizeSection()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	FScopedTransaction AutoSizeSectionTransaction(LOCTEXT("AutoSizeSection_Transaction", "Auto Size Section"));

	EMovieSceneSectionMovedResult SectionMovedResult = EMovieSceneSectionMovedResult::None;

	for (UMovieSceneSection* Section : Sequencer->GetViewModel()->GetSelection()->GetSelectedSections())
	{
		if (Section && Section->GetAutoSizeRange().IsSet())
		{
			TOptional<TRange<FFrameNumber> > DefaultSectionLength = Section->GetAutoSizeRange();

			if (DefaultSectionLength.IsSet())
			{
				Section->SetRange(DefaultSectionLength.GetValue());

				// Notify the track that the section was resized so it can update any dependent state
				if (UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>())
				{
					Track->Modify();
					SectionMovedResult |= Track->OnSectionMoved(*Section, FMovieSceneSectionMovedParams(EPropertyChangeType::ValueSet));
				}
			}
		}
	}

	EMovieSceneDataChangeType ChangeType = EnumHasAnyFlags(SectionMovedResult, EMovieSceneSectionMovedResult::SectionsChanged)
		? EMovieSceneDataChangeType::MovieSceneStructureItemAdded
		: EMovieSceneDataChangeType::TrackValueChanged;
	Sequencer->NotifyMovieSceneDataChanged(ChangeType);
}


void FSectionContextMenu::ReduceKeys()
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}
	
	const TSet<TSharedPtr<FChannelModel>> ChannelModels = Private::GetChannelModels(WeakSequencer);
	if (ChannelModels.IsEmpty())
	{
		return;
	}

	TSet<UMovieSceneSection*> Sections;
	TSet<FMovieSceneChannel*> Channels;
	Channels.Reserve(ChannelModels.Num());
	
	for (const TSharedPtr<FChannelModel>& ChannelModel: ChannelModels)
	{
		if (ChannelModel->GetLinkedOutlinerItem() && !ChannelModel->GetLinkedOutlinerItem()->IsFilteredOut())
		{
			const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea();
			if (KeyArea.IsValid())
			{
				FMovieSceneChannel* Channel = KeyArea->GetChannel().Get();
				UMovieSceneSection* Section = KeyArea->GetOwningSection();
				if (Channel && Section)
				{
					Channels.Add(Channel);
					Sections.Add(Section);
				}
			}
		}
	}

	if (!Sections.IsEmpty())
	{
		FScopedTransaction ReduceKeysTransaction(LOCTEXT("ReduceKeys_Transaction", "Reduce Keys"));

		for (UMovieSceneSection* Section: Sections)
		{
			Section->Modify();
		}

		FKeyDataOptimizationParams Params;
		Params.bAutoSetInterpolation = true;
		Params.Tolerance = Sequencer->GetSequencerSettings()->GetReduceKeysTolerance();
	
		for (FMovieSceneChannel* Channel: Channels)
		{
			Channel->Optimize(Params);
		}

		Sequencer->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::TrackValueChanged );
	}
}

bool FSectionContextMenu::CanAutoSize() const
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	for (UMovieSceneSection* Section : Sequencer->GetViewModel()->GetSelection()->GetSelectedSections())
	{
		if (Section && Section->GetAutoSizeRange().IsSet())
		{
			return true;
		}
	}

	return false;
}

bool FSectionContextMenu::CanReduceKeys() const
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	TSet<TSharedPtr<IKeyArea> > KeyAreas;
	for (const TWeakPtr<FViewModel>& WeakItem : Sequencer->GetViewModel()->GetSelection()->Outliner)
	{
		SequencerHelpers::GetAllKeyAreas(WeakItem.Pin(), KeyAreas);
	}

	if (KeyAreas.Num() == 0)
	{
		for (const TWeakViewModelPtr<IOutlinerExtension>& DisplayNode : Sequencer->GetViewModel()->GetSelection()->GetNodesWithSelectedKeysOrSections())
		{
			SequencerHelpers::GetAllKeyAreas(DisplayNode.Pin(), KeyAreas);
		}
	}

	return KeyAreas.Num() != 0;
}

void FSectionContextMenu::SetInterpTangentMode(ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode)
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	FScopedTransaction SetInterpTangentModeTransaction(LOCTEXT("SetInterpTangentMode_Transaction", "Set Interpolation and Tangent Mode"));

	TSet<TSharedPtr<IKeyArea> > KeyAreas;
	for (const TWeakPtr<FViewModel>& WeakItem : Sequencer->GetViewModel()->GetSelection()->Outliner)
	{
		SequencerHelpers::GetAllKeyAreas(WeakItem.Pin(), KeyAreas);
	}

	if (KeyAreas.Num() == 0)
	{
		for (const TWeakViewModelPtr<IOutlinerExtension>& DisplayNode : Sequencer->GetViewModel()->GetSelection()->GetNodesWithSelectedKeysOrSections())
		{
			SequencerHelpers::GetAllKeyAreas(DisplayNode.Pin(), KeyAreas);
		}
	}

	bool bAnythingChanged = false;

	for (TSharedPtr<IKeyArea> KeyArea : KeyAreas)
	{
		if (KeyArea.IsValid())
		{
			if (UMovieSceneSignedObject* OwningObject = Cast<UMovieSceneSignedObject>(KeyArea->GetOwningObject()))
			{
				OwningObject->Modify();
			}

			FMovieSceneChannelHandle Handle = KeyArea->GetChannel();
			if (Handle.GetChannelTypeName() == FMovieSceneFloatChannel::StaticStruct()->GetFName())
			{
				FMovieSceneFloatChannel* FloatChannel = static_cast<FMovieSceneFloatChannel*>(Handle.Get());
				TMovieSceneChannelData<FMovieSceneFloatValue> ChannelData = FloatChannel->GetData();
				TArrayView<FMovieSceneFloatValue> Values = ChannelData.GetValues();

				for (int32 KeyIndex = 0; KeyIndex < FloatChannel->GetNumKeys(); ++KeyIndex)
				{
					Values[KeyIndex].InterpMode = InterpMode;
					Values[KeyIndex].TangentMode = TangentMode;
					bAnythingChanged = true;
				}

				FloatChannel->AutoSetTangents();
			}
			else if (Handle.GetChannelTypeName() == FMovieSceneDoubleChannel::StaticStruct()->GetFName())
			{
				FMovieSceneDoubleChannel* DoubleChannel = static_cast<FMovieSceneDoubleChannel*>(Handle.Get());
				TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannel->GetData();
				TArrayView<FMovieSceneDoubleValue> Values = ChannelData.GetValues();

				for (int32 KeyIndex = 0; KeyIndex < DoubleChannel->GetNumKeys(); ++KeyIndex)
				{
					Values[KeyIndex].InterpMode = InterpMode;
					Values[KeyIndex].TangentMode = TangentMode;
					bAnythingChanged = true;
				}

				DoubleChannel->AutoSetTangents();
			}
		}
	}

	if (bAnythingChanged)
	{
		Sequencer->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::TrackValueChanged );
	}
}

bool FSectionContextMenu::CanSetInterpTangentMode() const
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	TSet<TSharedPtr<IKeyArea> > KeyAreas;
	for (const TWeakPtr<FViewModel>& WeakItem : Sequencer->GetViewModel()->GetSelection()->Outliner)
	{
		SequencerHelpers::GetAllKeyAreas(WeakItem.Pin(), KeyAreas);
	}

	if (KeyAreas.Num() == 0)
	{
		for (const TWeakViewModelPtr<IOutlinerExtension>& DisplayNode : Sequencer->GetViewModel()->GetSelection()->GetNodesWithSelectedKeysOrSections())
		{
			SequencerHelpers::GetAllKeyAreas(DisplayNode.Pin(), KeyAreas);
		}
	}

	for (TSharedPtr<IKeyArea> KeyArea : KeyAreas)
	{
		if (KeyArea.IsValid())
		{
			FMovieSceneChannelHandle Handle = KeyArea->GetChannel();
			return (Handle.GetChannelTypeName() == FMovieSceneFloatChannel::StaticStruct()->GetFName() ||
					Handle.GetChannelTypeName() == FMovieSceneDoubleChannel::StaticStruct()->GetFName());
		}
	}

	return false;
}			


void FSectionContextMenu::ToggleSectionActive()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	FScopedTransaction ToggleSectionActiveTransaction( LOCTEXT("ToggleSectionActive_Transaction", "Toggle Section Active") );
	bool bIsActive = !IsSectionActive();
	bool bAnythingChanged = false;

	for (UMovieSceneSection* Section : Sequencer->GetViewModel()->GetSelection()->GetSelectedSections())
	{
		bAnythingChanged = true;
		Section->Modify();
		Section->SetIsActive(bIsActive);
	}

	if (bAnythingChanged)
	{
		Sequencer->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::TrackValueChanged );
	}
	else
	{
		ToggleSectionActiveTransaction.Cancel();
	}
}

bool FSectionContextMenu::IsSectionActive() const
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	// Active only if all are active
	for (UMovieSceneSection* Section : Sequencer->GetViewModel()->GetSelection()->GetSelectedSections())
	{
		if (Section && !Section->IsActive())
		{
			return false;
		}
	}

	return true;
}


void FSectionContextMenu::ToggleSectionLocked()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	FScopedTransaction ToggleSectionLockedTransaction( NSLOCTEXT("Sequencer", "ToggleSectionLocked_Transaction", "Toggle Section Locked") );
	bool bIsLocked = !IsSectionLocked();
	bool bAnythingChanged = false;

	for (UMovieSceneSection* Section : Sequencer->GetViewModel()->GetSelection()->GetSelectedSections())
	{
		if (Section)
		{
			bAnythingChanged = true;
			Section->Modify();
			Section->SetIsLocked(bIsLocked);
		}
	}

	if (bAnythingChanged)
	{
		Sequencer->NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::TrackValueChanged );
	}
	else
	{
		ToggleSectionLockedTransaction.Cancel();
	}
}


bool FSectionContextMenu::IsSectionLocked() const
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	// Locked only if all are locked
	for (UMovieSceneSection* Section : Sequencer->GetViewModel()->GetSelection()->GetSelectedSections())
	{
		if (Section && !Section->IsLocked())
		{
			return false;
		}
	}

	return true;
}

bool FSectionContextMenu::CanDeleteSection() const
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	// Check if any selected section is deletable
	for (UMovieSceneSection* Section : Sequencer->GetViewModel()->GetSelection()->GetSelectedSections())
	{
		if (!Section || Section->IsLocked())
		{
			continue;
		}

		// Check if section interface allows deletion
		TSharedPtr<FSectionModel> SectionModel = Sequencer->GetNodeTree()->GetSectionModel(Section);
		if (SectionModel && SectionModel->GetSectionInterface() && SectionModel->GetSectionInterface()->IsDeletable())
		{
			return true;
		}
	}

	return false;
}

void FSectionContextMenu::DeleteSection()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	Sequencer->DeleteSections(Sequencer->GetViewModel()->GetSelection()->GetSelectedSections());
}

/** Information pertaining to a specific row in a track, required for z-ordering operations */
struct FTrackSectionRow
{
	/** The minimum z-order value for all the sections in this row */
	int32 MinOrderValue;

	/** The maximum z-order value for all the sections in this row */
	int32 MaxOrderValue;

	/** All the sections contained in this row */
	TArray<UMovieSceneSection*> Sections;

	/** A set of sections that are to be operated on */
	TSet<UMovieSceneSection*> SectionToReOrder;

	void AddSection(UMovieSceneSection* InSection)
	{
		Sections.Add(InSection);
		MinOrderValue = FMath::Min(MinOrderValue, InSection->GetOverlapPriority());
		MaxOrderValue = FMath::Max(MaxOrderValue, InSection->GetOverlapPriority());
	}
};


/** Generate the data required for re-ordering rows based on the current sequencer selection */
/** @note: Produces a map of track -> rows, keyed on row index. Only returns rows that contain selected sections */
TMap<UMovieSceneTrack*, TMap<int32, FTrackSectionRow>> GenerateTrackRowsFromSelection(FSequencer& Sequencer)
{
	TMap<UMovieSceneTrack*, TMap<int32, FTrackSectionRow>> TrackRows;

	for (UMovieSceneSection* Section : Sequencer.GetViewModel()->GetSelection()->GetSelectedSections())
	{
		UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>();
		if (!Track)
		{
			continue;
		}

		FTrackSectionRow& Row = TrackRows.FindOrAdd(Track).FindOrAdd(Section->GetRowIndex());
		Row.SectionToReOrder.Add(Section);
	}

	// Now ensure all rows that we're operating on are fully populated
	for (auto& Pair : TrackRows)
	{
		UMovieSceneTrack* Track = Pair.Key;
		for (auto& RowPair : Pair.Value)
		{
			const int32 RowIndex = RowPair.Key;
			for (UMovieSceneSection* Section : Track->GetAllSections())
			{
				if (Section->GetRowIndex() == RowIndex)
				{
					RowPair.Value.AddSection(Section);
				}
			}
		}
	}

	return TrackRows;
}


/** Modify all the sections contained within the specified data structure */
void ModifySections(TMap<UMovieSceneTrack*, TMap<int32, FTrackSectionRow>>& TrackRows)
{
	for (auto& Pair : TrackRows)
	{
		UMovieSceneTrack* Track = Pair.Key;
		for (auto& RowPair : Pair.Value)
		{
			for (UMovieSceneSection* Section : RowPair.Value.Sections)
			{
				Section->Modify();
			}
		}
	}
}

void FSectionContextMenu::BringToFront()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	TMap<UMovieSceneTrack*, TMap<int32, FTrackSectionRow>> TrackRows = GenerateTrackRowsFromSelection(*Sequencer.Get());
	if (TrackRows.Num() == 0)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("BringToFrontTransaction", "Bring to Front"));
	ModifySections(TrackRows);

	for (auto& Pair : TrackRows)
	{
		UMovieSceneTrack* Track = Pair.Key;
		TMap<int32, FTrackSectionRow>& Rows = Pair.Value;

		for (auto& RowPair : Rows)
		{
			FTrackSectionRow& Row = RowPair.Value;

			Row.Sections.StableSort([&](UMovieSceneSection& A, UMovieSceneSection& B){
				bool bIsActiveA = Row.SectionToReOrder.Contains(&A);
				bool bIsActiveB = Row.SectionToReOrder.Contains(&B);

				// Sort secondarily on overlap priority
				if (bIsActiveA == bIsActiveB)
				{
					return A.GetOverlapPriority() < B.GetOverlapPriority();
				}
				// Sort and primarily on whether we're sending to the back or not (bIsActive)
				else
				{
					return !bIsActiveA;
				}
			});

			int32 CurrentPriority = Row.MinOrderValue;
			for (UMovieSceneSection* Section : Row.Sections)
			{
				Section->SetOverlapPriority(CurrentPriority++);
			}
		}
	}

	Sequencer->SetLocalTimeDirectly(Sequencer->GetLocalTime().Time);
}

void FSectionContextMenu::SendToBack()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	TMap<UMovieSceneTrack*, TMap<int32, FTrackSectionRow>> TrackRows = GenerateTrackRowsFromSelection(*Sequencer.Get());
	if (TrackRows.Num() == 0)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SendToBackTransaction", "Send to Back"));
	ModifySections(TrackRows);

	for (auto& Pair : TrackRows)
	{
		UMovieSceneTrack* Track = Pair.Key;
		TMap<int32, FTrackSectionRow>& Rows = Pair.Value;

		for (auto& RowPair : Rows)
		{
			FTrackSectionRow& Row = RowPair.Value;

			Row.Sections.StableSort([&](UMovieSceneSection& A, UMovieSceneSection& B){
				bool bIsActiveA = Row.SectionToReOrder.Contains(&A);
				bool bIsActiveB = Row.SectionToReOrder.Contains(&B);

				// Sort secondarily on overlap priority
				if (bIsActiveA == bIsActiveB)
				{
					return A.GetOverlapPriority() < B.GetOverlapPriority();
				}
				// Sort and primarily on whether we're bringing to the front or not (bIsActive)
				else
				{
					return bIsActiveA;
				}
			});

			int32 CurrentPriority = Row.MinOrderValue;
			for (UMovieSceneSection* Section : Row.Sections)
			{
				Section->SetOverlapPriority(CurrentPriority++);
			}
		}
	}

	Sequencer->SetLocalTimeDirectly(Sequencer->GetLocalTime().Time);
}

void FSectionContextMenu::BringForward()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	TMap<UMovieSceneTrack*, TMap<int32, FTrackSectionRow>> TrackRows = GenerateTrackRowsFromSelection(*Sequencer.Get());
	if (TrackRows.Num() == 0)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("BringForwardTransaction", "Bring Forward"));
	ModifySections(TrackRows);

	for (auto& Pair : TrackRows)
	{
		UMovieSceneTrack* Track = Pair.Key;
		TMap<int32, FTrackSectionRow>& Rows = Pair.Value;

		for (auto& RowPair : Rows)
		{
			FTrackSectionRow& Row = RowPair.Value;

			Row.Sections.Sort([&](UMovieSceneSection& A, UMovieSceneSection& B){
				return A.GetOverlapPriority() < B.GetOverlapPriority();
			});

			for (int32 SectionIndex = Row.Sections.Num() - 2; SectionIndex > 0; --SectionIndex)
			{
				UMovieSceneSection* ThisSection = Row.Sections[SectionIndex];
				if (Row.SectionToReOrder.Contains(ThisSection))
				{
					UMovieSceneSection* OtherSection = Row.Sections[SectionIndex + 1];

					Row.Sections.Swap(SectionIndex, SectionIndex+1);

					const int32 SwappedPriority = OtherSection->GetOverlapPriority();
					OtherSection->SetOverlapPriority(ThisSection->GetOverlapPriority());
					ThisSection->SetOverlapPriority(SwappedPriority);
				}
			}
		}
	}

	Sequencer->SetLocalTimeDirectly(Sequencer->GetLocalTime().Time);
}

void FSectionContextMenu::SendBackward()
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	TMap<UMovieSceneTrack*, TMap<int32, FTrackSectionRow>> TrackRows = GenerateTrackRowsFromSelection(*Sequencer.Get());
	if (TrackRows.Num() == 0)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SendBackwardTransaction", "Send Backward"));
	ModifySections(TrackRows);

	for (auto& Pair : TrackRows)
	{
		UMovieSceneTrack* Track = Pair.Key;
		TMap<int32, FTrackSectionRow>& Rows = Pair.Value;

		for (auto& RowPair : Rows)
		{
			FTrackSectionRow& Row = RowPair.Value;

			Row.Sections.Sort([&](UMovieSceneSection& A, UMovieSceneSection& B){
				return A.GetOverlapPriority() < B.GetOverlapPriority();
			});

			for (int32 SectionIndex = 1; SectionIndex < Row.Sections.Num(); ++SectionIndex)
			{
				UMovieSceneSection* ThisSection = Row.Sections[SectionIndex];
				if (Row.SectionToReOrder.Contains(ThisSection))
				{
					UMovieSceneSection* OtherSection = Row.Sections[SectionIndex - 1];

					Row.Sections.Swap(SectionIndex, SectionIndex - 1);

					const int32 SwappedPriority = OtherSection->GetOverlapPriority();
					OtherSection->SetOverlapPriority(ThisSection->GetOverlapPriority());
					ThisSection->SetOverlapPriority(SwappedPriority);
				}
			}
		}
	}

	Sequencer->SetLocalTimeDirectly(Sequencer->GetLocalTime().Time);
}

bool FPasteContextMenu::BuildMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, TWeakPtr<FSequencer> InWeakSequencer, const FPasteContextMenuArgs& Args)
{
	TSharedRef<FPasteContextMenu> Menu = MakeShareable(new FPasteContextMenu(InWeakSequencer, Args));
	Menu->Setup();
	if (!Menu->IsValidPaste())
	{
		return false;
	}

	Menu->PopulateMenu(MenuBuilder, MenuExtender);
	return true;
}


TSharedRef<FPasteContextMenu> FPasteContextMenu::CreateMenu(TWeakPtr<FSequencer> InWeakSequencer, const FPasteContextMenuArgs& Args)
{
	TSharedRef<FPasteContextMenu> Menu = MakeShareable(new FPasteContextMenu(InWeakSequencer, Args));
	Menu->Setup();
	return Menu;
}


TArray<TSharedPtr<UE::Sequencer::FChannelGroupModel>> KeyAreaNodesBuffer;

void FPasteContextMenu::GatherPasteDestinationsForNode(const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InNode, UMovieSceneSection* InSection, const FName& CurrentScope, TMap<FName, FSequencerClipboardReconciler>& Map)
{
	using namespace UE::Sequencer;

	KeyAreaNodesBuffer.Reset();
	for (const TViewModelPtr<FChannelGroupModel>& ChannelNode : InNode.AsModel()->GetDescendantsOfType<FChannelGroupModel>(true))
	{
		KeyAreaNodesBuffer.Add(ChannelNode);
	}

	if (!KeyAreaNodesBuffer.Num())
	{
		return;
	}

	FName ThisScope;
	{
		FString ThisScopeString;
		if (!CurrentScope.IsNone())
		{
			ThisScopeString.Append(CurrentScope.ToString());
			ThisScopeString.AppendChar('.');
		}
		ThisScopeString.Append(InNode->GetIdentifier().ToString());
		ThisScope = *ThisScopeString;
	}

	FSequencerClipboardReconciler* Reconciler = Map.Find(ThisScope);
	if (!Reconciler)
	{
		Reconciler = &Map.Add(ThisScope, FSequencerClipboardReconciler(Args.Clipboard.ToSharedRef()));
	}

	FSequencerClipboardPasteGroup Group = Reconciler->AddDestinationGroup();
	for (const TSharedPtr<FChannelGroupModel>& KeyAreaNode : KeyAreaNodesBuffer)
	{
		TSharedPtr<FChannelModel> Channel = KeyAreaNode->GetChannel(InSection);
		if (Channel)
		{
			Group.Add(Channel);
		}
	}

	// Add children
	for (const TViewModelPtr<IOutlinerExtension>& Child : InNode.AsModel()->GetChildrenOfType<IOutlinerExtension>())
	{
		GatherPasteDestinationsForNode(Child, InSection, ThisScope, Map);
	}
}


void FPasteContextMenu::Setup()
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	if (!Args.Clipboard.IsValid())
	{
		if (Sequencer->GetClipboardStack().Num() != 0)
		{
			Args.Clipboard = Sequencer->GetClipboardStack().Last();
		}
		else
		{
			return;
		}
	}

	// Gather a list of sections we want to paste into
	TArray<TSharedPtr<FSectionModel>> SectionModels;

	if (Args.DestinationNodes.Num())
	{
		// If we have exactly one channel to paste, first check if we have exactly one valid target channel selected to support copying between channels e.g. from Tranform.x to Transform.y
		if (Args.Clipboard->GetKeyTrackGroups().Num() == 1)
		{
			for (const TViewModelPtr<IOutlinerExtension>& Node : Args.DestinationNodes)
			{
				TViewModelPtr<ISectionOwnerExtension> SectionOwner = Node.AsModel()->FindAncestorOfType<ISectionOwnerExtension>(true);
				if (!SectionOwner)
				{
					continue;
				}

				FPasteDestination& Destination = PasteDestinations[PasteDestinations.AddDefaulted()];

				for (UMovieSceneSection* Section : SectionOwner->GetSections())
				{
					if (Section)
					{
						GatherPasteDestinationsForNode(Node, Section, NAME_None, Destination.Reconcilers);
					}
				}

				// Reconcile and remove invalid pastes
				for (auto It = Destination.Reconcilers.CreateIterator(); It; ++It)
				{
					if (!It.Value().Reconcile() || !It.Value().CanAutoPaste())
					{
						It.RemoveCurrent();
					}
				}

				if (!Destination.Reconcilers.Num())
				{
					PasteDestinations.RemoveAt(PasteDestinations.Num() - 1, EAllowShrinking::No);
				}
			}

			int32 ExactMatchCount = 0;
			for (int32 PasteDestinationIndex = 0; PasteDestinationIndex < PasteDestinations.Num(); ++PasteDestinationIndex)
			{
				if (PasteDestinations[PasteDestinationIndex].Reconcilers.Num() == 1)
				{
					++ExactMatchCount;
				}
			}

			if (ExactMatchCount > 0 && ExactMatchCount == PasteDestinations.Num())
			{
				bPasteFirstOnly = false;
				return;
			}

			// Otherwise reset our list and move on
			PasteDestinations.Reset();
		}

		// Build a list of sections based on selected tracks
		for (const TViewModelPtr<IOutlinerExtension>& Node : Args.DestinationNodes)
		{
			TViewModelPtr<ISectionOwnerExtension> SectionOwner = Node.AsModel()->FindAncestorOfType<ISectionOwnerExtension>(true);
			if (!SectionOwner)
			{
				continue;
			}

			UMovieSceneSection* Section = MovieSceneHelpers::FindNearestSectionAtTime(SectionOwner->GetSections(), Args.PasteAtTime);
			TSharedPtr<FSectionModel> SectionModel = Sequencer->GetNodeTree()->GetSectionModel(Section);
			if (SectionModel)
			{
				SectionModels.Add(SectionModel);
			}
		}
	}
	else
	{
		// Use the selected sections
		for (UMovieSceneSection* WeakSection : Sequencer->GetViewModel()->GetSelection()->GetSelectedSections())
		{
			if (TSharedPtr<FSectionModel> SectionHandle = Sequencer->GetNodeTree()->GetSectionModel(WeakSection))
			{
				SectionModels.Add(SectionHandle);
			}
		}
	}

	TMap<FName, TArray<TSharedPtr<FSectionModel>>> SectionsByType;
	for (TSharedPtr<FSectionModel> SectionModel : SectionModels)
	{
		UMovieSceneTrack* Track = SectionModel->GetParentTrackExtension()->GetTrack();
		if (Track)
		{
			SectionsByType.FindOrAdd(Track->GetClass()->GetFName()).Add(SectionModel);
		}
	}

	for (const TTuple<FName, TArray<TSharedPtr<FSectionModel>>>& Pair : SectionsByType)
	{
		FPasteDestination& Destination = PasteDestinations[PasteDestinations.AddDefaulted()];
		if (Pair.Value.Num() == 1)
		{
			TSharedPtr<FViewModel> Model = Pair.Value[0]->FindAncestorOfTypes({ ISectionOwnerExtension::ID, IOutlinerExtension::ID});
			if (ensure(Model))
			{
				FString Path = IOutlinerExtension::GetPathName(Model);
				Destination.Name = FText::FromString(Path);
			}
		}
		else
		{
			Destination.Name = FText::Format(LOCTEXT("PasteMenuHeaderFormat", "{0} ({1} tracks)"), FText::FromName(Pair.Key), FText::AsNumber(Pair.Value.Num()));
		}

		for (TSharedPtr<FSectionModel> Section : Pair.Value)
		{
			FViewModelPtr Model = Section->FindAncestorOfTypes({ISectionOwnerExtension::ID, IOutlinerExtension::ID});
			GatherPasteDestinationsForNode(Model.ImplicitCast(), Section->GetSection(), NAME_None, Destination.Reconcilers);
		}

		// Reconcile and remove invalid pastes
		for (auto It = Destination.Reconcilers.CreateIterator(); It; ++It)
		{
			if (!It.Value().Reconcile())
			{
				It.RemoveCurrent();
			}
		}
		if (!Destination.Reconcilers.Num())
		{
			PasteDestinations.RemoveAt(PasteDestinations.Num() - 1, EAllowShrinking::No);
		}
	}
}


bool FPasteContextMenu::IsValidPaste() const
{
	return Args.Clipboard.IsValid() && PasteDestinations.Num() != 0;
}


void FPasteContextMenu::PopulateMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender)
{
	// Copy a reference to the context menu by value into each lambda handler to ensure the type stays alive until the menu is closed
	TSharedRef<FPasteContextMenu> Shared = AsShared();

	bool bElevateMenu = PasteDestinations.Num() == 1;
	for (int32 Index = 0; Index < PasteDestinations.Num(); ++Index)
	{
		if (bElevateMenu)
		{
			MenuBuilder.BeginSection("PasteInto", FText::Format(LOCTEXT("PasteIntoTitle", "Paste Into {0}"), PasteDestinations[Index].Name));
			AddPasteMenuForTrackType(MenuBuilder, Index);
			MenuBuilder.EndSection();
			break;
		}

		MenuBuilder.AddSubMenu(
			PasteDestinations[Index].Name,
			FText(),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder){ Shared->AddPasteMenuForTrackType(SubMenuBuilder, Index); })
		);
	}
}


void FPasteContextMenu::AddPasteMenuForTrackType(FMenuBuilder& MenuBuilder, int32 DestinationIndex)
{
	// Copy a reference to the context menu by value into each lambda handler to ensure the type stays alive until the menu is closed
	TSharedRef<FPasteContextMenu> Shared = AsShared();

	for (auto& Pair : PasteDestinations[DestinationIndex].Reconcilers)
	{
		MenuBuilder.AddMenuEntry(
			FText::FromName(Pair.Key),
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([=](){ 
				TSet<FSequencerSelectedKey> NewSelection;
				Shared->BeginPasteInto();
				const bool bAnythingPasted = Shared->PasteInto(DestinationIndex, Pair.Key, NewSelection); 
				Shared->EndPasteInto(bAnythingPasted, NewSelection);
				})
			)
		);
	}
}


bool FPasteContextMenu::AutoPaste()
{
	TSet<FSequencerSelectedKey> NewSelection;
	BeginPasteInto();

	bool bAnythingPasted = false;
	for (int32 PasteDestinationIndex = 0; PasteDestinationIndex < PasteDestinations.Num(); ++PasteDestinationIndex)
	{
		for (auto& Pair : PasteDestinations[PasteDestinationIndex].Reconcilers)
		{
			if (Pair.Value.CanAutoPaste())
			{
				if (PasteInto(PasteDestinationIndex, Pair.Key, NewSelection))
				{
					bAnythingPasted = true;

					if (bPasteFirstOnly)
					{
						break;
					}
				}
			}
		}
	}

	EndPasteInto(bAnythingPasted, NewSelection);

	return bAnythingPasted;
}

void FPasteContextMenu::BeginPasteInto()
{
	GEditor->BeginTransaction(LOCTEXT("PasteKeysTransaction", "Paste Keys"));
}

void FPasteContextMenu::EndPasteInto(bool bAnythingPasted, const TSet<FSequencerSelectedKey>& NewSelection)
{
	using namespace UE::Sequencer;

	if (!bAnythingPasted)
	{
		GEditor->CancelTransaction(0);
		return;
	}

	GEditor->EndTransaction();

	UE::Sequencer::SSequencerSection::ThrobKeySelection();

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	FSequencerSelection& Selection = *Sequencer->GetViewModel()->GetSelection();
	{
		FSelectionEventSuppressor EventSuppressor = Selection.SuppressEvents();

		Selection.TrackArea.Empty();
		Selection.KeySelection.Empty();

		for (const FSequencerSelectedKey& NewKey : NewSelection)
		{
			if (TSharedPtr<FChannelModel> Channel = NewKey.WeakChannel.Pin())
			{
				Selection.KeySelection.Select(Channel, NewKey.KeyHandle);
			}
		}
	}

	Sequencer->OnClipboardUsed(Args.Clipboard);
	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
}

bool FPasteContextMenu::PasteInto(int32 DestinationIndex, FName KeyAreaName, TSet<FSequencerSelectedKey>& NewSelection)
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	FSequencerClipboardReconciler& Reconciler = PasteDestinations[DestinationIndex].Reconcilers[KeyAreaName];

	FSequencerPasteEnvironment PasteEnvironment;
	PasteEnvironment.TickResolution = Sequencer->GetFocusedTickResolution();
	PasteEnvironment.CardinalTime = Args.PasteAtTime;
	PasteEnvironment.TimeTransform = Sequencer->GetFocusedMovieSceneSequenceTransform().LinearTransform;
	PasteEnvironment.OnKeyPasted = [&](FKeyHandle Handle, TSharedPtr<UE::Sequencer::FChannelModel> Channel){
		NewSelection.Add(FSequencerSelectedKey(*Channel->GetSection(), Channel, Handle));
	};

	return Reconciler.Paste(PasteEnvironment);
}


bool FPasteFromHistoryContextMenu::BuildMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, TWeakPtr<FSequencer> InWeakSequencer, const FPasteContextMenuArgs& Args)
{
	const TSharedPtr<FSequencer> Sequencer = InWeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	if (Sequencer->GetClipboardStack().Num() == 0)
	{
		return false;
	}

	TSharedRef<FPasteFromHistoryContextMenu> Menu = MakeShareable(new FPasteFromHistoryContextMenu(InWeakSequencer, Args));
	Menu->PopulateMenu(MenuBuilder, MenuExtender);
	return true;
}


TSharedPtr<FPasteFromHistoryContextMenu> FPasteFromHistoryContextMenu::CreateMenu(TWeakPtr<FSequencer> InWeakSequencer, const FPasteContextMenuArgs& Args)
{
	const TSharedPtr<FSequencer> Sequencer = InWeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return nullptr;
	}

	if (Sequencer->GetClipboardStack().Num() == 0)
	{
		return nullptr;
	}

	return MakeShareable(new FPasteFromHistoryContextMenu(InWeakSequencer, Args));
}


void FPasteFromHistoryContextMenu::PopulateMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender)
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	// Copy a reference to the context menu by value into each lambda handler to ensure the type stays alive until the menu is closed
	TSharedRef<FPasteFromHistoryContextMenu> Shared = AsShared();

	MenuBuilder.BeginSection("SequencerPasteHistory", LOCTEXT("PasteFromHistory", "Paste From History"));

	for (int32 Index = Sequencer->GetClipboardStack().Num() - 1; Index >= 0; --Index)
	{
		FPasteContextMenuArgs ThisPasteArgs = Args;
		ThisPasteArgs.Clipboard = Sequencer->GetClipboardStack()[Index];

		TSharedRef<FPasteContextMenu> PasteMenu = FPasteContextMenu::CreateMenu(Sequencer, ThisPasteArgs);

		MenuBuilder.AddSubMenu(
			ThisPasteArgs.Clipboard->GetDisplayText(),
			FText(),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder){ PasteMenu->PopulateMenu(SubMenuBuilder, MenuExtender); }),
			FUIAction (
				FExecuteAction(),
				FCanExecuteAction::CreateLambda([=]{ return PasteMenu->IsValidPaste(); })
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}

	MenuBuilder.EndSection();
}

void FEasingContextMenu::BuildMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, const TArray<UE::Sequencer::FEasingAreaHandle>& InEasings, TWeakPtr<FSequencer> InWeakSequencer, FFrameTime InMouseDownTime)
{
	TSharedRef<FEasingContextMenu> EasingMenu = MakeShareable(new FEasingContextMenu(InEasings, InWeakSequencer));
	EasingMenu->PopulateMenu(MenuBuilder, MenuExtender);

	MenuBuilder.AddMenuSeparator();

	FSectionContextMenu::BuildMenu(MenuBuilder, MenuExtender, InWeakSequencer, InMouseDownTime);
}

void FEasingContextMenu::PopulateMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender)
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
    if (!Sequencer.IsValid())
    {
    	return;
    }

	FText SectionText = Easings.Num() == 1 ? LOCTEXT("EasingCurve", "Easing Curve") : FText::Format(LOCTEXT("EasingCurvesFormat", "Easing Curves ({0} curves)"), FText::AsNumber(Easings.Num()));
	const bool bReadOnly = Algo::AnyOf(Easings, [](const FEasingAreaHandle& Handle) -> bool
		{
			const UMovieSceneSection* Section = Handle.WeakSectionModel.Pin()->GetSection();
			const UMovieSceneTrack* SectionTrack = Section->GetTypedOuter<UMovieSceneTrack>();
			FMovieSceneSupportsEasingParams Params(Section);
			return !EnumHasAllFlags(SectionTrack->SupportsEasing(Params), EMovieSceneTrackEasingSupportFlags::ManualEasing);
		});

	MenuBuilder.BeginSection("SequencerEasingEdit", SectionText);
	{
		// Copy a reference to the context menu by value into each lambda handler to ensure the type stays alive until the menu is closed
		TSharedRef<FEasingContextMenu> Shared = AsShared();

		auto OnBeginSliderMovement = [=]
		{
			GEditor->BeginTransaction(LOCTEXT("SetEasingTimeText", "Set Easing Length"));
		};
		auto OnEndSliderMovement = [=](double NewLength)
		{
			if (GEditor->IsTransactionActive())
			{
				GEditor->EndTransaction();
			}
		};
		auto OnValueCommitted = [=](double NewLength, ETextCommit::Type CommitInfo)
		{
			if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
			{
				FScopedTransaction Transaction(LOCTEXT("SetEasingTimeText", "Set Easing Length"));
				Shared->OnUpdateLength((int32)NewLength);
			}
		};

		TSharedRef<SWidget> SpinBox = SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.Padding(FMargin(5.f,0.f))
			[
				SNew(SBox)
				.HAlign(HAlign_Right)
				[
					SNew(SNumericEntryBox<double>)
					.SpinBoxStyle(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
					.EditableTextBoxStyle(&FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>("Sequencer.HyperlinkTextBox"))
					// Don't update the value when undetermined text changes
					.OnUndeterminedValueChanged_Lambda([](FText){})
					.AllowSpin(true)
					.IsEnabled(!bReadOnly)
					.MinValue(0.f)
					.MaxValue(TOptional<double>())
					.MaxSliderValue(TOptional<double>())
					.MinSliderValue(0.f)
					.Delta_Lambda([this]() -> double
						{
							const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
							if (!Sequencer.IsValid())
							{
								return 0;
							}
							return Sequencer->GetDisplayRateDeltaFrameCount();
						})
					.Value_Lambda([=]
						{
							TOptional<int32> Current = Shared->GetCurrentLength();
							if (Current.IsSet())
							{
								return TOptional<double>(Current.GetValue());
							}
							return TOptional<double>();
						})
					.OnValueChanged_Lambda([=](double NewLength){ Shared->OnUpdateLength(NewLength); })
					.OnValueCommitted_Lambda(OnValueCommitted)
					.OnBeginSliderMovement_Lambda(OnBeginSliderMovement)
					.OnEndSliderMovement_Lambda(OnEndSliderMovement)
					.BorderForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
					.TypeInterface(Sequencer->GetNumericTypeInterface())
				]
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsEnabled(!bReadOnly)
				.IsChecked_Lambda([=]{ return Shared->GetAutoEasingCheckState(); })
				.OnCheckStateChanged_Lambda([=](ECheckBoxState CheckState){ return Shared->SetAutoEasing(CheckState == ECheckBoxState::Checked); })
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AutomaticEasingText", "Auto?"))
				]
			];
		MenuBuilder.AddWidget(SpinBox, LOCTEXT("EasingAmountLabel", "Easing Length"));

		MenuBuilder.AddSubMenu(
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda([=]{ return Shared->GetEasingTypeText(); })),
			LOCTEXT("EasingTypeToolTip", "Change the type of curve used for the easing"),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder){ Shared->EasingTypeMenu(SubMenuBuilder); })
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("EasingOptions", "Options"),
			LOCTEXT("EasingOptionsToolTip", "Edit easing settings for this curve"),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder){ Shared->EasingOptionsMenu(SubMenuBuilder); })
		);
	}
	MenuBuilder.EndSection();
}

TOptional<int32> FEasingContextMenu::GetCurrentLength() const
{
	using namespace UE::Sequencer;

	TOptional<int32> Value;

	for (const FEasingAreaHandle& Handle : Easings)
	{
		UMovieSceneSection* Section = Handle.WeakSectionModel.Pin()->GetSection();
		if (Section)
		{
			if (Handle.EasingType == ESequencerEasingType::In && Section->Easing.GetEaseInDuration() == Value.Get(Section->Easing.GetEaseInDuration()))
			{
				Value = Section->Easing.GetEaseInDuration();
			}
			else if (Handle.EasingType == ESequencerEasingType::Out && Section->Easing.GetEaseOutDuration() == Value.Get(Section->Easing.GetEaseOutDuration()))
			{
				Value = Section->Easing.GetEaseOutDuration();
			}
			else
			{
				return TOptional<int32>();
			}
		}
	}

	return Value;
}

void FEasingContextMenu::OnUpdateLength(int32 NewLength)
{
	using namespace UE::Sequencer;

	for (const FEasingAreaHandle& Handle : Easings)
	{
		if (UMovieSceneSection* Section = Handle.WeakSectionModel.Pin()->GetSection())
		{
			Section->Modify();
			if (Handle.EasingType == ESequencerEasingType::In)
			{
				Section->Easing.bManualEaseIn = true;
				Section->Easing.ManualEaseInDuration = FMath::Min(UE::MovieScene::DiscreteSize(Section->GetRange()), NewLength);
			}
			else
			{
				Section->Easing.bManualEaseOut = true;
				Section->Easing.ManualEaseOutDuration = FMath::Min(UE::MovieScene::DiscreteSize(Section->GetRange()), NewLength);
			}
		}
	}
}

ECheckBoxState FEasingContextMenu::GetAutoEasingCheckState() const
{
	using namespace UE::Sequencer;

	TOptional<bool> IsChecked;
	for (const FEasingAreaHandle& Handle : Easings)
	{
		if (UMovieSceneSection* Section = Handle.WeakSectionModel.Pin()->GetSection())
		{
			if (Handle.EasingType == ESequencerEasingType::In)
			{
				if (IsChecked.IsSet() && IsChecked.GetValue() != !Section->Easing.bManualEaseIn)
				{
					return ECheckBoxState::Undetermined;
				}
				IsChecked = !Section->Easing.bManualEaseIn;
			}
			else
			{
				if (IsChecked.IsSet() && IsChecked.GetValue() != !Section->Easing.bManualEaseOut)
				{
					return ECheckBoxState::Undetermined;
				}
				IsChecked = !Section->Easing.bManualEaseOut;
			}
		}
	}
	return IsChecked.IsSet() ? IsChecked.GetValue() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked : ECheckBoxState::Undetermined;
}

void FEasingContextMenu::SetAutoEasing(bool bAutoEasing)
{
	using namespace UE::Sequencer;

	FScopedTransaction Transaction(LOCTEXT("SetAutoEasingText", "Set Automatic Easing"));

	TArray<UMovieSceneTrack*> AllTracks;

	for (const FEasingAreaHandle& Handle : Easings)
	{
		if (UMovieSceneSection* Section = Handle.WeakSectionModel.Pin()->GetSection())
		{
			AllTracks.AddUnique(Section->GetTypedOuter<UMovieSceneTrack>());

			Section->Modify();
			if (Handle.EasingType == ESequencerEasingType::In)
			{
				Section->Easing.bManualEaseIn = !bAutoEasing;
			}
			else
			{
				Section->Easing.bManualEaseOut = !bAutoEasing;
			}
		}
	}

	for (UMovieSceneTrack* Track : AllTracks)
	{
		Track->UpdateEasing();
	}
}

FText FEasingContextMenu::GetEasingTypeText() const
{
	using namespace UE::Sequencer;

	FText CurrentText;
	UClass* ClassType = nullptr;
	for (const FEasingAreaHandle& Handle : Easings)
	{
		if (UMovieSceneSection* Section = Handle.WeakSectionModel.Pin()->GetSection())
		{
			UObject* Object = Handle.EasingType == ESequencerEasingType::In ? Section->Easing.EaseIn.GetObject() : Section->Easing.EaseOut.GetObject();
			if (Object)
			{
				if (!ClassType)
				{
					ClassType = Object->GetClass();
				}
				else if (Object->GetClass() != ClassType)
				{
					CurrentText = LOCTEXT("MultipleEasingTypesText", "<Multiple>");
					break;
				}
			}
		}
	}
	if (CurrentText.IsEmpty())
	{
		CurrentText = ClassType ? ClassType->GetDisplayNameText() : LOCTEXT("NoneEasingText", "None");
	}

	return FText::Format(LOCTEXT("EasingTypeTextFormat", "Method ({0})"), CurrentText);
}

void FEasingContextMenu::EasingTypeMenu(FMenuBuilder& MenuBuilder)
{
	struct FFilter : IClassViewerFilter
	{
		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
		{
			bool bIsCorrectInterface = InClass->ImplementsInterface(UMovieSceneEasingFunction::StaticClass());
			bool bMatchesFlags = !InClass->HasAnyClassFlags(CLASS_Hidden | CLASS_HideDropDown | CLASS_Deprecated | CLASS_Abstract);
			return bIsCorrectInterface && bMatchesFlags;
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
		{
			bool bIsCorrectInterface = InUnloadedClassData->ImplementsInterface(UMovieSceneEasingFunction::StaticClass());
			bool bMatchesFlags = !InUnloadedClassData->HasAnyClassFlags(CLASS_Hidden | CLASS_HideDropDown | CLASS_Deprecated | CLASS_Abstract);
			return bIsCorrectInterface && bMatchesFlags;
		}
	};

	FClassViewerModule& ClassViewer = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	FClassViewerInitializationOptions InitOptions;
	InitOptions.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
	InitOptions.ClassFilters.Add(MakeShared<FFilter>());

	// Copy a reference to the context menu by value into each lambda handler to ensure the type stays alive until the menu is closed
	TSharedRef<FEasingContextMenu> Shared = AsShared();

	TSharedRef<SWidget> ClassViewerWidget = ClassViewer.CreateClassViewer(InitOptions, FOnClassPicked::CreateLambda([=](UClass* NewClass) { Shared->OnEasingTypeChanged(NewClass); }));

	MenuBuilder.AddWidget(ClassViewerWidget, FText(), true, false);
}

void FEasingContextMenu::OnEasingTypeChanged(UClass* NewClass)
{
	using namespace UE::Sequencer;

	FScopedTransaction Transaction(LOCTEXT("SetEasingType", "Set Easing Method"));

	for (const FEasingAreaHandle& Handle : Easings)
	{
		UMovieSceneSection* Section = Handle.WeakSectionModel.Pin()->GetSection();
		if (!Section)
		{
			continue;
		}

		Section->Modify();

		TScriptInterface<IMovieSceneEasingFunction>& EaseObject = Handle.EasingType == ESequencerEasingType::In ? Section->Easing.EaseIn : Section->Easing.EaseOut;
		if (!EaseObject.GetObject() || EaseObject.GetObject()->GetClass() != NewClass)
		{
			UObject* NewEasingFunction = NewObject<UObject>(Section, NewClass);

			EaseObject.SetObject(NewEasingFunction);
			EaseObject.SetInterface(Cast<IMovieSceneEasingFunction>(NewEasingFunction));
		}
	}
}

void FEasingContextMenu::EasingOptionsMenu(FMenuBuilder& MenuBuilder)
{
	using namespace UE::Sequencer;

	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowScrollBar = false;

	TSharedRef<IDetailsView> DetailsView = EditModule.CreateDetailView(DetailsViewArgs);
	
	TArray<UObject*> Objects;
	for (const FEasingAreaHandle& Handle : Easings)
	{
		if (UMovieSceneSection* Section = Handle.WeakSectionModel.Pin()->GetSection())
		{
			if (Handle.EasingType == ESequencerEasingType::In)
			{
				UObject* EaseInObject = Section->Easing.EaseIn.GetObject();
				EaseInObject->SetFlags(RF_Transactional);
				Objects.AddUnique(EaseInObject);
			}
			else
			{
				UObject* EaseOutObject = Section->Easing.EaseOut.GetObject();
				EaseOutObject->SetFlags(RF_Transactional);
				Objects.AddUnique(EaseOutObject);
			}
		}
	}

	DetailsView->SetObjects(Objects, true);

	MenuBuilder.AddWidget(DetailsView, FText(), true, false);
}




#undef LOCTEXT_NAMESPACE
