// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SequencerHotspots.h"
#include "Sequencer.h"
#include "SequencerClipboardReconciler.h"
#include "ScopedTransaction.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Curves/RealCurve.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/Extensions/IOutlinerExtension.h"

struct FSequencerSelectedKey;
class FMenuBuilder;
class FExtender;
class UMovieSceneSection;

/**
 * Class responsible for generating a menu for the currently selected sections.
 * This is a shared class that's entirely owned by the context menu handlers.
 * Once the menu is closed, all references to this class are removed, and the 
 * instance is cleaned up. To ensure no weak references are held, AsShared is hidden
 */
struct FSectionContextMenu : TSharedFromThis<FSectionContextMenu>
{
	static void BuildMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, TWeakPtr<FSequencer> InWeakSequencer, FFrameTime InMouseDownTime);
	static void BuildKeyEditMenu(FMenuBuilder& MenuBuilder, TWeakPtr<FSequencer> InWeakSequencer, FFrameTime InMouseDownTime);

private:

	/** Hidden AsShared() methods to discourage CreateSP delegate use. */
	using TSharedFromThis::AsShared;

	FSectionContextMenu(TWeakPtr<FSequencer> InWeakSequencer, FFrameTime InMouseDownTime);

	void PopulateMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender);

	/** Add edit menu for trim and split */
	void AddEditMenu(FMenuBuilder& MenuBuilder);

	void AddScalingMenu(FMenuBuilder& MenuBuilder);
	bool SelectionSupportsScaling() const;

	/** Add the Order sub-menu. */
	void AddOrderMenu(FMenuBuilder& MenuBuilder);

	void AddKeyInterpolationMenu(FMenuBuilder& MenuBuilder);
	void AddKeyEditMenu(FMenuBuilder& MenuBuilder);
	void AddBlendTypeMenu(FMenuBuilder& MenuBuilder);

	void SelectAllKeys();

	void CopyAllKeys();

	bool CanSelectAllKeys() const;

	void SetSectionToKey();

	bool IsSectionToKey() const;

	bool CanSetSectionToKey() const;

	void AutoSizeSection();

	void ReduceKeys();

	void SetInterpTangentMode(ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode);

	bool CanAutoSize() const;

	bool CanReduceKeys() const;

	bool CanSetInterpTangentMode() const;

	void ToggleSectionActive();

	bool IsSectionActive() const;

	void ToggleSectionLocked();

	bool IsSectionLocked() const;

	bool CanDeleteSection() const;

	void DeleteSection();

	void BringToFront();

	void SendToBack();

	void BringForward();

	void SendBackward();
	
	FMovieSceneBlendTypeField GetSupportedBlendTypes() const;

	/** The sequencer */
	TWeakPtr<FSequencer> WeakSequencer;

	/** The time that we clicked on to summon this menu */
	FFrameTime MouseDownTime;

	TMap<FName, TArray<FMovieSceneChannelHandle>> ChannelsByType;
	TMap<FName, TArray<UMovieSceneSection*>> SectionsByType;
};

/** Arguments required for a paste operation */
struct FPasteContextMenuArgs
{
	/** Paste the clipboard into the specified array of sequencer nodes, at the given time */
	static FPasteContextMenuArgs PasteInto(TArray<UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>> InNodes, FFrameNumber InTime, TSharedPtr<FMovieSceneClipboard> InClipboard = nullptr)
	{
		FPasteContextMenuArgs Args;
		Args.Clipboard = InClipboard;
		Args.DestinationNodes = MoveTemp(InNodes);
		Args.PasteAtTime = InTime;
		return Args;
	}

	/** Paste the clipboard at the given time, using the sequencer selection states to determine paste destinations */
	static FPasteContextMenuArgs PasteAt(FFrameNumber InTime, TSharedPtr<FMovieSceneClipboard> InClipboard = nullptr)
	{
		FPasteContextMenuArgs Args;
		Args.Clipboard = InClipboard;
		Args.PasteAtTime = InTime;
		return Args;
	}

	/** The clipboard to paste */
	TSharedPtr<FMovieSceneClipboard> Clipboard;

	/** The Time to paste at */
	FFrameNumber PasteAtTime;

	/** Optional user-supplied nodes to paste into */
	TArray<UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>> DestinationNodes;
};

struct FPasteContextMenu : TSharedFromThis<FPasteContextMenu>
{
	static bool BuildMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, TWeakPtr<FSequencer> InWeakSequencer, const FPasteContextMenuArgs& Args);

	static TSharedRef<FPasteContextMenu> CreateMenu(TWeakPtr<FSequencer> InWeakSequencer, const FPasteContextMenuArgs& Args);

	void PopulateMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender);

	bool IsValidPaste() const;

	bool AutoPaste();

private:

	using TSharedFromThis::AsShared;

	FPasteContextMenu(TWeakPtr<FSequencer> InWeakSequencer, const FPasteContextMenuArgs& InArgs)
		: WeakSequencer(InWeakSequencer)
		, bPasteFirstOnly(true)
		, Args(InArgs)
	{}

	void Setup();

	void AddPasteMenuForTrackType(FMenuBuilder& MenuBuilder, int32 DestinationIndex);

	void BeginPasteInto();
	bool PasteInto(int32 DestinationIndex, FName KeyAreaName, TSet<FSequencerSelectedKey>& NewSelection);
	void EndPasteInto(bool bAnythingPasted, const TSet<FSequencerSelectedKey>& NewSelection);

	void GatherPasteDestinationsForNode(const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InNode, UMovieSceneSection* InSection, const FName& CurrentScope, TMap<FName, FSequencerClipboardReconciler>& Map);

	/** The sequencer */
	TWeakPtr<FSequencer> WeakSequencer;

	/** Paste destinations are organized by track type primarily, then by key area name  */
	struct FPasteDestination
	{
		FText Name;
		TMap<FName, FSequencerClipboardReconciler> Reconcilers;
	};
	TArray<FPasteDestination> PasteDestinations;

	bool bPasteFirstOnly;

	/** Paste arguments */
	FPasteContextMenuArgs Args;
};

struct FPasteFromHistoryContextMenu : TSharedFromThis<FPasteFromHistoryContextMenu>
{
	static bool BuildMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, TWeakPtr<FSequencer> InWeakSequencer, const FPasteContextMenuArgs& Args);

	static TSharedPtr<FPasteFromHistoryContextMenu> CreateMenu(TWeakPtr<FSequencer> InWeakSequencer, const FPasteContextMenuArgs& Args);

	void PopulateMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender);

private:

	using TSharedFromThis::AsShared;

	FPasteFromHistoryContextMenu(TWeakPtr<FSequencer> InWeakSequencer, const FPasteContextMenuArgs& InArgs)
		: WeakSequencer(InWeakSequencer)
		, Args(InArgs)
	{}

	/** The sequencer */
	TWeakPtr<FSequencer> WeakSequencer;

	/** Paste arguments */
	FPasteContextMenuArgs Args;
};

/**
 * Class responsible for generating a menu for the currently selected keys.
 * This is a shared class that's entirely owned by the context menu handlers.
 * Once the menu is closed, all references to this class are removed, and the 
 * instance is cleaned up. To ensure no weak references are held, AsShared is hidden
 */
struct FKeyContextMenu : TSharedFromThis<FKeyContextMenu>
{
	static void BuildMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, TWeakPtr<FSequencer> InWeakSequencer);

private:

	FKeyContextMenu(TWeakPtr<FSequencer> InWeakSequencer)
		: WeakSequencer(InWeakSequencer)
	{}

	/** Hidden AsShared() methods to discourage CreateSP delegate use. */
	using TSharedFromThis::AsShared;

	/** Add the Properties sub-menu. */
	void AddPropertiesMenu(FMenuBuilder& MenuBuilder);

	/** Add the Quantization sub-menu. */
	void AddQuantizationMenu(FMenuBuilder& MenuBuilder);
	void QuantizeKeys(int32 QualtizationInterval);

	void PopulateMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender);

	/** The sequencer */
	TWeakPtr<FSequencer> WeakSequencer;

	TSharedPtr<FStructOnScope> KeyStruct;
	TWeakObjectPtr<UMovieSceneSection> KeyStructSection;
};


/**
 * Class responsible for generating a menu for a set of easing curves.
 * This is a shared class that's entirely owned by the context menu handlers.
 * Once the menu is closed, all references to this class are removed, and the 
 * instance is cleaned up. To ensure no weak references are held, AsShared is hidden
 */
struct FEasingContextMenu : TSharedFromThis<FEasingContextMenu>
{
	static void BuildMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, const TArray<UE::Sequencer::FEasingAreaHandle>& InEasings, TWeakPtr<FSequencer> InWeakSequencer, FFrameTime InMouseDownTime);

private:

	FEasingContextMenu(const TArray<UE::Sequencer::FEasingAreaHandle>& InEasings, TWeakPtr<FSequencer> InWeakSequencer)
		: Easings(InEasings)
		, WeakSequencer(InWeakSequencer)
	{}

	/** Hidden AsShared() methods to discourage CreateSP delegate use. */
	using TSharedFromThis::AsShared;

	void PopulateMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender);

	FText GetEasingTypeText() const;

	void EasingTypeMenu(FMenuBuilder& MenuBuilder);

	void EasingOptionsMenu(FMenuBuilder& MenuBuilder);

	void OnEasingTypeChanged(UClass* NewClass);

	void OnUpdateLength(int32 NewLength);

	TOptional<int32> GetCurrentLength() const;

	ECheckBoxState GetAutoEasingCheckState() const;

	void SetAutoEasing(bool bAutoEasing);

	TArray<UE::Sequencer::FEasingAreaHandle> Easings;

	/** The sequencer */
	TWeakPtr<FSequencer> WeakSequencer;

	/** A scoped transaction for a current operation */
	TUniquePtr<FScopedTransaction> ScopedTransaction;
};
