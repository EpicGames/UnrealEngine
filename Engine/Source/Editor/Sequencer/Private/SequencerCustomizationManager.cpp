// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerCustomizationManager.h"
#include "MovieSceneSequence.h"
#include "Containers/UnrealString.h"

FSequencerCustomizationManager::FSequencerCustomizationManager()
{
}

void FSequencerCustomizationManager::RegisterInstancedSequencerCustomization(const UClass* SequenceClass, FOnGetSequencerCustomizationInstance GetCustomizationDelegate)
{
    if (!ensure(SequenceClass != nullptr))
    {
        return;
    }

    const bool bFound = CustomizationRegistryEntries.ContainsByPredicate([=](const FCustomizationRegistryEntry& Entry) { return Entry.SequenceClass == SequenceClass; });
    ensureMsgf(!bFound, TEXT("Customization already registered for sequence class: %s"), *SequenceClass->GetName());
    if (bFound)
    {
        return;
    }

    FCustomizationRegistryEntry Entry;
    Entry.SequenceClass = SequenceClass;
    Entry.Factory = GetCustomizationDelegate;
    CustomizationRegistryEntries.Add(Entry);
}

void FSequencerCustomizationManager::UnregisterInstancedSequencerCustomization(const UClass* SequenceClass)
{
    if (!ensure(SequenceClass != nullptr))
    {
        return;
    }

    for (TArray<FCustomizationRegistryEntry>::SizeType Idx = 0; Idx < CustomizationRegistryEntries.Num(); ++Idx)
    {
        const FCustomizationRegistryEntry& Entry = CustomizationRegistryEntries[Idx];
        if (Entry.SequenceClass == SequenceClass)
        {
            CustomizationRegistryEntries.RemoveAt(Idx);
            return;
        }
    }
    ensureMsgf(false, TEXT("No customization found for sequence class: %s"), *SequenceClass->GetName());
}

static TAutoConsoleVariable<bool> CVarExactClassCustomizationsOnly(
	TEXT("Sequencer.ExactClassCustomizationsOnly"),
	false,
	TEXT("Enable/disable sequencer customizations with exact class matches only"));

void FSequencerCustomizationManager::GetSequencerCustomizations(UMovieSceneSequence* FocusedSequence, TArray<TUniquePtr<ISequencerCustomization>>& OutCustomizations)
{
	if (UNLIKELY(!FocusedSequence))
	{
		return;
	}

    UClass* SequenceClass = FocusedSequence->GetClass();

	const FCustomizationRegistryEntry* FoundEntry = nullptr;

	if (CVarExactClassCustomizationsOnly->GetBool())
	{
		FoundEntry = CustomizationRegistryEntries.FindByPredicate(
			[=](const FCustomizationRegistryEntry& Entry)
			{
				return Entry.SequenceClass == SequenceClass;
			});
	}
	else
	{
		UClass* CurrentClass = SequenceClass;
		while (CurrentClass != UObject::StaticClass())
		{
			FoundEntry = CustomizationRegistryEntries.FindByPredicate(
				[=](const FCustomizationRegistryEntry& Entry)
				{
					return Entry.SequenceClass == CurrentClass;
				});
			if (FoundEntry == nullptr)
			{
				CurrentClass = CurrentClass->GetSuperClass();
			}
			else
			{
				break;
			}
		}
	}

	if (FoundEntry != nullptr)
	{
		TUniquePtr<ISequencerCustomization> Instance(FoundEntry->Factory.Execute());
		OutCustomizations.Add(MoveTemp(Instance));
	}
}

bool FSequencerCustomizationManager::NeedsCustomizationChange(const UMovieSceneSequence* OldFocusedSequence, const UMovieSceneSequence* NewFocusedSequence) const
{
	if (OldFocusedSequence == nullptr || NewFocusedSequence == nullptr)
	{
		return true;
	}

	// For now, we only map customizations to the sequence's class, so we only need to rebuild customizations
	// if the class has changed between the old and new sequence.
    const UClass* OldSequenceClass = OldFocusedSequence->GetClass();
    const UClass* NewSequenceClass = NewFocusedSequence->GetClass();
	return OldSequenceClass != NewSequenceClass;
}

