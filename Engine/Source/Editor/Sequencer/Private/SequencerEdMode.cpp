// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerEdMode.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ObjectBindingModelStorageExtension.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/Extensions/ITrackRowExtension.h"
#include "MVVM/Views/STrackAreaView.h"
#include "MVVM/Selection/Selection.h"
#include "EditorViewportClient.h"
#include "Curves/KeyHandle.h"
#include "ISequencer.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "IKeyArea.h"
#include "SceneView.h"
#include "Sequencer.h"
#include "Framework/Application/SlateApplication.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "SequencerCommonHelpers.h"
#include "MovieSceneHitProxy.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Sections/MovieSceneAudioSection.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "SubtitleManager.h"
#include "EditorWorldExtension.h"
#include "ViewportWorldInteraction.h"
#include "SSequencer.h"
#include "MovieSceneTracksComponentTypes.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "SequencerSectionPainter.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneToolHelpers.h"
#include "Tools/MotionTrailOptions.h"
#include "SequencerCommands.h"
#include "SnappingUtils.h"
#include "SequencerNodeTree.h"
#include "SequencerSettings.h"
#include "UnrealEdGlobals.h"
#include "UnrealEdMisc.h"
#include "Editor/UnrealEdEngine.h"
#include "TextureResource.h"
#include "EditorModeManager.h"
#include "IMovieScenePlaybackClient.h"
#include "Components/PrimitiveComponent.h"
#include "SequencerSelectabilityTool.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ViewportInteractions/MoveSequencerTimeSliderViewportInteraction.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "Misc/InputState.h"

const FEditorModeID FSequencerEdMode::EM_SequencerMode(TEXT("EM_SequencerMode"));

//DEPRECRATED
TAutoConsoleVariable<bool> CVarUseOldSequencerMotionTrails(
	TEXT("Sequencer.UseOldSequencerTrails"),
	true,
	TEXT("DEPRECRATED:  Will show old trails outside animation mode, otherwise will show newer version for all objects."));


namespace UE
{


namespace SequencerEdMode
{

static const float DrawTrackTimeRes = 0.1f;

struct FTrackTransforms
{
	TArray<FFrameTime> Times;
	TArray<FTransform> Transforms;

	void Initialize(UObject* BoundObject, TArrayView<const FTrajectoryKey> TrajectoryKeys, ISequencer* Sequencer)
	{
		using namespace UE::MovieScene;

		// Hack: static system interrogator for now to avoid re-allocating UObjects all the time
		static FSystemInterrogator Interrogator;
		Interrogator.Reset();

		USceneComponent* SceneComponent = Cast<USceneComponent>(BoundObject);
		if (!SceneComponent)
		{
			AActor* Actor = Cast<AActor>(BoundObject);
			SceneComponent = Actor ? Actor->GetRootComponent() : nullptr;
		}

		if (!SceneComponent)
		{
			return;
		}

		FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();

		TRange<FFrameNumber> ViewRange(TickResolution.AsFrameNumber(Sequencer->GetViewRange().GetLowerBoundValue()), TickResolution.AsFrameNumber(Sequencer->GetViewRange().GetUpperBoundValue()));

		Times.Reserve(TrajectoryKeys.Num());

		FInterrogationChannel Channel = Interrogator.ImportTransformHierarchy(SceneComponent, Sequencer, Sequencer->GetFocusedTemplateID());

		TArray<FMovieSceneSequenceID> SubsequenceHierarchy = Sequencer->GetSubSequenceHierarchy();
		
		const FMovieSceneRootEvaluationTemplateInstance& RootInstance = Sequencer->GetEvaluationTemplate();
		const FMovieSceneSequenceHierarchy* Hierarchy = RootInstance.GetCompiledDataManager()->FindHierarchy(RootInstance.GetCompiledDataID());

		if (Hierarchy)
		{
			Interrogator.SetHierarchy(const_cast<FMovieSceneSequenceHierarchy*>(Hierarchy));

			FMovieSceneSequenceID OwningSequenceID = MovieSceneSequenceID::Root;
			for (const FMovieSceneSequenceID SubSequenceID : SubsequenceHierarchy)
			{
				if (const UMovieSceneSubSection* SubSection = Sequencer->FindSubSection(SubSequenceID))
				{
					if (UMovieSceneTrack* SubTrack = Cast<UMovieSceneTrack>(SubSection->GetOuter()))
					{
						Interrogator.ImportTrack(SubTrack, FInterrogationChannel::Default(), OwningSequenceID);
					}
				}
				OwningSequenceID = SubSequenceID;
			}
		}

		const int32 NumTrajectoryKeys = TrajectoryKeys.Num();
		for (int32 Index = 0; Index < TrajectoryKeys.Num(); ++Index)
		{
			const FTrajectoryKey& ThisKey = TrajectoryKeys[Index];

			Times.Add(ThisKey.Time);
			Interrogator.AddInterrogation(ThisKey.Time);

			const bool bIsConstantKey = ThisKey.Is(ERichCurveInterpMode::RCIM_Constant);
			if (!bIsConstantKey && Index != NumTrajectoryKeys-1)
			{
				const FTrajectoryKey& NextKey = TrajectoryKeys[Index+1];

				FFrameTime Diff = NextKey.Time - ThisKey.Time;
				int32 NumSteps = FMath::CeilToInt(TickResolution.AsSeconds(Diff) / DrawTrackTimeRes);
				// Limit the number of steps to prevent a rendering performance hit
				NumSteps = FMath::Min( 100, NumSteps );

				// Ensure that sub steps evaluate at equal points between the key times such that a NumSteps=2 results in:
				// PrevKey          step1          step2         ThisKey
				// |                  '              '              |
				NumSteps += 1;
				for (int32 Substep = 1; Substep < NumSteps; ++Substep)
				{
					FFrameTime Time = ThisKey.Time + (Diff * float(Substep)/NumSteps);

					Times.Add(Time);
					Interrogator.AddInterrogation(Time);
				}
			}
		}

		TArray<FTransform> OriginTransforms;

		// Get the Instance Data override from the world context object.
		const IMovieScenePlaybackClient*  Client = Sequencer->GetPlaybackClient();
		const UObject* InstanceData = Client ? Client->GetInstanceData() : nullptr;

		Interrogator.Update();
		Interrogator.QueryWorldSpaceTransforms(Channel, Transforms);
		Interrogator.QueryTransformOrigins(OriginTransforms, SubsequenceHierarchy, InstanceData);

		for (int32 Index = 0; Index < Times.Num(); ++Index)
		{
			if (Transforms.IsValidIndex(Index))
			{
				Transforms[Index] *= OriginTransforms[Index];
			}
		}

		check(Transforms.Num() == Times.Num());
		Interrogator.Reset();
	}
};

} // namespace SequencerEdMode
} // namespace UE

FSequencerEdMode::FSequencerEdMode()
{
	DefaultTool = MakeShared<FSequencerEdModeTool>(this);
	SelectabilityTool = MakeShared<FSequencerSelectabilityTool>(FOnGetWorld::CreateRaw(this, &FSequencerEdMode::GetWorld)
		, FOnIsObjectSelectableInViewport::CreateRaw(this, &FSequencerEdMode::IsObjectSelectableInViewport));

	Tools.Add(DefaultTool.Get());
	Tools.Add(SelectabilityTool.Get());
	SetCurrentTool(DefaultTool.Get());

	AudioTexture = LoadObject<UTexture2D>(NULL, TEXT("/Engine/EditorResources/AudioIcons/S_AudioComponent.S_AudioComponent"));
	check(AudioTexture);
}

FSequencerEdMode::~FSequencerEdMode()
{
}

void FSequencerEdMode::Enter()
{
	bIsTracking = false;
	NextPlayerStatus = EMovieScenePlayerStatus::Stopped;
	StartXValue.Reset();
	
	if (UModeManagerInteractiveToolsContext* Context = GetModeManager()->GetInteractiveToolsContext())
	{
		Context->OnBuildViewportInteractions().AddSPLambda(this, [this](UViewportInteractionsBehaviorSource* Source)
		{
			if (UMoveSequencerTimeSliderViewportInteraction* MoveInteraction = Source->AddInteraction<UMoveSequencerTimeSliderViewportInteraction>(true))
			{
				MoveInteraction->EdMode = this;
			}
		});
	}
	
	FEdMode::Enter();
}

void FSequencerEdMode::Exit()
{
	Sequencers.Reset();

	if (UModeManagerInteractiveToolsContext* Context = GetModeManager()->GetInteractiveToolsContext())
	{
		Context->OnBuildViewportInteractions().RemoveAll(this);
	}
	
	FEdMode::Exit();
}

bool FSequencerEdMode::IsCompatibleWith(FEditorModeID OtherModeID) const
{
	// Compatible with all modes so that we can take over with the sequencer hotkeys
	return true;
}

bool FSequencerEdMode::InputKey( FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event )
{
	UE::CurveEditor::InputState::SetCommandPressed(FSequencerCommands::Get().ScrubTimeViewport, Key, Event != IE_Released);

	if (Event != IE_Released && !ViewportClient->IsFlightCameraActive())
	{
		if (const TSharedPtr<ISequencer> ActiveSequencer = GetFirstActiveSequencer())
		{
			FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();

			if (ActiveSequencer->GetCommandBindings(ESequencerCommandBindings::Shared).Get()->ProcessCommandBindings(Key, KeyState, (Event == IE_Repeat) ))
			{
				return true;
			}
			if (IsPressingMoveTimeSlider(Viewport)) //this is needed to make sure we get all of the processed mouse events, for some reason the above may not return true
			{
				return true;
			}
		}
	}

	return FEdMode::InputKey(ViewportClient, Viewport, Key, Event);
}

bool FSequencerEdMode::IsPressingMoveTimeSlider(FViewport* InViewport) const
{
	const FSequencerCommands& Commands = FSequencerCommands::Get();
	bool bIsMovingTimeSlider = false;
	// Need to iterate through primary and secondary to make sure they are all pressed.
	for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
	{
		EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);
		const FInputChord& Chord = *Commands.ScrubTimeViewport->GetActiveChord(ChordIndex);
		bIsMovingTimeSlider |= Chord.IsValidChord() && InViewport->KeyState(Chord.Key) && 
			(Chord.NeedsAlt() ? IsAltDown(InViewport) : !IsAltDown(InViewport)) &&
			(Chord.NeedsShift() ? IsShiftDown(InViewport) : !IsShiftDown(InViewport)) &&
			(Chord.NeedsControl() ? IsCtrlDown(InViewport) : !IsCtrlDown(InViewport));

	}
	return bIsMovingTimeSlider;
}

//just get the first one.
USequencerSettings* FSequencerEdMode:: GetSequencerSettings() const
{
	if (const TSharedPtr<ISequencer> ActiveSequencer = GetFirstActiveSequencer())
	{
		return ActiveSequencer->GetSequencerSettings();
	}
	return nullptr;
}

bool FSequencerEdMode::IsMovingCamera(FViewport* InViewport) const
{
	const bool LeftMouseButtonDown = InViewport->KeyState(EKeys::LeftMouseButton);
	const bool bIsAltKeyDown = InViewport->KeyState(EKeys::LeftAlt) || InViewport->KeyState(EKeys::RightAlt);
	const USequencerSettings* SequencerSettings = GetSequencerSettings();

	return ((SequencerSettings ? SequencerSettings->GetLeftMouseDragDoesMarquee() : false) && LeftMouseButtonDown && bIsAltKeyDown);
}

bool FSequencerEdMode::IsDoingDrag(FViewport* InViewport) const
{
	const USequencerSettings* SequencerSettings = GetSequencerSettings();
	const bool LeftMouseButtonDown = InViewport->KeyState(EKeys::LeftMouseButton);
	const bool bIsCtrlKeyDown = InViewport->KeyState(EKeys::LeftControl) || InViewport->KeyState(EKeys::RightControl);
	const bool bIsAltKeyDown = InViewport->KeyState(EKeys::LeftAlt) || InViewport->KeyState(EKeys::RightAlt);
	EAxisList::Type CurrentAxis = GetCurrentWidgetAxis();

	//if shift is down we still want to drag

	return LeftMouseButtonDown && (CurrentAxis == EAxisList::None) && !bIsCtrlKeyDown  && !bIsAltKeyDown && (SequencerSettings ? SequencerSettings->GetLeftMouseDragDoesMarquee() : false);
}

TSharedPtr<ISequencer> FSequencerEdMode::GetFirstActiveSequencer() const
{
	for (const TWeakPtr<ISequencer> SequencerWeak : Sequencers)
	{
		if (const TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin())
		{
			return Sequencer;
		}
	}
	return nullptr;
}

bool FSequencerEdMode::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	if (Owner && Owner->GetInteractiveToolsContext()->InputRouter->HasActiveMouseCapture())
	{
		return false;
	}

	if (SelectabilityTool->IsSelectionLimited())
	{
		SelectabilityTool->StartTracking(InViewportClient, InViewport);
	}

	if (IsPressingMoveTimeSlider(InViewport))
	{
		UE::CurveEditor::InputState::SetCommandPressed(FSequencerCommands::Get().ScrubTimeViewport, InViewport, true);

		TSharedPtr<FSequencer> ActiveSequencer;
		for (TWeakPtr<FSequencer> WeakSequencer : Sequencers)
		{
			ActiveSequencer = WeakSequencer.Pin();
			if (ActiveSequencer.IsValid())
			{
				break;
			}
		}
		if (ActiveSequencer.IsValid())
		{
			NextPlayerStatus = (ActiveSequencer->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing) ?EMovieScenePlayerStatus::Playing :
				EMovieScenePlayerStatus::Stopped;
			ActiveSequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Scrubbing);
			ActiveSequencer->OnBeginScrubbingEvent().Broadcast();
		}
		return true;
	}
	else if (IsMovingCamera(InViewport))
	{
		bUpdatePivot = true;
		InViewportClient->SetCurrentWidgetAxis(EAxisList::None);
		return true;
	}
	else if (IsDoingDrag(InViewport))
	{
		bUpdatePivot = true;
		DragToolHandler.MakeDragTool(InViewportClient);
		return DragToolHandler.StartTracking(InViewportClient, InViewport);
	}
	return FEdMode::StartTracking(InViewportClient, InViewport);
}

bool FSequencerEdMode::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	if (Owner && Owner->GetInteractiveToolsContext()->InputRouter->HasActiveMouseCapture())
	{
		return false;
	}

	if (SelectabilityTool->IsSelectionLimited())
	{
		SelectabilityTool->EndTracking(InViewportClient, InViewport);
	}

	if (IsPressingMoveTimeSlider(InViewport))
	{
		return true;
	}
	else if (IsMovingCamera(InViewport))
	{
		bUpdatePivot = false;
		return true;
	}
	else if (DragToolHandler.EndTracking(InViewportClient, InViewport))
	{
		bUpdatePivot = false;
		return true;
	}
	return FEdMode::EndTracking(InViewportClient, InViewport);
}

void FSequencerEdMode::Tick(FEditorViewportClient* ViewportClient,float DeltaTime)
{
	if (bUpdatePivot)
	{
		GUnrealEd->UpdatePivotLocationForSelection();
	}
	return FEdMode::Tick(ViewportClient,DeltaTime);
}

bool FSequencerEdMode::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	if (Owner && Owner->GetInteractiveToolsContext()->InputRouter->HasActiveMouseCapture())
	{
		return false;
	}

	if (IsPressingMoveTimeSlider(InViewport))
	{
		return true;
	}
	else if (IsMovingCamera(InViewport))
	{
		if (InDrag.IsNearlyZero() == false || InRot.IsNearlyZero() == false || InScale.IsNearlyZero() == false)
		{
			InViewportClient->SetCurrentWidgetAxis(EAxisList::None);
			InViewportClient->PeformDefaultCameraMovement(InDrag, InRot, InScale);
			GUnrealEd->UpdatePivotLocationForSelection();
			GUnrealEd->RedrawLevelEditingViewports();
		}

		return true;
	}
	else if (IsDoingDrag(InViewport))
	{
		return DragToolHandler.InputDelta(InViewportClient, InViewport, InDrag, InRot, InScale);

	}
	return FEdMode::InputDelta(InViewportClient, InViewport, InDrag, InRot, InScale);
}

bool FSequencerEdMode::ProcessCapturedMouseMoves(FEditorViewportClient* InViewportClient, FViewport* InViewport, const TArrayView<FIntPoint>& CapturedMouseMoves)
{
	const TSharedPtr<ISequencer> ActiveSequencer = GetFirstActiveSequencer();
	const bool bTimeChange = IsPressingMoveTimeSlider(InViewport);
	auto SetNextPlayerStatus = [&ActiveSequencer](EMovieScenePlayerStatus::Type InNextPlayerStatus)
	{
		if (ActiveSequencer.IsValid())
		{
			if (InNextPlayerStatus == EMovieScenePlayerStatus::Stopped)
			{
				//Call Pause() instead of SetPlaybackStatus(EMovieScenePlayerStatus::Stopped)
				//so the correct delegates/evals can happen so all systems like audio are probably stopped (Pause set's Stopped status).
				ActiveSequencer->Pause();
			}
			else
			{
				ActiveSequencer->SetPlaybackStatus(InNextPlayerStatus);
			}
		}
	};
	if (CapturedMouseMoves.Num() > 0)
	{
		if (ActiveSequencer.IsValid())
		{
			if (bTimeChange)
			{
				for (int32 Index = 0; Index < CapturedMouseMoves.Num(); ++Index)
				{
					int32 X = CapturedMouseMoves[Index].X;
					if (StartXValue.IsSet() == false)
					{
						StartXValue = X;
						FQualifiedFrameTime CurrentTime = ActiveSequencer->GetLocalTime();
						StartFrameNumber = CurrentTime.Time.GetFrame();
					}
					else
					{
						int32 Diff = X - StartXValue.GetValue();
						if (Diff != 0)
						{
							FIntPoint Origin, Size;
							InViewportClient->GetViewportDimensions(Origin, Size);
							const float ViewPortSize = (float)Size.X;
							const float FloatViewDiff = (float)(Diff) / ViewPortSize;

							FFrameRate TickResolution = ActiveSequencer->GetFocusedTickResolution();
							TPair<FFrameNumber, FFrameNumber> ViewRange(TickResolution.AsFrameNumber(ActiveSequencer->GetViewRange().GetLowerBoundValue()), TickResolution.AsFrameNumber(ActiveSequencer->GetViewRange().GetUpperBoundValue()));
							FFrameNumber FrameDiff = ViewRange.Value - ViewRange.Key;
							FrameDiff = FrameDiff * FloatViewDiff;
							FFrameTime ScrubTime = StartFrameNumber + FrameDiff;
							ActiveSequencer->SnapSequencerTime(ScrubTime);
							ActiveSequencer->SetLocalTime(ScrubTime.GetFrame());
						}
					}
					if (bIsTracking == false)
					{
						NextPlayerStatus = (ActiveSequencer->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing) ?  EMovieScenePlayerStatus::Playing :
							EMovieScenePlayerStatus::Stopped;
						ActiveSequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Scrubbing);
						ActiveSequencer->OnBeginScrubbingEvent().Broadcast();
						bIsTracking = true;
					}
				}
			}
			else if(bIsTracking)
			{
				bIsTracking = false;
				SetNextPlayerStatus(NextPlayerStatus);
				ActiveSequencer->OnEndScrubbingEvent().Broadcast();
				StartXValue.Reset();
			}
		}
		else
		{
			bIsTracking = false;
			StartXValue.Reset();
		}
		return bIsTracking;
	}
	else if (bTimeChange == false && bIsTracking)
	{
		if (ActiveSequencer.IsValid())
		{
			bIsTracking = false;
			SetNextPlayerStatus(NextPlayerStatus);
			ActiveSequencer->OnEndScrubbingEvent().Broadcast();
			StartXValue.Reset();
		}
		else
		{
			bIsTracking = false;
			StartXValue.Reset();
		}
	}
	return false;
}

bool FSequencerEdMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy *InHitProxy, const FViewportClick &InClick)
{
	if (SelectabilityTool->IsSelectionLimited())
	{
		return SelectabilityTool->HandleClick(InViewportClient, InHitProxy, InClick);
	}

	return FEdMode::HandleClick(InViewportClient, InHitProxy, InClick);
}

bool FSequencerEdMode::BoxSelect(FBox& InBox, bool InSelect)
{
	if (SelectabilityTool->IsSelectionLimited())
	{
		return SelectabilityTool->BoxSelect(InBox, InSelect);
	}

	return FEdMode::BoxSelect(InBox, InSelect);
}

bool FSequencerEdMode::FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect)
{
	if (SelectabilityTool->IsSelectionLimited())
	{
		return SelectabilityTool->FrustumSelect(InFrustum, InViewportClient, InSelect);
	}

	return FEdMode::FrustumSelect(InFrustum, InViewportClient, InSelect);
}

bool FSequencerEdMode::GetCursor(EMouseCursor::Type& OutCursor) const
{
	if (SelectabilityTool->GetCursorForHovered(OutCursor))
	{
		return true;
	}

	return FEdMode::GetCursor(OutCursor);
}

bool FSequencerEdMode::MouseMove(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InX, int32 InY)
{
	if (InViewportClient && InViewportClient->Viewport && SelectabilityTool->IsSelectionLimited())
	{
		HHitProxy* const HitResult = InViewportClient->Viewport->GetHitProxy(InX, InY);
		SelectabilityTool->UpdateHoverFromHitProxy(HitResult);
	}

	return FEdMode::MouseMove(InViewportClient, InViewport, InX, InY);
}

void FSequencerEdMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	FEdMode::Render(View, Viewport, PDI);

	const FViewportClient* ViewportClient = Viewport ? Viewport->GetClient() : nullptr;
	const FViewportClient* FocusedViewportClient = Owner ? Owner->GetFocusedViewportClient() : nullptr;
	if (ViewportClient && ViewportClient == FocusedViewportClient)
	{
		DragToolHandler.Render3DDragTool(View, PDI);
	}

#if WITH_EDITORONLY_DATA
	if (PDI)
	{
		DrawAudioTracks(PDI);
	}

	// Draw spline trails using the PDI
	if (View->Family->EngineShowFlags.Splines)
	{
		DrawTracks3D(PDI);
	}
#endif
}

void FSequencerEdMode::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	FEdMode::DrawHUD(ViewportClient,Viewport,View,Canvas);

	if (ViewportClient)
	{
		const FViewportClient* FocusedViewportClient = Owner ? Owner->GetFocusedViewportClient() : nullptr;
		if (ViewportClient == FocusedViewportClient)
		{
			DragToolHandler.RenderDragTool(View, Canvas);
		}

		if ( ViewportClient->AllowsCinematicControl() )
		{
			// Get the size of the viewport
			const int32 SizeX = Viewport->GetSizeXY().X;
			const int32 SizeY = Viewport->GetSizeXY().Y;

			// Draw subtitles (toggle is handled internally)
			FVector2D MinPos(0.f, 0.f);
			FVector2D MaxPos(1.f, .9f);
			FIntRect SubtitleRegion(FMath::TruncToInt(SizeX * MinPos.X), FMath::TruncToInt(SizeY * MinPos.Y), FMath::TruncToInt(SizeX * MaxPos.X), FMath::TruncToInt(SizeY * MaxPos.Y));
			FSubtitleManager::GetSubtitleManager()->DisplaySubtitles( Canvas, SubtitleRegion, ViewportClient->GetWorld()->GetAudioTimeSeconds() );
		}
	}

	if (SelectabilityTool->IsSelectionLimited())
	{
		SelectabilityTool->DrawHUD(ViewportClient, Viewport, View, Canvas);
	}
}

void FSequencerEdMode::OnKeySelected(FViewport* Viewport, HMovieSceneKeyProxy* KeyProxy)
{
	using namespace UE::Sequencer;

	if (!KeyProxy)
	{
		return;
	}

	const bool bToggleSelection = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);
	const bool bAddToSelection = Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift);

	for (TWeakPtr<FSequencer> WeakSequencer : Sequencers)
	{
		TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
		if (Sequencer.IsValid())
		{
			Sequencer->SetLocalTimeDirectly(KeyProxy->Key.Time);

			FSequencerSelection& Selection = *Sequencer->GetViewModel()->GetSelection();

			FSelectionEventSuppressor EventSuppressor = Selection.SuppressEvents();

			if (!bAddToSelection && !bToggleSelection)
			{
				Selection.KeySelection.Empty();
			}

			for (const FTrajectoryKey::FData& KeyData : KeyProxy->Key.KeyData)
			{
				UMovieSceneSection* Section = KeyData.Section.Get();
				TSharedPtr<FSectionModel> SectionHandle = Sequencer->GetNodeTree()->GetSectionModel(Section);
				if (SectionHandle && KeyData.KeyHandle.IsSet())
				{
					TParentFirstChildIterator<FChannelGroupModel> KeyAreaNodes = SectionHandle->GetParentSectionOwnerModel().AsModel()->GetDescendantsOfType<FChannelGroupModel>();
					for (const TViewModelPtr<FChannelGroupModel>& KeyAreaNode : KeyAreaNodes)
					{
						TSharedPtr<FChannelModel> Channel = KeyAreaNode->GetChannel(Section);
						if (Channel && Channel->GetKeyArea()->GetName() == KeyData.ChannelName)
						{
							if (bToggleSelection && Selection.KeySelection.IsSelected(KeyData.KeyHandle.GetValue()))
							{
								Selection.KeySelection.Deselect(KeyData.KeyHandle.GetValue());
							}
							else
							{
								Selection.KeySelection.Select(Channel, KeyData.KeyHandle.GetValue());
							}

							break;
						}
					}
				}
			}
		}
	}
}

void FSequencerEdMode::DrawTransformTrack(const TSharedPtr<ISequencer>& Sequencer, FPrimitiveDrawInterface* PDI,
											UMovieScene3DTransformTrack* TransformTrack, TArrayView<const TWeakObjectPtr<>> BoundObjects, const bool bIsSelected)
{
	using namespace UE::MovieScene;
	using namespace UE::Sequencer;

	const bool bHitTesting = PDI && PDI->IsHitTesting();

	bool bShowTrajectory = TransformTrack->GetAllSections().ContainsByPredicate(
		[bIsSelected](UMovieSceneSection* Section)
		{
			UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(Section);
			if (TransformSection)
			{
				switch (TransformSection->GetShow3DTrajectory())
				{
				case EShow3DTrajectory::EST_Always:				return true;
				case EShow3DTrajectory::EST_Never:				return false;
				case EShow3DTrajectory::EST_OnlyWhenSelected:	return bIsSelected;
				}
			}
			return false;
		}
	);
	
	FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();

	if (!bShowTrajectory || !TransformTrack->GetAllSections().ContainsByPredicate([](UMovieSceneSection* In){ return In->IsActive(); }))
	{
		return;
	}

	TArray<UMovieScene3DTransformSection*> AllSectionsScratch;

	FLinearColor TrackColor = STrackAreaView::BlendDefaultTrackColor(TransformTrack->GetColorTint());
	FColor       KeyColor   = TrackColor.ToFColor(true);

	// Draw one line per-track (should only really ever be one)
	TRange<FFrameNumber> ViewRange(TickResolution.AsFrameNumber(Sequencer->GetViewRange().GetLowerBoundValue()), TickResolution.AsFrameNumber(Sequencer->GetViewRange().GetUpperBoundValue()));

	TArray<FTrajectoryKey> TrajectoryKeys = TransformTrack->GetTrajectoryData(Sequencer->GetLocalTime().Time.FrameNumber, Sequencer->GetSequencerSettings()->GetTrajectoryPathCap(), ViewRange);
	for (TWeakObjectPtr<> WeakBinding : BoundObjects)
	{
		UObject* BoundObject = WeakBinding.Get();
		if (!BoundObject)
		{
			continue;
		}

		UE::SequencerEdMode::FTrackTransforms TrackTransforms;
		TrackTransforms.Initialize(BoundObject, TrajectoryKeys, Sequencer.Get());

		int32 TransformIndex = 0;

		for (int32 TrajectoryIndex = 0; TrajectoryIndex < TrajectoryKeys.Num(); ++TrajectoryIndex)
		{
			const FTrajectoryKey& ThisKey = TrajectoryKeys[TrajectoryIndex];

			if (TransformIndex >= TrackTransforms.Transforms.Num())
			{
				continue;
			}

			FTransform ThisTransform = TrackTransforms.Transforms[TransformIndex];

			if (TrajectoryIndex < TrajectoryKeys.Num()-1)
			{
				FFrameTime NextKeyTime = TrajectoryKeys[TrajectoryIndex+1].Time;

				// Draw all the interpolated times between this and the next key
				FVector StartPosition = TrackTransforms.Transforms[TransformIndex].GetTranslation();
				++TransformIndex;

				const bool bIsConstantKey = ThisKey.Is(ERichCurveInterpMode::RCIM_Constant);
				if (bIsConstantKey)
				{
					if (PDI)
					{
						FVector EndPosition = TrackTransforms.Transforms[TransformIndex].GetTranslation();
						DrawDashedLine(PDI, StartPosition, EndPosition, TrackColor, 20, SDPG_Foreground);
					}
				}
				else
				{
					// Draw intermediate segments
					for ( ; TransformIndex < TrackTransforms.Times.Num() && TrackTransforms.Times[TransformIndex] < NextKeyTime; ++TransformIndex )
					{
						FTransform EndTransform = TrackTransforms.Transforms[TransformIndex];

						if (PDI)
						{
							PDI->DrawLine(StartPosition, EndTransform.GetTranslation(), TrackColor, SDPG_Foreground);
						}

						StartPosition = EndTransform.GetTranslation();
					}

					// Draw the final segment
					if (PDI && TrackTransforms.Times[TransformIndex] == NextKeyTime)
					{
						FTransform EndTransform = TrackTransforms.Transforms[TransformIndex];
						PDI->DrawLine(StartPosition, EndTransform.GetTranslation(), TrackColor, SDPG_Foreground);
					}
				}
			}

			// If this trajectory key does not have any key handles associated with it, we've nothing left to do
			if (ThisKey.KeyData.Num() == 0)
			{
				continue;
			}

			if (bHitTesting && PDI)
			{
				PDI->SetHitProxy(new HMovieSceneKeyProxy(TransformTrack, ThisKey));
			}

			// Drawing keys
			if (PDI != nullptr)
			{
				if (bHitTesting)
				{
					PDI->SetHitProxy(new HMovieSceneKeyProxy(TransformTrack, ThisKey));
				}

				PDI->DrawPoint(ThisTransform.GetTranslation(), KeyColor, 6.f, SDPG_Foreground);

				if (bHitTesting)
				{
					PDI->SetHitProxy(nullptr);
				}
			}
		}
	}
}


void FSequencerEdMode::DrawTracks3D(FPrimitiveDrawInterface* PDI)
{
	using namespace UE::Sequencer;

	//if in control rig mode(aka animation mode) we show new trails
	const FName ControlRigEditModeModeName("EditMode.ControlRig");
	const bool bIsInControlRigEditMode = GLevelEditorModeTools().GetActiveMode(ControlRigEditModeModeName) != nullptr;
	if (bIsInControlRigEditMode == true)
	{
		return;
	}
	for (TWeakPtr<FSequencer> WeakSequencer : Sequencers)
	{
		TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
		if (!Sequencer.IsValid())
		{
			continue;
		}

		UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
		if (!Sequence)
		{
			continue;
		}

		const UMovieScene* MovieScene = Sequence->GetMovieScene();
		if (!MovieScene)
		{
			continue;
		}

		// Gather a map of object bindings to their implict selection state
		TMap<const FMovieSceneBinding*, bool> ObjectBindingNodesSelectionMap;

		FObjectBindingModelStorageExtension* ObjectStorage = Sequencer->GetViewModel()->GetRootModel()->CastDynamic<FObjectBindingModelStorageExtension>();
		check(ObjectStorage);

		const FSequencerSelection& Selection = *Sequencer->GetViewModel()->GetSelection();
		const TSharedRef<FSequencerNodeTree>& NodeTree = Sequencer->GetNodeTree();
		for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
		{
			TSharedPtr<FObjectBindingModel> ObjectBindingNode = ObjectStorage->FindModelForObjectBinding(Binding.GetObjectGuid());
			if (!ObjectBindingNode)
			{
				continue;
			}

			bool bSelected = Selection.Outliner.IsSelected(ObjectBindingNode);
			if (!bSelected)
			{
				for (TViewModelPtr<IOutlinerExtension> Child : ObjectBindingNode->GetDescendantsOfType<IOutlinerExtension>())
				{
					if (Selection.Outliner.IsSelected(Child) || Selection.NodeHasSelectedKeysOrSections(Child))
					{
						bSelected = true;
						// Stop traversing
						break;
					}
				}

				// If one of our parent is selected, we're considered selected
				for (TViewModelPtr<IOutlinerExtension> Parent : ObjectBindingNode->GetAncestorsOfType<IOutlinerExtension>())
				{
					if (Selection.Outliner.IsSelected(Parent) || Selection.NodeHasSelectedKeysOrSections(Parent))
					{
						bSelected = true;
						break;
					}
				}
			}

			ObjectBindingNodesSelectionMap.Add(&Binding, bSelected);
		}

		// Gather up the transform track nodes from the object binding nodes
		for (TTuple<const FMovieSceneBinding*, bool>& Pair : ObjectBindingNodesSelectionMap)
		{
			for (UMovieSceneTrack* Track : Pair.Key->GetTracks())
			{
				UMovieScene3DTransformTrack* TransformTrack = Cast<UMovieScene3DTransformTrack>(Track);
				if (!TransformTrack)
				{
					continue;
				}

				const bool bIsSelected = Pair.Value;
				DrawTransformTrack(Sequencer, PDI, TransformTrack, Sequencer->FindObjectsInCurrentSequence(Pair.Key->GetObjectGuid()), bIsSelected);
			}
		}
	}
}

void FSequencerEdMode::DrawAudioTracks(FPrimitiveDrawInterface* PDI)
{
	using namespace UE::Sequencer;

	for (TWeakPtr<ISequencer> WeakSequencer : Sequencers)
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		if (!Sequencer.IsValid())
		{
			continue;
		}

		UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
		if (!Sequence)
		{
			continue;
		}

		FQualifiedFrameTime CurrentTime = Sequencer->GetLocalTime();

		TSet<UMovieSceneAudioTrack*> SelectedAudioTracks;

		for (TViewModelPtr<ITrackExtension> TrackModel : Sequencer->GetViewModel()->GetSelection()->Outliner.Filter<ITrackExtension>())
		{
			UMovieSceneAudioTrack* AudioTrack = Cast<UMovieSceneAudioTrack>(TrackModel->GetTrack());

			if (!AudioTrack)
			{
				continue;
			}
			SelectedAudioTracks.Add(AudioTrack);
		}
		
		for (TViewModelPtr<ITrackRowExtension> TrackRowModel : Sequencer->GetViewModel()->GetSelection()->Outliner.Filter<ITrackRowExtension>())
		{
			UMovieSceneAudioTrack* AudioTrack = Cast<UMovieSceneAudioTrack>(TrackRowModel->GetParentTrack());

			if (!AudioTrack)
			{
				continue;
			}
			SelectedAudioTracks.Add(AudioTrack);
		}
		
		for (UMovieSceneAudioTrack* AudioTrack : SelectedAudioTracks)
		{
			for (UMovieSceneSection* Section : AudioTrack->GetAudioSections())
			{
				UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(Section);
				const FMovieSceneActorReferenceChannel& AttachActorData = AudioSection->GetAttachActorData();

				TMovieSceneChannelData<const FMovieSceneActorReferenceKey> ChannelData = AttachActorData.GetData();
				TArrayView<const FMovieSceneActorReferenceKey> Values = ChannelData.GetValues().Num() != 0
					? ChannelData.GetValues()
					: MakeArrayView(&AttachActorData.GetDefault(), 1);
		
				FMovieSceneActorReferenceKey CurrentValue;
				AttachActorData.Evaluate(CurrentTime.Time, CurrentValue);

				for (int32 Index = 0; Index < Values.Num(); ++Index)
				{
					FMovieSceneObjectBindingID AttachBindingID = Values[Index].Object;
					FName AttachSocketName = Values[Index].SocketName;

					for (TWeakObjectPtr<> WeakObject : AttachBindingID.ResolveBoundObjects(Sequencer->GetFocusedTemplateID(), *Sequencer))
					{
						AActor* AttachActor = Cast<AActor>(WeakObject.Get());
						if (AttachActor)
						{
							USceneComponent* AttachComponent = AudioSection->GetAttachComponent(AttachActor, Values[Index]);
							if (AttachComponent)
							{
								FVector Location = AttachComponent->GetSocketLocation(AttachSocketName);
								bool bIsActive = CurrentValue == Values[Index];
								FColor Color = bIsActive ? FColor::Green : FColor::White;

								float Scale = PDI->View->WorldToScreen(Location).W * (4.0f / PDI->View->UnscaledViewRect.Width() / PDI->View->ViewMatrices.GetViewToClip().M[0][0]);
								Scale *= bIsActive ? 15.f : 10.f;

								PDI->DrawSprite(Location, Scale, Scale, AudioTexture->GetResource(), Color, SDPG_Foreground, 0.0, 0.0, 0.0, 0.0, SE_BLEND_Masked);
								break;
							}
						}
					}
				}
			}
		}
	}
}

bool FSequencerEdMode::IsViewportSelectionLimited() const
{
	return SelectabilityTool.IsValid() && SelectabilityTool->IsSelectionLimited();
}

void FSequencerEdMode::EnableSelectabilityTool(const bool bInEnabled)
{
	SelectabilityTool->EnableLimitedSelection(bInEnabled);
}

bool FSequencerEdMode::IsObjectSelectableInViewport(UObject* const InObject) const
{
	if (const TSharedPtr<ISequencer> Sequencer = GetFirstActiveSequencer())
	{
		return Sequencer->IsObjectSelectableInViewport(InObject);
	}
	return true;
}


FSequencerEdModeTool::FSequencerEdModeTool(FSequencerEdMode* InSequencerEdMode) :
	SequencerEdMode(InSequencerEdMode)
{
}

FSequencerEdModeTool::~FSequencerEdModeTool()
{
}

bool FSequencerEdModeTool::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	if( Key == EKeys::LeftMouseButton )
	{
		if( Event == IE_Pressed)
		{
			int32 HitX = ViewportClient->Viewport->GetMouseX();
			int32 HitY = ViewportClient->Viewport->GetMouseY();
			HHitProxy*HitResult = ViewportClient->Viewport->GetHitProxy(HitX, HitY);

			if(HitResult)
			{
				if( HitResult->IsA(HMovieSceneKeyProxy::StaticGetType()) )
				{
					HMovieSceneKeyProxy* KeyProxy = (HMovieSceneKeyProxy*)HitResult;
					SequencerEdMode->OnKeySelected(ViewportClient->Viewport, KeyProxy);
				}
			}
		}
	}

	return FModeTool::InputKey(ViewportClient, Viewport, Key, Event);
}

/*
*
*    Drag Tool Handler
*
*/

FMarqueeDragTool::FMarqueeDragTool()
	: bIsDeletingDragTool(false)
{
}

bool FMarqueeDragTool::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	return (DragTool.IsValid() && InViewportClient->GetCurrentWidgetAxis() == EAxisList::None);
}

bool FMarqueeDragTool::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	if (!bIsDeletingDragTool)
	{
		// Ending the drag tool may pop up a modal dialog which can cause unwanted reentrancy - protect against this.
		TGuardValue<bool> RecursionGuard(bIsDeletingDragTool, true);

		// Delete the drag tool if one exists.
		if (DragTool.IsValid())
		{
			if (DragTool->IsDragging())
			{
				DragTool->EndDrag();
			}
			DragTool.Reset();
			return true;
		}
	}
	
	return false;
}

bool FMarqueeDragTool::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	if (DragTool.IsValid() == false || InViewportClient->GetCurrentWidgetAxis() != EAxisList::None)
	{
		return false;
	}
	if (DragTool->IsDragging() == false)
	{
		int32 InX = InViewport->GetMouseX();
		int32 InY = InViewport->GetMouseY();
		FVector2D Start(InX, InY);

		DragTool->StartDrag(InViewportClient, GEditor->ClickLocation,Start);
	}
	const bool bUsingDragTool = UsingDragTool();
	if (bUsingDragTool == false)
	{
		return false;
	}

	DragTool->AddDelta(InDrag);
	return true;
}

/**
 * @return		true if a drag tool is being used by the tracker, false otherwise.
 */
bool FMarqueeDragTool::UsingDragTool() const
{
	return DragTool.IsValid() && DragTool->IsDragging();
}

/**
 * Renders the drag tool.  Does nothing if no drag tool exists.
 */
void FMarqueeDragTool::Render3DDragTool(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	if (DragTool.IsValid())
	{
		DragTool->Render3D(View, PDI);
	}
}

/**
 * Renders the drag tool.  Does nothing if no drag tool exists.
 */
void FMarqueeDragTool::RenderDragTool(const FSceneView* View, FCanvas* Canvas)
{
	if (DragTool.IsValid())
	{
		DragTool->Render(View, Canvas);
	}
}

void FMarqueeDragTool::MakeDragTool(FEditorViewportClient* InViewportClient)
{
	DragTool.Reset();
	if (InViewportClient->IsOrtho())
	{
		DragTool = InViewportClient->MakeDragTool(EDragTool::BoxSelect);
	}
	else
	{
		DragTool = InViewportClient->MakeDragTool(EDragTool::FrustumSelect);
	}

}
