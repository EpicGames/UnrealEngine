// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "EditorDragTools.h"
#include "EditorModeTools.h"
#include "EdMode.h"
#include "Misc/FrameTime.h"
#include "Engine/Texture2D.h"
#include "MovieSceneFwd.h"

class FCanvas;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class ISequencer;
class FSequencer;
class FViewport;
struct HMovieSceneKeyProxy;
class UMovieScene3DTransformSection;
class UMovieScene3DTransformTrack;
struct FMovieSceneEvaluationTrack;
struct FMovieSceneInterrogationData;
class USequencerSettings;
class FSequencerSelectabilityTool;

// This struct wraps up functionality for creating a marquee(frustum or box) selection drag tool
// It does so based upon the type of viewport being drawn.
// It's also agnostic to any key presses or mouse button type. 
// If it's created it's assumed it will track.
struct FMarqueeDragTool
{
	FMarqueeDragTool();
	~FMarqueeDragTool() = default;

	bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport);
	bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport);
	void MakeDragTool(FEditorViewportClient* InViewportClient);
	bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale);

	bool UsingDragTool() const;
	void Render3DDragTool(const FSceneView* View, FPrimitiveDrawInterface* PDI);
	void RenderDragTool(const FSceneView* View, FCanvas* Canvas);


private:
	
	/**
	 * If there is a dragging tool being used, this will point to it.
	 * Gets newed/deleted in StartTracking/EndTracking.
	 */
	TSharedPtr<FDragTool> DragTool;

	/** Tracks whether the drag tool is in the process of being deleted (to protect against reentrancy) */
	bool bIsDeletingDragTool = false;

};


/**
 * FSequencerEdMode is the editor mode for additional drawing and handling sequencer hotkeys in the editor
 */
class FSequencerEdMode : public FEdMode
{
public:
	static const FEditorModeID EM_SequencerMode;

public:
	FSequencerEdMode();
	virtual ~FSequencerEdMode();

	/* FEdMode interface */
	virtual void Enter() override;
	virtual void Exit() override;
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override;
	virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
	virtual void Render(const FSceneView* View,FViewport* Viewport,FPrimitiveDrawInterface* PDI) override;
	virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	virtual bool UsesTransformWidget() const override { return false; }
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override { return false; }

	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool GetCursor(EMouseCursor::Type& OutCursor) const override;
	virtual bool MouseMove(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InX, int32 InY) override;
	virtual bool ProcessCapturedMouseMoves(FEditorViewportClient* InViewportClient, FViewport* InViewport, const TArrayView<FIntPoint>& CapturedMouseMoves) override;
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy *InHitProxy, const FViewportClick &InClick) override;
	virtual bool BoxSelect(FBox& InBox, bool InSelect) override;
	virtual bool FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect) override;
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	virtual void Tick(FEditorViewportClient* ViewportClient,float DeltaTime) override;
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }

	bool IsPressingMoveTimeSlider(FViewport* InViewport) const;
	bool IsDoingDrag(FViewport* InViewport) const;
	bool IsMovingCamera(FViewport* InViewport) const;
	void AddSequencer(TWeakPtr<FSequencer> InSequencer) { Sequencers.AddUnique(InSequencer); }
	void RemoveSequencer(TWeakPtr<FSequencer> InSequencer) { Sequencers.Remove(InSequencer); }
	USequencerSettings* GetSequencerSettings() const;
	void OnSequencerReceivedFocus(TWeakPtr<FSequencer> InSequencer) { Sequencers.Sort([=](TWeakPtr<FSequencer> A, TWeakPtr<FSequencer> B){ return A == InSequencer; }); }

	void OnKeySelected(FViewport* Viewport, HMovieSceneKeyProxy* KeyProxy);

	bool IsViewportSelectionLimited() const;

	void EnableSelectabilityTool(const bool bInEnabled);

	bool IsObjectSelectableInViewport(UObject* const InObject) const;

	TSharedPtr<ISequencer> GetFirstActiveSequencer() const;

protected:
	void DrawTracks3D(FPrimitiveDrawInterface* PDI);
	void DrawTransformTrack(const TSharedPtr<ISequencer>& Sequencer, FPrimitiveDrawInterface* PDI, UMovieScene3DTransformTrack* TransformTrack, TArrayView<const TWeakObjectPtr<>> BoundObjects, const bool bIsSelected);
	void DrawAudioTracks(FPrimitiveDrawInterface* PDI);

private:
	TArray<TWeakPtr<FSequencer>> Sequencers;

	/** Interrogation data for extracting transforms */
	TSharedPtr<FMovieSceneInterrogationData> InterrogationData;

	/** The audio texture used for drawing the audio spatialization points */
	UTexture2D* AudioTexture;

	//params to handle mouse move for changing time
	/** If we are tracking */
	bool bIsTracking = false;

	/** Starting X Value*/
	TOptional<int32> StartXValue;
	/** Starting Time Value*/
	FFrameNumber StartFrameNumber;

	FMarqueeDragTool DragToolHandler;

	/** If the pivot location needs to be updated */
	bool bUpdatePivot = false;

	/**if dragging time need to reset to what it was */
	EMovieScenePlayerStatus::Type NextPlayerStatus; 

	TSharedPtr<class FSequencerEdModeTool> DefaultTool;
	TSharedPtr<FSequencerSelectabilityTool> SelectabilityTool;
};

/**
 * FSequencerEdMode is the editor mode tool for additional drawing and handling sequencer hotkeys in the editor
 */
class FSequencerEdModeTool : public FModeTool
{
public:
	FSequencerEdModeTool(FSequencerEdMode* InSequencerEdMode);
	virtual ~FSequencerEdModeTool();

	virtual FString GetName() const override { return TEXT("Sequencer Edit"); }

	/**
	 * @return		true if the key was handled by this editor mode tool.
	 */
	virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;

private:
	FSequencerEdMode* SequencerEdMode;
};
