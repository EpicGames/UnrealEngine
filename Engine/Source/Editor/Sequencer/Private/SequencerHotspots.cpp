// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerHotspots.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/ITimeDomainExtension.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/Selection/Selection.h"
#include "SequencerCommonHelpers.h"
#include "SequencerSettings.h"
#include "SSequencer.h"
#include "Tools/EditToolDragOperations.h"
#include "SequencerContextMenus.h"
#include "SequencerNodeTree.h"
#include "MVVM/Views/STrackAreaView.h"
#include "Tools/SequencerEditTool_Movement.h"
#include "Tools/SequencerEditTool_Selection.h"
#include "Tools/SequencerSnapField.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneTimeHelpers.h"
#include "SequencerCommonHelpers.h"
#include "ISequencerSection.h"
#include "ISequencerDecorationEditor.h"
#include "MovieSceneSignedObject.h"
#include "Decorations/MovieSceneDecorationContainer.h"

#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/VirtualTrackArea.h"

#define LOCTEXT_NAMESPACE "SequencerHotspots"

// Iterate section decorations and let registered decoration editors
// contribute to the context menu.
static void BuildDecorationContextMenu(
	FMenuBuilder& MenuBuilder,
	UMovieSceneSection* Section,
	const FGuid& ObjectBinding,
	TWeakPtr<ISequencer> Sequencer)
{
	if (!Section)
	{
		return;
	}

	TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
	if (!SequencerPtr)
	{
		return;
	}

	for (TObjectPtr<UObject> DecorationObject : Section->GetDecorations())
	{
		if (!DecorationObject)
		{
			continue;
		}

		if (ISequencerDecorationEditor* Editor = SequencerPtr->FindDecorationEditor(DecorationObject->GetClass()))
		{
			Editor->BuildSectionContextMenu(MenuBuilder, *Section, *DecorationObject, ObjectBinding, Sequencer);
		}
	}
}

namespace UE
{
namespace Sequencer
{

UE_SEQUENCER_DEFINE_CASTABLE(FKeyHotspot);
UE_SEQUENCER_DEFINE_CASTABLE(FKeyBarHotspot);
UE_SEQUENCER_DEFINE_CASTABLE(FSectionEasingAreaHotspot);
UE_SEQUENCER_DEFINE_CASTABLE(FSectionEasingHandleHotspot);
UE_SEQUENCER_DEFINE_CASTABLE(FSectionHotspot);
UE_SEQUENCER_DEFINE_CASTABLE(FSectionHotspotBase);
UE_SEQUENCER_DEFINE_CASTABLE(FSectionResizeHotspot);

UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IMouseHandlerHotspot);

FHotspotSelectionManager::FHotspotSelectionManager(const FPointerEvent* InMouseEvent, FSequencer* InSequencer)
	: MouseEvent(InMouseEvent)
	, Selection(InSequencer->GetViewModel()->GetSelection())
	, Sequencer(InSequencer)
{
	EventSuppressor = Selection->SuppressEventsLongRunning();

	bForceSelect = !MouseEvent->IsControlDown();
	bAddingToSelection = MouseEvent->IsShiftDown() || MouseEvent->IsControlDown();

	if (MouseEvent->GetEffectingButton() != EKeys::RightMouseButton)
	{
		// When single-clicking without the RMB, we always wipe the current selection
		ConditionallyClearSelection();
	}
}

FHotspotSelectionManager::~FHotspotSelectionManager()
{
	// Broadcast selection events by destroying the suppressor
	EventSuppressor = nullptr;
}

void FHotspotSelectionManager::ConditionallyClearSelection()
{
	if (!bAddingToSelection)
	{
		Selection->TrackArea.Empty();
		Selection->KeySelection.Empty();
		Selection->MarkedFrames.Empty();

		bAddingToSelection = true;
	}
}

void FHotspotSelectionManager::ToggleKeys(TArrayView<const FSequencerSelectedKey> InKeys)
{
	for (const FSequencerSelectedKey& Key : InKeys)
	{
		const bool bIsSelected = Selection->KeySelection.IsSelected(Key.KeyHandle);
		if (bIsSelected && bForceSelect)
		{
			continue;
		}

		if (!bIsSelected)
		{
			Selection->KeySelection.Select(Key.WeakChannel.Pin(), Key.KeyHandle);
		}
		else
		{
			Selection->KeySelection.Deselect(Key.KeyHandle);
		}
	}
}

void FHotspotSelectionManager::ToggleModel(TSharedPtr<FViewModel> InModel)
{
	const bool bIsSelected = Selection->TrackArea.IsSelected(InModel);
	if (bIsSelected && bForceSelect)
	{
		return;
	}

	TSharedPtr<ISelectableExtension> Selectable = InModel->CastThisShared<ISelectableExtension>();
	if (!Selectable)
	{
		return;
	}
	else if (MouseEvent->GetEffectingButton() == EKeys::RightMouseButton && !EnumHasAnyFlags(Selectable->IsSelectable(), ESelectionIntent::ContextMenu))
	{
		return;
	}
	else if (MouseEvent->GetEffectingButton() == EKeys::LeftMouseButton && !EnumHasAnyFlags(Selectable->IsSelectable(), ESelectionIntent::PersistentSelection))
	{
		return;
	}

	if (!bIsSelected)
	{
		Selection->TrackArea.Select(InModel);
	}
	else
	{
		Selection->TrackArea.Deselect(InModel);
	}
}

void FHotspotSelectionManager::SelectKeysExclusive(TArrayView<const FSequencerSelectedKey> InKeys)
{
	for (const FSequencerSelectedKey& Key : InKeys)
	{
		if (!Selection->KeySelection.IsSelected(Key.KeyHandle))
		{
			ConditionallyClearSelection();
			Selection->KeySelection.Select(Key.WeakChannel.Pin(), Key.KeyHandle);
		}
	}
}
void FHotspotSelectionManager::DefaultModelSelection(TSharedPtr<FViewModel> InModel)
{
	if (bForceSelect)
	{
		SelectModelExclusive(InModel);
	}
	else if (MouseEvent->IsControlDown())
	{
		ToggleModel(InModel);
	}
	else if (MouseEvent->IsShiftDown())
	{
		Selection->TrackArea.Select(InModel);
	}
}

void FHotspotSelectionManager::SelectModelExclusive(TSharedPtr<FViewModel> InModel)
{
	if (!Selection->TrackArea.IsSelected(InModel))
	{
		ConditionallyClearSelection();
		Selection->TrackArea.Select(InModel);
	}
}

FKeyHotspot::FKeyHotspot(const TArray<FSequencerSelectedKey>& InKeys, TWeakPtr<FSequencer> InWeakSequencer)
	: Keys(InKeys)
	, WeakSequencer(InWeakSequencer)
{ 
	RawKeys.Reserve(Keys.Num());
	for (const FSequencerSelectedKey& Key : InKeys)
	{
		RawKeys.Add(Key.KeyHandle);
	}
}

void FKeyHotspot::HandleMouseSelection(FHotspotSelectionManager& SelectionManager)
{
	if (SelectionManager.MouseEvent->GetEffectingButton() == EKeys::RightMouseButton)
	{
		SelectionManager.SelectKeysExclusive(Keys.Array());
	}
	else
	{
		// On mouse click, select only keys that are unique to a section and channel, ie. don't select multiple keys on the same channel that have the same time
		TArray<FSequencerSelectedKey> UniqueKeysArray;
		for (const FSequencerSelectedKey& Key : Keys)
		{
			if (Key.IsValid())
			{
				bool bFoundKey = false;
				for (const FSequencerSelectedKey& UniqueKey : UniqueKeysArray)
				{
					if (Key.Section == UniqueKey.Section && Key.WeakChannel == UniqueKey.WeakChannel)
					{
						bFoundKey = true;
						break;
					}
				}

				if (!bFoundKey)
				{
					UniqueKeysArray.Add(Key);
				}
			}
		}

		SelectionManager.ToggleKeys(UniqueKeysArray);
	}
}

void FKeyHotspot::UpdateOnHover(FTrackAreaViewModel& InTrackArea) const
{
	InTrackArea.AttemptToActivateTool(FSequencerEditTool_Movement::Identifier);
}

TOptional<ETimeDomain> FKeyHotspot::GetDomain() const
{
	TOptional<ETimeDomain> Domain;

	TSet<const FChannelModel*> VisitedChannels;
	for (const FSequencerSelectedKey& Key : Keys)
	{
		TSharedPtr<FChannelModel> Channel = Key.WeakChannel.Pin();
		if (!Channel || VisitedChannels.Contains(Channel.Get()))
		{
			continue;
		}

		VisitedChannels.Add(Channel.Get());

		TViewModelPtr<ITimeDomainExtension> TimeDomain = Channel->FindAncestorOfType<ITimeDomainExtension>(true);
		if (TimeDomain)
		{
			if (Domain && Domain != TimeDomain->GetDomain())
			{
				return TOptional<ETimeDomain>();
			}

			Domain = TimeDomain->GetDomain();
		}
	}

	if (!Domain)
	{
		Domain = ETimeDomain::Warped;
	}
	return Domain;
}

TOptional<FFrameNumber> FKeyHotspot::GetTime() const
{
	FFrameNumber Time = 0;

	if (auto It = Keys.CreateConstIterator(); It)
	{
		const FSequencerSelectedKey& Key = *It;
		TArrayView<const FSequencerSelectedKey> FirstKey(&Key, 1);
		TArrayView<FFrameNumber> FirstKeyTime(&Time, 1);
		GetKeyTimes(FirstKey, FirstKeyTime);
	}

	return Time;
}

bool FKeyHotspot::PopulateContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, FFrameTime MouseDownTime)
{
	FKeyContextMenu::BuildMenu(MenuBuilder, MenuExtender, WeakSequencer);
	return true;
}

void FKeyBarHotspot::UpdateOnHover(FTrackAreaViewModel& InTrackArea) const
{
}
TOptional<FFrameNumber> FKeyBarHotspot::GetTime() const
{
	FFrameNumber Time = 0;

	if (LeadingKeys.Num())
	{
		TArrayView<const FSequencerSelectedKey> FirstKey(&LeadingKeys.Last(), 1);
		TArrayView<FFrameNumber> FirstKeyTime(&Time, 1);
		GetKeyTimes(FirstKey, FirstKeyTime);
	}

	return Time;
}

bool FKeyBarHotspot::PopulateContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, FFrameTime MouseDownTime)
{
	TArrayView<const FSequencerSelectedKey> FirstKey(&LeadingKeys.Last(), 1);

	if (FirstKey.Num() == 0 || !FirstKey[0].Section)
	{
		return false;
	}

	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();

	TSharedPtr<FSectionModel> SectionModel = Sequencer->GetNodeTree()->GetSectionModel(FirstKey[0].Section);
	if (!SectionModel)
	{
		return false;
	}
	
	FSectionContextMenu::BuildMenu(MenuBuilder, MenuExtender, WeakSequencer, MouseDownTime);

	TSharedPtr<IObjectBindingExtension> ObjectBinding = SectionModel->FindAncestorOfType<IObjectBindingExtension>();
	FGuid ObjectBindingGuid = ObjectBinding ? ObjectBinding->GetObjectGuid() : FGuid();
	SectionModel->GetSectionInterface()->BuildSectionContextMenu(MenuBuilder, ObjectBindingGuid);
	BuildDecorationContextMenu(MenuBuilder, SectionModel->GetSection(), ObjectBindingGuid, Sequencer);

	return true;
}

FCursorReply FKeyBarHotspot::GetCursor() const
{
	return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
}

void FKeyBarHotspot::HandleMouseSelection(FHotspotSelectionManager& SelectionManager)
{
	TArrayView<const FSequencerSelectedKey> FirstKey(&LeadingKeys.Last(), 1);

	if (FirstKey.Num() == 0 || !FirstKey[0].Section)
	{
		return;
	}

	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();

	TSharedPtr<FSectionModel> SectionModel = Sequencer->GetNodeTree()->GetSectionModel(FirstKey[0].Section);
	if (!SectionModel)
	{
		return;
	}

	// Base-class only handles RMB selection so that the other handles and interactive controls
	// that act as hotspots and still operate correctly with Left click
	if (SectionModel && SelectionManager.MouseEvent->GetEffectingButton() == EKeys::RightMouseButton)
	{
		SelectionManager.SelectModelExclusive(SectionModel);
	}
}

TSharedPtr<ISequencerEditToolDragOperation> FKeyBarHotspot::InitiateDrag(const FPointerEvent& MouseEvent)
{
	struct FKeyBarDrag : ISequencerEditToolDragOperation, UE::Sequencer::ISnapCandidate
	{
		FFrameTime RelativeStartTime;

		TArray<FFrameTime> RelativeStartTimes;
		TArray<FSequencerSelectedKey> AllLinearKeys;

		TSet<FSequencerSelectedKey> AllKeys;

		TSharedPtr<FKeyBarHotspot> Hotspot;
		TUniquePtr<FScopedTransaction> Transaction;
		TSet<UObject*> ModifiedObjects;
		FSequencerSnapField SnapField;

		FKeyBarDrag(TSharedPtr<FKeyBarHotspot> InHotspot, TSharedPtr<FSequencer> InSequencer)
			: Hotspot(InHotspot)
		{
			AllKeys.Append(Hotspot->LeadingKeys);
			AllKeys.Append(Hotspot->TrailingKeys);
			FSequencerSelectedKey::AppendKeySelection(AllKeys, InSequencer->GetViewModel()->GetSelection()->KeySelection);

			AllLinearKeys = AllKeys.Array();

			SnapField = FSequencerSnapField(*InSequencer, *this);
			SnapField.SetSnapToInterval(InSequencer->GetSequencerSettings()->GetForceWholeFrames());
		}

		void OnBeginDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const FVirtualTrackArea& VirtualTrackArea) override
		{
			RelativeStartTime = VirtualTrackArea.PixelToFrame(LocalMousePos.X);

			// Cache off the starting times
			TArray<FFrameNumber> AbsoluteStartTimes;
			AbsoluteStartTimes.SetNumUninitialized(AllLinearKeys.Num());
			RelativeStartTimes.SetNumUninitialized(AllLinearKeys.Num());

			// Make the times relative to the initial drag position
			GetKeyTimes(AllLinearKeys, AbsoluteStartTimes);
			for (int32 Index = 0; Index < AbsoluteStartTimes.Num() ; ++Index)
			{
				RelativeStartTimes[Index] = (AbsoluteStartTimes[Index] - RelativeStartTime);
			}

			Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("DragKeyBarTransaction", "Move Keys"));

			for (const FSequencerSelectedKey& Key : AllLinearKeys)
			{
				TSharedPtr<FChannelModel> Channel = Key.WeakChannel.Pin();
				UObject* OwningObject = Channel ? Channel->GetOwningObject() : nullptr;

				if (OwningObject && !ModifiedObjects.Contains(OwningObject))
				{
					ModifiedObjects.Add(OwningObject);
				}
			}
		}
		void OnDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const FVirtualTrackArea& VirtualTrackArea) override
		{
			UE::MovieScene::FScopedSignedObjectModifyDefer Defer(true);

			const FFrameTime NewTime = VirtualTrackArea.PixelToFrame(LocalMousePos.X);

			for (UObject* Object : ModifiedObjects)
			{
				Object->Modify();
			}

			// Set the position of leading and trailing keys
			TArray<FFrameTime> NewKeyTimes;
			NewKeyTimes.SetNumUninitialized(RelativeStartTimes.Num());

			for (int32 Index = 0; Index < RelativeStartTimes.Num(); ++Index)
			{
				NewKeyTimes[Index] = NewTime + RelativeStartTimes[Index];
			}

			if (Hotspot.IsValid() && Hotspot->WeakSequencer.IsValid() && Hotspot->WeakSequencer.Pin()->GetSequencerSettings()->GetIsSnapEnabled())
			{
				const float PixelSnapWidth = 20.f;
				const int32 SnapThreshold = VirtualTrackArea.PixelDeltaToFrame(PixelSnapWidth).FloorToFrame().Value;
				if (TOptional<FSequencerSnapField::FSnapResult> SnappedTime = SnapField.Snap(NewKeyTimes, SnapThreshold))
				{
					FFrameTime SnappedDiff = SnappedTime->SnappedTime - SnappedTime->OriginalTime;
					for (int32 Index = 0; Index < RelativeStartTimes.Num(); ++Index)
					{
						NewKeyTimes[Index] = NewKeyTimes[Index] + SnappedDiff;
					}
				}
			}

			TArray<FFrameNumber> NewKeyFrames;
			NewKeyFrames.SetNumUninitialized(NewKeyTimes.Num());
			for (int32 Index = 0; Index < NewKeyTimes.Num(); ++Index)
			{
				NewKeyFrames[Index] = NewKeyTimes[Index].RoundToFrame();
			}
			SetKeyTimes(AllLinearKeys, NewKeyFrames);
		}
		void OnEndDrag( const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const FVirtualTrackArea& VirtualTrackArea) override
		{
			Transaction.Reset();
		}
		FCursorReply GetCursor() const override
		{
			return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
		}
		int32 OnPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const override
		{
			return LayerId;
		}
		bool IsKeyApplicable(FKeyHandle KeyHandle, const UE::Sequencer::FViewModelPtr& Owner) const override
		{
			using namespace UE::Sequencer;

			TSharedPtr<FChannelModel> Channel = Owner.ImplicitCast();
			return Channel && !AllKeys.Contains(FSequencerSelectedKey(*Channel->GetSection(), Channel, KeyHandle));
		}
		bool AreSectionBoundsApplicable(UMovieSceneSection* Section) const override
		{
			return true;
		}
	};

	return MakeShared<FKeyBarDrag>(SharedThis(this), WeakSequencer.Pin());
}

UMovieSceneSection* FSectionHotspotBase::GetSection() const
{
	if (TSharedPtr<FSectionModel> Model = WeakSectionModel.Pin())
	{
		return Model->GetSection();
	}
	return nullptr;
}

void FSectionHotspotBase::HandleMouseSelection(FHotspotSelectionManager& SelectionManager)
{
	// Base-class only handles RMB selection so that the other handles and interactive controls
	// that act as hotspots and still operate correctly with Left click
	TSharedPtr<FSectionModel> Section = WeakSectionModel.Pin();
	if (Section && SelectionManager.MouseEvent->GetEffectingButton() == EKeys::RightMouseButton)
	{
		SelectionManager.SelectModelExclusive(Section);
	}
}

TOptional<FFrameNumber> FSectionHotspotBase::GetTime() const
{
	UMovieSceneSection* ThisSection = GetSection();
	return ThisSection && ThisSection->HasStartFrame() ? ThisSection->GetInclusiveStartFrame() : TOptional<FFrameNumber>();
}

TOptional<FFrameTime> FSectionHotspotBase::GetOffsetTime() const
{
	UMovieSceneSection* ThisSection = GetSection();
	return ThisSection ? ThisSection->GetOffsetTime() : TOptional<FFrameTime>();
}

void FSectionHotspotBase::UpdateOnHover(FTrackAreaViewModel& InTrackArea) const
{
	UMovieSceneSection* ThisSection = GetSection();

	// Move sections if they are selected
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	if (Sequencer->GetViewModel()->GetSelection()->TrackArea.IsSelected(WeakSectionModel))
	{
		if (ThisSection && ThisSection->GetRange() != TRange<FFrameNumber>::All())
		{
			InTrackArea.AttemptToActivateTool(FSequencerEditTool_Movement::Identifier);
		}
	}
	else if (ThisSection)
	{
		// Check if the section interface wants to show key areas (and thus supports key selection)
		TSharedPtr<FSectionModel> SectionModel = WeakSectionModel.Pin();
		TSharedPtr<ISequencerSection> SectionInterface = SectionModel ? SectionModel->GetSectionInterface() : nullptr;
		const bool bShowKeyArea = !SectionInterface || SectionInterface->ShouldShowKeyAreaWhenCollapsed();

		// Activate selection mode if the section has keys and wants to show them
		if (bShowKeyArea)
		{
			for (const FMovieSceneChannelEntry& Entry : ThisSection->GetChannelProxy().GetAllEntries())
			{
				for (const FMovieSceneChannel* Channel : Entry.GetChannels())
				{
					if (Channel && Channel->GetNumKeys() != 0)
					{
						InTrackArea.AttemptToActivateTool(FSequencerEditTool_Selection::Identifier);
						return;
					}
				}
			}
		}

		if (ThisSection->GetRange() != TRange<FFrameNumber>::All())
		{
			InTrackArea.AttemptToActivateTool(FSequencerEditTool_Movement::Identifier);
		}
	}
}

bool FSectionHotspotBase::PopulateContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, FFrameTime MouseDownTime)
{
	TSharedPtr<FSectionModel> SectionModel = WeakSectionModel.Pin();
	UMovieSceneSection*       ThisSection  = SectionModel ? SectionModel->GetSection() : nullptr;
	if (ThisSection)
	{
		FSectionContextMenu::BuildMenu(MenuBuilder, MenuExtender, WeakSequencer, MouseDownTime);

		TSharedPtr<IObjectBindingExtension> ObjectBinding = SectionModel->FindAncestorOfType<IObjectBindingExtension>();
		FGuid ObjectBindingGuid = ObjectBinding ? ObjectBinding->GetObjectGuid() : FGuid();
		SectionModel->GetSectionInterface()->BuildSectionContextMenu(MenuBuilder, ObjectBindingGuid);
		BuildDecorationContextMenu(MenuBuilder, ThisSection, ObjectBindingGuid, WeakSequencer.Pin());
	}

	return true;
}

void FSectionHotspot::HandleMouseSelection(FHotspotSelectionManager& SelectionManager)
{
	TSharedPtr<FSectionModel> Section = WeakSectionModel.Pin();
	if (Section && SelectionManager.MouseEvent->GetEffectingButton() == EKeys::LeftMouseButton)
	{
		SelectionManager.ToggleModel(Section);
	}
	else
	{
		FSectionHotspotBase::HandleMouseSelection(SelectionManager);
	}
}

TOptional<FFrameNumber> FSectionResizeHotspot::GetTime() const
{
	UMovieSceneSection* ThisSection = GetSection();
	if (!ThisSection)
	{
		return TOptional<FFrameNumber>();
	}
	return HandleType == Left ? ThisSection->GetInclusiveStartFrame() : ThisSection->GetExclusiveEndFrame();
}

void FSectionResizeHotspot::UpdateOnHover(FTrackAreaViewModel& InTrackArea) const
{
	InTrackArea.AttemptToActivateTool(FSequencerEditTool_Movement::Identifier);
}

TSharedPtr<ISequencerEditToolDragOperation> FSectionResizeHotspot::InitiateDrag(const FPointerEvent& MouseEvent)
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	TSharedPtr<FSequencerSelection> Selection = Sequencer->GetViewModel()->GetSelection();

	if (!Selection->TrackArea.IsSelected(WeakSectionModel))
	{
		FSelectionEventSuppressor SuppressEvents = Selection->SuppressEvents();

		Selection->Empty();
		Selection->TrackArea.Select(WeakSectionModel);
	}

	const bool bIsSlipping = false;
	return MakeShareable( new FResizeSection(*Sequencer, HandleType == Right, bIsSlipping) );
}

const FSlateBrush* FSectionResizeHotspot::GetCursorDecorator(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	if (CursorEvent.IsControlDown())
	{
		return FAppStyle::Get().GetBrush(TEXT("Sequencer.CursorDecorator_Retime"));
	}
	else
	{
		return ITrackAreaHotspot::GetCursorDecorator(MyGeometry, CursorEvent);
	}
}

TOptional<FFrameNumber> FSectionEasingHandleHotspot::GetTime() const
{
	UMovieSceneSection* ThisSection = GetSection();
	if (ThisSection)
	{
		if (HandleType == ESequencerEasingType::In && !ThisSection->GetEaseInRange().IsEmpty())
		{
			return ThisSection->GetEaseInRange().GetUpperBoundValue();
		}
		else if (HandleType == ESequencerEasingType::Out && !ThisSection->GetEaseOutRange().IsEmpty())
		{
			return ThisSection->GetEaseOutRange().GetLowerBoundValue();
		}
	}
	return TOptional<FFrameNumber>();
}

void FSectionEasingHandleHotspot::UpdateOnHover(FTrackAreaViewModel& InTrackArea) const
{
	InTrackArea.AttemptToActivateTool(FSequencerEditTool_Movement::Identifier);
}

bool FSectionEasingHandleHotspot::PopulateContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, FFrameTime MouseDownTime)
{
	FEasingContextMenu::BuildMenu(MenuBuilder, MenuExtender, { FEasingAreaHandle{WeakSectionModel, HandleType} }, WeakSequencer, MouseDownTime);
	return true;
}

TSharedPtr<ISequencerEditToolDragOperation> FSectionEasingHandleHotspot::InitiateDrag(const FPointerEvent& MouseEvent)
{
	UMovieSceneSection* Section = GetSection();
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	return MakeShareable( new FManipulateSectionEasing(*static_cast<FSequencer*>(Sequencer.Get()), Section, HandleType == ESequencerEasingType::In) );
}

const FSlateBrush* FSectionEasingHandleHotspot::GetCursorDecorator(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	return FAppStyle::Get().GetBrush(TEXT("Sequencer.CursorDecorator_EasingHandle"));
}

bool FSectionEasingAreaHotspot::PopulateContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, FFrameTime MouseDownTime)
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	FEasingContextMenu::BuildMenu(MenuBuilder, MenuExtender, Easings, WeakSequencer, MouseDownTime);

	TSharedPtr<FSectionModel> SectionModel = WeakSectionModel.Pin();
	UMovieSceneSection*       ThisSection  = SectionModel ? SectionModel->GetSection() : nullptr;
	if (ThisSection)
	{
		TSharedPtr<IObjectBindingExtension> ObjectBinding = SectionModel->FindAncestorOfType<IObjectBindingExtension>();
		FGuid ObjectBindingGuid = ObjectBinding ? ObjectBinding->GetObjectGuid() : FGuid();
		SectionModel->GetSectionInterface()->BuildSectionContextMenu(MenuBuilder, ObjectBindingGuid);
		BuildDecorationContextMenu(MenuBuilder, ThisSection, ObjectBindingGuid, Sequencer);
	}

	return true;
}

void FSectionEasingAreaHotspot::HandleMouseSelection(FHotspotSelectionManager& SelectionManager)
{
	TSharedPtr<FSectionModel> Section = WeakSectionModel.Pin();
	if (Section && SelectionManager.MouseEvent->GetEffectingButton() == EKeys::LeftMouseButton)
	{
		SelectionManager.ToggleModel(Section);
	}
	else
	{
		FSectionHotspotBase::HandleMouseSelection(SelectionManager);
	}
}

bool FSectionEasingAreaHotspot::Contains(UMovieSceneSection* InSection) const
{
	return Easings.ContainsByPredicate([=](const FEasingAreaHandle& InHandle){ return InHandle.WeakSectionModel.IsValid() && InHandle.WeakSectionModel.Pin()->GetSection() == InSection; });
}

} // namespace Sequencer
} // namespace UE

#undef LOCTEXT_NAMESPACE
