// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/CursorReply.h"
#include "Styling/SlateBrush.h"
#include "SequencerSelectedKey.h"
#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "MVVM/Views/ITrackAreaHotspot.h"
#include "Sequencer.h"

class FMenuBuilder;
class FSequencerTrackNode;
class ISequencerSection;

namespace UE
{
namespace Sequencer
{

struct FSelectionEventSuppressor;

class FSectionModel;
class ISequencerEditToolDragOperation;

struct FHotspotSelectionManager
{
	const FPointerEvent* MouseEvent;
	TSharedPtr<FSequencerSelection> Selection;
	FSequencer* Sequencer;
	TUniquePtr<FSelectionEventSuppressor> EventSuppressor;

	bool bForceSelect;
	bool bAddingToSelection;

	FHotspotSelectionManager(const FPointerEvent* InMouseEvent, FSequencer* InSequencer);
	~FHotspotSelectionManager();

	void ConditionallyClearSelection();

	void ToggleKeys(TArrayView<const FSequencerSelectedKey> InKeys);
	void ToggleModel(TSharedPtr<FViewModel> InModel);
	void DefaultModelSelection(TSharedPtr<FViewModel> InModel);

	void SelectKeysExclusive(TArrayView<const FSequencerSelectedKey> InKeys);
	void SelectModelExclusive(TSharedPtr<FViewModel> InModel);
};

struct IMouseHandlerHotspot
{
	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID(IMouseHandlerHotspot);

	virtual ~IMouseHandlerHotspot()= default;

	virtual void HandleMouseSelection(FHotspotSelectionManager& SelectionManager) = 0;
};

enum class ESequencerEasingType
{
	In, Out
};

/** A hotspot representing a key */
struct FKeyHotspot
	: ITrackAreaHotspot, IMouseHandlerHotspot
{
	UE_SEQUENCER_DECLARE_CASTABLE(FKeyHotspot, ITrackAreaHotspot, IMouseHandlerHotspot);

	SEQUENCER_API FKeyHotspot(const TArray<FSequencerSelectedKey>& InKeys, TWeakPtr<FSequencer> InWeakSequencer);

	virtual void UpdateOnHover(FTrackAreaViewModel& InTrackArea) const override;
	virtual TOptional<ETimeDomain> GetDomain() const;
	virtual TOptional<FFrameNumber> GetTime() const override;
	virtual bool PopulateContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, FFrameTime MouseDownTime) override;

	virtual void HandleMouseSelection(FHotspotSelectionManager& SelectionManager) override;

	/** The keys that are part of this hotspot */
	TSet<FSequencerSelectedKey> Keys;
	TSet<FKeyHandle> RawKeys;
	TWeakPtr<FSequencer> WeakSequencer;
};

/** A hotspot representing a key bar */
struct FKeyBarHotspot
	: ITrackAreaHotspot, IMouseHandlerHotspot
	, TSharedFromThis<FKeyBarHotspot>

{
	UE_SEQUENCER_DECLARE_CASTABLE(FKeyBarHotspot, ITrackAreaHotspot, IMouseHandlerHotspot);

	FKeyBarHotspot(const TRange<FFrameTime>& InRange, TArray<FSequencerSelectedKey>&& InLeadingKeys, TArray<FSequencerSelectedKey>&& InTrailingKeys, TWeakPtr<FSequencer> InWeakSequencer)
		: LeadingKeys(MoveTemp(InLeadingKeys))
		, TrailingKeys(MoveTemp(InTrailingKeys))
		, WeakSequencer(InWeakSequencer)
		, Range(InRange)
	{ }

	virtual void UpdateOnHover(FTrackAreaViewModel& InTrackArea) const override;
	virtual TOptional<FFrameNumber> GetTime() const override;
	virtual bool PopulateContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, FFrameTime MouseDownTime) override;
	virtual TSharedPtr<ISequencerEditToolDragOperation> InitiateDrag(const FPointerEvent& MouseEvent) override;
	virtual FCursorReply GetCursor() const override;

	/** IMouseHandlerHotspot */
	virtual void HandleMouseSelection(FHotspotSelectionManager& SelectionManager) override;

	/** The keys that are part of this hotspot */
	TArray<FSequencerSelectedKey> LeadingKeys;
	TArray<FSequencerSelectedKey> TrailingKeys;
	TWeakPtr<FSequencer> WeakSequencer;
	TRange<FFrameTime> Range;
};

/** A hotspot representing a section */
struct FSectionHotspotBase
	: ITrackAreaHotspot, IMouseHandlerHotspot
{
	UE_SEQUENCER_DECLARE_CASTABLE(FSectionHotspotBase, ITrackAreaHotspot, IMouseHandlerHotspot);

	FSectionHotspotBase(TWeakPtr<FSectionModel> InSectionModel, TWeakPtr<FSequencer> InWeakSequencer)
		: WeakSectionModel(InSectionModel)
		, WeakSequencer(InWeakSequencer)
	{ }

	virtual void UpdateOnHover(FTrackAreaViewModel& InTrackArea) const override;
	virtual TOptional<FFrameNumber> GetTime() const override;
	virtual TOptional<FFrameTime> GetOffsetTime() const override;
	virtual TSharedPtr<ISequencerEditToolDragOperation> InitiateDrag(const FPointerEvent& MouseEvent) override { return nullptr; }
	virtual bool PopulateContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, FFrameTime MouseDownTime) override;

	/** IMouseHandlerHotspot */
	virtual void HandleMouseSelection(FHotspotSelectionManager& SelectionManager) override;

	UMovieSceneSection* GetSection() const;

	/** The section model */
	TWeakPtr<FSectionModel> WeakSectionModel;
	TWeakPtr<FSequencer> WeakSequencer;
};

/** A hotspot representing a section */
struct FSectionHotspot : FSectionHotspotBase
{
	UE_SEQUENCER_DECLARE_CASTABLE(FSectionHotspot, FSectionHotspotBase);

	FSectionHotspot(TWeakPtr<FSectionModel> InSectionModel, TWeakPtr<FSequencer> InWeakSequencer)
		: FSectionHotspotBase(InSectionModel, InWeakSequencer)
	{ }

	/** IMouseHandlerHotspot */
	virtual void HandleMouseSelection(FHotspotSelectionManager& SelectionManager) override;
};


/** A hotspot representing a resize handle on a section */
struct FSectionResizeHotspot : FSectionHotspotBase
{
	UE_SEQUENCER_DECLARE_CASTABLE(FSectionResizeHotspot, FSectionHotspotBase);

	enum EHandle
	{
		Left,
		Right
	};

	FSectionResizeHotspot(EHandle InHandleType, TWeakPtr<FSectionModel> InSectionModel, TWeakPtr<FSequencer> InWeakSequencer)
		: FSectionHotspotBase(InSectionModel, InWeakSequencer)
		, HandleType(InHandleType)
	{}

	virtual void UpdateOnHover(FTrackAreaViewModel& InTrackArea) const override;
	virtual TOptional<FFrameNumber> GetTime() const override;
	virtual TSharedPtr<ISequencerEditToolDragOperation> InitiateDrag(const FPointerEvent& MouseEvent) override;
	virtual FCursorReply GetCursor() const { return FCursorReply::Cursor( EMouseCursor::ResizeLeftRight ); }
	virtual const FSlateBrush* GetCursorDecorator(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const;

	EHandle HandleType;
};


/** A hotspot representing a resize handle on a section's easing */
struct FSectionEasingHandleHotspot : FSectionHotspotBase
{
	UE_SEQUENCER_DECLARE_CASTABLE(FSectionEasingHandleHotspot, FSectionHotspotBase);

	FSectionEasingHandleHotspot(ESequencerEasingType InHandleType, TWeakPtr<FSectionModel> InSectionModel, TWeakPtr<FSequencer> InWeakSequencer)
		: FSectionHotspotBase(InSectionModel, InWeakSequencer)
		, HandleType(InHandleType)
	{}

	virtual void UpdateOnHover(FTrackAreaViewModel& InTrackArea) const override;
	virtual bool PopulateContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, FFrameTime MouseDownTime) override;
	virtual TOptional<FFrameNumber> GetTime() const override;
	virtual TSharedPtr<ISequencerEditToolDragOperation> InitiateDrag(const FPointerEvent& MouseEvent) override;
	virtual FCursorReply GetCursor() const { return FCursorReply::Cursor( EMouseCursor::ResizeLeftRight ); }
	virtual const FSlateBrush* GetCursorDecorator(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const;

	ESequencerEasingType HandleType;
};


struct FEasingAreaHandle
{
	TWeakPtr<FSectionModel> WeakSectionModel;
	ESequencerEasingType EasingType;
};

/** A hotspot representing an easing area for multiple sections */
struct FSectionEasingAreaHotspot : FSectionHotspotBase
{
	UE_SEQUENCER_DECLARE_CASTABLE(FSectionEasingAreaHotspot, FSectionHotspotBase);

	FSectionEasingAreaHotspot(const TArray<FEasingAreaHandle>& InEasings, TWeakPtr<FSectionModel> InSectionModel, TWeakPtr<FSequencer> InWeakSequencer)
		: FSectionHotspotBase(InSectionModel, InWeakSequencer)
		, Easings(InEasings)
	{}

	virtual bool PopulateContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, FFrameTime MouseDownTime) override;

	/** IMouseHandlerHotspot */
	virtual void HandleMouseSelection(FHotspotSelectionManager& SelectionManager) override;

	bool Contains(UMovieSceneSection* InSection) const;

	/** Handles to the easings that exist on this hotspot */
	TArray<FEasingAreaHandle> Easings;
};


} // namespace Sequencer
} // namespace UE
