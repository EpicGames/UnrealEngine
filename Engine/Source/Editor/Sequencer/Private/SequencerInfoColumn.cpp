// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerInfoColumn.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Styling/AppStyle.h"
#include "ISceneOutliner.h"
#include "Sequencer.h"
#include "LevelEditorSequencerIntegration.h"

#include "EditorClassUtils.h"
#include "SortHelper.h"

#include "ActorTreeItem.h"
#include "SceneOutlinerHelpers.h"

#define LOCTEXT_NAMESPACE "SequencerInfoColumn"

namespace Sequencer
{

/** Functor which retrieves actor string information for sorting */
struct FGetActorInfo
{
	FGetActorInfo(const FSequencerInfoColumn& InColumn)
		: WeakColumn(StaticCastSharedRef<const FSequencerInfoColumn>(InColumn.AsShared()))
	{}

	FString operator()(const ISceneOutlinerTreeItem& Item) const
	{
		if (!WeakColumn.IsValid())
		{
			return FString();
		}

		const FSequencerInfoColumn& Column = *WeakColumn.Pin();
		if (const TWeakObjectPtr<AActor> Actor = Column.GetActorFromItem(Item); Actor.IsValid())
		{
			return Column.GetTextForActor(Actor.Get());
		}
		return FString();
	}


	/** Weak reference to the sequencer info column */
	TWeakPtr< const FSequencerInfoColumn > WeakColumn;
};

FSequencerInfoColumn::FSequencerInfoColumn(ISceneOutliner& InSceneOutliner, FSequencer& InSequencer, const FLevelEditorSequencerBindingData& InBindingData)
	: WeakSceneOutliner(StaticCastSharedRef<ISceneOutliner>(InSceneOutliner.AsShared())) 
	, WeakSequencer(StaticCastSharedRef<FSequencer>(InSequencer.AsShared()))
	, WeakBindingData(ConstCastSharedRef<FLevelEditorSequencerBindingData>(InBindingData.AsShared()))
{
}

FSequencerInfoColumn::FSequencerInfoColumn(ISceneOutliner& InSceneOutliner)
	: WeakSceneOutliner(StaticCastSharedRef<ISceneOutliner>(InSceneOutliner.AsShared()))
{
}

FSequencerInfoColumn::~FSequencerInfoColumn()
{
}

FName FSequencerInfoColumn::GetID()
{
	static FName IDName("Sequence");
	return IDName;
}

FName FSequencerInfoColumn::GetColumnID()
{
	return GetID();
}

SHeaderRow::FColumn::FArguments FSequencerInfoColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.DefaultLabel(LOCTEXT("SequencerColumn", "Sequence"))
		.DefaultTooltip(LOCTEXT("SequencerColumnTooltip", "The sequence that this actor is referenced from"))
		.FillWidth( 5.0f );
}

const TSharedRef< SWidget > FSequencerInfoColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	auto SceneOutliner = WeakSceneOutliner.Pin();
	check(SceneOutliner.IsValid());

	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	TSharedRef<STextBlock> MainText = SNew( STextBlock )
		.Text( this, &FSequencerInfoColumn::GetTextForItem, TWeakPtr<ISceneOutlinerTreeItem>(TreeItem) )
		.HighlightText( SceneOutliner->GetFilterHighlightText() )
		.ColorAndOpacity( FSlateColor::UseSubduedForeground() );

	HorizontalBox->AddSlot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		MainText
	];

	return HorizontalBox;
}

void FSequencerInfoColumn::PopulateSearchStrings( const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings ) const
{
	OutSearchStrings.Add(Item.GetDisplayString());
}

void FSequencerInfoColumn::SortItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const
{
	if (WeakBindingData.IsValid())
	{
		FSceneOutlinerSortHelper<FString>()
			.Primary(FGetActorInfo(*this), SortMode)
			.Sort(OutItems);
	}
}

TWeakObjectPtr<AActor> FSequencerInfoColumn::GetActorFromItem(const ISceneOutlinerTreeItem& Item) const
{
	return SceneOutliner::FSceneOutlinerHelpers::GetActorFromOutlinerTreeItem(Item, WeakSceneOutliner.Pin());
}

FString FSequencerInfoColumn::GetTextForActor(AActor* InActor) const
{
	if (WeakBindingData.IsValid() && WeakSequencer.IsValid())
	{
		return WeakBindingData.Pin()->GetLevelSequencesForActor(WeakSequencer.Pin(), InActor);
	}

	return FString();
}

FText FSequencerInfoColumn::GetTextForItem( TWeakPtr<ISceneOutlinerTreeItem> TreeItem ) const
{
	auto Item = TreeItem.Pin();
	return Item.IsValid() && WeakBindingData.IsValid() ? FText::FromString(FGetActorInfo(*this)(*Item)) : FText::GetEmpty();
}

}

#undef LOCTEXT_NAMESPACE
