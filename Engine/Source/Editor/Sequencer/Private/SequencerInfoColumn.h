// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerFwd.h"
#include "Misc/EnumRange.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerColumn.h"

class AActor;
class ISceneOutliner;
struct ISceneOutlinerTreeItem;
class FSequencer;
class FLevelEditorSequencerBindingData;

namespace Sequencer
{

/**
 * A custom column for the SceneOutliner which is capable of displaying a variety of Actor details
 */
class FSequencerInfoColumn : public ISceneOutlinerColumn
{

public:

	FSequencerInfoColumn(ISceneOutliner& InSceneOutliner, FSequencer& InSequencer, const FLevelEditorSequencerBindingData& InBindingData);
	FSequencerInfoColumn(ISceneOutliner& InSceneOutliner);
	
	virtual ~FSequencerInfoColumn();

	static FName GetID();

	//////////////////////////////////////////////////////////////////////////
	// Begin ISceneOutlinerColumn Implementation

	virtual FName GetColumnID() override;

	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	
	virtual const TSharedRef< SWidget > ConstructRowWidget( FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row ) override;

	virtual void PopulateSearchStrings( const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings ) const override;

	virtual bool SupportsSorting() const override { return true; }

	virtual void SortItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const override;
	
	// End ISceneOutlinerColumn Implementation
	//////////////////////////////////////////////////////////////////////////
		
	FString GetTextForActor(AActor* InActor) const;
	TWeakObjectPtr<AActor> GetActorFromItem(const ISceneOutlinerTreeItem& Item) const;

private:

	FText GetTextForItem( const TWeakPtr<ISceneOutlinerTreeItem> TreeItem ) const;

	/** Weak reference to the outliner widget that owns our list */
	TWeakPtr< ISceneOutliner > WeakSceneOutliner;

	/** Weak reference to sequencer */
	TWeakPtr< FSequencer > WeakSequencer;

	/** Weak reference to binding data */
	TWeakPtr< FLevelEditorSequencerBindingData > WeakBindingData;
};

}