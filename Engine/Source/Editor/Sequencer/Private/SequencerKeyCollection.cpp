// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerKeyCollection.h"

#include "Algo/BinarySearch.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "IKeyArea.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/SharedList.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Math/Range.h"
#include "Math/RangeBound.h"
#include "Math/UnrealMathUtility.h"
#include "MovieSceneSection.h"
#include "SequencerCoreFwd.h"
#include "Templates/Tuple.h"

FSequencerKeyCollectionSignature FSequencerKeyCollectionSignature::FromNodes(const TArray<TSharedRef<FViewModel>>& InNodes, FFrameNumber InDuplicateThresholdTime)
{
	using namespace UE::Sequencer;

	FSequencerKeyCollectionSignature Result;
	Result.DuplicateThresholdTime = InDuplicateThresholdTime;

	for (const TSharedRef<FViewModel>& Node : InNodes)
	{
		for (TViewModelPtr<const FChannelGroupModel> ChannelGroupModel : Node->GetChildrenOfType<FChannelGroupModel>())
		{
			const IOutlinerExtension* OutlinerExtension = ChannelGroupModel->CastThis<IOutlinerExtension>();
			if (!OutlinerExtension || OutlinerExtension->IsFilteredOut() == false)
			{
				for (const TSharedRef<IKeyArea>& KeyArea : ChannelGroupModel->GetAllKeyAreas())
				{
					const UMovieSceneSection* Section = KeyArea->GetOwningSection();
					Result.KeyAreaToSignature.Add(KeyArea, Section ? Section->GetSignature() : FGuid());
				}
			}
		}

		for (TSharedPtr<FSectionModel> SectionModel : Node->GetChildrenOfType<FSectionModel>())
		{
			if (UMovieSceneSection* Section = SectionModel->GetSection())
			{
				TRange<FFrameNumber> SectionRange = SectionModel->GetRange();
				Result.SignatureToSectionBounds.Add(Section->GetSignature(), SectionRange);
			}
		}
	}

	return Result;
}

FSequencerKeyCollectionSignature FSequencerKeyCollectionSignature::FromNodesRecursive(const TArray<TSharedRef<FViewModel>>& InNodes, FFrameNumber InDuplicateThresholdTime)
{
	using namespace UE::Sequencer;

	FSequencerKeyCollectionSignature Result;
	Result.DuplicateThresholdTime = InDuplicateThresholdTime;

	TArray<TSharedPtr<FChannelGroupModel>> AllKeyAreaNodes;
	AllKeyAreaNodes.Reserve(36);
	for (const TSharedRef<FViewModel>& Node : InNodes)
	{
		const bool bIncludeThis = true;
		for (const TViewModelPtr<FChannelGroupModel>& KeyAreaNode : Node->GetDescendantsOfType<FChannelGroupModel>(bIncludeThis))
		{
			IOutlinerExtension* OutlinerExtension = KeyAreaNode->CastThis<IOutlinerExtension>();
			if (!OutlinerExtension || OutlinerExtension->IsFilteredOut() == false)
			{
				AllKeyAreaNodes.Add(KeyAreaNode);
			}
		}

		for (TSharedPtr<FSectionModel> SectionModel : Node->GetDescendantsOfType<FSectionModel>())
		{
			if (UMovieSceneSection* Section = SectionModel->GetSection())
			{
				TRange<FFrameNumber> SectionRange = SectionModel->GetRange();
				Result.SignatureToSectionBounds.Add(Section->GetSignature(), SectionRange);
			}
		}
	}

	for (const TSharedPtr<FChannelGroupModel>& Node : AllKeyAreaNodes)
	{
		for (const TSharedRef<IKeyArea>& KeyArea : Node->GetAllKeyAreas())
		{
			const UMovieSceneSection* Section = KeyArea->GetOwningSection();
			Result.KeyAreaToSignature.Add(KeyArea, Section ? Section->GetSignature() : FGuid());
		}
	}

	return Result;
}

FSequencerKeyCollectionSignature FSequencerKeyCollectionSignature::FromNodeRecursive(TSharedRef<FViewModel> InNode, UMovieSceneSection* InSection, FFrameNumber InDuplicateThresholdTime)
{
	using namespace UE::Sequencer;

	FSequencerKeyCollectionSignature Result;
	Result.DuplicateThresholdTime = InDuplicateThresholdTime;

	TArray<TSharedPtr<FChannelGroupModel>> AllKeyAreaNodes;
	AllKeyAreaNodes.Reserve(36);
	for (TSharedPtr<FChannelGroupModel> KeyAreaNode : InNode->GetDescendantsOfType<FChannelGroupModel>(true))
	{
		IOutlinerExtension* OutlinerExtension = KeyAreaNode->CastThis<IOutlinerExtension>();
		if (!OutlinerExtension || OutlinerExtension->IsFilteredOut() == false)
		{
			AllKeyAreaNodes.Add(KeyAreaNode);
		}
	}

	for (TSharedPtr<FSectionModel> SectionModel : InNode->GetChildrenOfType<FSectionModel>())
	{
		if (UMovieSceneSection* Section = SectionModel->GetSection())
		{
			TRange<FFrameNumber> SectionRange = SectionModel->GetRange();
			Result.SignatureToSectionBounds.Add(Section->GetSignature(), SectionRange);
		}
	}

	for (const TSharedPtr<FChannelGroupModel>& Node : AllKeyAreaNodes)
	{
		TSharedPtr<IKeyArea> KeyArea = Node->GetKeyArea(InSection);
		if (KeyArea.IsValid())
		{
			Result.KeyAreaToSignature.Add(KeyArea.ToSharedRef(), InSection ? InSection->GetSignature() : FGuid());
		}
	}

	return Result;
}

bool FSequencerKeyCollectionSignature::HasUncachableContent() const
{
	for (auto& Pair : KeyAreaToSignature)
	{
		if (!Pair.Value.IsValid())
		{
			return true;
		}
	}
	for (auto& Pair : SignatureToSectionBounds)
	{
		if (!Pair.Key.IsValid())
		{
			return true;
		}
	}
	return false;
}

bool operator!=(const FSequencerKeyCollectionSignature& A, const FSequencerKeyCollectionSignature& B)
{
	if (A.HasUncachableContent() || B.HasUncachableContent())
	{
		return true;
	}

	if (A.KeyAreaToSignature.Num() != B.KeyAreaToSignature.Num() || A.DuplicateThresholdTime != B.DuplicateThresholdTime || A.SignatureToSectionBounds.Num() != B.SignatureToSectionBounds.Num())
	{
		return true;
	}

	for (auto& Pair : A.KeyAreaToSignature)
	{
		const FGuid* BSig = B.KeyAreaToSignature.Find(Pair.Key);
		if (!BSig || *BSig != Pair.Value)
		{
			return true;
		}
	}

	for (auto& Pair : A.SignatureToSectionBounds)
	{
		const TRange<FFrameNumber>* SectionBounds = B.SignatureToSectionBounds.Find(Pair.Key);
		if (!SectionBounds || *SectionBounds != Pair.Value)
		{
			return true;
		}
	}

	return false;
}

bool operator==(const FSequencerKeyCollectionSignature& A, const FSequencerKeyCollectionSignature& B)
{
	if (A.HasUncachableContent() || B.HasUncachableContent())
	{
		return false;
	}

	if (A.KeyAreaToSignature.Num() != B.KeyAreaToSignature.Num() || A.DuplicateThresholdTime != B.DuplicateThresholdTime || A.SignatureToSectionBounds.Num() != B.SignatureToSectionBounds.Num())
	{
		return false;
	}

	for (auto& Pair : A.KeyAreaToSignature)
	{
		const FGuid* BSig = B.KeyAreaToSignature.Find(Pair.Key);
		if (!BSig || *BSig != Pair.Value)
		{
			return false;
		}
	}

	for (auto& Pair : A.SignatureToSectionBounds)
	{
		const TRange<FFrameNumber>* SectionBounds = B.SignatureToSectionBounds.Find(Pair.Key);
		if (!SectionBounds || *SectionBounds != Pair.Value)
		{
			return false;
		}
	}

	return true;
}

bool FSequencerKeyCollection::Update(const FSequencerKeyCollectionSignature& InSignature)
{
	if (InSignature == Signature)
	{
		return false;
	}

	TArray<FFrameNumber> AllTimes;
	TArray<FFrameNumber> AllSectionTimes;

	// Get all the key times for the key areas
	for (auto& Pair : InSignature.GetKeyAreas())
	{
		if (UMovieSceneSection* Section = Pair.Key->GetOwningSection())
		{
			Pair.Key->GetKeyTimes(AllTimes, Section->GetRange());
		}
	}
			
	for (auto& Pair : InSignature.GetSectionBounds())
	{
		if (Pair.Value.HasLowerBound())
		{
			AllSectionTimes.Add(Pair.Value.GetLowerBoundValue());
		}
		if (Pair.Value.HasUpperBound())
		{
			AllSectionTimes.Add(Pair.Value.GetUpperBoundValue());
		}
	}

	AllTimes.Sort();
	AllSectionTimes.Sort();

	GroupedTimes.Reset(AllTimes.Num());
	int32 Index = 0;
	while ( Index < AllTimes.Num() )
	{
		FFrameNumber PredicateTime = AllTimes[Index];
		GroupedTimes.Add(PredicateTime);
		while (Index < AllTimes.Num() && FMath::Abs(AllTimes[Index] - PredicateTime) <= InSignature.GetDuplicateThreshold())
		{
			++Index;
		}
	}
	GroupedTimes.Shrink();

	GroupedSectionTimes.Reset(AllSectionTimes.Num());
	Index = 0;
	while (Index < AllSectionTimes.Num())
	{
		FFrameNumber PredicateTime = AllSectionTimes[Index];
		GroupedSectionTimes.Add(PredicateTime);
		while (Index < AllSectionTimes.Num() && FMath::Abs(AllSectionTimes[Index] - PredicateTime) <= InSignature.GetDuplicateThreshold())
		{
			++Index;
		}
	}
	GroupedSectionTimes.Shrink();

	AllTimes = GroupedTimes;
	if (GroupedSectionTimes.Num() > 0)
	{
		AllTimes.Insert(GroupedSectionTimes, 0);
		AllTimes.Sort();
	}
	AllGroupedTimes.Reset(AllTimes.Num());
	Index = 0;
	while (Index < AllTimes.Num())
	{
		FFrameNumber PredicateTime = AllTimes[Index];
		AllGroupedTimes.Add(PredicateTime);
		while (Index < AllTimes.Num() && FMath::Abs(AllTimes[Index] - PredicateTime) <= InSignature.GetDuplicateThreshold())
		{
			++Index;
		}
	}
	AllGroupedTimes.Shrink();

	Signature = InSignature;

	return true;
}

TArrayView<const FFrameNumber> GetKeysInRangeInternal(const TArray<FFrameNumber>&Times, const TRange<FFrameNumber>& Range)
{
	// Binary search the first time that's >= the lower bound
	int32 FirstVisibleIndex = Range.GetLowerBound().IsClosed() ? Algo::LowerBound(Times, Range.GetLowerBoundValue()) : 0;
	// Binary search the last time that's > the upper bound
	int32 LastVisibleIndex = Range.GetUpperBound().IsClosed() ? Algo::UpperBound(Times, Range.GetUpperBoundValue()) : Times.Num();

	int32 Num = LastVisibleIndex - FirstVisibleIndex;
	if (Times.IsValidIndex(FirstVisibleIndex) && LastVisibleIndex <= Times.Num() && Num > 0)
	{
		return MakeArrayView(&Times[FirstVisibleIndex], Num);
	}

	return TArrayView<const FFrameNumber>();
}

TOptional<FFrameNumber> GetNextKeyInternal(const TArray<FFrameNumber>& Times, FFrameNumber FrameNumber, EFindKeyDirection Direction, const TRange<FFrameNumber>& Range, const bool bAllowLoop)
{
	TArrayView<const FFrameNumber> KeysInRange = GetKeysInRangeInternal(Times, Range);

	int32 Index = INDEX_NONE;
	if (Direction == EFindKeyDirection::Forwards)
	{
		Index = Algo::UpperBound(KeysInRange, FrameNumber);
	}
	else
	{
		Index = Algo::LowerBound(KeysInRange, FrameNumber) - 1;
	}

	if (KeysInRange.IsValidIndex(Index))
	{
		return KeysInRange[Index];
	}
	else if (KeysInRange.Num() > 0 && bAllowLoop)
	{
		if (Direction == EFindKeyDirection::Forwards)
		{
			return KeysInRange[0];
		}
		else
		{
			return KeysInRange.Last();
		}
	}

	return TOptional<FFrameNumber>();

}

TOptional<FFrameNumber> FindFirstKeyInRangeInternal(const TArray<FFrameNumber>& Times, const TRange<FFrameNumber>& Range, EFindKeyDirection Direction)
{
	TArrayView<const FFrameNumber> KeysInRange = GetKeysInRangeInternal(Times, Range);
	if (KeysInRange.Num())
	{
		return Direction == EFindKeyDirection::Forwards ? KeysInRange[0] : KeysInRange[KeysInRange.Num() - 1];
	}
	return TOptional<FFrameNumber>();
}

TOptional<FFrameNumber> FSequencerKeyCollection::FindFirstKeyInRange(const TRange<FFrameNumber>& Range, EFindKeyDirection Direction, EFindKeyType FindKeyType) const
{
	if (FindKeyType == EFindKeyType::FKT_Keys)
	{
		return FindFirstKeyInRangeInternal(GroupedTimes, Range, Direction);
	}
	else if (FindKeyType == EFindKeyType::FKT_Sections)
	{
		return FindFirstKeyInRangeInternal(GroupedSectionTimes, Range, Direction);
	}
	else if (FindKeyType == EFindKeyType::FKT_All)
	{
		return FindFirstKeyInRangeInternal(AllGroupedTimes, Range, Direction);
	}
	return TOptional<FFrameNumber>();
}

TArrayView<const FFrameNumber> FSequencerKeyCollection::GetKeysInRange(const TRange<FFrameNumber>& Range, EFindKeyType FindKeyType) const
{
	if (FindKeyType == EFindKeyType::FKT_Keys)
	{
		return GetKeysInRangeInternal(GroupedTimes, Range);
	}
	else if (FindKeyType == EFindKeyType::FKT_Sections)
	{
		return GetKeysInRangeInternal(GroupedSectionTimes, Range);
	}
	else if (FindKeyType == EFindKeyType::FKT_All)
	{
		return GetKeysInRangeInternal(AllGroupedTimes, Range);
	}
	return TArrayView<const FFrameNumber>();
}

TOptional<FFrameNumber> FSequencerKeyCollection::GetNextKey(FFrameNumber FrameNumber, EFindKeyDirection Direction, const TRange<FFrameNumber>& Range, EFindKeyType FindKeyType, const bool bAllowLoop) const
{
	if (FindKeyType == EFindKeyType::FKT_Keys)
	{
		return GetNextKeyInternal(GroupedTimes, FrameNumber, Direction, Range, bAllowLoop);
	}
	else if (FindKeyType == EFindKeyType::FKT_Sections)
	{
		return GetNextKeyInternal(GroupedSectionTimes, FrameNumber, Direction, Range, bAllowLoop);
	}
	else if (FindKeyType == EFindKeyType::FKT_All)
	{
		return GetNextKeyInternal(AllGroupedTimes, FrameNumber, Direction, Range, bAllowLoop);
	}
	return TOptional<FFrameNumber>();
}
