// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerKeyParams.h"

#include "Algo/Find.h"
#include "Algo/IndexOf.h"
#include "Algo/Sort.h"
#include "CoreTypes.h"
#include "IKeyArea.h"
#include "ISequencerSection.h"
#include "Math/Range.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "Templates/Function.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

namespace UE
{
namespace Sequencer
{


void FKeyOperation::IterateOperations(TFunctionRef<CallbackType> Callback) const
{
	for (const TTuple<UMovieSceneTrack*, FSectionCandidates>& Pair : CandidatesByTrack)
	{
		UMovieSceneTrack*                      Track      = Pair.Key;
		TArrayView<const FKeySectionOperation> Operations = Pair.Value.Operations;

		Callback(Track, Operations);
	}
}

void FKeyOperation::InitializeOperation(FFrameNumber InKeyTime)
{
	for (auto It = CandidatesByTrack.CreateIterator(); It; ++It)
	{
		UMovieSceneTrack*   Track      = It.Key();
		FSectionCandidates& Candidates = It.Value();

		Candidates.FilterOperations(Track, InKeyTime);
		if (Candidates.Operations.Num() == 0)
		{
			It.RemoveCurrent();
			continue;
		}

		for (const FKeySectionOperation& Operation : Candidates.Operations)
		{
			UMovieSceneSection* SectionObject = Operation.Section->GetSectionObject();
			SectionObject->Modify();
			for (TSharedPtr<IKeyArea> KeyArea : Operation.KeyAreas)
			{
				UObject* OwningObject = KeyArea->GetOwningObject();
				if (OwningObject != SectionObject)
				{
					OwningObject->Modify();
				}
			}
			SectionObject->ExpandToFrame(InKeyTime);
		}
	}
}

void FKeyOperation::ApplyDefault(FFrameNumber InKeyTime, ISequencer& InSequencer, TArray<FAddKeyResult>* OutResults) const
{
	auto Iterator = [InKeyTime, &InSequencer, &OutResults](UMovieSceneTrack* Track, TArrayView<const FKeySectionOperation> Operations)
	{
		ApplyOperations(InKeyTime, Operations, Track->FindObjectBindingGuid(), InSequencer, OutResults);
	};

	IterateOperations(Iterator);
}

void FKeyOperation::ApplyOperations(FFrameNumber InKeyTime, TArrayView<const FKeySectionOperation> InOperations, const FGuid& ObjectBindingID, ISequencer& InSequencer, TArray<FAddKeyResult>* OutResults)
{
	// @todo: Do we even want to support multiple???
	for (const FKeySectionOperation& Operation : InOperations)
	{
		for (TSharedPtr<IKeyArea> KeyArea : Operation.KeyAreas)
		{
			FKeyHandle KeyHandle = KeyArea->AddOrUpdateKey(InKeyTime, ObjectBindingID, InSequencer);

			if (OutResults)
			{
				FAddKeyResult Result;
				Result.KeyArea = KeyArea;
				Result.KeyHandle = KeyHandle;
				OutResults->Add(Result);
			}
		}
	}
}

void FKeyOperation::Populate(UMovieSceneTrack* InTrack, TSharedPtr<ISequencerSection> InSection, TSharedPtr<IKeyArea> InKeyArea)
{
	UMovieSceneSection* SectionObject = InSection->GetSectionObject();
	if (SectionObject->IsReadOnly())
	{
		return;
	}

	if (!MovieSceneHelpers::IsSectionKeyable(SectionObject))
	{
		return;
	}

	FSectionCandidates&   Candidates        = CandidatesByTrack.FindOrAdd(InTrack);
	FKeySectionOperation* ExistingOperation = Algo::FindBy(Candidates.Operations, InSection, &FKeySectionOperation::Section);
	if (ExistingOperation)
	{
		ExistingOperation->KeyAreas.Add(InKeyArea);
	}
	else
	{
		Candidates.Operations.Add(FKeySectionOperation{ InSection, { InKeyArea } });
	}
}

void FKeyOperation::FSectionCandidates::FilterOperations(UMovieSceneTrack* Track, FFrameNumber KeyTime)
{
	checkf(Operations.Num() != 0, TEXT("FSectionCandidates should never have been constructed with empty sections"));

	UMovieSceneSection* SectionToKey = Track->GetSectionToKey();
	if (SectionToKey)
	{
		// If we have a section to key, and it is one of the prospective sections, it should always receive the full weighting
		const int32 SectionToKeyIndex = Algo::IndexOfBy(Operations, SectionToKey, [](const FKeySectionOperation& In) { return In.Section->GetSectionObject(); });
		if (SectionToKeyIndex != INDEX_NONE)
		{
			FKeySectionOperation Op = MoveTemp(Operations[SectionToKeyIndex]);
			Operations.Empty();
			Operations.Add(MoveTemp(Op));
			return;
		}
	}

	if (Operations.Num() > 1)
	{
		TArray<FKeySectionOperation, TInlineAllocator<1>> OldOperations;
		Swap(OldOperations, Operations);

		TArray<UMovieSceneSection*, TInlineAllocator<1>> Sections;
		for (const FKeySectionOperation& Operation : OldOperations)
		{
			UMovieSceneSection* Section = Operation.Section->GetSectionObject();
			Sections.Add(Section);
		}

		// Sort by row index so we can choose one per row
		Algo::SortBy(Sections, &UMovieSceneSection::GetRowIndex);

		bool bFoundOverlap = false;

		// Find a sections that overlaps the current time
		for (int32 Index = 0; Index < Sections.Num(); )
		{
			const int32 StartIndex = Index;
			const int32 RowIndex   = Sections[Index]->GetRowIndex();

			UMovieSceneSection* ThisSectionToKey = nullptr;

			// Increment over all the sections on the current row
			for (; Index < Sections.Num() && Sections[Index]->GetRowIndex() == RowIndex; ++Index)
			{
				if (Sections[Index]->GetRange().Contains(KeyTime))
				{
					if (!ThisSectionToKey || ThisSectionToKey->GetOverlapPriority() > Sections[Index]->GetOverlapPriority())
					{
						ThisSectionToKey = Sections[Index];
					}
				}
			}

			if (ThisSectionToKey)
			{
				bFoundOverlap = true;

				const int32 ThisSectionToKeyIndex = Algo::IndexOfBy(OldOperations, ThisSectionToKey, [](const FKeySectionOperation& In) { return In.Section->GetSectionObject(); });
				Operations.Add(MoveTemp(OldOperations[ThisSectionToKeyIndex]));
				OldOperations.RemoveAt(ThisSectionToKeyIndex, EAllowShrinking::No);
			}
		}

		// If nothing was overlapping, we find the nearest section on each row, and expand those
		if (!bFoundOverlap)
		{
			for (int32 Index = 0; Index < Sections.Num(); )
			{
				const int32 StartIndex = Index;
				const int32 RowIndex   = Sections[Index]->GetRowIndex();

				// Increment over all the sections on the current row
				for (; Index < Sections.Num() && Sections[Index]->GetRowIndex() == RowIndex; ++Index)
				{}

				TArrayView<UMovieSceneSection* const> Row = MakeArrayView(Sections.GetData() + StartIndex, Index - StartIndex);

				UMovieSceneSection* ThisSectionToKey = MovieSceneHelpers::FindNearestSectionAtTime(Row, KeyTime);
				if (ThisSectionToKey)
				{
					const int32 ThisSectionToKeyIndex = Algo::IndexOfBy(OldOperations, ThisSectionToKey, [](const FKeySectionOperation& In) { return In.Section->GetSectionObject(); });
					Operations.Add(MoveTemp(OldOperations[ThisSectionToKeyIndex]));
					OldOperations.RemoveAt(ThisSectionToKeyIndex, EAllowShrinking::No);
				}
			}
		}
	}
}



} // namespace Sequencer
} // namespace UE
