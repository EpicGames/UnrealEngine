// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerKeyStructGenerator.h"

#include "Misc/AssertionMacros.h"
#include "UObject/Field.h"
#include "UObject/Package.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequencerKeyStructGenerator)

UMovieSceneKeyStructType::UMovieSceneKeyStructType(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	SourceValuesProperty = nullptr;
	SourceTimesProperty  = nullptr;
	DestValueProperty    = nullptr;
	DestTimeProperty     = nullptr;

	SetSuperStruct(FGeneratedMovieSceneKeyStruct::StaticStruct());
	ensureMsgf(FGeneratedMovieSceneKeyStruct::StaticStruct()->PropertyLink == nullptr, TEXT("FGeneratedMovieSceneKeyStruct must not have any UPROPERTY members."));
}

void UMovieSceneKeyStructType::InitializeStruct(void* InDest, int32 ArrayDim) const
{
	const int32 Stride = GetStructureSize();

	FMemory::Memzero(InDest, ArrayDim*Stride);

	// Initialize the native super struct first
	for (int32 Index = 0; Index < ArrayDim; ++Index)
	{
		uint8* ElementBytes = reinterpret_cast<uint8*>(InDest) + Index*Stride;
		new (ElementBytes) FGeneratedMovieSceneKeyStruct();

		// Initialize any additional properties on this struct
		for (FProperty* Property = PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			Property->InitializeValue_InContainer(ElementBytes);
		}
	}
}

void UMovieSceneKeyStructType::DestroyStruct(void* InDest, int32 ArrayDim) const
{
	const int32 Stride = GetStructureSize();

	for (int32 Index = 0; Index < ArrayDim; ++Index)
	{
		uint8* ElementBytes = reinterpret_cast<uint8*>(InDest) + Index*Stride;

		// Destroy any additional properties on this struct
		for (FProperty* Property = PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			Property->DestroyValue_InContainer(ElementBytes);
		}

		// Destroy the base class
		reinterpret_cast<FGeneratedMovieSceneKeyStruct*>(ElementBytes)->~FGeneratedMovieSceneKeyStruct();
	}
}

FSequencerKeyStructGenerator& FSequencerKeyStructGenerator::Get()
{
	static FSequencerKeyStructGenerator Instance;
	return Instance;
}

UMovieSceneKeyStructType* FSequencerKeyStructGenerator::AllocateNewKeyStruct()
{
	UMovieSceneKeyStructType* NewStruct = NewObject<UMovieSceneKeyStructType>(GetTransientPackage(), NAME_None, RF_Public | RF_Standalone);
	return NewStruct;
}

UMovieSceneKeyStructType* FSequencerKeyStructGenerator::AllocateNewKeyStruct(UScriptStruct* ChannelType)
{
	static const FName TimesMetaDataTag("KeyTimes");
	static const FName ValuesMetaDataTag("KeyValues");

	FArrayProperty* SourceTimes  = FindArrayPropertyWithTag(ChannelType, TimesMetaDataTag);
	FArrayProperty* SourceValues = FindArrayPropertyWithTag(ChannelType, ValuesMetaDataTag);

	if (!ensureMsgf(SourceTimes, TEXT("No times property could be found for channel type %s. Please add KeyTimes meta data to the array containing the channel's key time."), *ChannelType->GetName()))
	{
		return nullptr;
	}
	else if (!ensureMsgf(SourceValues, TEXT("No value property could be found for channel type %s. Please add KeyValues meta data to the array containing the channel's key values."), *ChannelType->GetName()))
	{
		return nullptr;
	}

	UMovieSceneKeyStructType* NewStruct = AllocateNewKeyStruct();

	NewStruct->SourceTimesProperty  = SourceTimes;
	NewStruct->SourceValuesProperty = SourceValues;

	return NewStruct;
}

UMovieSceneKeyStructType* FSequencerKeyStructGenerator::DefaultInstanceGeneratedStruct(UScriptStruct* ChannelType)
{
	UMovieSceneKeyStructType* Existing = FindGeneratedStruct(ChannelType->GetFName());
	if (Existing)
	{
		return Existing;
	}
	else
	{
		UMovieSceneKeyStructType* NewStruct = FSequencerKeyStructGenerator::AllocateNewKeyStruct(ChannelType);
		if (!NewStruct)
		{
			return nullptr;
		}

		FProperty* NewValueProperty = CastField<FProperty>(FField::Duplicate(NewStruct->SourceValuesProperty->Inner, NewStruct, "Value"));
		NewValueProperty->SetPropertyFlags(CPF_Edit);
		NewValueProperty->SetMetaData("Category", TEXT("Key"));
		NewValueProperty->SetMetaData("ShowOnlyInnerProperties", TEXT("true"));
		NewValueProperty->ArrayDim = 1;

		NewStruct->AddCppProperty(NewValueProperty);
		NewStruct->DestValueProperty = NewValueProperty;

		FSequencerKeyStructGenerator::FinalizeNewKeyStruct(NewStruct);

		AddGeneratedStruct(ChannelType->GetFName(), NewStruct);
		return NewStruct;
	}
}

void FSequencerKeyStructGenerator::FinalizeNewKeyStruct(UMovieSceneKeyStructType* InStruct)
{
	check(InStruct);

	// Add the time property to the head of the property linked list (so it shows first)
	FStructProperty* NewTimeProperty = new FStructProperty(InStruct, "Time");
	NewTimeProperty->SetPropertyFlags(CPF_Edit);
	NewTimeProperty->SetMetaData("Category", TEXT("Key"));
	NewTimeProperty->ArrayDim = 1;
	NewTimeProperty->Struct = TBaseStructure<FFrameNumber>::Get();

	InStruct->AddCppProperty(NewTimeProperty);
	InStruct->DestTimeProperty = NewTimeProperty;

	// Finalize the struct
	InStruct->Bind();
	InStruct->StaticLink(true);

	check(InStruct->IsComplete());
}

void FSequencerKeyStructGenerator::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObjects(InstanceNameToGeneratedStruct);
}

void FSequencerKeyStructGenerator::AddGeneratedStruct(FName InstancedStructName, UMovieSceneKeyStructType* Struct)
{
	check(!InstanceNameToGeneratedStruct.Contains(InstancedStructName));
	InstanceNameToGeneratedStruct.Add(InstancedStructName, Struct);
}

UMovieSceneKeyStructType* FSequencerKeyStructGenerator::FindGeneratedStruct(FName InstancedStructName)
{
	return InstanceNameToGeneratedStruct.FindRef(InstancedStructName);
}

FArrayProperty* FSequencerKeyStructGenerator::FindArrayPropertyWithTag(UScriptStruct* ChannelStruct, FName MetaDataTag)
{
	for (FArrayProperty* ArrayProperty : TFieldRange<FArrayProperty>(ChannelStruct))
	{
		if (ArrayProperty->HasMetaData(MetaDataTag))
		{
			return ArrayProperty;
		}
	}

	return nullptr;
}

TSharedPtr<FStructOnScope> FSequencerKeyStructGenerator::CreateInitialStructInstance(const void* SourceChannel, UMovieSceneKeyStructType* GeneratedStructType, int32 InitialKeyIndex)
{
	check(InitialKeyIndex != INDEX_NONE);

	TSharedPtr<FStructOnScope>     Struct    = MakeShared<FStructOnScope>(GeneratedStructType);
	FGeneratedMovieSceneKeyStruct* StructPtr = reinterpret_cast<FGeneratedMovieSceneKeyStruct*>(Struct->GetStructMemory());

	// Copy the initial time into the struct
	{
		const uint8* SrcTimeData  = GeneratedStructType->SourceTimesProperty->ContainerPtrToValuePtr<uint8>(SourceChannel);
		uint8*       DestTimeData = GeneratedStructType->DestTimeProperty->ContainerPtrToValuePtr<uint8>(StructPtr);

		FScriptArrayHelper SourceTimesArray(GeneratedStructType->SourceTimesProperty.Get(), SrcTimeData);
		GeneratedStructType->SourceTimesProperty->Inner->CopyCompleteValue(DestTimeData, SourceTimesArray.GetRawPtr(InitialKeyIndex));
	}

	return Struct;
}
