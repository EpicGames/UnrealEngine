// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerMarkedFrameHelper.h"

#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/MovieSceneSectionParameters.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "Evaluation/MovieSceneTimeArray.h"
#include "Evaluation/MovieSceneTimeTransform.h"
#include "HAL/PlatformCrt.h"
#include "ISequencer.h"
#include "Math/Range.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "MovieScene.h"
#include "MovieSceneFrameMigration.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceID.h"
#include "MovieSceneTimeHelpers.h"
#include "Sections/MovieSceneSubSection.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

namespace UE
{
namespace MovieScene
{

static void FindGlobalMarkedFrames(
		const ISequencer& Sequencer, const FMovieSceneSequenceHierarchy* SequenceHierarchy,
		FMovieSceneSequenceIDRef FocusedSequenceID, FMovieSceneSequenceIDRef SequenceID, 
		TRange<FFrameNumber> GatherRange,
		const FMovieSceneInverseSequenceTransform& SequenceToRootTransform,
		const FMovieSceneTransformBreadcrumbs& StartBreadcrumbs,
		const FMovieSceneTransformBreadcrumbs& EndBreadcrumbs,
		TMovieSceneTimeArray<FMovieSceneMarkedFrame>& OutTimestampedGlobalMarkedFrames)
{
	// Find the current sequence in the hierarchy.
	const FMovieSceneSubSequenceData* const SequenceSubData = SequenceHierarchy->FindSubData(SequenceID);
	const UMovieSceneSequence* const Sequence = SequenceSubData ? SequenceSubData->GetSequence() : Sequencer.GetRootMovieSceneSequence();
	const UMovieScene* const MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (UNLIKELY(!ensure(MovieScene)))
	{
		return;
	}

	// Get the marked frames of the current sequence if it's not the focused sequence.
	if (SequenceID != FocusedSequenceID && MovieScene->GetGloballyShowMarkedFrames())
	{
		const TArray<FMovieSceneMarkedFrame>& MarkedFrames = MovieScene->GetMarkedFrames();
		for (const FMovieSceneMarkedFrame& MarkedFrame : MarkedFrames)
		{
			if (GatherRange.Contains(MarkedFrame.FrameNumber))
			{
				// Iterate all mappings in the root
				auto VisitMarkedFrame = [&OutTimestampedGlobalMarkedFrames, &MarkedFrame](FFrameTime RootTime)
				{
					OutTimestampedGlobalMarkedFrames.Add(RootTime, MarkedFrame);
					return true;
				};

				SequenceToRootTransform.TransformTimeWithinRange(MarkedFrame.FrameNumber, VisitMarkedFrame, StartBreadcrumbs, EndBreadcrumbs);
			}
		}
	}

	// Dive into the current sequence's sub-sequences.
	const FMovieSceneSequenceHierarchyNode* const SequenceNode = SequenceHierarchy->FindNode(SequenceID);
	if (ensure(SequenceNode))
	{
		for (const FMovieSceneSequenceID ChildID : SequenceNode->Children)
		{
			const FMovieSceneSubSequenceData* const ChildSubData = SequenceHierarchy->FindSubData(ChildID);
			if (UNLIKELY(!ensure(ChildSubData)))
			{
				continue;
			}

			const UMovieSceneSequence* const ChildSequence = ChildSubData->GetSequence();
			const UMovieScene* const ChildMovieScene = ChildSequence ? ChildSequence->GetMovieScene() : nullptr;
			if (UNLIKELY(!ensure(ChildMovieScene)))
			{
				continue;
			}

			// Gather marked frames in this "window".
			FindGlobalMarkedFrames(
				Sequencer,
				SequenceHierarchy,
				FocusedSequenceID,
				ChildID,
				ChildSubData->PlayRange.Value,
				ChildSubData->RootToSequenceTransform.Inverse(),
				ChildSubData->StartTimeBreadcrumbs,
				ChildSubData->EndTimeBreadcrumbs,
				OutTimestampedGlobalMarkedFrames
			);

		}
	}
}

} // namespace MovieScene
} // namespace UE

void FSequencerMarkedFrameHelper::FindGlobalMarkedFrames(ISequencer& Sequencer, TArray<FMovieSceneMarkedFrame>& OutGlobalMarkedFrames)
{
	// Get the focused sequence info. We want to gather all the marked frames that are in the subset of the sequence hierarchy
	// that hangs below this focused sequence.
	UMovieSceneSequence* FocusedMovieSequence = Sequencer.GetFocusedMovieSceneSequence();
	const FMovieSceneSequenceID FocusedMovieSequenceID = Sequencer.GetFocusedTemplateID();

	UMovieSceneSequence* RootMovieSequence = Sequencer.GetRootMovieSceneSequence();

	if (!FocusedMovieSequence || !RootMovieSequence)
	{
		return;
	}

	// Get the sequence hierarchy so that we can iterate it.
	const FMovieSceneRootEvaluationTemplateInstance& EvalTemplate = Sequencer.GetEvaluationTemplate();
	const FMovieSceneSequenceHierarchy* SequenceHierarchy = EvalTemplate.GetHierarchy();
	if (!SequenceHierarchy)
	{
		return;
	}
	
	// All the marked frames will be added using their root time, but we want to actually display them in the time space of whatever
	// is the currently focused sequence. We therefore add the inverse time transform of the focused sequence at the top of the
	// transform stack if the focused sequence isn't the root sequence (which has no time transform).

	TMovieSceneTimeArray<FMovieSceneMarkedFrame> TimestampedGlobalMarkedFrames;

	// Grab the marked frames from the root sequence, and recursively across the whole hierarchy.
	UE::MovieScene::FindGlobalMarkedFrames(
		Sequencer,
		SequenceHierarchy,
		FocusedMovieSequenceID,
		MovieSceneSequenceID::Root,
		TRange<FFrameNumber>::All(),
		FMovieSceneInverseSequenceTransform(),
		FMovieSceneTransformBreadcrumbs(),
		FMovieSceneTransformBreadcrumbs(),
		TimestampedGlobalMarkedFrames);

	FMovieSceneSequenceTransform RootToFocusedTransform;
	if (const FMovieSceneSubSequenceData* SubData = SequenceHierarchy->FindSubData(FocusedMovieSequenceID))
	{
		RootToFocusedTransform = SubData->RootToSequenceTransform;
	}

	// Export the modified timestamped entries.
	for (const TMovieSceneTimeArrayEntry<FMovieSceneMarkedFrame>& Entry : TimestampedGlobalMarkedFrames.GetEntries())
	{
		FMovieSceneMarkedFrame MarkedFrame = Entry.Datum;
		MarkedFrame.FrameNumber = RootToFocusedTransform.TransformTime(Entry.RootTime).FrameNumber;
		OutGlobalMarkedFrames.Add(MarkedFrame);
	}
}

void FSequencerMarkedFrameHelper::ClearGlobalMarkedFrames(ISequencer& Sequencer)
{
	const FMovieSceneRootEvaluationTemplateInstance& EvalTemplate = Sequencer.GetEvaluationTemplate();

	ClearGlobalMarkedFrames(EvalTemplate.GetRootSequence());

	const FMovieSceneSequenceHierarchy* SequenceHierarchy = EvalTemplate.GetHierarchy();
	if (SequenceHierarchy)
	{
		for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : SequenceHierarchy->AllSubSequenceData())
		{
			ClearGlobalMarkedFrames(Pair.Value.GetSequence());
		}
	}
}

void FSequencerMarkedFrameHelper::ClearGlobalMarkedFrames(UMovieSceneSequence* Sequence)
{
	if (Sequence)
	{
		if (UMovieScene* MovieScene = Sequence->GetMovieScene())
		{
			MovieScene->SetGloballyShowMarkedFrames(false);
		}
	}
}

