// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"

class ISequencer;
class UMovieSceneSequence;
struct FMovieSceneMarkedFrame;

/**
 * Helper class to manage marked frames across a sequence hierarchy.
 */
class FSequencerMarkedFrameHelper
{
public:
	/** Find all marked frames in a sequence hierarchy, relative to the Sequencer's currently focused sequence */
	static void FindGlobalMarkedFrames(ISequencer& Sequencer, TArray<FMovieSceneMarkedFrame>& OutGlobalMarkedFrames);

	/** Clear the setting to show marked frames globally across an entire sequence hierarchy */
	static void ClearGlobalMarkedFrames(ISequencer& Sequencer);

private:
	static void ClearGlobalMarkedFrames(UMovieSceneSequence* Sequence);
};

