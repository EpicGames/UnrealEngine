// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"
#include "EditorModeRegistry.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "IMovieRendererInterface.h"
#include "ISequencer.h"
#include "ISequencerModule.h"
#include "SequencerCommands.h"
#include "ISequencerObjectChangeListener.h"
#include "Sequencer.h"
#include "SequencerCustomizationManager.h"
#include "SequencerEdMode.h"
#include "SequencerObjectChangeListener.h"
#include "SimpleView/Menus/SequencerToolbarExtension.h"
#include "IDetailKeyframeHandler.h"
#include "IDetailTreeNode.h"
#include "IDetailsView.h"
#include "Tree/CurveEditorTreeFilter.h"
#include "AnimatedPropertyKey.h"
#include "MovieSceneSignedObject.h"
#include "MovieSceneSequence.h"
#include "Sections/MovieSceneSubSection.h"

#include "EntitySystem/MovieScenePropertyRegistry.h"
#include "EntitySystem/BuiltInComponentTypes.h"

#include "MVVM/CurveEditorExtension.h"
#include "MVVM/CurveEditorIntegrationExtension.h"
#include "MVVM/FolderModelStorageExtension.h"
#include "MVVM/ObjectBindingModelStorageExtension.h"
#include "MVVM/DecorationModelStorageExtension.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/TrackModelStorageExtension.h"
#include "MVVM/TrackRowModelStorageExtension.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"

#include "MVVM/ViewModels/OutlinerColumns/OutlinerIndicatorColumn.h"
#include "MVVM/ViewModels/OutlinerColumns/DeactivateOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerColumns/LockOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerColumns/MuteOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerColumns/PinOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerColumns/SoloOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerColumns/LabelOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerColumns/EditOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerColumns/AddOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerColumns/NavOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerColumns/KeyFrameOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerColumns/ColorPickerOutlinerColumn.h"

#include "MVVM/ViewModels/OutlinerIndicators/ConditionOutlinerIndicatorBuilder.h"
#include "MVVM/ViewModels/OutlinerIndicators/TimeWarpOutlinerIndicatorBuilder.h"

#include "ToolMenus.h"
#include "ContentBrowserMenuContexts.h"
#include "SequencerUtilities.h"
#include "FileHelpers.h"
#include "LevelSequence.h"
#include "ActorObjectSchema.h"
#include "SkeletalMeshComponentSchema.h"

#include "Misc/CoreDelegates.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"

#include "Algo/RemoveIf.h"
#include "Engine/Font.h"
#include "CanvasTypes.h"

namespace UE::Sequencer::Private
{
	static const TMap<EPropertyKeyedStatus, FName> KeyedStatusStyleNames =
		{
			{ EPropertyKeyedStatus::NotKeyed, "Sequencer.KeyedStatus.NotKeyed" },
			{ EPropertyKeyedStatus::KeyedInOtherFrame, "Sequencer.KeyedStatus.Animated" },
			{ EPropertyKeyedStatus::KeyedInFrame, "Sequencer.KeyedStatus.Keyed" },
			{ EPropertyKeyedStatus::PartiallyKeyed, "Sequencer.KeyedStatus.PartialKey" },
		};
}


#define LOCTEXT_NAMESPACE "SequencerEditor"

// Destructor defined in CPP to avoid having to #include SequencerChannelInterface.h in the main module definition
ISequencerModule::~ISequencerModule()
{
}

ECurveEditorTreeFilterType ISequencerModule::GetSequencerSelectionFilterType()
{
	static ECurveEditorTreeFilterType FilterType = FCurveEditorTreeFilter::RegisterFilterType();
	return FilterType;
}

static TSharedPtr<IDetailKeyframeHandler> GetKeyframeHandler(TWeakPtr<IDetailTreeNode> OwnerTreeNode)
{
	TSharedPtr<IDetailTreeNode> OwnerTreeNodePtr = OwnerTreeNode.Pin();
	if (!OwnerTreeNodePtr.IsValid())
	{
		return TSharedPtr<IDetailKeyframeHandler>();
	}

	TSharedPtr<IDetailsView> DetailsView = OwnerTreeNodePtr->GetNodeDetailsViewSharedPtr();
	if (DetailsView == nullptr)
	{
		return TSharedPtr<IDetailKeyframeHandler>();
	}

	return DetailsView->GetKeyframeHandler();
}

static FSlateIcon GetKeyframeIcon(TWeakPtr<IDetailTreeNode> OwnerTreeNode, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	if (!PropertyHandle.IsValid())
	{
		return FSlateIcon();
	}

	EPropertyKeyedStatus KeyedStatus = EPropertyKeyedStatus::NotKeyed;

	if (TSharedPtr<IDetailKeyframeHandler> KeyframeHandler = GetKeyframeHandler(OwnerTreeNode))
	{
		KeyedStatus = KeyframeHandler->GetPropertyKeyedStatus(*PropertyHandle);
	}

	return FSlateIcon(FAppStyle::GetAppStyleSetName(), UE::Sequencer::Private::KeyedStatusStyleNames[KeyedStatus]);
}

static bool IsKeyframeButtonVisible(TWeakPtr<IDetailTreeNode> OwnerTreeNode, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	TSharedPtr<IDetailKeyframeHandler> KeyframeHandler = GetKeyframeHandler(OwnerTreeNode);
	if (!KeyframeHandler.IsValid() || !PropertyHandle.IsValid())
	{
		return false;
	}

	const UClass* ObjectClass = PropertyHandle->GetOuterBaseClass();
	if (ObjectClass == nullptr)
	{
		return false;
	}

	return KeyframeHandler->IsPropertyKeyable(ObjectClass, *PropertyHandle);
}

static bool IsKeyframeButtonEnabled(TWeakPtr<IDetailTreeNode> OwnerTreeNode)
{
	TSharedPtr<IDetailKeyframeHandler> KeyframeHandler = GetKeyframeHandler(OwnerTreeNode);
	if (!KeyframeHandler.IsValid())
	{
		return false;
	}

	return KeyframeHandler->IsPropertyKeyingEnabled();
}

static void OnAddKeyframeClicked(TWeakPtr<IDetailTreeNode> OwnerTreeNode, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	TSharedPtr<IDetailKeyframeHandler> KeyframeHandler = GetKeyframeHandler(OwnerTreeNode);
	if (!KeyframeHandler.IsValid() || !PropertyHandle.IsValid())
	{
		return;
	}

	KeyframeHandler->OnKeyPropertyClicked(*PropertyHandle);
}

static void RegisterKeyframeExtensionHandler(const FOnGenerateGlobalRowExtensionArgs& Args, TArray<FPropertyRowExtensionButton>& OutExtensionButtons)
{
	// local copy for capturing in handlers below
	TSharedPtr<IPropertyHandle> PropertyHandle = Args.PropertyHandle;
	if (!PropertyHandle.IsValid())
	{
		return;
	}

	TWeakPtr<IDetailTreeNode> OwnerTreeNode = Args.OwnerTreeNode;

	FPropertyRowExtensionButton& CreateKey = OutExtensionButtons.AddDefaulted_GetRef();

	CreateKey.Icon = TAttribute<FSlateIcon>::Create(TAttribute<FSlateIcon>::FGetter::CreateStatic(&GetKeyframeIcon, OwnerTreeNode, PropertyHandle));
	CreateKey.Label = NSLOCTEXT("PropertyEditor", "CreateKey", "Create Key");
	CreateKey.ToolTip = NSLOCTEXT("PropertyEditor", "CreateKeyToolTip", "Add a keyframe for this property.");
	CreateKey.UIAction = FUIAction(
		FExecuteAction::CreateStatic(&OnAddKeyframeClicked, OwnerTreeNode, PropertyHandle),
		FCanExecuteAction::CreateStatic(&IsKeyframeButtonEnabled, OwnerTreeNode),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateStatic(&IsKeyframeButtonVisible, OwnerTreeNode, PropertyHandle)
	);
}

namespace UE::SequencerModule::Private
{
	int32 RenderTimecode(FCanvas* Canvas, int32 X, int32 Y, const FTimecode& Timecode, const FString& SequenceName)
	{
		UFont* Font = FPlatformProperties::SupportsWindowedMode() ? GEngine->GetSmallFont() : GEngine->GetMediumFont();
		const int32 RowHeight = FMath::TruncToInt(Font->GetMaxCharHeight());

		const bool bForceSignDisplay = false;
		const bool bAlwaysDisplaySubframe = true;

		FString TimecodeStr = Timecode.ToString(bForceSignDisplay, bAlwaysDisplaySubframe);
		float CharWidth, CharHeight;
		Font->GetCharSize(TEXT(' '), CharWidth, CharHeight);
		int32 NewX = X - Font->GetStringSize(*SequenceName) - (int32)CharWidth*14;

		Canvas->DrawShadowedString(NewX, Y, *FString::Printf(TEXT("%s TC: %s"), *SequenceName, *TimecodeStr), Font, FColor::Green);
		Y += RowHeight;

		return Y;
	};

	int32 RenderTimeForSequences(FCanvas* Canvas, int32 X, int32 Y, TSharedPtr<ISequencer>& InSequencer)
	{
		const FFrameRate RootDisplayRate = InSequencer->GetRootDisplayRate();
		const FFrameRate LocalDisplayRate = InSequencer->GetFocusedDisplayRate();
		const FQualifiedFrameTime LocalCurrentTime = InSequencer->GetLocalTime();
		const FQualifiedFrameTime RootCurrentTime = InSequencer->GetGlobalTime();

		const FTimecode LocalTimecode = FTimecode::FromFrameTime(LocalCurrentTime.ConvertTo(LocalDisplayRate), LocalDisplayRate);
		const FTimecode RootTimecode = FTimecode::FromFrameTime(RootCurrentTime.ConvertTo(RootDisplayRate), RootDisplayRate);

		const TArray<FMovieSceneSequenceID>& SubSequenceHierarchy = InSequencer->GetSubSequenceHierarchy();
		if (SubSequenceHierarchy.Num() > 0)
		{
			// The first one is the root sequence.
			UMovieSceneSequence* Sequence = FSequencerUtilities::GetMovieSceneSequence(InSequencer, SubSequenceHierarchy[0]);
			check(Sequence);
			FString SequenceName = Sequence->GetDisplayName().ToString();
			Y = RenderTimecode(Canvas, X, Y, RootTimecode, SequenceName);

			if (SubSequenceHierarchy.Num() > 1)
			{
				// The current sequence is always the first in the list.
				Sequence = FSequencerUtilities::GetMovieSceneSequence(InSequencer, SubSequenceHierarchy.Last());
				check(Sequence);
				SequenceName = Sequence->GetDisplayName().ToString();
				Y = RenderTimecode(Canvas, X, Y, LocalTimecode, SequenceName);
			}
		}

		return Y;
	}

	static FOpenSequencerWatcher SequencerWatcher;

	/** Render the sequencer time to the viewport HUD. */
	int32 RenderStatSequencerTime(UWorld* World, FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const FVector* ViewLocation, const FRotator* ViewRotation)
	{
		for (const FOpenSequencerWatcher::FOpenSequencerData& OpenSequencer : SequencerWatcher.OpenSequencers)
		{
			if (TSharedPtr<ISequencer> Sequencer = OpenSequencer.WeakSequencer.Pin())
			{
				Y = RenderTimeForSequences(Canvas, X, Y, Sequencer);
			}
		}
		return Y;
	}

	void InitStatCommands()
	{
		auto StartupComplete = []()
		{
			check(GEngine);
			if (GIsEditor)
			{
				const bool bIsRHS = true;
				GEngine->AddEngineStat(TEXT("STAT_SequencerTimecode"), TEXT("STATCAT_Sequencer"),
									   LOCTEXT("SequencerTimeDisplay", "Displays current timecode, rate, and frame for active sequencer editor."),
									   UEngine::FEngineStatRender::CreateStatic(&RenderStatSequencerTime),
									   nullptr, bIsRHS);
			}
		};

		SequencerWatcher.DoStartup(StartupComplete);
	}
}
/**
 * SequencerModule implementation (private)
 */
class FSequencerModule
	: public ISequencerModule
{
public:

	// ISequencerModule interface

	virtual TSharedRef<ISequencer> CreateSequencer(const FSequencerInitParams& InitParams) override
	{
		using namespace UE::Sequencer;

		TSharedRef<FSequencer> Sequencer = MakeShared<FSequencer>();
		TSharedRef<ISequencerObjectChangeListener> ObjectChangeListener = MakeShared<FSequencerObjectChangeListener>(Sequencer);

		OnPreSequencerInit.Broadcast(Sequencer, ObjectChangeListener, InitParams);

		Sequencer->InitSequencer(InitParams, ObjectChangeListener, TrackEditorDelegates, DecorationEditorDelegates, EditorObjectBindingDelegates, OutlinerColumnDelegates, OutlinerIndicatorDelegates);

		OnSequencerCreated.Broadcast(Sequencer);

		return Sequencer;
	}
	
	virtual FDelegateHandle RegisterTrackEditor( FOnCreateTrackEditor InOnCreateTrackEditor, TArrayView<FAnimatedPropertyKey> AnimatedPropertyTypes ) override
	{
		TrackEditorDelegates.Add( InOnCreateTrackEditor );
		FDelegateHandle Handle = TrackEditorDelegates.Last().GetHandle();
		for (const FAnimatedPropertyKey& Key : AnimatedPropertyTypes)
		{
			PropertyAnimators.Add(Key);
		}

		if (AnimatedPropertyTypes.Num() > 0)
		{
			FAnimatedTypeCache CachedTypes;
			CachedTypes.FactoryHandle = Handle;
			for (const FAnimatedPropertyKey& Key : AnimatedPropertyTypes)
			{
				CachedTypes.AnimatedTypes.Add(Key);
			}
			AnimatedTypeCache.Add(CachedTypes);
		}
		return Handle;
	}

	virtual void UnRegisterTrackEditor( FDelegateHandle InHandle ) override
	{
		TrackEditorDelegates.RemoveAll( [=](const FOnCreateTrackEditor& Delegate){ return Delegate.GetHandle() == InHandle; } );
		int32 CacheIndex = AnimatedTypeCache.IndexOfByPredicate([=](const FAnimatedTypeCache& In) { return In.FactoryHandle == InHandle; });
		if (CacheIndex != INDEX_NONE)
		{
			for (const FAnimatedPropertyKey& Key : AnimatedTypeCache[CacheIndex].AnimatedTypes)
			{
				PropertyAnimators.Remove(Key);
			}
			AnimatedTypeCache.RemoveAtSwap(CacheIndex);
		}
	}

	virtual FDelegateHandle RegisterDecorationEditor(FOnCreateDecorationEditor InOnCreateDecorationEditor) override
	{
		DecorationEditorDelegates.Add(InOnCreateDecorationEditor);
		return DecorationEditorDelegates.Last().GetHandle();
	}

	virtual void UnRegisterDecorationEditor(FDelegateHandle InHandle) override
	{
		DecorationEditorDelegates.RemoveAll([=](const FOnCreateDecorationEditor& Delegate) { return Delegate.GetHandle() == InHandle; });
	}

	virtual FDelegateHandle RegisterTrackModel(FOnCreateTrackModel InCreator) override
	{
		TrackModelDelegates.Add(InCreator);
		return TrackModelDelegates.Last().GetHandle();
	}

	virtual void UnregisterTrackModel(FDelegateHandle InHandle) override
	{
		TrackModelDelegates.RemoveAll([=](const FOnCreateTrackModel& Delegate) { return Delegate.GetHandle() == InHandle; });
	}

	virtual FDelegateHandle RegisterOutlinerColumn(FOnCreateOutlinerColumn InCreator) override {
		OutlinerColumnDelegates.Add(InCreator);
		return OutlinerColumnDelegates.Last().GetHandle();
	}

	virtual void UnregisterOutlinerColumn(FDelegateHandle InHandle) override {
		OutlinerColumnDelegates.RemoveAll([=](const FOnCreateOutlinerColumn& Delegate) { return Delegate.GetHandle() == InHandle; });
	}

	virtual FDelegateHandle RegisterOutlinerIndicator(FOnCreateOutlinerIndicator InCreator) override {
		OutlinerIndicatorDelegates.Add(InCreator);
		return OutlinerIndicatorDelegates.Last().GetHandle();
	}

	virtual void UnregisterOutlinerIndicator(FDelegateHandle InHandle) override {
		OutlinerIndicatorDelegates.RemoveAll([=](const FOnCreateOutlinerIndicator& Delegate) { return Delegate.GetHandle() == InHandle; });
	}

	virtual FDelegateHandle RegisterOnSequencerCreated(FOnSequencerCreated::FDelegate InOnSequencerCreated) override
	{
		return OnSequencerCreated.Add(InOnSequencerCreated);
	}

	virtual void UnregisterOnSequencerCreated(FDelegateHandle InHandle) override
	{
		OnSequencerCreated.Remove(InHandle);
	}

	virtual FDelegateHandle RegisterOnPreSequencerInit(FOnPreSequencerInit::FDelegate InOnPreSequencerInit) override
	{
		return OnPreSequencerInit.Add(InOnPreSequencerInit);
	}

	virtual void UnregisterOnPreSequencerInit(FDelegateHandle InHandle) override
	{
		OnPreSequencerInit.Remove(InHandle);
	}

	virtual FDelegateHandle RegisterEditorObjectBinding(FOnCreateEditorObjectBinding InOnCreateEditorObjectBinding) override
	{
		EditorObjectBindingDelegates.Add(InOnCreateEditorObjectBinding);
		return EditorObjectBindingDelegates.Last().GetHandle();
	}

	virtual void UnRegisterEditorObjectBinding(FDelegateHandle InHandle) override
	{
		EditorObjectBindingDelegates.RemoveAll([=](const FOnCreateEditorObjectBinding& Delegate) { return Delegate.GetHandle() == InHandle; });
	}

	void RegisterMenus()
	{
		UToolMenus* ToolMenus = UToolMenus::Get();
		UToolMenu* Menu = ToolMenus->ExtendMenu("ContentBrowser.AssetContextMenu.LevelSequence");
		if (!Menu)
		{
			return;
		}

		FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
		Section.AddDynamicEntry("SequencerActions", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>();
			if (!Context)
			{
				return;
			}

			if (Context->SelectedAssets.Num() == 1 && Context->SelectedAssets[0].IsInstanceOf(ULevelSequence::StaticClass()))
			{
				const FAssetData LevelSequenceAsset = Context->SelectedAssets[0];
			
				// if this LevelSequence has associated maps, offer to load them
				TArray<FString> AssociatedMaps = FSequencerUtilities::GetAssociatedLevelSequenceMapPackages(LevelSequenceAsset.PackageName);

				if (AssociatedMaps.Num() > 0)
				{
					InSection.AddSubMenu(
						"SequencerOpenMap_Label",
						LOCTEXT("SequencerOpenMap_Label", "Open Map"),
						LOCTEXT("SequencerOpenMap_Tooltip", "Open a map associated with this Level Sequence Asset"),
						FNewMenuDelegate::CreateLambda(
							[AssociatedMaps](FMenuBuilder& SubMenuBuilder)
							{
								for (const FString& AssociatedMap : AssociatedMaps)
								{
									SubMenuBuilder.AddMenuEntry(
										FText::FromString(FPaths::GetBaseFilename(AssociatedMap)),
										FText(),
										FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Levels"),
										FExecuteAction::CreateLambda(
											[AssociatedMap]
											{
												FEditorFileUtils::LoadMap(AssociatedMap);
											}
										)
									);
								}
							}
						),
						false,
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Levels")
					);
				}
			}
		}));

		UE::Sequencer::SimpleView::FSequencerToolbarExtension::AddExtension();
	}

	void RegisterObjectSchemas()
	{
		RegisterObjectSchema(MakeShared<UE::Sequencer::FActorSchema>());
		RegisterObjectSchema(MakeShared<UE::Sequencer::FSkeletalMeshComponentSchema>());
	}

	virtual void StartupModule() override
	{
		using namespace UE::Sequencer;
		using namespace UE::MovieScene;

		if (GIsEditor)
		{
			FEditorModeRegistry::Get().RegisterMode<FSequencerEdMode>(
				FSequencerEdMode::EM_SequencerMode,
				NSLOCTEXT("Sequencer", "SequencerEditMode", "Sequencer Mode"),
				FSlateIcon(),
				false);

			if (UToolMenus::TryGet())
			{
				FSequencerCommands::Register();
				RegisterMenus();
			}
			else
			{
				FCoreDelegates::GetOnPostEngineInit().AddStatic(&FSequencerCommands::Register);
				FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FSequencerModule::RegisterMenus);
			}
			UE::SequencerModule::Private::InitStatCommands();

			FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
			OnGetGlobalRowExtensionHandle = EditModule.GetGlobalRowExtensionDelegate().AddStatic(&RegisterKeyframeExtensionHandler);

			// Register far left gutter columns
			OutlinerIndicatorColumnHandle = RegisterOutlinerColumn(FOnCreateOutlinerColumn::CreateStatic([] { return TSharedRef<IOutlinerColumn>(MakeShared<FOutlinerIndicatorColumn>()); }));

			// Register left gutter columns
			PinOutlinerColumnHandle  = RegisterOutlinerColumn(FOnCreateOutlinerColumn::CreateStatic([]{ return TSharedRef<IOutlinerColumn>(MakeShared<FPinOutlinerColumn>()); }));
			LockOutlinerColumnHandle = RegisterOutlinerColumn(FOnCreateOutlinerColumn::CreateStatic([]{ return TSharedRef<IOutlinerColumn>(MakeShared<FLockOutlinerColumn>()); }));
			DeactivateOutlinerColumnHandle = RegisterOutlinerColumn(FOnCreateOutlinerColumn::CreateStatic([]{ return TSharedRef<IOutlinerColumn>(MakeShared<FDeactivateOutlinerColumn>()); }));

			// Register left gutter columns
			MuteOutlinerColumnHandle = RegisterOutlinerColumn(FOnCreateOutlinerColumn::CreateStatic([]{ return TSharedRef<IOutlinerColumn>(MakeShared<FMuteOutlinerColumn>()); }));
			SoloOutlinerColumnHandle = RegisterOutlinerColumn(FOnCreateOutlinerColumn::CreateStatic([]{ return TSharedRef<IOutlinerColumn>(MakeShared<FSoloOutlinerColumn>()); }));

			// Register center columns
			LabelOutlinerColumnHandle = RegisterOutlinerColumn(FOnCreateOutlinerColumn::CreateStatic([]{ return TSharedRef<IOutlinerColumn>(MakeShared<FLabelOutlinerColumn>()); }));
			EditOutlinerColumnHandle  = RegisterOutlinerColumn(FOnCreateOutlinerColumn::CreateStatic([]{ return TSharedRef<IOutlinerColumn>(MakeShared<FEditOutlinerColumn>()); }));
			AddOutlinerColumnHandle   = RegisterOutlinerColumn(FOnCreateOutlinerColumn::CreateStatic([]{ return TSharedRef<IOutlinerColumn>(MakeShared<FAddOutlinerColumn>()); }));

			// Register right gutter columns
			KeyFrameOutlinerColumnHandle     = RegisterOutlinerColumn(FOnCreateOutlinerColumn::CreateStatic([]{ return TSharedRef<IOutlinerColumn>(MakeShared<FKeyFrameOutlinerColumn>()); }));
			NavOutlinerColumnHandle          = RegisterOutlinerColumn(FOnCreateOutlinerColumn::CreateStatic([]{ return TSharedRef<IOutlinerColumn>(MakeShared<FNavOutlinerColumn>()); }));
			ColorPickerOutlinerColumnHandle  = RegisterOutlinerColumn(FOnCreateOutlinerColumn::CreateStatic([]{ return TSharedRef<IOutlinerColumn>(MakeShared<FColorPickerOutlinerColumn>()); }));

			// Register outliner indicator items
			ConditionOutlinerIndicatorHandle = RegisterOutlinerIndicator(FOnCreateOutlinerIndicator::CreateStatic([] { return TSharedRef<IOutlinerIndicatorBuilder>(MakeShared<FConditionOutlinerIndicatorBuilder>()); }));
			TimeWarpOutlinerIndicatorHandle = RegisterOutlinerIndicator(FOnCreateOutlinerIndicator::CreateStatic([] { return TSharedRef<IOutlinerIndicatorBuilder>(MakeShared<FTimeWarpOutlinerIndicatorBuilder>()); }));

			RegisterObjectSchemas();
		}

		FSequenceModel::CreateExtensionsEvent.AddLambda(
			[&](TSharedPtr<FEditorViewModel> InEditor, TSharedPtr<FSequenceModel> InModel)
			{
				InModel->AddDynamicExtension(FFolderModelStorageExtension::ID);
				InModel->AddDynamicExtension(FObjectBindingModelStorageExtension::ID);
				InModel->AddDynamicExtension(FTrackModelStorageExtension::ID, TrackModelDelegates);
				InModel->AddDynamicExtension(FTrackRowModelStorageExtension::ID);
				InModel->AddDynamicExtension(FSectionModelStorageExtension::ID);
				InModel->AddDynamicExtension(FDecorationModelStorageExtension::ID);
			}
		);

		ObjectBindingContextMenuExtensibilityManager = MakeShareable( new FExtensibilityManager );
		AddTrackMenuExtensibilityManager = MakeShareable( new FExtensibilityManager );
		ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);
		ActionsMenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		ViewMenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		SidebarExtensibilityManager = MakeShareable(new FExtensibilityManager);

		SequencerCustomizationManager = MakeShareable(new FSequencerCustomizationManager);
	}

	virtual void ShutdownModule() override
	{
		UE::Sequencer::SimpleView::FSequencerToolbarExtension::RemoveExtension();

		if (GIsEditor)
		{
			UMovieSceneSignedObject::SetDeferredHandler(nullptr);

			FSequencerCommands::Unregister();

			if (FPropertyEditorModule* EditModulePtr = FModuleManager::Get().GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
			{
				EditModulePtr->GetGlobalRowExtensionDelegate().Remove(OnGetGlobalRowExtensionHandle);
			}

			FEditorModeRegistry::Get().UnregisterMode(FSequencerEdMode::EM_SequencerMode);

			// unregister outliner columns
			UnregisterOutlinerColumn(OutlinerIndicatorColumnHandle);
			UnregisterOutlinerColumn(DeactivateOutlinerColumnHandle);
			UnregisterOutlinerColumn(PinOutlinerColumnHandle);
			UnregisterOutlinerColumn(MuteOutlinerColumnHandle);
			UnregisterOutlinerColumn(LockOutlinerColumnHandle);
			UnregisterOutlinerColumn(SoloOutlinerColumnHandle);
			UnregisterOutlinerColumn(LabelOutlinerColumnHandle);
			UnregisterOutlinerColumn(EditOutlinerColumnHandle);
			UnregisterOutlinerColumn(AddOutlinerColumnHandle);
			UnregisterOutlinerColumn(KeyFrameOutlinerColumnHandle);
			UnregisterOutlinerColumn(NavOutlinerColumnHandle);
			UnregisterOutlinerColumn(ColorPickerOutlinerColumnHandle);

			// unregister outliner indicator items
			UnregisterOutlinerIndicator(ConditionOutlinerIndicatorHandle);
			UnregisterOutlinerIndicator(TimeWarpOutlinerIndicatorHandle);
		}
	}

	virtual void RegisterPropertyAnimator(FAnimatedPropertyKey Key) override
	{
		PropertyAnimators.Add(Key);
	}

	virtual void UnRegisterPropertyAnimator(FAnimatedPropertyKey Key) override
	{
		PropertyAnimators.Remove(Key);
	}

	virtual bool CanAnimateProperty(FProperty* Property) override
	{
		if (PropertyAnimators.Contains(FAnimatedPropertyKey::FromProperty(Property)))
		{
			return true;
		}

		const UE::MovieScene::FPropertyRegistry& PropertyRegistry = UE::MovieScene::FBuiltInComponentTypes::Get()->PropertyRegistry;

		// Find the property that applies to this
		for (const UE::MovieScene::FPropertyDefinition& PropertyDefinition : PropertyRegistry.GetProperties())
		{
			if (PropertyDefinition.Handler->SupportsProperty(PropertyDefinition, *Property))
			{
				return true;
			}
		}


		FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property);

		// Check each level of the property hierarchy
		FFieldClass* PropertyType = Property->GetClass();
		while (PropertyType && PropertyType != FProperty::StaticClass())
		{
			FAnimatedPropertyKey Key = FAnimatedPropertyKey::FromPropertyTypeName(PropertyType->GetFName());

			// For object properties, check each parent type of the object (ie, so a track that animates UBaseClass ptrs can be used with a UDerivedClass property)
			UClass* ClassType = (ObjectProperty && ObjectProperty->PropertyClass) ? ObjectProperty->PropertyClass->GetSuperClass() : nullptr;
			while (ClassType)
			{
				Key.ObjectTypeName = ClassType->GetFName();
				if (PropertyAnimators.Contains(Key))
				{
					return true;
				}
				ClassType = ClassType->GetSuperClass();
			}

			Key.ObjectTypeName = NAME_None;
			if (PropertyAnimators.Contains(Key))
			{
				return true;
			}

			// Look at the property's super class
			PropertyType = PropertyType->GetSuperClass();
		}

		return false;
	}

	virtual TSharedPtr<FExtensibilityManager> GetObjectBindingContextMenuExtensibilityManager() const override { return ObjectBindingContextMenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetAddTrackMenuExtensibilityManager() const override { return AddTrackMenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() const override { return ToolBarExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetActionsMenuExtensibilityManager() const override { return ActionsMenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetViewMenuExtensibilityManager() const override { return ViewMenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetSidebarExtensibilityManager() const override { return SidebarExtensibilityManager; }

	virtual TSharedPtr<FSequencerCustomizationManager> GetSequencerCustomizationManager() const override { return SequencerCustomizationManager; }

	virtual void RegisterObjectSchema(TSharedPtr<UE::Sequencer::IObjectSchema> InSchema) override
	{
		ObjectSchemas.Add(InSchema);
	}

	virtual void UnregisterObjectSchema(TSharedPtr<UE::Sequencer::IObjectSchema> InSchema) override
	{
		ObjectSchemas.Remove(InSchema);
	}

	virtual TSharedPtr<UE::Sequencer::IObjectSchema> FindObjectSchema(const UObject* Object) const override
	{
		using namespace UE::Sequencer;

		FObjectSchemaRelevancy Relevancy;
		TSharedPtr<IObjectSchema> RelevantSchema;

		for (const TSharedPtr<IObjectSchema>& Schema : ObjectSchemas)
		{
			FObjectSchemaRelevancy ThisRelevancy = Schema->GetRelevancy(Object);
			if (ThisRelevancy > Relevancy)
			{
				Relevancy = ThisRelevancy;
				RelevantSchema = Schema;
			}
		}
		return RelevantSchema;
	}

	virtual FDelegateHandle RegisterMovieRenderer(TUniquePtr<IMovieRendererInterface>&& InMovieRenderer) override
	{
		FDelegateHandle NewHandle(FDelegateHandle::GenerateNewHandle);
		MovieRenderers.Add(FMovieRendererEntry{ NewHandle, MoveTemp(InMovieRenderer) });
		return NewHandle;
	}

	virtual void UnregisterMovieRenderer(FDelegateHandle InDelegateHandle) override
	{
		MovieRenderers.RemoveAll([InDelegateHandle](const FMovieRendererEntry& In){ return In.Handle == InDelegateHandle; });
	}

	virtual IMovieRendererInterface* GetMovieRenderer(const FString& InMovieRendererName) override
	{
		for (const FMovieRendererEntry& MovieRenderer : MovieRenderers)
		{
			if (MovieRenderer.Renderer->GetDisplayName() == InMovieRendererName)
			{
				return MovieRenderer.Renderer.Get();
			}
		}

		return nullptr;
	}

	virtual TArray<FString> GetMovieRendererNames() override
	{
		TArray<FString> MovieRendererNames;
		for (const FMovieRendererEntry& MovieRenderer : MovieRenderers)
		{
			MovieRendererNames.Add(MovieRenderer.Renderer->GetDisplayName());
		}
		return MovieRendererNames;
	}

	TArrayView<const TSharedPtr<UE::Sequencer::IObjectSchema>> GetObjectSchemas() const override
	{
		return ObjectSchemas;
	}

private:

	TSet<FAnimatedPropertyKey> PropertyAnimators;

	/** List of auto-key handler delegates sequencers will execute when they are created */
	TArray< FOnCreateTrackEditor > TrackEditorDelegates;

	/** List of decoration editor factory delegates sequencers will execute when they are created */
	TArray< FOnCreateDecorationEditor > DecorationEditorDelegates;

	/** List of object binding handler delegates sequencers will execute when they are created */
	TArray< FOnCreateEditorObjectBinding > EditorObjectBindingDelegates;

	/** List of track model creators */
	TArray<FOnCreateTrackModel> TrackModelDelegates;

	/** List of outliner column creators */
	TArray<FOnCreateOutlinerColumn> OutlinerColumnDelegates;

	/** List of outliner indicator item creators */
	TArray<FOnCreateOutlinerIndicator> OutlinerIndicatorDelegates;

	TArray<TSharedPtr<UE::Sequencer::IObjectSchema>> ObjectSchemas;

	/** Global details row extension delegate; */
	FDelegateHandle OnGetGlobalRowExtensionHandle;

	/** Multicast delegate used to notify others of sequencer initialization params and allow modification. */
	FOnPreSequencerInit OnPreSequencerInit;

	/** Multicast delegate used to notify others of sequencer creations */
	FOnSequencerCreated OnSequencerCreated;

	struct FAnimatedTypeCache
	{
		FDelegateHandle FactoryHandle;
		TArray<FAnimatedPropertyKey, TInlineAllocator<4>> AnimatedTypes;
	};

	/** Map of all track editor factories to property types that they have registered to animated */
	TArray<FAnimatedTypeCache> AnimatedTypeCache;

	TSharedPtr<FExtensibilityManager> ObjectBindingContextMenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> AddTrackMenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ActionsMenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ViewMenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> SidebarExtensibilityManager;

	TSharedPtr<FSequencerCustomizationManager> SequencerCustomizationManager;

	struct FMovieRendererEntry
	{
		FDelegateHandle Handle;
		TUniquePtr<IMovieRendererInterface> Renderer;
	};

	/** Array of movie renderers */
	TArray<FMovieRendererEntry> MovieRenderers;

	// Outliner Column Delegate Handles
	FDelegateHandle OutlinerIndicatorColumnHandle;
	FDelegateHandle DeactivateOutlinerColumnHandle;
	FDelegateHandle PinOutlinerColumnHandle;
	FDelegateHandle MuteOutlinerColumnHandle;
	FDelegateHandle LockOutlinerColumnHandle;
	FDelegateHandle SoloOutlinerColumnHandle;
	FDelegateHandle LabelOutlinerColumnHandle;
	FDelegateHandle EditOutlinerColumnHandle;
	FDelegateHandle AddOutlinerColumnHandle;
	FDelegateHandle KeyFrameOutlinerColumnHandle;
	FDelegateHandle NavOutlinerColumnHandle;
	FDelegateHandle ColorPickerOutlinerColumnHandle;

	// Outliner Indicator Item Delegate Handles
	FDelegateHandle ConditionOutlinerIndicatorHandle;
	FDelegateHandle TimeWarpOutlinerIndicatorHandle;
};

IMPLEMENT_MODULE(FSequencerModule, Sequencer);

#undef LOCTEXT_NAMESPACE
