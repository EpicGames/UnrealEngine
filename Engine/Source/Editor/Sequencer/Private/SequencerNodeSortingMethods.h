// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Widgets/Views/STableRow.h"

/**
* This sorting delegate sorts based on the vertical position of the node represented by where it falls
* in the TreeView, ignoring collapsed nodes. This allows you to get a sorting order that matches the visual
* representation which is more logical for the user.
*/
//struct FDisplayNodeTreePositionSorter
//{
//	bool operator()(const TSharedRef<FSequencerDisplayNode>& A, const TSharedRef <FSequencerDisplayNode>& B) const
//	{
//		return A->GetVirtualTop() < B->GetVirtualTop();
//	}
//};


/**
* Sorts the supplied unsorted nodes and inserts them into the existing sorted nodes before assigning
* a sorting order to the newly combined array. Inserted nodes can optionally be relative to an existing
* node that comes from the ExistingSortedNodes array; this allows you to sort them and then insert them
* mid array.
*
* This function modifies the sorting order of all nodes passed in from both lists, but does not change
* their actual hierarchy.
*
* @param UnsortedNodesToInsert A list of nodes to be sorted by Predicate
* @param ExistingSortedNdoes A list of nodes that are already sorted that you want to have their indexes
* adjusted for having the unsorted nodes inserted among them.
* @param ItemDropZone Specifies the offset for the RelativeToNode (above, below or at the end). This is
* useful as many of the places this gets called from have this information already.
* @param Predicate Sorting predicate to sort UnsortedNodesToSort before insertion.
* @param RelativeToNode An optional node from the ExistingSortedNodes list to insert before/after. If null
* unsorted nodes will be inserted at the start and all nodes in ExistingSortedNodes will have their sorting
* order offset by that many.
*/
//template <class PREDICATE_CLASS>
//void SortAndSetSortingOrder(const TArray<TSharedRef<FSequencerDisplayNode>>& UnsortedNodesToInsert, const TArray<TSharedRef<FSequencerDisplayNode>>& ExistingSortedNodes, TOptional<EItemDropZone> ItemDropZone, const PREDICATE_CLASS& Predicate, TSharedPtr<FSequencerDisplayNode> RelativeToNode)
//{
//	TArray<TSharedRef<FSequencerDisplayNode>> NewSortingOrder = ExistingSortedNodes;
//
//	// ExistingSortedNodes may contain nodes that we do not consider for sorting order (spacers) or nodes we wish to re-sort.
//	TArray<TSharedRef<FSequencerDisplayNode>> ChildrenToRemove;
//	for (TSharedRef<FSequencerDisplayNode> Node : ExistingSortedNodes)
//	{
//		// Discard any of our children that aren't folders, objects or tracks (this ignores spacers, etc.)
//		if (Node->GetType() != ESequencerNode::Folder && Node->GetType() != ESequencerNode::Object && Node->GetType() != ESequencerNode::Track)
//		{
//			ChildrenToRemove.Add(Node);
//			continue;
//		}
//
//		// If we're trying to sort this child remove it from our searchable list as well.
//		if (UnsortedNodesToInsert.Contains(Node))
//		{
//			ChildrenToRemove.Add(Node);
//		}
//	}
//
//	for (TSharedRef<FSequencerDisplayNode> Node : ChildrenToRemove)
//	{
//		NewSortingOrder.Remove(Node);
//	}
//
//	// Now get our index and insert the dragged nodes either before/after/at the end of that index depending on the ItemDropZone.
//	int32 DropAdjustedIndex = NewSortingOrder.Num();
//	int32 RelativeToIndex = NewSortingOrder.IndexOfByKey(RelativeToNode);
//	if (RelativeToIndex >= 0 && ItemDropZone.IsSet())
//	{
//		switch (ItemDropZone.GetValue())
//		{
//		case EItemDropZone::AboveItem:
//			DropAdjustedIndex = RelativeToIndex;
//			break;
//		case EItemDropZone::BelowItem:
//			DropAdjustedIndex = RelativeToIndex + 1;
//			break;
//			// Folders will emit this drop zone which means we want to put it at the end of the folder.
//		case EItemDropZone::OntoItem:
//			DropAdjustedIndex = NewSortingOrder.Num();
//			break;
//		}
//	}
//
//	// Allow our caller to specify the sorting order of new nodes. This allows us to sort incoming nodes by category or vertical position on the treeview, etc.
//	TArray<TSharedRef<FSequencerDisplayNode>> SortedNodes = UnsortedNodesToInsert;
//	SortedNodes.Sort(Predicate);
//
//	// Insert our sorted nodes into our child list so that we have the new absolute order for all items.
//	NewSortingOrder.Insert(SortedNodes, DropAdjustedIndex);
//
//	// And then re-assign the sorting order index of everything in the folder according to the new list.
//	for (int i = 0; i < NewSortingOrder.Num(); i++)
//	{
//		NewSortingOrder[i]->ModifyAndSetSortingOrder(i);
//	}
//};
