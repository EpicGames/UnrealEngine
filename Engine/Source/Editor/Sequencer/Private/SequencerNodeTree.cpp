// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerNodeTree.h"

#include "Filters/Evaluation/FilterEvaluator.h"
#include "Filters/SequencerFilterBar.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "MVVM/Selection/Selection.h"
#include "MovieSceneBinding.h"
#include "GameFramework/Actor.h"
#include "MovieScene.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/ICurveEditorTreeItemExtension.h"
#include "MVVM/Extensions/IPinnableExtension.h"
#include "MVVM/Extensions/ISoloableExtension.h"
#include "MVVM/Extensions/IMutableExtension.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/ViewModels/FolderModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/OutlinerSpacer.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/CurveEditorIntegrationExtension.h"
#include "MVVM/ObjectBindingModelStorageExtension.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/TrackModelStorageExtension.h"
#include "IKeyArea.h"
#include "ISequencerSection.h"
#include "MovieSceneSequence.h"
#include "Sequencer.h"
#include "MovieSceneFolder.h"
#include "ISequencerTrackEditor.h"
#include "Widgets/Views/STableRow.h"
#include "CurveEditor.h"
#include "Channels/MovieSceneChannel.h"
#include "ScopedTransaction.h"
#include "SequencerUtilities.h"
#include "SequencerLog.h"
#include "SequencerCommonHelpers.h"
#include "SSequencer.h"

FSequencerNodeTree::~FSequencerNodeTree()
{
}

FSequencerNodeTree::FSequencerNodeTree(FSequencer& InSequencer)
	: Sequencer(InSequencer)
	, FilterEvaluator(MakeShared<UE::Sequencer::FFilterEvaluator>())
{
}

TSharedPtr<UE::Sequencer::FObjectBindingModel> FSequencerNodeTree::FindObjectBindingNode(const FGuid& BindingID) const
{
	using namespace UE::Sequencer;

	const FObjectBindingModelStorageExtension* ObjectBindingStorage = RootNode->CastDynamic<FObjectBindingModelStorageExtension>();
	if (ObjectBindingStorage)
	{
		TSharedPtr<FObjectBindingModel> Model = ObjectBindingStorage->FindModelForObjectBinding(BindingID);
		return Model;
	}

	return nullptr;
}

bool FSequencerNodeTree::UpdateFiltersOnTrackValueChanged()
{
	return FilterEvaluator->UpdateFiltersOnTrackValueChanged();
}

void FSequencerNodeTree::Update()
{
	using namespace UE::Sequencer;

	FViewModelHierarchyOperation UpdateOp(RootNode->GetSharedData());

	if (!ensure(RootNode))
	{
		return;
	}

	TSharedPtr<FSequenceModel> SequenceModel = RootNode->CastThisShared<FSequenceModel>();
	check(SequenceModel);

	// Cache pinned state of nodes, this needs to happen before UpdateFilters() below as some filters will look at the pinned state of child nodes
	FPinnableExtensionShim::UpdateCachedPinnedState(RootNode);

	// Re-filter the tree after updating 
	// @todo sequencer: Newly added sections may need to be visible even when there is a filter
	constexpr bool bForce = true;
	FilterEvaluator->UpdateFilters(bForce);

	// Sort all nodes
	const bool bIncludeRootNode = true;
	for (TSharedPtr<ISortableExtension> SortableChild : RootNode->GetDescendantsOfType<ISortableExtension>(bIncludeRootNode))
	{
		SortableChild->SortChildren();
	}

	// Avoid updating geometry during an undo/redo, as we may have changed the nodes and they won't get updated until next frame.
	// Any deleted nodes will be present in the hierarchy but garbage.
	if (!GIsTransacting)
	{
		// Update all virtual geometries
		// This must happen after the sorting
		IGeometryExtension::UpdateVirtualGeometry(0.f, RootNode);
	}

	// This will cause relevant systems to update. E.g. curve editor, if enabled, will updated based on the new filtered hierarchy.
	OnUpdatedDelegate.Broadcast();
}

UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension> FindNodeWithPath(UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension> InNode, const FString& NodePath)
{
	using namespace UE::Sequencer;

	if (!InNode)
	{
		return nullptr;
	}

	FString HeadPath, TailPath;
	const bool bHasDelimiter = NodePath.Split(".", &HeadPath, &TailPath);
	const FString NodeIdentifier = InNode->GetIdentifier().ToString();

	if (bHasDelimiter)
	{
		if (NodeIdentifier != HeadPath)
		{
			// The node we're looking for is not in this sub-branch.
			return nullptr;
		}
	}
	else
	{
		// NodePath is just a name, so simply check with our node's name.
		return (NodeIdentifier == NodePath) ? InNode : nullptr;
	}

	check(bHasDelimiter && !TailPath.IsEmpty());

	for (TViewModelPtr<IOutlinerExtension> Child : InNode.AsModel()->GetChildrenOfType<IOutlinerExtension>())
	{
		TViewModelPtr<IOutlinerExtension> FoundNode = FindNodeWithPath(Child, TailPath);
		if (FoundNode)
		{
			return FoundNode;
		}
	}

	return nullptr;
}


UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension> FSequencerNodeTree::GetNodeAtPath(const FString& NodePath) const
{
	using namespace UE::Sequencer;

	for (const TViewModelPtr<IOutlinerExtension>& RootChild : RootNode->GetChildrenOfType<IOutlinerExtension>())
	{
		TViewModelPtr<IOutlinerExtension> NodeAtPath = FindNodeWithPath(RootChild, NodePath);
		if (NodeAtPath)
		{
			return NodeAtPath;
		}
	}
	return nullptr;
}

void FSequencerNodeTree::SetRootNode(const UE::Sequencer::FViewModelPtr& InRootNode)
{
	ensureMsgf(!RootNode, TEXT("Re-assinging the root node is currently an undefined behavior"));
	RootNode = InRootNode;
}

UE::Sequencer::FViewModelPtr FSequencerNodeTree::GetRootNode() const
{
	return RootNode;
}

TArray<UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>> FSequencerNodeTree::GetRootNodes() const
{
	using namespace UE::Sequencer;

	TArray<TViewModelPtr<IOutlinerExtension>> RootNodes;
	for (const TViewModelPtr<IOutlinerExtension>& Child : RootNode->GetChildrenOfType<IOutlinerExtension>())
	{
		RootNodes.Add(Child);
	}
	return RootNodes;
}

void FSequencerNodeTree::ClearCustomSortOrders()
{
	using namespace UE::Sequencer;

	const bool bIncludeRootNode = true;
	for (TSharedPtr<FViewModel> Child : RootNode->GetDescendants(bIncludeRootNode))
	{
		if (ISortableExtension* SortableExtension = Child->CastThis<ISortableExtension>())
		{
			SortableExtension->SetCustomOrder(-1);
		}
	}
}

void FSequencerNodeTree::SortAllNodesAndDescendants()
{
	using namespace UE::Sequencer;

	const bool bIncludeRootNode = true;
	TArray<ISortableExtension*> SortableChildren;
	for (TSharedPtr<FViewModel> Child : RootNode->GetDescendants(bIncludeRootNode))
	{
		if (ISortableExtension* SortableExtension = Child->CastThis<ISortableExtension>())
		{
			SortableChildren.Add(SortableExtension);
		}
	}
	for (ISortableExtension* SortableChild : SortableChildren)
	{
		SortableChild->SortChildren();
	}
}

void FSequencerNodeTree::SaveExpansionState(const UE::Sequencer::FViewModel& Node, bool bExpanded)
{
	using namespace UE::Sequencer;

	// @todo Sequencer - This should be moved to the sequence level
	UMovieScene* MovieScene = Sequencer.GetFocusedMovieSceneSequence()->GetMovieScene();
	FMovieSceneEditorData& EditorData = MovieScene->GetEditorData();

	EditorData.ExpansionStates.Add(IOutlinerExtension::GetPathName(Node), FMovieSceneExpansionState(bExpanded));
}

TOptional<bool> FSequencerNodeTree::GetSavedExpansionState( const FViewModel& Node )
{
	using namespace UE::Sequencer;

	UMovieScene* MovieScene = Sequencer.GetFocusedMovieSceneSequence()->GetMovieScene();
	FMovieSceneEditorData& EditorData = MovieScene->GetEditorData();
	FMovieSceneExpansionState* ExpansionState = EditorData.ExpansionStates.Find(IOutlinerExtension::GetPathName(Node));

	return ExpansionState ? ExpansionState->bExpanded : TOptional<bool>();
}

void FSequencerNodeTree::SavePinnedState(const UE::Sequencer::FViewModel& Node, bool bPinned)
{
	using namespace UE::Sequencer;

	UMovieScene* MovieScene = Sequencer.GetFocusedMovieSceneSequence()->GetMovieScene();
	FMovieSceneEditorData& EditorData = MovieScene->GetEditorData();

	if (bPinned)
	{
		EditorData.PinnedNodes.AddUnique(IOutlinerExtension::GetPathName(Node));
	}
	else
	{
		EditorData.PinnedNodes.RemoveSingle(IOutlinerExtension::GetPathName(Node));
	}
}


bool FSequencerNodeTree::GetSavedPinnedState(const UE::Sequencer::FViewModel& Node) const
{
	using namespace UE::Sequencer;

	UMovieScene* MovieScene = Sequencer.GetFocusedMovieSceneSequence()->GetMovieScene();
	FMovieSceneEditorData& EditorData = MovieScene->GetEditorData();
	bool bPinned = EditorData.PinnedNodes.Contains(IOutlinerExtension::GetPathName(Node));

	return bPinned;
}

bool FSequencerNodeTree::IsNodeFiltered(const TSharedPtr<UE::Sequencer::FViewModel>& Node) const
{
	using namespace UE::Sequencer;

	const TViewModelPtr<IOutlinerExtension> OutlinerItem = CastViewModel<IOutlinerExtension>(Node);
	return OutlinerItem && !OutlinerItem->IsFilteredOut();
}

void FSequencerNodeTree::RequestFilterUpdate()
{
	FilterEvaluator->RequestFilterUpdate(Sequencer.GetFilterBar());
}

bool FSequencerNodeTree::NeedsFilterUpdate() const
{
	return FilterEvaluator->NeedsFilterUpdate();
}

TSharedPtr<UE::Sequencer::FSectionModel> FSequencerNodeTree::GetSectionModel(const UMovieSceneSection* Section) const
{
	using namespace UE::Sequencer;

	FSectionModelStorageExtension* SectionStorage = RootNode->CastThis<FSectionModelStorageExtension>();
	if (ensure(SectionStorage))
	{
		return SectionStorage->FindModelForSection(Section);
	}
	return nullptr;
}

int32 FSequencerNodeTree::GetTotalDisplayNodeCount() const 
{ 
	return Sequencer.GetFilterBar()->GetFilterData().GetTotalNodeCount();
}

int32 FSequencerNodeTree::GetFilteredDisplayNodeCount() const 
{ 
	return Sequencer.GetFilterBar()->GetFilterData().GetDisplayNodeCount();
}

void FSequencerNodeTree::NodeGroupsCollectionChanged()
{
	if (Sequencer.GetFocusedMovieSceneSequence()->GetMovieScene()->GetNodeGroups().HasAnyActiveFilter() || bFilteringOnNodeGroups)
	{
		RequestFilterUpdate();
	}
}

void FSequencerNodeTree::GetAllNodes(TArray<TSharedPtr<UE::Sequencer::FViewModel>>& OutNodes) const
{
	using namespace UE::Sequencer;

	const bool bIncludeRootNode = false;
	for (const FViewModelPtr& It : RootNode->GetDescendants(bIncludeRootNode))
	{
		OutNodes.Add(It.AsModel());
	}
}

void FSequencerNodeTree::GetAllNodes(TArray<TSharedRef<UE::Sequencer::FViewModel>>& OutNodes) const
{
	using namespace UE::Sequencer;

	const bool bIncludeRootNode = false;
	for (const FViewModelPtr& It : RootNode->GetDescendants(bIncludeRootNode))
	{
		OutNodes.Add(It.AsModel().ToSharedRef());
	}
}
