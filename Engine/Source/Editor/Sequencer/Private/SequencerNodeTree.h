// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "Misc/Guid.h"
#include "UObject/ObjectKey.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/OutlinerItemModel.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MovieSceneSequence.h"

namespace UE::Sequencer
{
class FFilterEvaluator;
}

class FSequencer;
class FSequencerFilterBar;
class FSequencerTrackFilterCollection;
class FSequencerTrackFilter_Level;
class ISequencerTrackEditor;
class UMovieScene;
class UMovieSceneFolder;
class UMovieSceneTrack;
struct FMovieSceneBinding;

/**
 * Represents a tree of sequencer display nodes, used to populate the Sequencer UI with MovieScene data
 */
class FSequencerNodeTree : public TSharedFromThis<FSequencerNodeTree>
{
public:

	DECLARE_MULTICAST_DELEGATE(FOnUpdated);

	using FSectionModel = UE::Sequencer::FSectionModel;
	using FViewModel = UE::Sequencer::FViewModel;
	using FObjectBindingModel = UE::Sequencer::FObjectBindingModel;

public:

	FSequencerNodeTree(FSequencer& InSequencer);
	
	~FSequencerNodeTree();

	/**
	 * Updates the tree with sections from a MovieScene
	 */
	void Update();

	/**
	 * Set this tree's symbolic root node
	 */
	void SetRootNode(const UE::Sequencer::FViewModelPtr& InRootNode);

	/**
	 * Access this tree's symbolic root node
	 */
	UE::Sequencer::FViewModelPtr GetRootNode() const;

	/**
	 * @return The root nodes of the tree
	 */
	TArray<UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>> GetRootNodes() const;

	/** 
	 * Checks if filters should be updated on track value changes, and if so updates them.
	 * @return Whether the filtered node list was modified
	 */
	bool UpdateFiltersOnTrackValueChanged();

	/**
	 * Returns whether or not a node is filtered
	 *
	 * @param Node	The node to check if it is filtered
	 */
	bool IsNodeFiltered(const TSharedPtr<FViewModel>& Node) const;

	/** Schedules an update of the Sequencer outliner filter bar. */
	void RequestFilterUpdate();

	/** @return Whether any filter bar needs to be filtered. */
	bool NeedsFilterUpdate() const;


	/** Called when the active MovieScene's node group colletion has been modifed */
	void NodeGroupsCollectionChanged();

	/**
	 * @return All nodes in a flat array
	 */
	void GetAllNodes(TArray<TSharedPtr<FViewModel>>& OutNodes) const;

	/**
	 * @return All nodes in a flat array
	 */
	void GetAllNodes(TArray<TSharedRef<FViewModel>>& OutNodes) const;

	/** Gets the parent sequencer of this tree */
	FSequencer& GetSequencer() {return Sequencer;}

	/**
	 * Saves the expansion state of a display node
	 *
	 * @param Node		The node whose expansion state should be saved
	 * @param bExpanded	The new expansion state of the node
	 */
	void SaveExpansionState( const FViewModel& Node, bool bExpanded );

	/**
	 * Gets the saved expansion state of a display node
	 *
	 * @param Node	The node whose expansion state may have been saved
	 * @return true if the node should be expanded, false otherwise, unset if it wasn't saved
	 */
	TOptional<bool> GetSavedExpansionState( const FViewModel& Node );

	/**
	 * Saves the pinned state of a display node
	 *
	 * @param Node		The node whose pinned state should be saved
	 * @param bPinned	The new pinned state of the node
	 */
	void SavePinnedState( const FViewModel& Node, bool bPinned );

	/**
	 * Gets the saved pinned state of a display node
	 *
	 * @param Node	The node whose pinned state may have been saved
	 * @return true if the node should be pinned, false otherwise	
	 */
	bool GetSavedPinnedState( const FViewModel& Node ) const;

	/*
	 * Find the object binding node with the specified GUID
	 */
	TSharedPtr<FObjectBindingModel> FindObjectBindingNode(const FGuid& BindingID) const;

	/*
	 * Gets a multicast delegate which is called whenever the node tree has been updated.
	 */
	FOnUpdated& OnUpdated() { return OnUpdatedDelegate; }

	/** Clears all custom sort orders so that sorting will be by category then alphabetically. */
	void ClearCustomSortOrders();

	/** Sorts all nodes and their descendants*/
	void SortAllNodesAndDescendants();

	/**
	 * Attempt to get a node at the specified path
	 *
	 * @param NodePath The path of the node to search for
	 * @return The node located at NodePath, or nullptr if not found
	 */
	UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension> GetNodeAtPath(const FString& NodePath) const;
	
	/** @return  */
	const TSharedRef<UE::Sequencer::FFilterEvaluator>& GetFilterEvaluator() const { return FilterEvaluator; }

public:

	/**
	 * Finds the section handle relating to the specified section object, or Null if one was not found (perhaps, if it's filtered away)
	 */
	TSharedPtr<FSectionModel> GetSectionModel(const UMovieSceneSection* Section) const;

public:

	int32 GetTotalDisplayNodeCount() const;
	int32 GetFilteredDisplayNodeCount() const;

private:

	/** Symbolic root node that contains the actual displayed root nodes as children */
	UE::Sequencer::FViewModelPtr RootNode;

	/** Sequencer interface */
	FSequencer& Sequencer;
	
	/** Knows about all filter bars that are dirty and handles evaluating them. */
	const TSharedRef<UE::Sequencer::FFilterEvaluator> FilterEvaluator;
	
	/** A multicast delegate which is called whenever the node tree has been updated. */
	FOnUpdated OnUpdatedDelegate;

	bool bFilteringOnNodeGroups;
};
