// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerObjectChangeListener.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "PropertyPermissionList.h"
#include "IPropertyChangeListener.h"
#include "MovieSceneSequence.h"
#include "EntitySystem/MovieScenePropertySystemTypes.h"
#include "ScopedTransaction.h"

DEFINE_LOG_CATEGORY_STATIC(LogSequencerTools, Log, All);

namespace UE::MovieScene
{
	enum class EPropertyAccessibility : uint8
	{
		Inaccessible,
		ExplicitlyAccessible,
		Unknown,
	};
	static EPropertyAccessibility GetPropertyAccessibility(const FProperty& Property)
	{
		// Skip deprecated properties
		static const FName NAME_DeprecatedProperty = TEXT("DeprecatedProperty");
		if (Property.HasAnyPropertyFlags(CPF_Deprecated) || Property.HasMetaData(NAME_DeprecatedProperty))
		{
			return EPropertyAccessibility::Inaccessible;
		}

		// Interp properties are always keyable
		if (Property.HasAnyPropertyFlags(CPF_Interp))
		{
			return EPropertyAccessibility::ExplicitlyAccessible;
		}
		// Properties with a setter function are always keyable
		if (Property.HasSetter())
		{
			return EPropertyAccessibility::ExplicitlyAccessible;
		}
		return EPropertyAccessibility::Unknown;
	}
}

FSequencerObjectChangeListener::FSequencerObjectChangeListener( TSharedRef<ISequencer> InSequencer )
	: Sequencer( InSequencer )
{
	FCoreUObjectDelegates::OnPreObjectPropertyChanged.AddRaw(this, &FSequencerObjectChangeListener::OnObjectPreEditChange);
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FSequencerObjectChangeListener::OnObjectPostEditChange);
	//GEditor->OnPreActorMoved.AddRaw(this, &FSequencerObjectChangeListener::OnActorPreEditMove);
	GEditor->OnActorMoved().AddRaw( this, &FSequencerObjectChangeListener::OnActorPostEditMove );
}

FSequencerObjectChangeListener::~FSequencerObjectChangeListener()
{
	FCoreUObjectDelegates::OnPreObjectPropertyChanged.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	GEditor->OnActorMoved().RemoveAll( this );
}

void FSequencerObjectChangeListener::OnPropertyChanged(const TArray<UObject*>& ChangedObjects, const IPropertyHandle& PropertyHandle) const
{
	if (Sequencer.IsValid() && !Sequencer.Pin()->IsAllowedToChange())
	{
		return;
	}

	BroadcastPropertyChanged(FKeyPropertyParams(ChangedObjects, PropertyHandle, ESequencerKeyMode::AutoKey));

	for (UObject* Object : ChangedObjects)
	{
		if (Object)
		{
			const FOnObjectPropertyChanged* Event = ObjectToPropertyChangedEvent.Find(Object);
			if (Event)
			{
				Event->Broadcast(*Object);
			}
		}
	}
}

void FSequencerObjectChangeListener::BroadcastPropertyChanged( FKeyPropertyParams KeyPropertyParams ) const
{
	if (Sequencer.IsValid() && !Sequencer.Pin()->IsAllowedToChange())
	{
		return;
	}

	// Filter out objects that actually have the property path that will be keyable. 
	// Otherwise, this might try to key objects that don't have the requested property.
	// For example, a property changed for the FieldOfView property will be sent for 
	// both the CameraActor and the CameraComponent.
	TArray<UObject*> KeyableObjects;
	FOnAnimatablePropertyChanged Delegate;
	const FProperty* Property = nullptr;
	FPropertyPath PropertyPath;
	for (auto ObjectToKey : KeyPropertyParams.ObjectsToKey)
	{
		if (KeyPropertyParams.PropertyPath.GetNumProperties() > 0)
		{
			for (TFieldIterator<FProperty> PropertyIterator(ObjectToKey->GetClass()); PropertyIterator; ++PropertyIterator)
			{
				FProperty* CheckProperty = *PropertyIterator;

				for (int32 Index = 0; Index < KeyPropertyParams.PropertyPath.GetNumProperties(); ++Index)
				{
					const FPropertyInfo& PropertyInfo = KeyPropertyParams.PropertyPath.GetPropertyInfo(Index);

					if (CheckProperty == PropertyInfo.Property.Get())
					{
						FPropertyPath TrimmedPropertyPath;
						for (int32 TrimmedIndex = Index; TrimmedIndex < KeyPropertyParams.PropertyPath.GetNumProperties(); ++TrimmedIndex)
						{
							TrimmedPropertyPath.AddProperty(KeyPropertyParams.PropertyPath.GetPropertyInfo(TrimmedIndex));
						}

						if (CanKeyProperty_Internal(FCanKeyPropertyParams(ObjectToKey->GetClass(), TrimmedPropertyPath), Delegate, Property, PropertyPath))
						{
							KeyableObjects.Add(ObjectToKey);
							break;
						}
					}
				}
			}
		}
	}

	if (!KeyableObjects.Num())
	{
		return;
	}

	if (Delegate.IsBound() && PropertyPath.GetNumProperties() > 0 && Property != nullptr)
	{
		// If the property path is not truncated, then we are keying the leafmost property anyways, so set to NAME_None
		// Otherwise always set to leafmost property of the non-truncated property path, so we correctly pick up struct members
		FPropertyPath StructPathToKey;
		if (PropertyPath.GetNumProperties() < KeyPropertyParams.PropertyPath.GetNumProperties())
		{
			StructPathToKey = *KeyPropertyParams.PropertyPath.TrimRoot(PropertyPath.GetNumProperties());
		}

		// Create a transaction record because we are about to add keys/tracks
		const bool bShouldActuallyTransact = !GIsTransacting;		// Don't transact if we're recording in a PIE world.  That type of keyframe capture cannot be undone.
		FScopedTransaction PropertyChangedTransaction(NSLOCTEXT("Sequencer", "PropertyChanged", "Animatable Property Changed"), bShouldActuallyTransact);

		FPropertyChangedParams Params(KeyableObjects, PropertyPath, StructPathToKey, KeyPropertyParams.KeyMode);
		Delegate.Broadcast(Params);
	}
}

bool FSequencerObjectChangeListener::IsObjectValidForListening( UObject* Object ) const
{
	// @todo Sequencer - Pre/PostEditChange is sometimes called for inner objects of other objects (like actors with components)
	// We only want the outer object so assume it's an actor for now
	if (Sequencer.IsValid())
	{
		TSharedRef<ISequencer> PinnedSequencer = Sequencer.Pin().ToSharedRef();
		return (PinnedSequencer->GetFocusedMovieSceneSequence() && PinnedSequencer->GetFocusedMovieSceneSequence()->CanAnimateObject(*Object));
	}

	return false;
}

FOnAnimatablePropertyChanged& FSequencerObjectChangeListener::GetOnAnimatablePropertyChanged( FAnimatedPropertyKey PropertyKey )
{
	return PropertyChangedEventMap.FindOrAdd( PropertyKey );
}

FOnAnimatablePropertyChanged& FSequencerObjectChangeListener::GetOnAnimatablePropertyChanged(const FProperty* Property)
{
	return PropertyPathChangedEventMap.FindOrAdd(Property);
}

FOnPropagateObjectChanges& FSequencerObjectChangeListener::GetOnPropagateObjectChanges()
{
	return OnPropagateObjectChanges;
}

FOnObjectPropertyChanged& FSequencerObjectChangeListener::GetOnAnyPropertyChanged(UObject& Object)
{
	return ObjectToPropertyChangedEvent.FindOrAdd(&Object);
}

void FSequencerObjectChangeListener::AddFindPropertyChangedHandler(FFindAnimatablePropertyChangedHandler&& Handler)
{
	FindPropertyChangedHandlers.Emplace(MoveTemp(Handler));
}

void FSequencerObjectChangeListener::ReportObjectDestroyed(UObject& Object)
{
	ObjectToPropertyChangedEvent.Remove(&Object);
}

FName GetFunctionName(FAnimatedPropertyKey PropertyKey, const FString& InPropertyVarName)
{
	FString PropertyVarName = InPropertyVarName;

	// If this is a bool property, strip off the 'b' so that the "Set" functions to be 
	// found are, for example, "SetHidden" instead of "SetbHidden"
	if (PropertyKey.PropertyTypeName == "BoolProperty")
	{
		PropertyVarName.RemoveFromStart("b", ESearchCase::CaseSensitive);
	}

	static const FString Set(TEXT("Set"));

	const FString FunctionString = Set + PropertyVarName;

	FName FunctionName = FName(*FunctionString);

	return FunctionName;
}

bool IsHiddenFunction(const UStruct& PropertyStructure, FAnimatedPropertyKey PropertyKey, const FString& InPropertyVarName)
{
	FName FunctionName = GetFunctionName(PropertyKey, InPropertyVarName);

	static const FName HideFunctionsName(TEXT("HideFunctions"));
	bool bIsHiddenFunction = false;
	TArray<FString> HideFunctions;
	if (const UClass* Class = Cast<const UClass>(&PropertyStructure))
	{
		Class->GetHideFunctions(HideFunctions);
	}

	return HideFunctions.Contains(FunctionName.ToString());
}

TOptional<FOnAnimatablePropertyChanged> FSequencerObjectChangeListener::FindPropertySetter(const UStruct& PropertyStructure, FAnimatedPropertyKey PropertyKey, const FProperty& Property) const
{
	using namespace UE::MovieScene;

	const FArrayProperty* ArrayOwner = Property.GetTypedOwner<FArrayProperty>();
	const FProperty* PropertyOrContainer = ArrayOwner ? ArrayOwner : &Property;

	// If we are trying to set a property that exists within a container type (ie an array),
	// we check for flags on the outer property but must not use setter functions that would not know the index to set
	const bool bCanApplyFunction = PropertyOrContainer == &Property;

	// Early return if explicitly supported
	if (const FOnAnimatablePropertyChanged* DelegatePtr = PropertyPathChangedEventMap.Find(&Property))
	{
		return *DelegatePtr;
	}

	// Else find a generic property changed event
	TOptional<FOnAnimatablePropertyChanged> Result;
	if (const FOnAnimatablePropertyChanged* DelegatePtr = PropertyChangedEventMap.Find(PropertyKey))
	{
		Result = *DelegatePtr;
	}
	else for (const FFindAnimatablePropertyChangedHandler& FindHandler : FindPropertyChangedHandlers)
	{
		FOnAnimatablePropertyChanged PropertyChanged = FindHandler.Execute(&Property);
		if (PropertyChanged.IsBound())
		{
			Result = MoveTemp(PropertyChanged);
			break;
		}
	}

	if (Result.IsSet())
	{
		// If the function is hidden, we cannot use it
		const bool bHiddenFunction = bCanApplyFunction && IsHiddenFunction(PropertyStructure, FAnimatedPropertyKey::FromProperty(&Property), Property.GetName());
		if (bHiddenFunction)
		{
			return {};
		}

		switch (GetPropertyAccessibility(*PropertyOrContainer))
		{
		case EPropertyAccessibility::Inaccessible:
			return {};
		case EPropertyAccessibility::ExplicitlyAccessible:
			return Result;
		case EPropertyAccessibility::Unknown:
			break;
		}

		if (bCanApplyFunction)
		{
			// If the function is hidden, we cannot use it
			if (IsHiddenFunction(PropertyStructure, FAnimatedPropertyKey::FromProperty(&Property), Property.GetName()))
			{
				return {};
			}

			// If there is a native setter we can always animate the property
			if (Property.HasSetter())
			{
				return Result;
			}
	
			// Check to see if we have a function of the form Set<PropertyName> that we can use
			if (const UClass* Class = Cast<const UClass>(&PropertyStructure))
			{
				static const FString Set(TEXT("Set"));
				static const FName DeprecatedFunctionName(TEXT("DeprecatedFunction"));

				FString PropertyVarName = PropertyOrContainer->GetName();

				// If this is a bool property, strip off the 'b' so that the "Set" functions to be 
				// found are, for example, "SetHidden" instead of "SetbHidden"
				if (PropertyKey.PropertyTypeName == "BoolProperty")
				{
					PropertyVarName.RemoveFromStart("b", ESearchCase::CaseSensitive);
				}

				FName FunctionName = FName(*(Set + PropertyVarName));
				UFunction* Function = Class->FindFunctionByName(FunctionName);

				// @TODO: should we early out of our property path iteration if we find an "edit defaults only" property?
				const bool bEditable           = Property.HasAnyPropertyFlags(CPF_Edit) && !Property.HasAnyPropertyFlags(CPF_DisableEditOnInstance);
				const bool bFoundValidFunction = Function && !Function->HasMetaData(DeprecatedFunctionName);
			
				// Valid if there's a setter function and the property is editable.
				if (bFoundValidFunction && bEditable)
				{
					return Result;
				}
			}
		}
	}

	return {};
}

bool FSequencerObjectChangeListener::CanKeyProperty(FCanKeyPropertyParams CanKeyPropertyParams) const
{
	FOnAnimatablePropertyChanged Delegate;
	const FProperty* Property = nullptr;
	FPropertyPath PropertyPath;
	return CanKeyProperty_Internal(CanKeyPropertyParams, Delegate, Property, PropertyPath);
}

bool FSequencerObjectChangeListener::CanKeyProperty(FCanKeyPropertyParams KeyPropertyParams, FPropertyPath& OutPropertyPath) const
{
	const FProperty* Property = nullptr;
	FOnAnimatablePropertyChanged Delegate;
	return CanKeyProperty_Internal(KeyPropertyParams, Delegate, Property, OutPropertyPath);
}

bool FSequencerObjectChangeListener::CanKeyProperty_Internal(FCanKeyPropertyParams CanKeyPropertyParams, FOnAnimatablePropertyChanged& InOutDelegate, const FProperty*& InOutProperty, FPropertyPath& InOutPropertyPath) const
{
	using namespace UE::MovieScene;

	if (CanKeyPropertyParams.PropertyPath.GetNumProperties() == 0)
	{
		return false;
	}

	// iterate over the property path trying to find keyable properties
	InOutPropertyPath = FPropertyPath();
	for (int32 Index = 0; Index < CanKeyPropertyParams.PropertyPath.GetNumProperties(); ++Index)
	{
		const FPropertyInfo& PropertyInfo = CanKeyPropertyParams.PropertyPath.GetPropertyInfo(Index);

		// Add this to our 'potentially truncated' path
		InOutPropertyPath.AddProperty(PropertyInfo);

		const FProperty* Property = CanKeyPropertyParams.PropertyPath.GetPropertyInfo(Index).Property.Get();
		if (Property)
		{
			if (Property->IsA<FArrayProperty>())
			{
				continue;
			}

			const UStruct* PropertyOwner = CanKeyPropertyParams.FindPropertyOwner(Property);
			if (!PropertyOwner)
			{
				continue;
			}

			if (!FPropertyEditorPermissionList::Get().DoesPropertyPassFilter(PropertyOwner, Property->GetFName()))
			{
				continue;
			}

			const UStruct* PropertyContainer = CanKeyPropertyParams.FindPropertyContainer(Property);
			if (!PropertyContainer)
			{
				continue;
			}

			FAnimatedPropertyKey PropertyKey = FAnimatedPropertyKey::FromProperty(Property);

			// If there is a custom accessor for this specific property path, it is animatable (as long as there is a supported track editor registered for the property type)
			if (UE::MovieScene::GlobalCustomAccessorExists(CanKeyPropertyParams.ObjectClass, InOutPropertyPath.ToString(TEXT("."))))
			{
				if (const FOnAnimatablePropertyChanged* DelegatePtr = PropertyPathChangedEventMap.Find(Property))
				{
					InOutProperty = Property;
					InOutDelegate = *DelegatePtr;
					return true;
				}
				if (const FOnAnimatablePropertyChanged* DelegatePtr = PropertyChangedEventMap.Find(PropertyKey))
				{
					InOutProperty = Property;
					InOutDelegate = *DelegatePtr;
					return true;
				}
				for (const FFindAnimatablePropertyChangedHandler& FindHandler : FindPropertyChangedHandlers)
				{
					FOnAnimatablePropertyChanged PropertyChanged = FindHandler.Execute(Property);
					if (PropertyChanged.IsBound())
					{
						InOutProperty = Property;
						InOutDelegate = MoveTemp(PropertyChanged);
						return true;
					}
				}
			}

			// Otherwise we check for magic named functions using the default logic
			if (TOptional<FOnAnimatablePropertyChanged> Delegate = FindPropertySetter(*PropertyContainer, PropertyKey, *Property))
			{
				InOutProperty = Property;
				InOutDelegate = MoveTemp(Delegate.GetValue());
				return true;
			}

			const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property);

			// Check each level of the property hierarchy
			FFieldClass* PropertyType = Property->GetClass();
			while (PropertyType && PropertyType != FProperty::StaticClass())
			{
				FAnimatedPropertyKey Key = FAnimatedPropertyKey::FromPropertyTypeName(PropertyType->GetFName());

				// For object properties, check each parent type of the object (ie, so a track that animates UBaseClass ptrs can be used with a UDerivedClass property)
				UClass* ClassType = (ObjectProperty && ObjectProperty->PropertyClass) ? ObjectProperty->PropertyClass->GetSuperClass() : nullptr;
				while (ClassType)
				{
					Key.ObjectTypeName = ClassType->GetFName();

					if (TOptional<FOnAnimatablePropertyChanged> Delegate = FindPropertySetter(*PropertyContainer, Key, *Property))
					{
						InOutProperty = Property;
						InOutDelegate = MoveTemp(Delegate.GetValue());
						return true;
					}

					ClassType = ClassType->GetSuperClass();
				}

				Key.ObjectTypeName = NAME_None;
				if (TOptional<FOnAnimatablePropertyChanged> Delegate = FindPropertySetter(*PropertyContainer, Key, *Property))
				{
					InOutProperty = Property;
					InOutDelegate = MoveTemp(Delegate.GetValue());
					return true;
				}

				// Look at the property's super class
				PropertyType = PropertyType->GetSuperClass();
			}
		}
	}

	return false;
}

void FSequencerObjectChangeListener::KeyProperty(FKeyPropertyParams KeyPropertyParams) const
{
	BroadcastPropertyChanged(KeyPropertyParams);
}

void FSequencerObjectChangeListener::OnObjectPreEditChange( UObject* Object, const FEditPropertyChain& PropertyChain )
{
	if (Sequencer.IsValid() && !Sequencer.Pin()->IsAllowedToChange())
	{
		return;
	}

	// We only care if we are not attempting to change properties of a CDO (which cannot be animated)
	if( Sequencer.IsValid() && !Object->HasAnyFlags(RF_ClassDefaultObject) && PropertyChain.GetActiveMemberNode() )
	{
		// Register with the property editor module that we'd like to know about animated float properties that change
		FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>( "PropertyEditor" );

		// Sometimes due to edit inline new the object is not actually the object that contains the property
		if ( IsObjectValidForListening(Object) && Object->GetClass()->HasProperty(PropertyChain.GetActiveMemberNode()->GetValue()) )
		{
			TSharedPtr<IPropertyChangeListener> PropertyChangeListener = ActivePropertyChangeListeners.FindRef( Object );

			if( !PropertyChangeListener.IsValid() )
			{
				PropertyChangeListener = PropertyEditor.CreatePropertyChangeListener();
				
				ActivePropertyChangeListeners.Add( Object, PropertyChangeListener );

				PropertyChangeListener->GetOnPropertyChangedDelegate().AddRaw( this, &FSequencerObjectChangeListener::OnPropertyChanged );

				FPropertyListenerSettings Settings;
				// Ignore array and object properties
				Settings.bIgnoreArrayProperties = true;
				Settings.bIgnoreObjectProperties = false;
				// Property flags which must be on the property
				Settings.RequiredPropertyFlags = 0;
				// Property flags which cannot be on the property
				Settings.DisallowedPropertyFlags = CPF_EditConst;

				PropertyChangeListener->SetObject( *Object, Settings );
			}
		}
	}

	// Call add key/track before the property changes so that pre-animated state can be saved off.

	TArray<FPropertyInfo, TInlineAllocator<16>> ReversePropertyPath;
	FEditPropertyChain::TDoubleLinkedListNode* ActiveNode = PropertyChain.GetActiveNode();
	while (ActiveNode && ActiveNode->GetValue())
	{
		ReversePropertyPath.Add(FPropertyInfo(ActiveNode->GetValue()));
		ActiveNode = ActiveNode->GetPrevNode();
	}

	FPropertyPath PropertyPath;
	for (int32 Index = ReversePropertyPath.Num() - 1; Index >= 0; --Index)
	{
		PropertyPath.AddProperty(ReversePropertyPath[Index]);
	}

	TArray<UObject*> Objects;
	Objects.Add(Object);

	BroadcastPropertyChanged(FKeyPropertyParams(Objects, PropertyPath, ESequencerKeyMode::AutoKey));
}

void FSequencerObjectChangeListener::OnObjectPostEditChange( UObject* Object, FPropertyChangedEvent& PropertyChangedEvent )
{
	if( Object )
	{
		bool bIsObjectValid = IsObjectValidForListening( Object );

		bool bShouldPropagateChanges = bIsObjectValid;

		// We only care if we are not attempting to change properties of a CDO (which cannot be animated)
		if( Sequencer.IsValid() && bIsObjectValid && !Object->HasAnyFlags(RF_ClassDefaultObject) )
		{
			TSharedPtr< IPropertyChangeListener > Listener;
			ActivePropertyChangeListeners.RemoveAndCopyValue( Object, Listener );

			if( Listener.IsValid() )
			{
				check( Listener.IsUnique() );
					
				// Don't recache new values, the listener will be destroyed after this call
				const bool bRecacheNewValues = false;

				const bool bFoundChanges = Listener->ScanForChanges( bRecacheNewValues );

				// If the listener did not find any changes we care about, propagate changes to puppets
				// @todo Sequencer - We might need to check per changed property
				bShouldPropagateChanges = !bFoundChanges;
			}
		}

		if( bShouldPropagateChanges )
		{
			OnPropagateObjectChanges.Broadcast( Object );
		}
	}
}


void FSequencerObjectChangeListener::OnActorPostEditMove( AActor* Actor )
{
	// @todo sequencer actors: Currently this only fires on a "final" move.  For our purposes we probably
	// want to get an update every single movement, even while dragging an object.
	FPropertyChangedEvent PropertyChangedEvent(nullptr);
	OnObjectPostEditChange( Actor, PropertyChangedEvent );
}

void FSequencerObjectChangeListener::TriggerAllPropertiesChanged(UObject* Object)
{
	if( Object )
	{
		// @todo Sequencer - Pre/PostEditChange is sometimes called for inner objects of other objects (like actors with components)
		// We only want the outer object so assume it's an actor for now
		bool bObjectIsActor = Object->IsA( AActor::StaticClass() );

		// Default to propagating changes to objects only if they are actors
		// if this change is handled by auto-key we will not propagate changes
		bool bShouldPropagateChanges = bObjectIsActor;

		// We only care if we are not attempting to change properties of a CDO (which cannot be animated)
		if( Sequencer.IsValid() && bObjectIsActor && !Object->HasAnyFlags(RF_ClassDefaultObject) )
		{
			TSharedPtr<IPropertyChangeListener> PropertyChangeListener = ActivePropertyChangeListeners.FindRef( Object );
			
			if( !PropertyChangeListener.IsValid() )
			{
				// Register with the property editor module that we'd like to know about animated float properties that change
				FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>( "PropertyEditor" );

				PropertyChangeListener = PropertyEditor.CreatePropertyChangeListener();
				
				PropertyChangeListener->GetOnPropertyChangedDelegate().AddRaw( this, &FSequencerObjectChangeListener::OnPropertyChanged );

				FPropertyListenerSettings Settings;
				// Ignore array and object properties
				Settings.bIgnoreArrayProperties = true;
				Settings.bIgnoreObjectProperties = true;
				// Property flags which must be on the property
				Settings.RequiredPropertyFlags = 0;
				// Property flags which cannot be on the property
				Settings.DisallowedPropertyFlags = CPF_EditConst;

				PropertyChangeListener->SetObject( *Object, Settings );
			}

			PropertyChangeListener->TriggerAllPropertiesChangedDelegate();
		}
	}
}
