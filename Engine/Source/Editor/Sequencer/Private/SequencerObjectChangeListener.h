// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//#include "ISequencerObjectChangeListener.h"

#include "CoreMinimal.h"
#include "KeyPropertyParams.h"
#include "UObject/ObjectKey.h"
#include "ISequencer.h"
#include "ISequencerObjectChangeListener.h"
#include "AnimatedPropertyKey.h"

class AActor;
class IPropertyChangeListener;
class IPropertyHandle;

/**
 * Listens for changes objects and calls delegates when those objects change
 */
class FSequencerObjectChangeListener : public ISequencerObjectChangeListener
{
public:
	FSequencerObjectChangeListener( TSharedRef<ISequencer> InSequencer );
	virtual ~FSequencerObjectChangeListener();

	/** ISequencerObjectChangeListener interface */
	virtual FOnAnimatablePropertyChanged& GetOnAnimatablePropertyChanged( FAnimatedPropertyKey PropertyKey ) override;
	virtual FOnAnimatablePropertyChanged& GetOnAnimatablePropertyChanged(const FProperty* Property) override;
	virtual FOnPropagateObjectChanges& GetOnPropagateObjectChanges() override;
	virtual FOnObjectPropertyChanged& GetOnAnyPropertyChanged(UObject& Object) override;
	virtual void AddFindPropertyChangedHandler(FFindAnimatablePropertyChangedHandler&& Handler) override;
	virtual void ReportObjectDestroyed(UObject& Object) override;
	UE_DEPRECATED(5.4, "CanKeyProperty taking in only FCanKeyPropertyParams is deprecated. Please use the alternative CanKeyProperty that takes in FCanKeyPropertyParams and an out FPropertyPath.")
	virtual bool CanKeyProperty(FCanKeyPropertyParams KeyPropertyParams) const override;
	virtual bool CanKeyProperty(FCanKeyPropertyParams KeyPropertyParams, FPropertyPath& OutPropertyPath) const override;
	virtual void KeyProperty(FKeyPropertyParams KeyPropertyParams) const override;
	virtual void TriggerAllPropertiesChanged(UObject* Object) override;

private:
	/**
	 * Called when PreEditChange is called on an object
	 *
	 * @param Object	The object that PreEditChange was called on
	 * @param Property	The property that is about to change
	 */
	void OnObjectPreEditChange( UObject* Object, const FEditPropertyChain& PropertyChain );

	/**
	 * Called when PostEditChange is called on an object
	 *
	 * @param Object	The object that PostEditChange was called on
	 */
	void OnObjectPostEditChange( UObject* Object, FPropertyChangedEvent& PropertyChangedEvent );

	/**
	 * Called after an actor is moved (PostEditMove)
	 *
	 * @param Actor The actor that PostEditMove was called on
	 */
	void OnActorPostEditMove( AActor* Actor );

	/**
	 * Called when a property changes on an object that is being observed
	 *
	 * @param Object	The object that PostEditChange was called on
	 */
	void OnPropertyChanged( const TArray<UObject*>& ChangedObjects, const IPropertyHandle& PropertyHandle ) const;

	/** Broadcasts the property change callback to the appropriate listeners. */
	void BroadcastPropertyChanged(FKeyPropertyParams KeyPropertyParams) const;

	/**
	 * @return True if an object is valid for listening to property changes 
	 */
	bool IsObjectValidForListening( UObject* Object ) const;

	/** @return A valid delegate if property a setter could be found for a property on a class */
	TOptional<FOnAnimatablePropertyChanged> FindPropertySetter(const UStruct& PropertyStructure, FAnimatedPropertyKey PropertyKey, const FProperty& Property) const;

	/** 
	 * Internal implementation of CanKeyProperty that allows us to also retrieve a delegate that can be used to broadcast keyable properties.
	 * This traverses the property path until it finds the first property that we can key.
	 * @param	KeyPropertyParams	Parameters describing what to key
	 * @param	InOutDelegate		Delegate to call to key a property (if any)
	 * @param	InOutProperty		The property that we will actually key
	 * @param	InOutPropertyPath	The property path of the proepty we will actually key
	 * @return true if we can key the property
	 */
	bool CanKeyProperty_Internal(FCanKeyPropertyParams KeyPropertyParams, FOnAnimatablePropertyChanged& InOutDelegate, const FProperty*& InOutProperty, FPropertyPath& InOutPropertyPath) const;

private:

	/** Array of delegates that are called before any other method to attempt to retrieve a property changed handler */
	TArray<FFindAnimatablePropertyChangedHandler> FindPropertyChangedHandlers;

	/** Mapping of object to a listener used to check for property changes */
	TMap< TWeakObjectPtr<UObject>, TSharedPtr<class IPropertyChangeListener> > ActivePropertyChangeListeners;

	/** A mapping of property classes to multi-cast delegate that is broadcast when properties of that type change */
	TMap< FAnimatedPropertyKey, FOnAnimatablePropertyChanged > PropertyChangedEventMap;

	/** A mapping of an explicitly supported property path to multi-cast delegate that is broadcast when properties of that type change */
	TMap<const FProperty*, FOnAnimatablePropertyChanged> PropertyPathChangedEventMap;

	/** A mapping of object instance to property change event */
	TMap< FObjectKey, FOnObjectPropertyChanged > ObjectToPropertyChangedEvent;

	/** Delegate to call when object changes should be propagated */
	FOnPropagateObjectChanges OnPropagateObjectChanges;

	/** The owning sequencer */
	TWeakPtr< ISequencer > Sequencer;
};
