// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerOutlinerItemDragDropOp.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceID.h"
#include "MovieScene.h"
#include "Sequencer.h"
#include "K2Node_GetSequenceBinding.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "EdGraph/EdGraph.h"

#define LOCTEXT_NAMESPACE "SequencerOutlinerDragDropOp"

namespace UE
{
namespace Sequencer
{

TSharedRef<FSequencerOutlinerDragDropOp> FSequencerOutlinerDragDropOp::New(TArray<TWeakViewModelPtr<IOutlinerExtension>>&& InDraggedNodes, FText InDefaultText, const FSlateBrush* InDefaultIcon)
{
	TSharedRef<FSequencerOutlinerDragDropOp> NewOp = MakeShared<FSequencerOutlinerDragDropOp>();

	NewOp->WeakViewModels = MoveTemp(InDraggedNodes);
	NewOp->DefaultHoverText = NewOp->CurrentHoverText = InDefaultText;
	NewOp->DefaultHoverIcon = NewOp->CurrentIconBrush = InDefaultIcon;

	NewOp->Construct();
	return NewOp;
}

TArray<MovieScene::FFixedObjectBindingID> FSequencerOutlinerDragDropOp::GetDraggedBindings() const
{
	return GetDraggedBindingsImpl(
		[](const TWeakViewModelPtr<IOutlinerExtension>&)
		{
			return true;
		}
	);
}

TArray<MovieScene::FFixedObjectBindingID> FSequencerOutlinerDragDropOp::GetDraggedRebindableBindings() const
{
	return GetDraggedBindingsImpl(
		[](const TWeakViewModelPtr<IOutlinerExtension>& WeakModel)
		{
			TViewModelPtr<FObjectBindingModel> Object = WeakModel.ImplicitPin();
			return Object && Object->SupportsRebinding();
		}
	);
}

TArray<MovieScene::FFixedObjectBindingID> FSequencerOutlinerDragDropOp::GetDraggedBindingsImpl(TFunctionRef<bool(const TWeakViewModelPtr<IOutlinerExtension>&)> InFilter) const
{
	using namespace MovieScene;

	TArray<FFixedObjectBindingID> Bindings;

	for (const TWeakViewModelPtr<IOutlinerExtension>& WeakModel : GetDraggedViewModels())
	{
		TViewModelPtr<IObjectBindingExtension> ObjectBinding = WeakModel.ImplicitPin();
		if (!ObjectBinding || !InFilter(WeakModel))
		{
			continue;
		}

		FGuid ObjectBindingID = ObjectBinding->GetObjectGuid();
		TViewModelPtr<FSequenceModel> Sequence = WeakModel.Pin().AsModel()->FindAncestorOfType<FSequenceModel>();

		if (Sequence && ObjectBindingID.IsValid())
		{
			Bindings.Emplace(MovieScene::FFixedObjectBindingID(ObjectBindingID, Sequence->GetSequenceID()));
		}
	}

	return Bindings;
}

void FSequencerOutlinerDragDropOp::HoverTargetChanged()
{
	if (GetHoveredGraph() && GetDraggedBindings().Num() > 0)
	{
		CurrentHoverText = LOCTEXT("CreateNode", "Add binding ID to graph");
		CurrentIconBrush = FAppStyle::GetBrush( TEXT( "Graph.ConnectorFeedback.NewNode" ) );
	}
	else
	{
		ResetToDefaultToolTip();
	}
}

FReply FSequencerOutlinerDragDropOp::DroppedOnPanel( const TSharedRef< class SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{
	using namespace MovieScene;

	const UEdGraphSchema* Schema = Graph.GetSchema();
	if (Schema)
	{
		UK2Node_GetSequenceBinding* Template = NewObject<UK2Node_GetSequenceBinding>(GetTransientPackage());

		FEdGraphSchemaAction_NewNode Action;
		Action.NodeTemplate = Template;

		FVector2f AddGraphPosition = GraphPosition;
		for (const TWeakViewModelPtr<IOutlinerExtension>& WeakModel : GetDraggedViewModels())
		{
			FViewModelPtr Model = WeakModel.Pin();
			TViewModelPtr<FObjectBindingModel> ObjectBinding = Model.ImplicitCast();
			if (!ObjectBinding || !ObjectBinding->SupportsRebinding())
			{
				continue;
			}

			TViewModelPtr<FSequenceModel> SequenceModel = Model->FindAncestorOfType<FSequenceModel>();
			if (SequenceModel)
			{
				// Fixed bindings always resolve from the root
				Template->SetSequence(SequenceModel->GetSequencer()->GetRootMovieSceneSequence());
				Template->Binding = FFixedObjectBindingID(ObjectBinding->GetObjectGuid(), SequenceModel->GetSequenceID());
				UEdGraphNode* NewNode = Action.PerformAction(&Graph, GetHoveredPin(), FDeprecateSlateVector2D(AddGraphPosition), false);

				int32 Offset = FMath::Max(NewNode->NodeHeight, 100);
				Offset += Offset % 16;
				AddGraphPosition.Y += Offset;
			}
		}
	}
	
	return FReply::Handled();
}

} // namespace Sequencer
} // namespace UE

#undef LOCTEXT_NAMESPACE

