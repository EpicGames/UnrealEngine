// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Input/DragAndDrop.h"
#include "SequencerObjectBindingDragDropOp.h"

class FSequencer;

namespace UE
{
namespace MovieScene
{
	struct FFixedObjectBindingID;
}

namespace Sequencer
{

/** A decorated drag drop operation object for dragging sequencer display nodes. */
class FSequencerOutlinerDragDropOp : public FSequencerObjectBindingDragDropOp
{
public:

	DRAG_DROP_OPERATOR_TYPE( FSequencerOutlinerDragDropOp, FSequencerObjectBindingDragDropOp )

	/**
	 * Construct a new drag/drop operation for dragging a selection of display nodes
	 */
	static TSharedRef<FSequencerOutlinerDragDropOp> New(TArray<TWeakViewModelPtr<IOutlinerExtension>>&& InDraggedNodes, FText InDefaultText, const FSlateBrush* InDefaultIcon);

public:

	//~ FSequencerObjectBindingDragDropOp interface
	virtual TArray<MovieScene::FFixedObjectBindingID> GetDraggedBindings() const override;
	virtual TArray<MovieScene::FFixedObjectBindingID> GetDraggedRebindableBindings() const override;

	TArray<MovieScene::FFixedObjectBindingID> GetDraggedBindingsImpl(TFunctionRef<bool(const TWeakViewModelPtr<IOutlinerExtension>&)> InFilter) const;

	//~ FGraphEditorDragDropAction interface
	virtual void HoverTargetChanged() override;
	virtual FReply DroppedOnPanel( const TSharedRef< class SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph) override;
};


} // namespace Sequencer
} // namespace UE