// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerPropertyKeyedStatus.h"
#include "IDetailKeyframeHandler.h"
#include "ISequencer.h"
#include "ISequencerObjectChangeListener.h"
#include "MovieScene.h"
#include "PropertyHandle.h"
#include "SequencerCommonHelpers.h"
#include "Tracks/MovieScenePropertyTrack.h"

FSequencerPropertyKeyedStatusHandler::FPropertyParameters::FPropertyParameters(ISequencer& InSequencer, const IPropertyHandle& ActualProperty)
	: Sequencer(InSequencer)
	, ActualProperty(ActualProperty)
{
	CurrentFrameRange = TRange<FFrameNumber>(Sequencer.GetLocalTime().Time.FrameNumber);

	FPropertyPath PropertyPath = SequencerHelpers::PropertyHandleToPropertyPath(ActualProperty);
	TrackPropertyName = FindTrackPropertyName(PropertyPath);
	SubPropertyPath = BuildSubPropertyPath();
	TrackPropertyStructName = FindTrackPropertyStructName(PropertyPath);
}

FString FSequencerPropertyKeyedStatusHandler::FPropertyParameters::FindTrackPropertyName(FPropertyPath& InOutPropertyPath) const
{
	const UClass* ObjectClass = ActualProperty.GetOuterBaseClass();
	Sequencer.GetObjectChangeListener().CanKeyProperty(FCanKeyPropertyParams(ObjectClass, InOutPropertyPath), InOutPropertyPath);
	return InOutPropertyPath.ToString(TEXT("."));
}

FName FSequencerPropertyKeyedStatusHandler::FPropertyParameters::FindTrackPropertyStructName(const FPropertyPath& PropertyPath)
{
	if (PropertyPath.GetNumProperties() > 0)
	{
		const FStructProperty* StructProperty = CastField<FStructProperty>(PropertyPath.GetLeafMostProperty().Property.Get());
		if (StructProperty && StructProperty->Struct)
		{
			return StructProperty->Struct->GetFName();
		}
	}
	return NAME_None;
}

FName FSequencerPropertyKeyedStatusHandler::FPropertyParameters::BuildSubPropertyPath() const
{
	// Property Path shows up with the format "Euler->Location->X".
	// This should be converted to "Euler.Location.X"
	FString PropertyPath = FString(ActualProperty.GetPropertyPath()).Replace(TEXT("->"), TEXT("."));

	int32 Index = PropertyPath.Find(TrackPropertyName);
	if (Index != INDEX_NONE)
	{
		// Remove the Track Property name from the property path.
		// From example above, "Euler.Location.X" should be changed to "Location.X" to obtain the sub-property path relative to the Track property
		// If the Actual Property is the Track Property, SubPropertyPath should be left empty
		// Additionally, remove an extra character for the dot (i.e. remove first dot in ".Location.X"). Safe to call as it gets clamped out in RightChop
		PropertyPath.RightChopInline(Index + TrackPropertyName.Len() + 1, EAllowShrinking::No);
	}

	return *PropertyPath;
}

FSequencerPropertyKeyedStatusHandler::FSequencerPropertyKeyedStatusHandler(TSharedRef<ISequencer> InSequencer)
	: SequencerWeak(InSequencer)
{
	InSequencer->OnMovieSceneDataChanged().AddRaw(this, &FSequencerPropertyKeyedStatusHandler::OnMovieSceneDataChanged);
	InSequencer->OnGlobalTimeChanged().AddRaw(this, &FSequencerPropertyKeyedStatusHandler::OnGlobalTimeChanged);
	InSequencer->OnEndScrubbingEvent().AddRaw(this, &FSequencerPropertyKeyedStatusHandler::ResetCachedData);
	InSequencer->OnChannelChangedWithTime().AddRaw(this, &FSequencerPropertyKeyedStatusHandler::OnChannelChanged);
	InSequencer->OnStopEvent().AddRaw(this, &FSequencerPropertyKeyedStatusHandler::ResetCachedData);
}

FSequencerPropertyKeyedStatusHandler::~FSequencerPropertyKeyedStatusHandler()
{
	if (TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin())
	{
		Sequencer->OnMovieSceneDataChanged().RemoveAll(this);
		Sequencer->OnGlobalTimeChanged().RemoveAll(this);
		Sequencer->OnEndScrubbingEvent().RemoveAll(this);
		Sequencer->OnChannelChangedWithTime().RemoveAll(this);
		Sequencer->OnStopEvent().RemoveAll(this);
	}
}

EPropertyKeyedStatus FSequencerPropertyKeyedStatusHandler::GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const
{
	if (const EPropertyKeyedStatus* ExistingKeyedStatus = CachedPropertyKeyedStatusMap.Find(&PropertyHandle))
	{
		return *ExistingKeyedStatus;
	}

	EPropertyKeyedStatus KeyedStatus;

	const FOnGetPropertyKeyedStatus* ExternalHandler = ExternalHandlers.Find(PropertyHandle.GetProperty());
	if (ExternalHandler && ExternalHandler->IsBound())
	{
		KeyedStatus = ExternalHandler->Execute(PropertyHandle);
	}
	else
	{
		KeyedStatus = CalculatePropertyKeyedStatus(PropertyHandle);
	}

	CachedPropertyKeyedStatusMap.Add(&PropertyHandle, KeyedStatus);
	return KeyedStatus;
}

ISequencerPropertyKeyedStatusHandler::FOnGetPropertyKeyedStatus& FSequencerPropertyKeyedStatusHandler::GetExternalHandler(const FProperty* Property)
{
	return ExternalHandlers.FindOrAdd(Property);
}

void FSequencerPropertyKeyedStatusHandler::ResetCachedData()
{
	CachedPropertyKeyedStatusMap.Reset();
}

void FSequencerPropertyKeyedStatusHandler::OnGlobalTimeChanged()
{
	// Only reset cached data when not playing
	TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin();
	if (Sequencer.IsValid() && Sequencer->GetPlaybackStatus() != EMovieScenePlayerStatus::Playing)
	{
		ResetCachedData();
	}
}

void FSequencerPropertyKeyedStatusHandler::OnMovieSceneDataChanged(EMovieSceneDataChangeType DataChangeType)
{
	if (DataChangeType == EMovieSceneDataChangeType::MovieSceneStructureItemAdded
		|| DataChangeType == EMovieSceneDataChangeType::MovieSceneStructureItemRemoved
		|| DataChangeType == EMovieSceneDataChangeType::MovieSceneStructureItemsChanged
		|| DataChangeType == EMovieSceneDataChangeType::ActiveMovieSceneChanged
		|| DataChangeType == EMovieSceneDataChangeType::RefreshAllImmediately)
	{
		ResetCachedData();
	}
}

void FSequencerPropertyKeyedStatusHandler::OnChannelChanged(const TArray<FFrameNumber>&,const FMovieSceneChannelMetaData*, UMovieSceneSection*)
{
	ResetCachedData();
}

EPropertyKeyedStatus FSequencerPropertyKeyedStatusHandler::CalculatePropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSequencerPropertyKeyedStatus::CalculatePropertyKeyedStatus);

	TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin();
	if (!Sequencer.IsValid())
	{
		return EPropertyKeyedStatus::NotKeyed;
	}

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return EPropertyKeyedStatus::NotKeyed;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return EPropertyKeyedStatus::NotKeyed;
	}

	TArray<UObject*> OuterObjects;
	PropertyHandle.GetOuterObjects(OuterObjects);
	if (OuterObjects.IsEmpty())
	{
		return EPropertyKeyedStatus::NotKeyed;
	}

	const FPropertyParameters Parameters(*Sequencer, PropertyHandle);

	EPropertyKeyedStatus KeyedStatus = EPropertyKeyedStatus::NotKeyed;

	// List of Tracks that had no sections at the current frame that require an additional check by going through all its sections and whether there's a key in any of them.
	// Used only to determine whether to return "Not Keyed" or "Keyed In Other Frame".
	TArray<const UMovieScenePropertyTrack*> TracksToCheck;
	TracksToCheck.Reserve(OuterObjects.Num());

	TSet<const UMovieSceneSection*> ProcessedSections;
	ProcessedSections.Reserve(OuterObjects.Num());

	for (UObject* Object : OuterObjects)
	{
		// Object can be null here, but is handled gracefully by GetHandleToObject
		constexpr bool bCreateHandleIfMissing = false;
		FGuid ObjectHandle = Parameters.Sequencer.GetHandleToObject(Object, bCreateHandleIfMissing);
		if (!ObjectHandle.IsValid())
		{
			continue;
		}

		UMovieScenePropertyTrack* PropertyTrack = MovieScene->FindTrack<UMovieScenePropertyTrack>(ObjectHandle, *Parameters.TrackPropertyName);
		if (!PropertyTrack || PropertyTrack->IsEmpty())
		{
			continue;
		}

		TArray<UMovieSceneSection*, TInlineAllocator<5>> Sections;
		Sections = PropertyTrack->FindAllSections(Parameters.CurrentFrameRange.GetLowerBoundValue());

		UMovieSceneSection* const SectionToKey = PropertyTrack->GetSectionToKey();

		// if Section to Key is valid, ensure it is in Sections as the first element, as it takes precedence (and returns immediately) when it exists
		if (SectionToKey)
		{
			int32 Index = Sections.Find(SectionToKey);
			if (Index == INDEX_NONE)
			{
				Index = Sections.Add(SectionToKey);
			}

			if (Index != INDEX_NONE)
			{
				// section order shouldn't matter
				Sections.Swap(0, Index);	
			}
		}

		for (const UMovieSceneSection* Section : Sections)
		{
			check(Section);
			ProcessedSections.Add(Section);

			constexpr EPropertyKeyedStatus MaxStatus = EPropertyKeyedStatus::KeyedInFrame;
			EPropertyKeyedStatus SectionKeyedStatus = GetKeyedStatusInSection(Parameters, *Section, MaxStatus);

			// If the Section is the Section to Key, prioritize it and ignore the rest
			if (Section == SectionToKey)
			{
				return SectionKeyedStatus;
			}

			KeyedStatus = FMath::Max(KeyedStatus, SectionKeyedStatus);

			// Return if max status already reached
			if (KeyedStatus >= MaxStatus)
			{
				return KeyedStatus;
			}
		}

		// If this Track had no keys in any of its sections,
		// add it to check for all sections (as this only checked for sections in the current time)
		if (KeyedStatus == EPropertyKeyedStatus::NotKeyed)
		{
			TracksToCheck.Add(PropertyTrack);
		}
	}

	// If there's no key in the provided sections look through all sections of the tracks
	// And return "KeyedInOtherFrame" as soon as there's a keyed section
	if (KeyedStatus == EPropertyKeyedStatus::NotKeyed)
	{
		for (const UMovieScenePropertyTrack* PropertyTrack : TracksToCheck)
		{
			if (!PropertyTrack)
			{
				continue;
			}

			for (const UMovieSceneSection* Section : PropertyTrack->GetAllSections())
			{
				// Skip sections that were already processed
				if (Section && !ProcessedSections.Contains(Section))
				{
					constexpr EPropertyKeyedStatus MaxStatus = EPropertyKeyedStatus::KeyedInOtherFrame;
					EPropertyKeyedStatus SectionKeyedStatus = GetKeyedStatusInSection(Parameters, *Section, MaxStatus);

					KeyedStatus = FMath::Max(KeyedStatus, SectionKeyedStatus);

					// Maximum Status Reached no need to iterate further
					if (KeyedStatus >= MaxStatus)
					{
						return KeyedStatus;
					}
				}
			}
		}
	}

	return KeyedStatus;
}

EPropertyKeyedStatus FSequencerPropertyKeyedStatusHandler::GetKeyedStatusInSection(const FPropertyParameters& Parameters, const UMovieSceneSection& Section, EPropertyKeyedStatus MaxPropertyKeyedStatus) const
{
	EPropertyKeyedStatus KeyedStatus = EPropertyKeyedStatus::NotKeyed;

	int32 EmptyChannelCount = 0;

	for (const FMovieSceneChannelEntry& ChannelEntry : Section.GetChannelProxy().GetAllEntries())
	{
		TConstArrayView<FMovieSceneChannel*> Channels = ChannelEntry.GetChannels();

		TConstArrayView<FMovieSceneChannelMetaData> ChannelMetaData = ChannelEntry.GetMetaData();

		for (int32 ChannelIndex = 0; ChannelIndex < Channels.Num(); ++ChannelIndex)
		{
			if (KeyedStatus >= MaxPropertyKeyedStatus)
			{
				return KeyedStatus;
			}

			FMovieSceneChannel* const Channel = Channels[ChannelIndex];
			if (!Channel)
			{
				continue;
			}

			const FMovieSceneChannelMetaData& MetaData = ChannelMetaData[ChannelIndex];

			FName SubPropertyPath = MetaData.SubPropertyPath;
			if (SubPropertyPath.IsNone())
			{
				// If sub property path is none, try finding the entry from the sub-property path map
				SubPropertyPath = MetaData.SubPropertyPathMap.FindRef(Parameters.TrackPropertyStructName);
			}

			// Skip Channel if there was a Struct name for the Track (expected a valid Sub-property path) but didn't find one
			const bool bSkipChannel = !Parameters.TrackPropertyStructName.IsNone() && SubPropertyPath.IsNone();

			if (bSkipChannel || !HasMatchingProperty(Parameters, SubPropertyPath))
			{
				continue;
			}

			// Check if Channel is disabled, as these could still have Keys, but should be viewed as empty
			if (Channel->GetNumKeys() == 0 || !MetaData.bEnabled)
			{
				++EmptyChannelCount;
				continue;
			}

			// There's at least 1 key in this Channel, so status is at least KeyedInOtherFrame
			KeyedStatus = FMath::Max(KeyedStatus, EPropertyKeyedStatus::KeyedInOtherFrame);
			if (KeyedStatus >= MaxPropertyKeyedStatus)
			{
				return KeyedStatus;
			}

			TArray<FFrameNumber> KeyTimes;
			Channel->GetKeys(Parameters.CurrentFrameRange, &KeyTimes, nullptr);
			if (KeyTimes.IsEmpty())
			{
				++EmptyChannelCount;
			}
			else
			{
				// Key Times found, updated the KeyedStatus
				KeyedStatus = FMath::Max(KeyedStatus, EPropertyKeyedStatus::PartiallyKeyed);
			}
		}
	}

	if (KeyedStatus == EPropertyKeyedStatus::PartiallyKeyed && EmptyChannelCount == 0)
	{
		KeyedStatus = EPropertyKeyedStatus::KeyedInFrame;
	}
	return KeyedStatus;
}

bool FSequencerPropertyKeyedStatusHandler::HasMatchingProperty(const FPropertyParameters& Parameters, FName SubPropertyPath) const
{
	FProperty* Property = Parameters.ActualProperty.GetProperty();
	if (Property->GetFName() == SubPropertyPath || Parameters.SubPropertyPath == SubPropertyPath)
	{
		return true;
	}

	// At this point, the actual property is NOT the channel's property,
	// but the actual property could be a parent of the channel's property -- so iterate downward and see if there's a match.

	// If it isn't a struct property, it can't iterate further down, return no match
	const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
	if (!StructProperty)
	{
		return false;
	}

	// Return early no match if the Property Path does not even begin with the pre-calculated Sub Property Path
	if (!Parameters.SubPropertyPath.IsNone() && !SubPropertyPath.ToString().StartsWith(Parameters.SubPropertyPath.ToString()))
	{
		return false;
	}

	TArray<FString> PropertySegments;
	SubPropertyPath.ToString().ParseIntoArray(PropertySegments, TEXT("."));
	return HasMatchingSubProperty(*StructProperty, PropertySegments);
}

bool FSequencerPropertyKeyedStatusHandler::HasMatchingSubProperty(const FStructProperty& StructProperty, TConstArrayView<FString> InPropertySegments) const
{
	for (const FProperty* Property : TFieldRange<FProperty>(StructProperty.Struct))
	{
		int32 SegmentIndex = InPropertySegments.Find(Property->GetName());
		if (SegmentIndex == INDEX_NONE)
		{
			continue;
		}

		// If it's last index, then the property is found
		if (SegmentIndex == InPropertySegments.Num() - 1)
		{
			return true;
		}

		// If it's not the last segment that matches, there's more to match, recurse downward
		if (const FStructProperty* SubStructProperty = CastField<FStructProperty>(Property))
		{
			if (HasMatchingSubProperty(*SubStructProperty, InPropertySegments.RightChop(SegmentIndex + 1)))
			{
				return true;
			}
		}
	}
	return false;
}
