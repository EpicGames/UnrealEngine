// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "ISequencerPropertyKeyedStatus.h"
#include "Math/Range.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FPropertyPath;
class FSequencer;
class FStructProperty;
class IPropertyHandle;
class ISequencer;
class UMovieSceneSection;
enum class EMovieSceneDataChangeType;
enum class EPropertyKeyedStatus : uint8;
struct FMovieSceneChannelMetaData;

class FSequencerPropertyKeyedStatusHandler : public ISequencerPropertyKeyedStatusHandler
{
	struct FPropertyParameters
	{
		explicit FPropertyParameters(ISequencer& InSequencer, const IPropertyHandle& ActualProperty);

		ISequencer& Sequencer;

		/** the actual Property whose keyed status is being queried */
		const IPropertyHandle& ActualProperty;

		/** the name of the property that the Property Track is made for. Could be the same as the ActualProperty name, or the name of a parent property (e.g. a Struct containing the Actual Property) */
		FString TrackPropertyName;

		/** The possible underlying struct for the Track's Property, can be none */
		FName TrackPropertyStructName;

		/**
		 * The Sub Property Path going from TrackProperty->ActualProperty.
		 * If the Actual Property is the Track Property, this is Empty
		 * Else, if Track's Property is "A", and ActualProperty is in the form "A->B->C", the generated SubPropertyPath becomes "B.C"
		 */
		FName SubPropertyPath;

		/** Current Sequencer Time, converted into a Frame Range */
		TRange<FFrameNumber> CurrentFrameRange;

		FPropertyPath BuildPropertyPath() const;

		FString FindTrackPropertyName(FPropertyPath& InOutPropertyPath) const;

		FName FindTrackPropertyStructName(const FPropertyPath& PropertyPath);

		FName BuildSubPropertyPath() const;
	};

public:
	explicit FSequencerPropertyKeyedStatusHandler(TSharedRef<ISequencer> InSequencer);

	virtual ~FSequencerPropertyKeyedStatusHandler() override;

	EPropertyKeyedStatus GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const;

	//~ Begin ISequencerPropertyKeyedStatusHandler
	virtual FOnGetPropertyKeyedStatus& GetExternalHandler(const FProperty* Property) override;
	//~ End ISequencerPropertyKeyedStatusHandler

private:
	void ResetCachedData();

	/** Delegates Resetting Cached Data */
	void OnGlobalTimeChanged();
	void OnMovieSceneDataChanged(EMovieSceneDataChangeType);
	void OnChannelChanged(const TArray<FFrameNumber>&, const FMovieSceneChannelMetaData*, UMovieSceneSection*);

	EPropertyKeyedStatus CalculatePropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const;

	EPropertyKeyedStatus GetKeyedStatusInSection(const FPropertyParameters& Parameters, const UMovieSceneSection& Section, EPropertyKeyedStatus MaxPropertyKeyedStatus) const;

	bool HasMatchingProperty(const FPropertyParameters& Parameters, FName SubPropertyPath) const;

	bool HasMatchingSubProperty(const FStructProperty& StructProperty, TConstArrayView<FString> InPropertySegments) const;

	/** Sequencer owning this instance */
	TWeakPtr<ISequencer> SequencerWeak;

	/** Map of Properties and their delegates used to get the Property Keyed Status via External Logic */
	TMap<const FProperty*, FOnGetPropertyKeyedStatus> ExternalHandlers;

	/** Map to the last calculated property keyed status. Resets when Scrubbing, changing Movie Scene Data, etc */
	mutable TMap<const IPropertyHandle*, EPropertyKeyedStatus> CachedPropertyKeyedStatusMap;
};
