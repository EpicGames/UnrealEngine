// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerSectionPainter.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

FSequencerSectionPainter::FSequencerSectionPainter(FSlateWindowElementList& OutDrawElements, const FGeometry& InSectionGeometry, TSharedPtr<UE::Sequencer::FSectionModel> InSection)
	: SectionModel(InSection)
	, DrawElements(OutDrawElements)
	, SectionGeometry(InSectionGeometry)
	, LayerId(0)
	, bParentEnabled(true)
	, bIsHighlighted(false)
	, bIsSelected(false)
	, GhostAlpha(1.f)
{
	HeaderGeometry = SectionGeometry.MakeChild(
		FVector2f(
			SectionGeometry.GetLocalSize().X,
			InSection->GetLinkedOutlinerItem()->GetOutlinerSizing().Height
		),
		FSlateLayoutTransform()
	);

}

FSequencerSectionPainter::~FSequencerSectionPainter()
{
}

int32 FSequencerSectionPainter::PaintSectionBackground()
{
	FLinearColor TrackColor = FLinearColor(GetTrack()->GetColorTint());
	FLinearColor SectionColor = FLinearColor(SectionModel->GetSection()->GetColorTint());

	const float Alpha = SectionColor.A;
	SectionColor.A = 1.f;

	FLinearColor BackgroundColor = TrackColor * (1.f - Alpha) + SectionColor * Alpha;
	return PaintSectionBackground(BackgroundColor);
}

UMovieSceneTrack* FSequencerSectionPainter::GetTrack() const
{
	return SectionModel->GetSection()->GetTypedOuter<UMovieSceneTrack>();
}

FLinearColor FSequencerSectionPainter::BlendColor(FLinearColor InColor)
{
	static FLinearColor BaseColor(FColor(71,71,71));

	const float Alpha = InColor.A;
	InColor.A = 1.f;
	
	return BaseColor * (1.f - Alpha) + InColor * Alpha;
}

FLinearColor FSequencerSectionPainter::GetSectionColor() const
{
	using namespace UE::Sequencer;

	TSharedPtr<ITrackExtension> Track = SectionModel->FindAncestorOfType<ITrackExtension>();
	UMovieSceneTrack* TrackObject = Track ? Track->GetTrack() : nullptr;

	FLinearColor Tint = FLinearColor::White;
	if (TrackObject)
	{
		return BlendColor(TrackObject->GetColorTint()).CopyWithNewOpacity(1.f);
	}

	return Tint;
}