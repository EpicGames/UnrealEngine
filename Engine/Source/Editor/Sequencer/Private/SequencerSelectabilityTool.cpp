// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerSelectabilityTool.h"
#include "LevelEditorViewport.h"
#include "ScopedTransaction.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "SequencerCommands.h"

#define LOCTEXT_NAMESPACE "SequencerSelectabilityTool"

FSequencerSelectabilityTool::FSequencerSelectabilityTool(const FOnGetWorld& InOnGetWorld, const FOnIsObjectSelectableInViewport& InOnIsObjectSelectableInViewport)
	: FEditorViewportSelectability(InOnGetWorld, InOnIsObjectSelectableInViewport)
{
}

bool FSequencerSelectabilityTool::BoxSelect(FBox& InBox, const bool InSelect)
{
	if (!bSelectionLimited
		|| !GCurrentLevelEditingViewportClient
		|| GCurrentLevelEditingViewportClient->IsInGameView())
	{
		return false;
	}

	return BoxSelectWorldActors(InBox, GCurrentLevelEditingViewportClient, InSelect);
}

bool FSequencerSelectabilityTool::FrustumSelect(const FConvexVolume& InFrustum
	, FEditorViewportClient* const InEditorViewportClient
	, const bool InSelect)
{
	if (!bSelectionLimited
		|| !InEditorViewportClient
		|| InEditorViewportClient->IsInGameView())
	{
		return false;
	}

	// Need to check for a zero frustum since ComponentIsTouchingSelectionFrustum will return true, selecting everything, when this is the case
	const bool bAreTopBottomMalformed = InFrustum.Planes[0].IsNearlyZero() && InFrustum.Planes[2].IsNearlyZero();
	const bool bAreRightLeftMalformed = InFrustum.Planes[1].IsNearlyZero() && InFrustum.Planes[3].IsNearlyZero();
	if (bAreTopBottomMalformed || bAreRightLeftMalformed || InEditorViewportClient->IsInGameView())
	{
		return false;
	}

	return FrustumSelectWorldActors(InFrustum, InEditorViewportClient, InSelect);
}

#undef LOCTEXT_NAMESPACE
