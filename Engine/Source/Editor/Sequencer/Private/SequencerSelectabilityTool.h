// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorModeTools.h"
#include "EditorViewportSelectability.h"

class FSequencerSelectabilityTool : public FModeTool, public FEditorViewportSelectability
{
public:
	FSequencerSelectabilityTool() = delete;
	FSequencerSelectabilityTool(const FOnGetWorld& InOnGetWorld, const FOnIsObjectSelectableInViewport& InOnIsObjectSelectableInViewport);

	//~ Begin FModeTool
	virtual FString GetName() const override
	{
		return TEXT("Sequencer Selectability");
	}
	virtual bool BoxSelect(FBox& InBox, const bool InSelect) override;
	virtual bool FrustumSelect(const FConvexVolume& InFrustum
		, FEditorViewportClient* const InEditorViewportClient
		, const bool InSelect) override;
	//~ End FModeTool
};
