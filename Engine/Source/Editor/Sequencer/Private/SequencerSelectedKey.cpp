// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerSelectedKey.h"

#include "Channels/MovieSceneChannel.h"
#include "Containers/Map.h"
#include "IKeyArea.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/Selection/Selection.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FrameNumber.h"

void FSequencerSelectedKey::AppendKeySelection(TSet<FSequencerSelectedKey>& OutSelectedKeys, const UE::Sequencer::FKeySelection& InKeySelection)
{
	using namespace UE::Sequencer;

	for (FKeyHandle Key : InKeySelection)
	{
		TSharedPtr<FChannelModel> Channel = InKeySelection.GetModelForKey(Key);
		UMovieSceneSection*       Section = Channel ? Channel->GetSection() : nullptr;
		if (Channel && Section)
		{
			OutSelectedKeys.Emplace(FSequencerSelectedKey(*Section, Channel, Key));
		}
	}
}

FSelectedKeysByChannel::FSelectedKeysByChannel(const UE::Sequencer::FKeySelection& KeySelection)
{
	using namespace UE::Sequencer;

	TMap<const IKeyArea*, int32> KeyAreaToChannelIndex;

	int32 Index = 0;
	for (FKeyHandle Key : KeySelection)
	{
		TSharedPtr<FChannelModel> Channel = KeySelection.GetModelForKey(Key);

		if (Channel)
		{
			const int32* ChannelArrayIndex = KeyAreaToChannelIndex.Find(Channel->GetKeyArea().Get());
			if (!ChannelArrayIndex)
			{
				int32 NewIndex = SelectedChannels.Add(FSelectedChannelInfo(Channel->GetKeyArea()->GetChannel(), Channel->GetSection(), Channel->GetOwningObject()));
				ChannelArrayIndex = &KeyAreaToChannelIndex.Add(Channel->GetKeyArea().Get(), NewIndex);
			}

			FSelectedChannelInfo& ThisChannelInfo = SelectedChannels[*ChannelArrayIndex];
			ThisChannelInfo.KeyHandles.Add(Key);
			ThisChannelInfo.OriginalIndices.Add(Index);

			++Index;
		}
	}
}

FSelectedKeysByChannel::FSelectedKeysByChannel(TArrayView<const FSequencerSelectedKey> InSelectedKeys)
{
	using namespace UE::Sequencer;

	TMap<const IKeyArea*, int32> KeyAreaToChannelIndex;

	for (int32 Index = 0; Index < InSelectedKeys.Num(); ++Index)
	{
		FSequencerSelectedKey Key = InSelectedKeys[Index];
		TSharedPtr<FChannelModel> Channel = Key.WeakChannel.Pin();

		if (Channel && Key.IsValid())
		{
			const int32* ChannelArrayIndex = KeyAreaToChannelIndex.Find(Channel->GetKeyArea().Get());
			if (!ChannelArrayIndex)
			{
				int32 NewIndex = SelectedChannels.Add(FSelectedChannelInfo(Channel->GetKeyArea()->GetChannel(), Channel->GetSection(), Channel->GetOwningObject()));
				ChannelArrayIndex = &KeyAreaToChannelIndex.Add(Channel->GetKeyArea().Get(), NewIndex);
			}

			FSelectedChannelInfo& ThisChannelInfo = SelectedChannels[*ChannelArrayIndex];
			ThisChannelInfo.KeyHandles.Add(Key.KeyHandle);
			ThisChannelInfo.OriginalIndices.Add(Index);
		}
	}
}

void GetKeyTimes(TArrayView<const FSequencerSelectedKey> InSelectedKeys, TArrayView<FFrameNumber> OutTimes)
{
	check(InSelectedKeys.Num() == OutTimes.Num());

	FSelectedKeysByChannel KeysByChannel(InSelectedKeys);

	TArray<FFrameNumber> KeyTimesScratch;

	for (const FSelectedChannelInfo& ChannelInfo : KeysByChannel.SelectedChannels)
	{
		const FMovieSceneChannelMetaData* MetaData = ChannelInfo.Channel.GetMetaData();
		FMovieSceneChannel* Channel = ChannelInfo.Channel.Get();
		UMovieSceneSection* Section = ChannelInfo.OwningSection;
		if (Channel && MetaData && Section)
		{
			// Resize the scratch buffer to the correct size
			const int32 NumKeys = ChannelInfo.KeyHandles.Num();
			KeyTimesScratch.Reset(NumKeys);
			KeyTimesScratch.SetNum(NumKeys);

			// Populating the key times scratch buffer with the times for these handles
			Channel->GetKeyTimes(ChannelInfo.KeyHandles, KeyTimesScratch);

			FFrameNumber Offset = MetaData->GetOffsetTime(Section);
			for(int32 Index = 0; Index < KeyTimesScratch.Num(); ++Index)
			{
				int32 OriginalIndex = ChannelInfo.OriginalIndices[Index];
				OutTimes[OriginalIndex] = KeyTimesScratch[Index] + Offset;
			}
		}
	}
}

void SetKeyTimes(TArrayView<const FSequencerSelectedKey> InSelectedKeys, TArrayView<const FFrameNumber> InTimes)
{
	check(InSelectedKeys.Num() == InTimes.Num());

	FSelectedKeysByChannel KeysByChannel(InSelectedKeys);

	TArray<FFrameNumber> KeyTimesScratch;

	for (const FSelectedChannelInfo& ChannelInfo : KeysByChannel.SelectedChannels)
	{
		const FMovieSceneChannelMetaData* MetaData = ChannelInfo.Channel.GetMetaData();
		FMovieSceneChannel* Channel = ChannelInfo.Channel.Get();
		UMovieSceneSection* Section = ChannelInfo.OwningSection;
		if (Channel && MetaData && Section)
		{
			// Expand to frame first, then offset
			KeyTimesScratch.Reset(ChannelInfo.OriginalIndices.Num());
			for (int32 Index : ChannelInfo.OriginalIndices)
			{
				KeyTimesScratch.Add(InTimes[Index]);

				if (!Section->GetRange().Contains(InTimes[Index]))
				{
					Section->ExpandToFrame(InTimes[Index]);
				}
			}

			FFrameNumber Offset = MetaData->GetOffsetTime(Section);

			if (Offset != 0)
			{
				for (FFrameNumber& Frame : KeyTimesScratch)
				{
					Frame -= Offset;
				}
			}

			Channel->SetKeyTimes(ChannelInfo.KeyHandles, KeyTimesScratch);
		}
	}
}

void DuplicateKeys(TArrayView<const FSequencerSelectedKey> InSelectedKeys, TArrayView<FKeyHandle> OutNewHandles)
{
	check(InSelectedKeys.Num() == OutNewHandles.Num());

	FSelectedKeysByChannel KeysByChannel(InSelectedKeys);

	TArray<FKeyHandle> KeyHandlesScratch;
	for (const FSelectedChannelInfo& ChannelInfo : KeysByChannel.SelectedChannels)
	{
		FMovieSceneChannel* Channel = ChannelInfo.Channel.Get();
		if (Channel)
		{
			// Resize the scratch buffer to the correct size
			const int32 NumKeys = ChannelInfo.KeyHandles.Num();
			KeyHandlesScratch.Reset(NumKeys);
			KeyHandlesScratch.SetNum(NumKeys);

			// Duplicate the keys, populating the handles scratch buffer
			Channel->DuplicateKeys(ChannelInfo.KeyHandles, KeyHandlesScratch);

			// Copy the duplicated key handles to the output array view
			for(int32 Index = 0; Index < KeyHandlesScratch.Num(); ++Index)
			{
				int32 OriginalIndex = ChannelInfo.OriginalIndices[Index];
				OutNewHandles[OriginalIndex] = KeyHandlesScratch[Index];
			}
		}
	}
}
