// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelHandle.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "Curves/KeyHandle.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"

class UMovieSceneSection;
struct FFrameNumber;

namespace UE
{
namespace Sequencer
{

struct FKeySelection;
class FChannelModel;

} // namespace Sequencer 
} // namespace UE

/**
 * Represents a selected key in the sequencer
 */
struct FSequencerSelectedKey
{	
	/** Section that the key belongs to */
	UMovieSceneSection* Section;

	/** Key area providing the key */
	TWeakPtr<UE::Sequencer::FChannelModel> WeakChannel;

	/** Index of the key in the key area */
	FKeyHandle KeyHandle;

public:

	static void AppendKeySelection(TSet<FSequencerSelectedKey>& OutSelectedKeys, const UE::Sequencer::FKeySelection& InKeySelection);

	/** Create and initialize a new instance. */
	FSequencerSelectedKey(UMovieSceneSection& InSection, TWeakPtr<UE::Sequencer::FChannelModel> InChannel, FKeyHandle InKeyHandle)
		: Section(&InSection)
		, WeakChannel(MoveTemp(InChannel))
		, KeyHandle(InKeyHandle)
	{}

	/** Default constructor. */
	FSequencerSelectedKey()
		: Section(nullptr)
		, KeyHandle(FKeyHandle::Invalid())
	{}

	/** @return Whether or not this is a valid selected key */
	bool IsValid() const
	{
		return Section != nullptr && WeakChannel.Pin().IsValid()
			&& KeyHandle != FKeyHandle::Invalid();
	}

	friend uint32 GetTypeHash(const FSequencerSelectedKey& SelectedKey)
	{
		return GetTypeHash(SelectedKey.Section) ^ GetTypeHash(SelectedKey.WeakChannel)
			 ^ GetTypeHash(SelectedKey.KeyHandle);
	} 

	bool operator==(const FSequencerSelectedKey& OtherKey) const
	{
		return Section == OtherKey.Section && WeakChannel == OtherKey.WeakChannel
			&& KeyHandle == OtherKey.KeyHandle;
	}
};

/**
 * Structure representing a number of keys selected on a movie scene channel
 */
struct FSelectedChannelInfo
{
	explicit FSelectedChannelInfo(FMovieSceneChannelHandle InChannel, UMovieSceneSection* InOwningSection, UObject* InOwningObject)
		: Channel(InChannel), OwningSection(InOwningSection), OwningObject(InOwningObject)
	{}

	/** The channel on which the keys are selected */
	FMovieSceneChannelHandle Channel;

	/** The section that owns this channel */
	UMovieSceneSection* OwningSection;

	/** The object that owns this channel. Maybe the same as OwningSection, maybe not. */
	UObject* OwningObject;

	/** The key handles that are selected on this channel */
	TArray<FKeyHandle> KeyHandles;

	/** The index of each key handle in the original unordered key array supplied to FSelectedKeysByChannel */
	TArray<int32> OriginalIndices;
};

/**
 * Structure that groups an arbitrarily ordered array of selected keys into their respective channels
 */
struct FSelectedKeysByChannel
{
	explicit FSelectedKeysByChannel(const UE::Sequencer::FKeySelection& KeySelection);
	explicit FSelectedKeysByChannel(TArrayView<const FSequencerSelectedKey> InSelectedKeys);

	/** Array storing all selected keys for each channel */
	TArray<FSelectedChannelInfo> SelectedChannels;
};

/**
 * Populate the specified key times array with the times of all the specified keys. Array sizes must match.
 *
 * @param InSelectedKeys    Array of selected keys
 * @param OutTimes          Pre-allocated array of key times to fill with the times of the above keys
 */
void GetKeyTimes(TArrayView<const FSequencerSelectedKey> InSelectedKeys, TArrayView<FFrameNumber> OutTimes);

/**
 * Set the key times for each of the specified keys. Array sizes must match.
 *
 * @param InSelectedKeys    Array of selected keys
 * @param InTimes           Array of times to apply, one per selected key index
 */
void SetKeyTimes(TArrayView<const FSequencerSelectedKey> InSelectedKeys, TArrayView<const FFrameNumber> InTimes);

/**
 * Duplicate the specified keys, populating another array with the duplicated key handles. Array sizes must match.
 *
 * @param InSelectedKeys    Array of selected keys
 * @param OutNewHandles     Pre-allocated array to receive duplicated key handles, one per input key.
 */
void DuplicateKeys(TArrayView<const FSequencerSelectedKey> InSelectedKeys, TArrayView<FKeyHandle> OutNewHandles);