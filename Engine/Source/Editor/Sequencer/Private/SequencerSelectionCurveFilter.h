// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISequencerModule.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "Tree/CurveEditorTreeFilter.h"

namespace UE::Sequencer
{

/**
 * A specialized filter for showing items in the curve editor selected from the sequencer panel.
 * This filter will store the selected nodes and all the parents of the selected nodes in a NodesToFilter set.
 * An item will pass if it is either directly selected or has a parent in the set.
 */
struct FSequencerSelectionCurveFilter : FCurveEditorTreeFilter
{
	static const int32 FilterPass = -1000;

	FSequencerSelectionCurveFilter()
		: FCurveEditorTreeFilter(ISequencerModule::GetSequencerSelectionFilterType(), FilterPass)
	{}

	/**
	 * Adds all selected nodes and their object parents to the NodesToFilter set
	 */
	void Update(TSharedPtr<FSequencerSelection> Selection, const bool bExpandTreeToSelectedNodes = true)
	{
		NodesToFilter.Empty(Selection->Outliner.Num());

		for (TViewModelPtr<IOutlinerExtension> SelectedNode : Selection->Outliner)
		{
			NodesToFilter.Add(SelectedNode.AsModel());

			for (TViewModelPtr<IObjectBindingExtension> ParentObject : SelectedNode.AsModel()->GetAncestorsOfType<IObjectBindingExtension>())
			{
				NodesToFilter.Add(ParentObject.AsModel());
			}
		}

		bExpandToMatchedItems = bExpandTreeToSelectedNodes;
	}

	bool Match(TSharedRef<const FViewModel> InNode) const
	{
		return NodesToFilter.Contains(InNode);
	}

private:

	TSet<TWeakPtr<const FViewModel>> NodesToFilter;
};

} // namespace UE::Sequencer