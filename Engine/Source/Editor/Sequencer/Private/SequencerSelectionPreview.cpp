// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerSelectionPreview.h"

#include "HAL/PlatformCrt.h"
#include "SequencerSelectedKey.h"
#include "Templates/TypeHash.h"

namespace UE::Sequencer { class FViewModel; }

void FSequencerSelectionPreview::SetSelectionState(const UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::FChannelModel>& Channel, FKeyHandle Key, ESelectionPreviewState InState)
{
	if (InState == ESelectionPreviewState::Undefined)
	{
		RawDefinedKeyStates.Remove(Key);
		KeyHandleToModel.Remove(Key);
	}
	else
	{
		RawDefinedKeyStates.Add(Key, InState);
		KeyHandleToModel.Add(Key, Channel);
	}

	CachedSelectionHash.Reset();
}

void FSequencerSelectionPreview::SetSelectionState(TWeakPtr<UE::Sequencer::FViewModel> InModel, ESelectionPreviewState InState)
{
	if (InState == ESelectionPreviewState::Undefined)
	{
		DefinedModelStates.Remove(InModel);
	}
	else
	{
		DefinedModelStates.Add(InModel, InState);
	}
	CachedSelectionHash.Reset();
}

ESelectionPreviewState FSequencerSelectionPreview::GetSelectionState(FKeyHandle Key) const
{
	if (const ESelectionPreviewState* State = RawDefinedKeyStates.Find(Key))
	{
		return *State;
	}
	return ESelectionPreviewState::Undefined;
}


ESelectionPreviewState FSequencerSelectionPreview::GetSelectionState(TWeakPtr<UE::Sequencer::FViewModel> InModel) const
{
	if (const ESelectionPreviewState* State = DefinedModelStates.Find(InModel))
	{
		return *State;
	}
	return ESelectionPreviewState::Undefined;
}

UE::Sequencer::TViewModelPtr<UE::Sequencer::FChannelModel> FSequencerSelectionPreview::GetChannelForKey(FKeyHandle KeyHandle) const
{
	return KeyHandleToModel.FindRef(KeyHandle).Pin();
}

void FSequencerSelectionPreview::Empty()
{
	EmptyDefinedKeyStates();
	EmptyDefinedModelStates();
}

void FSequencerSelectionPreview::EmptyDefinedKeyStates()
{
	RawDefinedKeyStates.Reset();
	KeyHandleToModel.Reset();
	CachedSelectionHash.Reset();
}

void FSequencerSelectionPreview::EmptyDefinedModelStates()
{
	DefinedModelStates.Reset();
	CachedSelectionHash.Reset();
}

uint32 FSequencerSelectionPreview::GetSelectionHash() const
{
	if (!CachedSelectionHash.IsSet())
	{
		uint32 NewHash = 0;

		for (TPair<FKeyHandle, ESelectionPreviewState> Pair : RawDefinedKeyStates)
		{
			NewHash = HashCombine(NewHash, HashCombine(GetTypeHash(Pair.Key), GetTypeHash(Pair.Value)));
		}
		for (TPair<TWeakPtr<UE::Sequencer::FViewModel>, ESelectionPreviewState> Pair : DefinedModelStates)
		{
			NewHash = HashCombine(NewHash, HashCombine(GetTypeHash(Pair.Key), GetTypeHash(Pair.Value)));
		}

		CachedSelectionHash = NewHash;
	}

	return CachedSelectionHash.GetValue();
}
