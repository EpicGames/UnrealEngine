// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Misc/Optional.h"
#include "Curves/KeyHandle.h"
#include "Templates/SharedPointer.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModelPtr.h"

namespace UE::Sequencer { class FViewModel; }

enum class ESelectionPreviewState
{
	Undefined,
	Selected,
	NotSelected
};


/**
 * Manages the selection of keys, sections, and outliner nodes for the sequencer.
 */
class FSequencerSelectionPreview
{
public:

	/** Access the defined key states */
	const TMap<FKeyHandle, ESelectionPreviewState>& GetDefinedKeyStates() const { return RawDefinedKeyStates; }

	/** Access the defined model states */
	const TMap<TWeakPtr<UE::Sequencer::FViewModel>, ESelectionPreviewState>& GetDefinedModelStates() const { return DefinedModelStates; }

	/** Adds a key to the selection */
	void SetSelectionState(const UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::FChannelModel>& Channel, FKeyHandle Key, ESelectionPreviewState InState);

	/** Adds a model to the selection */
	void SetSelectionState(TWeakPtr<UE::Sequencer::FViewModel> InModel, ESelectionPreviewState InState);

	/** Returns the selection state for the the specified key. */
	ESelectionPreviewState GetSelectionState(FKeyHandle Key) const;

	/** Returns the selection state for the the specified model. */
	ESelectionPreviewState GetSelectionState(TWeakPtr<UE::Sequencer::FViewModel> InModel) const;

	/** Finds the channel associated with the specified key. */
	UE::Sequencer::TViewModelPtr<UE::Sequencer::FChannelModel> GetChannelForKey(FKeyHandle KeyHandle) const;

	/** Empties all selections. */
	void Empty();

	/** Empties the key selection. */
	void EmptyDefinedKeyStates();

	/** Empties the model selection. */
	void EmptyDefinedModelStates();

	/** Hash the contents of this selection preview. */
	uint32 GetSelectionHash() const;

private:

	TMap<FKeyHandle, UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::FChannelModel>> KeyHandleToModel;
	TMap<FKeyHandle, ESelectionPreviewState> RawDefinedKeyStates;

	TMap<TWeakPtr<UE::Sequencer::FViewModel>, ESelectionPreviewState> DefinedModelStates;

	/** Cached hash of this whole selection preview state. */
	mutable TOptional<uint32> CachedSelectionHash;
};
