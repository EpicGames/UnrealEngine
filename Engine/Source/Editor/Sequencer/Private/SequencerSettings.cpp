// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerSettings.h"
#include "KeyParams.h"
#include "ISequencer.h"
#include "SSequencer.h"
#include "Misc/ConsoleVariables.h"
#include "MovieScene.h"
#include "MVVM/ViewModels/ViewDensity.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequencerSettings)

USequencerSettings::USequencerSettings( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	AutoChangeMode = EAutoChangeMode::None;
	AllowEditsMode = EAllowEditsMode::AllEdits;
	KeyGroupMode = EKeyGroupMode::KeyChanged;
	KeyInterpolation = EMovieSceneKeyInterpolation::Auto;
	bAutoSetTrackDefaults = false;
	SpawnPosition = SSP_Origin;
	bCreateSpawnableCameras = true;
	bShowRangeSlider = false;
	bShowBottomTimeSlider = true;
	bIsSnapEnabled = true;
	bSnapKeyTimesToElements = false;
	bSnapSectionTimesToElements = false;
	bSnapPlayTimeToKeys = false;
	bSnapPlayTimeToSections = false;
	bSnapPlayTimeToMarkers = false;
	bSnapPlayTimeToPressedKey = true;
	bSnapPlayTimeToDraggedKey = true;
	bForceWholeFrames = true;
	CurveValueSnapInterval = 0.1f;
	GridSpacing = TOptional<float>();
	bSnapCurveValueToInterval = false;
	bShowSelectedNodesOnly = false;
	ZoomPosition = ESequencerZoomPosition::SZP_CurrentTime;
	bAutoScrollEnabled = false;
	bLinkCurveEditorTimeRange = false;
	bSynchronizeCurveEditorSelection = true;
	bSynchronizeOutlinerSelectionToCurveEditor = true;
	bIsolateCurveEditorToSelection = true;
	bCurveEditorVisible = false;
	LoopMode = ESequencerLoopMode::SLM_NoLoop;
	bResetPlayheadWhenNavigating = false;
	PlayheadScrubbingLimit = ESequencerPlayheadScrubbingLimit::SPSL_None;
	bKeepPlayRangeInSectionBounds = true;
	bCompileDirectorOnEvaluate = true;
	bLeftMouseDragDoesMarquee = false;
	ZeroPadFrames = 0;
	JumpFrameIncrement = FFrameNumber(5);
	TimeWarpDisplay = ESequencerTimeWarpDisplay::Both;
	bShowLayerBars = true;
	bShowKeyBars = true;
	bInfiniteKeyAreas = false;
	bShowChannelColors = false;
	bShowInfoButton = true;
	ReduceKeysTolerance = KINDA_SMALL_NUMBER;
	KeyAreaHeightWithCurves = SequencerLayoutConstants::KeyAreaHeight;
	bDeleteKeysWhenTrimming = true;
	bDisableSectionsAfterBaking = true;
	PlaybackRangeStartColor = FLinearColor(0.1f, 0.5f, 0.1f, 1.0f);
	PlaybackRangeEndColor = FLinearColor(0.5f, 0.01f, 0.01f, 1.0f);
	MarkedFrameColor = FLinearColor(0.f, 1.f, 1.f, 0.4f);
	SectionColorAlpha = 1.0f;
	bDrawHorizontalTrackSeparatorLines = false;
	bCleanPlaybackMode = true;
	bActivateRealtimeViewports = true;
	bEvaluateSubSequencesInIsolation = false;
	bRerunConstructionScripts = true;
	bScrubTimeStartFromCursor = true;
	bStopPlayingWhenJumpingToStartOrEnd = false;
	bVisualizePreAndPostRoll = true;
	TrajectoryPathCap = 250;
	FrameNumberDisplayFormat = EFrameNumberDisplayFormats::Seconds;
	bAutoExpandNodesOnSelection = true;
	bAutoExpandOnCurveEditorSelection = true;
	bRestoreOriginalViewportOnCameraCutUnlock = true;
	TreeViewWidth = SSequencer::DefaultOutlinerWidth;
	bShowTickLines = true;
	bShowSequencerToolbar = true;
	bShowMarkedFrames = true;
	bShowScalingAnchors = true;
	ViewDensity = "Relaxed";
	AssetBrowserWidth = 500.f;
	AssetBrowserHeight = 300.f;
	bFilterBarVisible = true;
	bNavigationToolVisible = false;
	bSelectionDetailsPoppedOut = false;
	bShowTransportControls = true;
	bTransportControlsPoppedOut = false;
	bRestoreTransportControlsPoppedOut = false;

	SectionColorTints.Add(FColor(88, 102, 142, 255)); // blue
	SectionColorTints.Add(FColor(99, 137, 132, 255)); // blue-green
	SectionColorTints.Add(FColor(110, 127, 92, 255)); // green
	SectionColorTints.Add(FColor(151, 142, 102, 255)); // yellow
	SectionColorTints.Add(FColor(147, 119, 101, 255)); // orange
	SectionColorTints.Add(FColor(139, 95, 108, 255)); // red 
	SectionColorTints.Add(FColor(109, 74, 121, 255)); // purple
	
	// Positive x -> zooming out, Negative x -> zooming in. X is in seconds.
	// Don't add any factors for zoom-in. Zoom-in is already very fast with a zoom factor of 1.
	CurveEditorZoomScaling.HorizontalZoomScale.EditorCurveData.AddKey(100, 6); // ... e.g. showing 100 seconds worth of frames, zoom out factor is 6.
	CurveEditorZoomScaling.VerticalZoomScale.EditorCurveData.AddKey(100, 6);
	CurveEditorZoomScaling.bLimitHorizontalZoomOut = false;
	CurveEditorZoomScaling.bLimitVerticalZoomOut = false;
	CurveEditorZoomScaling.MaxHorizontalZoomOut = 900; // Cannot zoom out more than 900 seconds = 15 minutes.
	CurveEditorZoomScaling.MaxVerticalZoomOut = 10000;
}

EAutoChangeMode USequencerSettings::GetAutoChangeMode() const
{
	return AutoChangeMode;
}

void USequencerSettings::SetAutoChangeMode(EAutoChangeMode InAutoChangeMode)
{
	if ( AutoChangeMode != InAutoChangeMode )
	{
		AutoChangeMode = InAutoChangeMode;
		SaveConfig();
	}
}

EAllowEditsMode USequencerSettings::GetAllowEditsMode() const
{
	return AllowEditsMode;
}

void USequencerSettings::SetAllowEditsMode(EAllowEditsMode InAllowEditsMode)
{
	if ( AllowEditsMode != InAllowEditsMode )
	{
		AllowEditsMode = InAllowEditsMode;
		SaveConfig();

		OnAllowEditsModeChangedEvent.Broadcast(InAllowEditsMode);
	}
}

EKeyGroupMode USequencerSettings::GetKeyGroupMode() const
{
	return KeyGroupMode;
}

void USequencerSettings::SetKeyGroupMode(EKeyGroupMode InKeyGroupMode)
{
	if (KeyGroupMode != InKeyGroupMode)
	{
		KeyGroupMode = InKeyGroupMode;
		SaveConfig();
	}
}

EMovieSceneKeyInterpolation USequencerSettings::GetKeyInterpolation() const
{
	return KeyInterpolation;
}

void USequencerSettings::SetKeyInterpolation(EMovieSceneKeyInterpolation InKeyInterpolation)
{
	if ( KeyInterpolation != InKeyInterpolation)
	{
		KeyInterpolation = InKeyInterpolation;
		SaveConfig();
	}
}

ESequencerSpawnPosition USequencerSettings::GetSpawnPosition() const
{
	return SpawnPosition;
}

void USequencerSettings::SetSpawnPosition(ESequencerSpawnPosition InSpawnPosition)
{
	if ( SpawnPosition != InSpawnPosition)
	{
		SpawnPosition = InSpawnPosition;
		SaveConfig();
	}
}

bool USequencerSettings::GetCreateSpawnableCameras() const
{
	return bCreateSpawnableCameras;
}

void USequencerSettings::SetCreateSpawnableCameras(bool bInCreateSpawnableCameras)
{
	if ( bCreateSpawnableCameras != bInCreateSpawnableCameras)
	{
		bCreateSpawnableCameras = bInCreateSpawnableCameras;
		SaveConfig();
	}
}

bool USequencerSettings::GetShowRangeSlider() const
{
	return bShowRangeSlider;
}

void USequencerSettings::SetShowRangeSlider(bool InbShowRangeSlider)
{
	if ( bShowRangeSlider != InbShowRangeSlider )
	{
		bShowRangeSlider = InbShowRangeSlider;
		SaveConfig();
	}
}

bool USequencerSettings::GetShowBottomTimeSlider() const
{
	return bShowBottomTimeSlider;
}

void USequencerSettings::SetShowBottomTimeSlider(const bool bInShow)
{
	if (bShowBottomTimeSlider != bInShow)
	{
		bShowBottomTimeSlider = bInShow;
		SaveConfig();
	}
}

bool USequencerSettings::GetIsSnapEnabled() const
{
	return bIsSnapEnabled;
}

void USequencerSettings::SetIsSnapEnabled(bool InbIsSnapEnabled)
{
	if ( bIsSnapEnabled != InbIsSnapEnabled )
	{
		bIsSnapEnabled = InbIsSnapEnabled;
		SaveConfig();
	}
}

bool USequencerSettings::GetSnapKeyTimesToElements() const
{
	return bSnapKeyTimesToElements;
}

void USequencerSettings::SetSnapKeyTimesToElements(bool InbSnapKeyTimesToElements)
{
	if ( bSnapKeyTimesToElements != InbSnapKeyTimesToElements)
	{
		bSnapKeyTimesToElements = InbSnapKeyTimesToElements;
		SaveConfig();
	}
}

bool USequencerSettings::GetLeftMouseDragDoesMarquee() const
{
	return bLeftMouseDragDoesMarquee;
}
void USequencerSettings::SetLeftMouseDragDoesMarque(bool bDoMarque)
{
	if (bLeftMouseDragDoesMarquee != bDoMarque)
	{
		bLeftMouseDragDoesMarquee = bDoMarque;
		SaveConfig();
	}
}

bool USequencerSettings::GetSnapSectionTimesToElements() const
{
	return bSnapSectionTimesToElements;
}

void USequencerSettings::SetSnapSectionTimesToElements( bool InbSnapSectionTimesToElements )
{
	if (bSnapSectionTimesToElements != InbSnapSectionTimesToElements)
	{
		bSnapSectionTimesToElements = InbSnapSectionTimesToElements;
		SaveConfig();
	}
}

bool USequencerSettings::GetSnapPlayTimeToKeys() const
{
	return bSnapPlayTimeToKeys;
}

void USequencerSettings::SetSnapPlayTimeToKeys(bool InbSnapPlayTimeToKeys)
{
	if ( bSnapPlayTimeToKeys != InbSnapPlayTimeToKeys )
	{
		bSnapPlayTimeToKeys = InbSnapPlayTimeToKeys;
		SaveConfig();
	}
}

bool USequencerSettings::GetSnapPlayTimeToSections() const
{
	return bSnapPlayTimeToSections;
}

void USequencerSettings::SetSnapPlayTimeToSections(bool InbSnapPlayTimeToSections)
{
	if (bSnapPlayTimeToSections != InbSnapPlayTimeToSections)
	{
		bSnapPlayTimeToSections = InbSnapPlayTimeToSections;
		SaveConfig();
	}
}

bool USequencerSettings::GetSnapPlayTimeToMarkers() const
{
	return bSnapPlayTimeToMarkers;
}

void USequencerSettings::SetSnapPlayTimeToMarkers(bool InbSnapPlayTimeToMarkers)
{
	if ( bSnapPlayTimeToMarkers != InbSnapPlayTimeToMarkers )
	{
		bSnapPlayTimeToMarkers = InbSnapPlayTimeToMarkers;
		SaveConfig();
	}
}

bool USequencerSettings::GetSnapPlayTimeToPressedKey() const
{
	return bSnapPlayTimeToPressedKey;
}

void USequencerSettings::SetSnapPlayTimeToPressedKey(bool InbSnapPlayTimeToPressedKey)
{
	if ( bSnapPlayTimeToPressedKey != InbSnapPlayTimeToPressedKey )
	{
		bSnapPlayTimeToPressedKey = InbSnapPlayTimeToPressedKey;
		SaveConfig();
	}
}

bool USequencerSettings::GetSnapPlayTimeToDraggedKey() const
{
	return bSnapPlayTimeToDraggedKey;
}

void USequencerSettings::SetSnapPlayTimeToDraggedKey(bool InbSnapPlayTimeToDraggedKey)
{
	if ( bSnapPlayTimeToDraggedKey != InbSnapPlayTimeToDraggedKey )
	{
		bSnapPlayTimeToDraggedKey = InbSnapPlayTimeToDraggedKey;
		SaveConfig();
	}
}

float USequencerSettings::GetCurveValueSnapInterval() const
{
	return CurveValueSnapInterval;
}

void USequencerSettings::SetCurveValueSnapInterval( float InCurveValueSnapInterval )
{
	if ( CurveValueSnapInterval != InCurveValueSnapInterval )
	{
		CurveValueSnapInterval = InCurveValueSnapInterval;
		SaveConfig();
	}
}

TOptional<float> USequencerSettings::GetGridSpacing() const
{
	return GridSpacing;
}

void USequencerSettings::SetGridSpacing(TOptional<float> InGridSpacing)
{
	if (InGridSpacing != GridSpacing)
	{
		GridSpacing = InGridSpacing;
		SaveConfig();
	}
}

bool USequencerSettings::GetSnapCurveValueToInterval() const
{
	return bSnapCurveValueToInterval;
}

void USequencerSettings::SetSnapCurveValueToInterval( bool InbSnapCurveValueToInterval )
{
	if ( bSnapCurveValueToInterval != InbSnapCurveValueToInterval )
	{
		bSnapCurveValueToInterval = InbSnapCurveValueToInterval;
		SaveConfig();
	}
}

bool USequencerSettings::GetForceWholeFrames() const
{
	return bForceWholeFrames;
}

void USequencerSettings::SetForceWholeFrames(bool InbForceWholeFrames)
{
	if (bForceWholeFrames != InbForceWholeFrames)
	{
		bForceWholeFrames = InbForceWholeFrames;
		SaveConfig();
	}
}

bool USequencerSettings::GetShowSelectedNodesOnly() const
{
	return bShowSelectedNodesOnly;
}

void USequencerSettings::SetShowSelectedNodesOnly(bool Visible)
{
	if (bShowSelectedNodesOnly != Visible)
	{
		bShowSelectedNodesOnly = Visible;
		SaveConfig();

		OnShowSelectedNodesOnlyChangedEvent.Broadcast();
	}
}

ESequencerZoomPosition USequencerSettings::GetZoomPosition() const
{
	return ZoomPosition;
}

void USequencerSettings::SetZoomPosition(ESequencerZoomPosition InZoomPosition)
{
	if ( ZoomPosition != InZoomPosition)
	{
		ZoomPosition = InZoomPosition;
		SaveConfig();
	}
}

bool USequencerSettings::GetAutoScrollEnabled() const
{
	return bAutoScrollEnabled;
}

void USequencerSettings::SetAutoScrollEnabled(bool bInAutoScrollEnabled)
{
	if (bAutoScrollEnabled != bInAutoScrollEnabled)
	{
		bAutoScrollEnabled = bInAutoScrollEnabled;
		SaveConfig();
	}
}

bool USequencerSettings::GetScrubTimeStartFromCursor() const
{
	return bScrubTimeStartFromCursor;
}

void USequencerSettings::SetScrubTimeStartFromCursor(bool bInValue)
{
	if (bScrubTimeStartFromCursor != bInValue)
	{
		bScrubTimeStartFromCursor = bInValue;
		SaveConfig();
	}
}

bool USequencerSettings::GetStopPlayingWhenJumpingToStartOrEnd() const
{
	return bStopPlayingWhenJumpingToStartOrEnd;
}

void USequencerSettings::SetStopPlayingWhenJumpingToStartOrEnd(bool bInValue)
{
	if (bStopPlayingWhenJumpingToStartOrEnd != bInValue)
	{
		bStopPlayingWhenJumpingToStartOrEnd = bInValue;
		SaveConfig();
	}
}

void USequencerSettings::SetCurveEditorZoomScaling(const FCurveEditorZoomScaleConfig& Value)
{
	CurveEditorZoomScaling = Value;
	SaveConfig();
}

ESequencerLoopMode USequencerSettings::GetLoopMode() const
{
	return LoopMode;
}

void USequencerSettings::SetLoopMode(ESequencerLoopMode InLoopMode)
{
	if (LoopMode != InLoopMode)
	{
		LoopMode = InLoopMode;
		OnLoopStateChangedEvent.Broadcast();
		SaveConfig();
	}
}

bool USequencerSettings::ShouldResetPlayheadWhenNavigating() const
{
	return bResetPlayheadWhenNavigating;
}

void USequencerSettings::SetResetPlayheadWhenNavigating(bool bInResetPlayheadWhenNavigating)
{
	if (bResetPlayheadWhenNavigating != bInResetPlayheadWhenNavigating)
	{
		bResetPlayheadWhenNavigating = bInResetPlayheadWhenNavigating;
		SaveConfig();
	}
}

ESequencerPlayheadScrubbingLimit USequencerSettings::GetPlayheadScrubbingLimit() const
{
	return PlayheadScrubbingLimit;
}

void USequencerSettings::SetPlayheadScrubbingLimit(ESequencerPlayheadScrubbingLimit InPlayheadScrubbingLimit)
{
	if (PlayheadScrubbingLimit != InPlayheadScrubbingLimit)
	{
		PlayheadScrubbingLimit = InPlayheadScrubbingLimit;
		SaveConfig();
	}
}

TRange<FFrameNumber> USequencerSettings::GetPlayheadScrubbingRange(TWeakPtr<ISequencer> InSequencer) const
{
	TSharedPtr<ISequencer> Sequencer = InSequencer.Pin();
	if (!Sequencer)
	{
		return TRange<FFrameNumber>::Empty();
	}

	TRange<FFrameNumber> PlayheadScrubbingRange;
	if (PlayheadScrubbingLimit == ESequencerPlayheadScrubbingLimit::SPSL_PlaybackRange)
	{
		return PlayheadScrubbingRange = Sequencer->GetSubSequenceRange().Get(Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange());
	}
	else if (PlayheadScrubbingLimit == ESequencerPlayheadScrubbingLimit::SPSL_SelectionRange)
	{
		return PlayheadScrubbingRange = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetSelectionRange();
	}
	return TRange<FFrameNumber>::Empty();
}

bool USequencerSettings::ShouldKeepPlayRangeInSectionBounds() const
{
	return bKeepPlayRangeInSectionBounds;
}

void USequencerSettings::SetKeepPlayRangeInSectionBounds(bool bInKeepPlayRangeInSectionBounds)
{
	if (bKeepPlayRangeInSectionBounds != bInKeepPlayRangeInSectionBounds)
	{
		bKeepPlayRangeInSectionBounds = bInKeepPlayRangeInSectionBounds;
		SaveConfig();
	}
}


bool USequencerSettings::GetLinkCurveEditorTimeRange() const
{
	return bLinkCurveEditorTimeRange;
}

void USequencerSettings::SetLinkCurveEditorTimeRange(bool InbLinkCurveEditorTimeRange)
{
	if (bLinkCurveEditorTimeRange != InbLinkCurveEditorTimeRange)
	{
		bLinkCurveEditorTimeRange = InbLinkCurveEditorTimeRange;
		SaveConfig();
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool USequencerSettings::GetLinkFiltersWithCurveEditor() const
{
	return bLinkFiltersWithCurveEditor_DEPRECATED;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void USequencerSettings::SetLinkFiltersWithCurveEditor(bool bInLinkFiltersWithCurveEditor)
{
	if (bLinkFiltersWithCurveEditor_DEPRECATED != bInLinkFiltersWithCurveEditor)
	{
		bLinkFiltersWithCurveEditor_DEPRECATED = bInLinkFiltersWithCurveEditor;
		SaveConfig();
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void USequencerSettings::SyncCurveEditorSelection(bool bInSynchronizeCurveEditorSelection)
{
	if (bSynchronizeCurveEditorSelection != bInSynchronizeCurveEditorSelection)
	{
		bSynchronizeCurveEditorSelection = bInSynchronizeCurveEditorSelection;
		SaveConfig();
	}
}

void USequencerSettings::SyncOutlinerSelectionToCurveEditor(bool bInValue)
{
	if (bSynchronizeOutlinerSelectionToCurveEditor != bInValue)
	{
		bSynchronizeOutlinerSelectionToCurveEditor = bInValue;
		SaveConfig();
	}
}

void USequencerSettings::IsolateCurveEditorToSelection(bool bInIsolateCurveEditorToSelection)
{
	if (bIsolateCurveEditorToSelection != bInIsolateCurveEditorToSelection)
	{
		bIsolateCurveEditorToSelection = bInIsolateCurveEditorToSelection;
		SaveConfig();
	}
}

void USequencerSettings::SetCurveEditorVisible(bool bInCurveEditorVisible)
{
	if (bCurveEditorVisible != bInCurveEditorVisible)
	{
		bCurveEditorVisible = bInCurveEditorVisible;
		SaveConfig();
	}
}

uint8 USequencerSettings::GetZeroPadFrames() const
{
	return ZeroPadFrames;
}

void USequencerSettings::SetZeroPadFrames(uint8 InZeroPadFrames)
{
	if (ZeroPadFrames != InZeroPadFrames)
	{
		ZeroPadFrames = InZeroPadFrames;
		SaveConfig();
		OnTimeDisplayFormatChangedEvent.Broadcast();
	}
}

FFrameNumber USequencerSettings::GetJumpFrameIncrement() const
{
	return JumpFrameIncrement;
}

void USequencerSettings::SetJumpFrameIncrement(FFrameNumber InJumpFrameIncrement)
{
	if (JumpFrameIncrement != InJumpFrameIncrement)
	{
		JumpFrameIncrement = InJumpFrameIncrement;
		SaveConfig();
	}
}

ESequencerTimeWarpDisplay USequencerSettings::GetTimeWarpDisplayMode() const
{
	return TimeWarpDisplay;
}

void USequencerSettings::SetTimeWarpDisplayMode(ESequencerTimeWarpDisplay InTimeWarpDisplay)
{
	if (TimeWarpDisplay != InTimeWarpDisplay)
	{
		TimeWarpDisplay = InTimeWarpDisplay;
		SaveConfig();
	}
}

bool USequencerSettings::GetShowLayerBars() const
{
	return bShowLayerBars;
}

void USequencerSettings::SetShowLayerBars(bool InbShowLayerBars)
{
	if (bShowLayerBars != InbShowLayerBars)
	{
		bShowLayerBars = InbShowLayerBars;
		SaveConfig();
	}
}

bool USequencerSettings::GetShowKeyBars() const
{
	return bShowKeyBars;
}

void USequencerSettings::SetShowKeyBars(bool InbShowKeyBars)
{
	if (bShowKeyBars != InbShowKeyBars)
	{
		bShowKeyBars = InbShowKeyBars;
		SaveConfig();
	}
}

bool USequencerSettings::GetInfiniteKeyAreas() const
{
	return bInfiniteKeyAreas;
}

void USequencerSettings::SetInfiniteKeyAreas(bool InbInfiniteKeyAreas)
{
	if (bInfiniteKeyAreas != InbInfiniteKeyAreas)
	{
		bInfiniteKeyAreas = InbInfiniteKeyAreas;
		SaveConfig();
	}
}


bool USequencerSettings::GetShowChannelColors() const
{
	return bShowChannelColors;
}

void USequencerSettings::SetShowChannelColors(bool InbShowChannelColors)
{
	if (bShowChannelColors != InbShowChannelColors)
	{
		bShowChannelColors = InbShowChannelColors;
		SaveConfig();
	}
}

bool USequencerSettings::GetShowInfoButton() const
{
	return bShowInfoButton;
}

void USequencerSettings::SetShowInfoButton(bool InbShowInfoButton)
{
	if (bShowInfoButton != InbShowInfoButton)
	{
		bShowInfoButton = InbShowInfoButton;
		SaveConfig();
	}
}

bool USequencerSettings::GetShowTickLines() const
{
	return bShowTickLines;
}

void USequencerSettings::SetShowTickLines(bool bInDrawTickLines)
{
	if(bShowTickLines != bInDrawTickLines)
	{
		bShowTickLines = bInDrawTickLines;
		SaveConfig();
	}
}

bool USequencerSettings::GetShowSequencerToolbar() const
{
	return bShowSequencerToolbar;
}

void USequencerSettings::SetShowSequencerToolbar(bool bInShowSequencerToolbar)
{
	if (bShowSequencerToolbar != bInShowSequencerToolbar)
	{
		bShowSequencerToolbar = bInShowSequencerToolbar;
		SaveConfig();
	}
}

bool USequencerSettings::GetShowMarkedFrames() const
{
	return bShowMarkedFrames;
}

void USequencerSettings::SetShowMarkedFrames(bool bInShowMarkedFrames)
{
	if (bShowMarkedFrames != bInShowMarkedFrames)
	{
		bShowMarkedFrames = bInShowMarkedFrames;
		SaveConfig();
	}
}

bool USequencerSettings::GetShowScalingAnchors() const
{
	return bShowScalingAnchors;
}

void USequencerSettings::SetShowScalingAnchors(bool bInShowScalingAnchors)
{
	if (bShowScalingAnchors != bInShowScalingAnchors)
	{
		bShowScalingAnchors = bInShowScalingAnchors;
		SaveConfig();
	}
}

bool USequencerSettings::HasKeyAreaCurveExtents(const FString& ChannelName) const
{
	TArray<FString> ChannelsArray;
	KeyAreaCurveExtents.ParseIntoArray(ChannelsArray, TEXT(":"));

	for (int32 ChannelIndex = 0; ChannelIndex < ChannelsArray.Num(); ++ChannelIndex)
	{
		TArray<FString> ExtentsArray;
		ChannelsArray[ChannelIndex].ParseIntoArray(ExtentsArray, TEXT(","));	

		if (ExtentsArray.Num() == 3 && ExtentsArray[0] == ChannelName)
		{
			return true;
		}
	}
	return false;
}

void USequencerSettings::RemoveKeyAreaCurveExtents(const FString& ChannelName)
{
	TArray<FString> ChannelsArray;
	KeyAreaCurveExtents.ParseIntoArray(ChannelsArray, TEXT(":"));

	FString NewKeyAreaCurveExtents;
	for (int32 ChannelIndex = 0; ChannelIndex < ChannelsArray.Num(); ++ChannelIndex)
	{
		TArray<FString> ExtentsArray;
		ChannelsArray[ChannelIndex].ParseIntoArray(ExtentsArray, TEXT(","));	

		if (ExtentsArray.Num() == 3 && ExtentsArray[0] == ChannelName)
		{
			continue;
		}

		NewKeyAreaCurveExtents.Append(TEXT(":"));
		NewKeyAreaCurveExtents.Append(ChannelsArray[ChannelIndex]);
	}

	KeyAreaCurveExtents = NewKeyAreaCurveExtents;
	SaveConfig();
}

void USequencerSettings::SetKeyAreaCurveExtents(const FString& ChannelName, double InMin, double InMax)
{
	RemoveKeyAreaCurveExtents(ChannelName);

	FString NewChannelExtents = FString::Printf(TEXT("%s,%0.3f,%0.3f"), *ChannelName, InMin, InMax);
	KeyAreaCurveExtents.Append(TEXT(":"));
	KeyAreaCurveExtents.Append(NewChannelExtents);

	SaveConfig();
}

void USequencerSettings::GetKeyAreaCurveExtents(const FString& ChannelName, double& OutMin, double& OutMax) const
{
	TArray<FString> ChannelsArray;
	KeyAreaCurveExtents.ParseIntoArray(ChannelsArray, TEXT(":"));

	for (int32 ChannelIndex = 0; ChannelIndex < ChannelsArray.Num(); ++ChannelIndex)
	{
		TArray<FString> ExtentsArray;
		ChannelsArray[ChannelIndex].ParseIntoArray(ExtentsArray, TEXT(","));	

		if (ExtentsArray.Num() == 3 && ExtentsArray[0] == ChannelName)
		{
			OutMin = FCString::Atod(*ExtentsArray[1]);
			OutMax = FCString::Atod(*ExtentsArray[2]);
			return;
		}
	}
}

float USequencerSettings::GetKeyAreaHeightWithCurves() const
{
	return KeyAreaHeightWithCurves;
}

void USequencerSettings::SetKeyAreaHeightWithCurves(float InKeyAreaHeightWithCurves)
{
	if (KeyAreaHeightWithCurves != InKeyAreaHeightWithCurves)
	{
		KeyAreaHeightWithCurves = InKeyAreaHeightWithCurves;
		SaveConfig();
	}
}

float USequencerSettings::GetReduceKeysTolerance() const
{
	return ReduceKeysTolerance;
}

void USequencerSettings::SetReduceKeysTolerance(float InReduceKeysTolerance)
{
	if (ReduceKeysTolerance != InReduceKeysTolerance)
	{
		ReduceKeysTolerance = InReduceKeysTolerance;
		SaveConfig();
	}
}

bool USequencerSettings::GetDeleteKeysWhenTrimming() const
{
	return bDeleteKeysWhenTrimming;
}

void USequencerSettings::SetDeleteKeysWhenTrimming(bool bInDeleteKeysWhenTrimming)
{
	if (bDeleteKeysWhenTrimming != bInDeleteKeysWhenTrimming)
	{
		bDeleteKeysWhenTrimming = bInDeleteKeysWhenTrimming;
		SaveConfig();
	}
}

bool USequencerSettings::GetDisableSectionsAfterBaking() const
{
	return bDisableSectionsAfterBaking;
}

void USequencerSettings::SetDisableSectionsAfterBaking(bool bInDisableSectionsAfterBaking)
{
	if (bDisableSectionsAfterBaking != bInDisableSectionsAfterBaking)
	{
		bDisableSectionsAfterBaking = bInDisableSectionsAfterBaking;
		SaveConfig();
	}
}

FLinearColor USequencerSettings::GetPlaybackRangeStartColor() const
{
	return PlaybackRangeStartColor;
}

void USequencerSettings::SetPlaybackRangeStartColor(const FLinearColor& InPlaybackRangeStartColor)
{
	if (PlaybackRangeStartColor != InPlaybackRangeStartColor)
	{
		PlaybackRangeStartColor = InPlaybackRangeStartColor;
		SaveConfig();
	}
}

FLinearColor USequencerSettings::GetPlaybackRangeEndColor() const
{
	return PlaybackRangeEndColor;
}

void USequencerSettings::SetPlaybackRangeEndColor(const FLinearColor& InPlaybackRangeEndColor)
{
	if (PlaybackRangeEndColor != InPlaybackRangeEndColor)
	{
		PlaybackRangeEndColor = InPlaybackRangeEndColor;
		SaveConfig();
	}
}

FLinearColor USequencerSettings::GetMarkedFrameColor() const
{
	return MarkedFrameColor;
}

void USequencerSettings::SetMarkedFrameColor(const FLinearColor& InMarkedFrameColor)
{
	if (MarkedFrameColor != InMarkedFrameColor)
	{
		MarkedFrameColor = InMarkedFrameColor;
		SaveConfig();
	}
}

TArray<FColor> USequencerSettings::GetSectionColorTints() const
{
	return SectionColorTints;
}

void USequencerSettings::SetSectionColorTints(const TArray<FColor>& InSectionColorTints)
{
	if (SectionColorTints != InSectionColorTints)
	{
		SectionColorTints = InSectionColorTints;
		SaveConfig();
	}
}

float USequencerSettings::GetSectionColorAlpha() const
{
	return SectionColorAlpha;
}

void USequencerSettings::SetSectionColorAlpha(const float InAlpha)
{
	if (SectionColorAlpha != InAlpha)
	{
		SectionColorAlpha = InAlpha;
		SaveConfig();
	}
}

bool USequencerSettings::ShouldDrawHorizontalTrackSeparatorLines() const
{
	return bDrawHorizontalTrackSeparatorLines;
}

void USequencerSettings::SetDrawHorizontalTrackSeparatorLines(const bool bInDraw)
{
	if (bDrawHorizontalTrackSeparatorLines != bInDraw)
	{
		bDrawHorizontalTrackSeparatorLines = bInDraw;
		SaveConfig();
	}
}

bool USequencerSettings::GetCleanPlaybackMode() const
{
	return bCleanPlaybackMode;
}

void USequencerSettings::SetCleanPlaybackMode(bool bInCleanPlaybackMode)
{
	if (bInCleanPlaybackMode != bCleanPlaybackMode)
	{
		bCleanPlaybackMode = bInCleanPlaybackMode;
		SaveConfig();
	}
}

bool USequencerSettings::ShouldActivateRealtimeViewports() const
{
	return bActivateRealtimeViewports;
}

void USequencerSettings::SetActivateRealtimeViewports(bool bInActivateRealtimeViewports)
{
	if (bInActivateRealtimeViewports != bActivateRealtimeViewports)
	{
		bActivateRealtimeViewports = bInActivateRealtimeViewports;
		SaveConfig();
	}
}

bool USequencerSettings::GetAutoSetTrackDefaults() const
{
	return bAutoSetTrackDefaults;
}

void USequencerSettings::SetAutoSetTrackDefaults(bool bInAutoSetTrackDefaults)
{
	if (bInAutoSetTrackDefaults != bAutoSetTrackDefaults)
	{
		bAutoSetTrackDefaults = bInAutoSetTrackDefaults;
		SaveConfig();
	}
}

bool USequencerSettings::ShouldShowDebugVisualization() const
{
	return bShowDebugVisualization;
}

void USequencerSettings::SetShowDebugVisualization(bool bInShowDebugVisualization)
{
	if (bShowDebugVisualization != bInShowDebugVisualization)
	{
		bShowDebugVisualization = bInShowDebugVisualization;
		SaveConfig();
	}
}

bool USequencerSettings::ShouldEvaluateSubSequencesInIsolation() const
{
	return bEvaluateSubSequencesInIsolation;
}

void USequencerSettings::SetEvaluateSubSequencesInIsolation(bool bInEvaluateSubSequencesInIsolation)
{
	if (bEvaluateSubSequencesInIsolation != bInEvaluateSubSequencesInIsolation)
	{
		bEvaluateSubSequencesInIsolation = bInEvaluateSubSequencesInIsolation;
		SaveConfig();

		OnEvaluateSubSequencesInIsolationChangedEvent.Broadcast();
	}
}

bool USequencerSettings::ShouldRerunConstructionScripts() const
{
	return bRerunConstructionScripts;
}

void USequencerSettings::SetRerunConstructionScripts(bool bInRerunConstructionScripts)
{
	if (bRerunConstructionScripts != bInRerunConstructionScripts)
	{
		bRerunConstructionScripts = bInRerunConstructionScripts;
		SaveConfig();
	}
}

bool USequencerSettings::ShouldShowPrePostRoll() const
{
	return bVisualizePreAndPostRoll;
}

void USequencerSettings::SetShouldShowPrePostRoll(bool bInVisualizePreAndPostRoll)
{
	if (bInVisualizePreAndPostRoll != bVisualizePreAndPostRoll)
	{
		bVisualizePreAndPostRoll = bInVisualizePreAndPostRoll;
		SaveConfig();
	}
}


bool USequencerSettings::ShouldCompileDirectorOnEvaluate() const
{
	return bCompileDirectorOnEvaluate;
}

void USequencerSettings::SetCompileDirectorOnEvaluate(bool bInCompileDirectorOnEvaluate)
{
	if (bInCompileDirectorOnEvaluate != bCompileDirectorOnEvaluate)
	{
		bCompileDirectorOnEvaluate = bInCompileDirectorOnEvaluate;
		SaveConfig();
	}
}

USequencerSettings::FOnLoopStateChanged& USequencerSettings::GetOnLoopStateChanged()
{
	return OnLoopStateChangedEvent;
}

USequencerSettings::FOnTimeDisplayFormatChanged& USequencerSettings::GetOnTimeDisplayFormatChanged()
{
	return OnTimeDisplayFormatChangedEvent;
}

void USequencerSettings::SetTimeDisplayFormat(EFrameNumberDisplayFormats InFormat)
{
	if (InFormat != FrameNumberDisplayFormat)
	{
		FrameNumberDisplayFormat = InFormat;
		SaveConfig();
		OnTimeDisplayFormatChangedEvent.Broadcast();
	}
}

void USequencerSettings::SetMovieRendererName(const FString& InMovieRendererName)
{
	if (InMovieRendererName != MovieRendererName)
	{
		MovieRendererName = InMovieRendererName;
		SaveConfig();
	}
}

void USequencerSettings::SetAutoExpandNodesOnSelection(bool bInAutoExpandNodesOnSelection)
{
	if (bInAutoExpandNodesOnSelection != bAutoExpandNodesOnSelection)
	{
		bAutoExpandNodesOnSelection = bInAutoExpandNodesOnSelection;
		SaveConfig();
	}
}

void USequencerSettings::SetAutoExpandCurveEditorOnSelection(bool bValue)
{
	if (bValue != bAutoExpandOnCurveEditorSelection)
	{
		bAutoExpandOnCurveEditorSelection = bValue;
		SaveConfig();
	}
}

void USequencerSettings::SetRestoreOriginalViewportOnCameraCutUnlock(bool bInRestoreOriginalViewportOnCameraCutUnlock)
{
	if (bInRestoreOriginalViewportOnCameraCutUnlock != bRestoreOriginalViewportOnCameraCutUnlock)
	{
		bRestoreOriginalViewportOnCameraCutUnlock = bInRestoreOriginalViewportOnCameraCutUnlock;
		SaveConfig();
	}
}

void USequencerSettings::SetTreeViewWidth(float InTreeViewWidth)
{
	if (InTreeViewWidth != TreeViewWidth)
	{
		TreeViewWidth = InTreeViewWidth;
		SaveConfig();
	}
}

UE::Sequencer::EViewDensity USequencerSettings::GetViewDensity() const
{
	static FName NAME_Compact("Compact");
	static FName NAME_Relaxed("Relaxed");
	if (ViewDensity == NAME_Compact)
	{
		return UE::Sequencer::EViewDensity::Compact;
	}
	if (ViewDensity == NAME_Relaxed)
	{
		return UE::Sequencer::EViewDensity::Relaxed;
	}
	return UE::Sequencer::EViewDensity::Variable;
}

void USequencerSettings::SetViewDensity(FName InViewDensity)
{
	if (InViewDensity != ViewDensity)
	{
		ViewDensity = InViewDensity;
		SaveConfig();
	}
}

void USequencerSettings::SetAssetBrowserWidth(float InAssetBrowserWidth)
{
	if (InAssetBrowserWidth != AssetBrowserWidth)
	{
		AssetBrowserWidth = InAssetBrowserWidth;
		SaveConfig();
	}
}

void USequencerSettings::SetAssetBrowserHeight(float InAssetBrowserHeight)
{
	if (InAssetBrowserHeight != AssetBrowserHeight)
	{
		AssetBrowserHeight = InAssetBrowserHeight;
		SaveConfig();
	}
}

FSidebarState& USequencerSettings::GetSidebarState()
{
	return SidebarState.FindOrAdd(GetFName());
}

void USequencerSettings::SetSidebarState(const FSidebarState& InSidebarState)
{
	FSidebarState& State = SidebarState.FindOrAdd(GetFName());
	State = InSidebarState;
	SaveConfig();
}

void USequencerSettings::SetOutlinerColumnVisibility(const TArray<FColumnVisibilitySetting>& InColumnVisibilitySettings)
{
	if (InColumnVisibilitySettings != ColumnVisibilitySettings)
	{
		ColumnVisibilitySettings = InColumnVisibilitySettings;
		SaveConfig();
	}
}

FSequencerFilterBarConfig& USequencerSettings::FindOrAddTrackFilterBar(const FName InIdentifier, const bool bInSaveConfig)
{
	FSequencerFilterBarConfig& FilterBarSettings = TrackFilterBars.FindOrAdd(InIdentifier);

	if (bInSaveConfig)
	{
		SaveConfig();
	}

	return FilterBarSettings;
}

FSequencerFilterBarConfig* USequencerSettings::FindTrackFilterBar(const FName InIdentifier)
{
	return TrackFilterBars.Find(InIdentifier);
}

bool USequencerSettings::RemoveTrackFilterBar(const FName InIdentifier)
{
	const int32 RemovedCount = TrackFilterBars.Remove(InIdentifier) > 0;
	SaveConfig();
	return RemovedCount > 0;
}

FSequencerFilterAreaConfig& USequencerSettings::FindOrAddTrackFilterArea(const FName InIdentifier, bool bInSaveConfig)
{
	FSequencerFilterAreaConfig& FilterBarSettings = TrackFilterAreas.FindOrAdd(InIdentifier);

	if (bInSaveConfig)
	{
		SaveConfig();
	}

	return FilterBarSettings;
}

FSequencerFilterAreaConfig* USequencerSettings::FindTrackFilterArea(const FName InIdentifier)
{
	return TrackFilterAreas.Find(InIdentifier);
}

bool USequencerSettings::RemoveTrackFilterArea(const FName InIdentifier)
{
	const int32 RemovedCount = TrackFilterBars.Remove(InIdentifier) > 0;
	const bool bHasChanged = RemovedCount > 0;
	if (bHasChanged)
	{
		SaveConfig();
	}
	return bHasChanged;
}

bool USequencerSettings::GetIncludePinnedInFilter() const
{
	return bIncludePinnedInFilter;
}

void USequencerSettings::SetIncludePinnedInFilter(const bool bInIncludePinned)
{
	bIncludePinnedInFilter = bInIncludePinned;
	SaveConfig();
}

bool USequencerSettings::GetAutoExpandNodesOnFilterPass() const
{
	return bAutoExpandNodesOnFilterPass;
}

void USequencerSettings::SetAutoExpandNodesOnFilterPass(const bool bInIncludeParents)
{
	bAutoExpandNodesOnFilterPass = bInIncludeParents;
	SaveConfig();
}

bool USequencerSettings::GetUseFilterSubmenusForCategories() const
{
	return bUseFilterSubmenusForCategories;
}

void USequencerSettings::SetUseFilterSubmenusForCategories(const bool bInUseFilterSubmenusForCategories)
{
	bUseFilterSubmenusForCategories = bInUseFilterSubmenusForCategories;
	SaveConfig();
}

bool USequencerSettings::IsFilterBarVisible() const
{
	return bFilterBarVisible;
}

void USequencerSettings::SetFilterBarVisible(const bool bInVisible)
{
	bFilterBarVisible = bInVisible;
	SaveConfig();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
EFilterBarLayout USequencerSettings::GetFilterBarLayout() const
{
	return LastFilterBarLayout_DEPRECATED;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void USequencerSettings::SetFilterBarLayout(const EFilterBarLayout InLayout)
{
	LastFilterBarLayout_DEPRECATED = InLayout;
	SaveConfig();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

float USequencerSettings::GetLastFilterBarSizeCoefficient() const
{
	return LastFilterBarSizeCoefficient;
}

void USequencerSettings::SetLastFilterBarSizeCoefficient(const float bInSizeCoefficient, const bool bInSaveConfig)
{
	if (LastFilterBarSizeCoefficient != bInSizeCoefficient)
	{
		LastFilterBarSizeCoefficient = bInSizeCoefficient;

		if (bInSaveConfig)
		{
			SaveConfig();
		}
	}
}

void USequencerSettings::SetThumbnailCaptureSettings(const FSequencerThumbnailCaptureSettings& InNewValue)
{
	ThumbnailCaptureSettings = InNewValue;
	SaveConfig();
}

bool USequencerSettings::ShouldShowThumbnailCaptureSettings()
{
	return UE::Sequencer::CVarEnableRelevantThumbnails.GetValueOnGameThread();
}

void USequencerSettings::SetNavigationToolVisible(const bool bInVisible)
{
	if (bNavigationToolVisible != bInVisible)
	{
		bNavigationToolVisible = bInVisible;
		SaveConfig();
	}
}

void USequencerSettings::SetSelectionDetailsPoppedOut(const bool bInPoppedOut)
{
	if (bSelectionDetailsPoppedOut != bInPoppedOut)
	{
		bSelectionDetailsPoppedOut = bInPoppedOut;
		SaveConfig();
	}
}

bool USequencerSettings::ShouldShowTransportControls() const
{
	return bShowTransportControls;
}

void USequencerSettings::SetShowTransportControls(const bool bInShowTransportControls)
{
	if (bShowTransportControls != bInShowTransportControls)
	{
		bShowTransportControls = bInShowTransportControls;
		SaveConfig();
	}
}

bool USequencerSettings::AreTransportControlsPoppedOut() const
{
	return bTransportControlsPoppedOut;
}

void USequencerSettings::SetTransportControlsPoppedOut(const bool bInPoppedOut)
{
	if (bTransportControlsPoppedOut != bInPoppedOut)
	{
		bTransportControlsPoppedOut = bInPoppedOut;
		SaveConfig();
	}
}

bool USequencerSettings::ShouldRestoreTransportControlsPoppedOut() const
{
	return bRestoreTransportControlsPoppedOut;
}

void USequencerSettings::SetRestoreTransportControlsPoppedOut(const bool bInRestorePoppedOut)
{
	if (bRestoreTransportControlsPoppedOut != bInRestorePoppedOut)
	{
		bRestoreTransportControlsPoppedOut = bInRestorePoppedOut;
		SaveConfig();
	}
}

void USequencerSettings::SetIsSimpleView(const bool bInSimpleView)
{
	if (bSimpleView != bInSimpleView)
	{
		bSimpleView = bInSimpleView;
		SaveConfig();
	}
}
