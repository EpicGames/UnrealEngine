// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerSpawnableColumn.h"

#include "ActorTreeItem.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "ISceneOutlinerTreeItem.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"

class SWidget;
struct FSlateBrush;
template <typename ItemType> class STableRow;

#define LOCTEXT_NAMESPACE "SpawnableInfoColumn"

namespace Sequencer
{

FName FSequencerSpawnableColumn::GetID()
{
	static FName IDName("Spawnable");
	return IDName;
}

FName FSequencerSpawnableColumn::GetColumnID()
{
	return GetID();
}

SHeaderRow::FColumn::FArguments FSequencerSpawnableColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FixedWidth(20.f)
		.HAlignHeader(HAlign_Left) // Note the icon has some baked in right alignment, so we are compensating here
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Left) // Note the icon has some baked in right alignment, so we are compensating here
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(LOCTEXT("SpawnableColumnTooltip", "Whether this actor is spawned by Sequencer"))
		.HeaderContentPadding(FMargin(0.0))
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Sequencer.SpawnableIconOverlay"))
		];
}

const TSharedRef< SWidget > FSequencerSpawnableColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	return SNew(SImage).Image(this, &FSequencerSpawnableColumn::GetSpawnableIcon, TreeItem);
}

const FSlateBrush* FSequencerSpawnableColumn::GetSpawnableIcon( FSceneOutlinerTreeItemRef TreeItem ) const
{
	static const FSlateBrush* SpawnedBrush  = FAppStyle::Get().GetBrush("Sequencer.SpawnableIconOverlay");
	static const FSlateBrush* ReplaceableBrush = FAppStyle::Get().GetBrush("Sequencer.ReplaceableIconOverlay");
	static const FSlateBrush* NoSpawnedBrush  = FAppStyle::Get().GetBrush("NoBrush");

	bool bIsSpawned = false;
	bool bIsPreviewActor = false;
	{
		static const FName SequencerActorTag(TEXT("SequencerActor"));
		static const FName SequencerPreviewActorTag(TEXT("SequencerPreviewActor"));
		if (const FActorTreeItem* ActorItem = (*TreeItem).CastTo<FActorTreeItem>())
		{
			AActor* Actor = ActorItem->Actor.Get();
			bIsSpawned = Actor && Actor->ActorHasTag(SequencerActorTag);
			bIsPreviewActor = Actor && Actor->ActorHasTag(SequencerPreviewActorTag);
		}
	}

	return bIsPreviewActor ? ReplaceableBrush : bIsSpawned ? SpawnedBrush : NoSpawnedBrush;
}

}// End Sequencer namespace

#undef LOCTEXT_NAMESPACE
