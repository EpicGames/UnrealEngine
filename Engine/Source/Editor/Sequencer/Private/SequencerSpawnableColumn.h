// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// #include "SceneOutlinerPublicTypes.h"
#include "ISceneOutlinerColumn.h"
#include "SceneOutlinerFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/Views/SHeaderRow.h"

class ISceneOutliner;
class SWidget;
struct FSlateBrush;
template <typename ItemType> class STableRow;

namespace Sequencer
{

/**
 * A custom column for the SceneOutliner to display if the Actor row is Spawnable 
 */
class FSequencerSpawnableColumn : public ISceneOutlinerColumn
{

public:
	FSequencerSpawnableColumn() {}
	FSequencerSpawnableColumn(ISceneOutliner& InSceneOutliner) {}

	static FName GetID();

	virtual FName GetColumnID() override;

	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	
	virtual const TSharedRef< SWidget > ConstructRowWidget( FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row ) override;

	const FSlateBrush* GetSpawnableIcon( FSceneOutlinerTreeItemRef TreeItem ) const;
	
};

}