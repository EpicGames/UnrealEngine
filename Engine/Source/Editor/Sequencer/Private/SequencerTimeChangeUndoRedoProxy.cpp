// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTimeChangeUndoRedoProxy.h"
#include "Sequencer.h"
#include "ScopedTransaction.h"
#include "MovieScene.h"
#include "CoreGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequencerTimeChangeUndoRedoProxy)

#define LOCTEXT_NAMESPACE "Sequencer"

FSequencerTimeChangedHandler::~FSequencerTimeChangedHandler()
{
	if (OnActivateSequenceChangedHandle.IsValid() && WeakSequencer.IsValid())
	{
		WeakSequencer.Pin()->OnActivateSequence().Remove(OnActivateSequenceChangedHandle);
	}
}

void FSequencerTimeChangedHandler::SetSequencer(TSharedRef<FSequencer> InSequencer)
{
	WeakSequencer = InSequencer;
	if (UndoRedoProxy)
	{
		UndoRedoProxy->WeakSequencer = InSequencer;
	}
	OnActivateSequenceChangedHandle = InSequencer->OnActivateSequence().AddRaw(this, &FSequencerTimeChangedHandler::OnActivateSequenceChanged);
	OnActivateSequenceChanged(InSequencer->GetFocusedTemplateID());
}

void FSequencerTimeChangedHandler::OnActivateSequenceChanged(FMovieSceneSequenceIDRef ID)
{
	if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
	{
		if (SequencerPtr->GetFocusedMovieSceneSequence() && SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene())
		{
			using namespace UE::MovieScene;
			if (UndoRedoProxy)
			{
				UndoRedoProxy->bTimeWasSet = false;
			}
			MovieSceneModified.Unlink();
			SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene()->UMovieSceneSignedObject::EventHandlers.Link(MovieSceneModified, this);
		}
	}
}

void FSequencerTimeChangedHandler::OnModifiedIndirectly(UMovieSceneSignedObject* Object)
{
	if (UndoRedoProxy == nullptr)
	{
		return;
	}
	if (Object->IsA<UMovieSceneSection>() && UndoRedoProxy->WeakSequencer.IsValid())
	{
		FQualifiedFrameTime InTime = UndoRedoProxy->WeakSequencer.Pin()->GetGlobalTime();

		if (UndoRedoProxy->bTimeWasSet)
		{
			if (!GIsTransacting && (InTime.Time != UndoRedoProxy->Time.Time || InTime.Rate != UndoRedoProxy->Time.Rate))
			{
				const FScopedTransaction Transaction(LOCTEXT("TimeChanged", "Time Changed"));
				UndoRedoProxy->Modify();
			}
		}
		UndoRedoProxy->bTimeWasSet = true;
		UndoRedoProxy->Time = InTime;
	}
}

void USequencerTimeChangeUndoRedoProxy::PostEditUndo()
{
	if (WeakSequencer.IsValid())
	{
		WeakSequencer.Pin()->SetGlobalTime(Time.Time, true);
	}
}



#undef LOCTEXT_NAMESPACE

