// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "EventHandlers/ISignedObjectEventHandler.h"
#include "Delegates/IDelegateInstance.h"
#include "Misc/QualifiedFrameTime.h"
#include "MovieSceneSequenceID.h"
#include "SequencerTimeChangeUndoRedoProxy.generated.h"

class FSequencer;



UCLASS()
class  USequencerTimeChangeUndoRedoProxy : public UObject
{
public:
	GENERATED_BODY()
	USequencerTimeChangeUndoRedoProxy() :bTimeWasSet(false), WeakSequencer(nullptr) {  };
	~USequencerTimeChangeUndoRedoProxy() = default;

	/*~ UObject */
	virtual void PostEditUndo() override;

	UPROPERTY(Transient)
	FQualifiedFrameTime Time;
	
	//no TOptional UPROPERTY so use this instead
	UPROPERTY(Transient)
	bool bTimeWasSet = false;
	
	TWeakPtr<FSequencer> WeakSequencer;

};


class FSequencerTimeChangedHandler: public UE::MovieScene::ISignedObjectEventHandler
{
public:
	FSequencerTimeChangedHandler() : UndoRedoProxy(nullptr), WeakSequencer(nullptr){};
	virtual ~FSequencerTimeChangedHandler();

	/*~ ISignedObjectEventHandler Interface */
	virtual void  OnModifiedIndirectly(UMovieSceneSignedObject*) override;

	void SetSequencer(TSharedRef<FSequencer> InSequencer);
	void OnActivateSequenceChanged(FMovieSceneSequenceIDRef ID);

	TObjectPtr<USequencerTimeChangeUndoRedoProxy> UndoRedoProxy;
	UE::MovieScene::TNonIntrusiveEventHandler<UE::MovieScene::ISignedObjectEventHandler> MovieSceneModified;
	TWeakPtr<FSequencer> WeakSequencer;
	FDelegateHandle OnActivateSequenceChangedHandle;

};

