// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTimeSliderController.h"
#include "MVVM/ViewModels/OutlinerSpacer.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/Views/SOutlinerView.h"
#include "MVVM/Extensions/IClockExtension.h"
#include "TrackEditors/TimeWarpTrackEditor.h"
#include "Channels/MovieSceneTimeWarpChannel.h"
#include "SequencerTimeDomainOverride.h"
#include "SequencerToolMenuContext.h"
#include "SSequencer.h"
#include "Fonts/SlateFontInfo.h"
#include "Rendering/DrawElements.h"
#include "Misc/Paths.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Misc/InputState.h"
#include "Fonts/FontMeasure.h"
#include "Styling/CoreStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "SequencerCommands.h"
#include "SequencerCommonHelpers.h"
#include "SequencerSettings.h"
#include "Misc/KeyHelperUtils.h"
#include "Misc/QualifiedFrameTime.h"
#include "MovieSceneTimeHelpers.h"
#include "CommonFrameRates.h"
#include "ScopedTransaction.h"
#include "Sequencer.h"
#include "SequencerUtilities.h"
#include "Modules/ModuleManager.h"
#include "Decorations/MovieSceneScalingAnchors.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "SequencerSelectedKey.h"

#include "Misc/NotifyHook.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "ISinglePropertyView.h"
#include "IStructureDetailsView.h"
#include "FrameNumberDetailsCustomization.h"

#define LOCTEXT_NAMESPACE "TimeSlider"

namespace ScrubConstants
{
	/** The minimum amount of pixels between each major ticks on the widget */
	const int32 MinPixelsPerDisplayTick = 12;

	/**The smallest number of units between between major tick marks */
	const float MinDisplayTickSpacing = 0.001f;

	/**The fraction of the current view range to scroll per unit delta  */
	const float ScrollPanFraction = 0.1f;

	/** Marked frame label box margin */
	const int32 MarkLabelBoxMargin = 2;

	/** Marked frame label box margin on the opposite side of the marker time */
	const int32 MarkLabelBoxWideMargin = 4;
}

bool GSequencerShowTimeWarpScrubberLink = false;
FAutoConsoleVariableRef CVarSequencerShowTimeWarpScrubberLink(
	TEXT("Sequencer.ShowTimeWarpScrubberLink"),
	GSequencerShowTimeWarpScrubberLink,
	TEXT("(Default: false) When enabled, shows a dashed link to link warped and unwarped time scrubbers."),
	ECVF_Default
);


FSequencerTimeSliderController::FSequencerTimeSliderController( const FTimeSliderArgs& InArgs, TWeakPtr<FSequencer> InWeakSequencer )
	: WeakSequencer(InWeakSequencer)
	, TimeSliderArgs(InArgs)
	, DistanceDragged(0.0f)
	, MouseDragType(DRAG_NONE)
	, bMouseDownInRegion(false)
	, bPanning(false)
	, HoverMarkIndex(INDEX_NONE)
{
	ScrubFillBrush                 = FAppStyle::GetBrush(TEXT("Sequencer.Timeline.ScrubFill"));
	FrameBlockScrubHandleDownBrush = FAppStyle::GetBrush(TEXT("Sequencer.Timeline.FrameBlockScrubHandle"));
	VanillaScrubHandleDownBrush    = FAppStyle::GetBrush(TEXT("Sequencer.Timeline.ScrubHandle"));

	FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService().ToSharedPtr();
	SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);
	SmallBoldLayoutFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);

	ContextMenuSuppression = 0;

	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		Sequencer->OnGlobalTimeChanged().AddRaw(this, &FSequencerTimeSliderController::SetIsEvaluating);
	}
}

FSequencerTimeSliderController::~FSequencerTimeSliderController()
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		Sequencer->OnGlobalTimeChanged().RemoveAll(this);
	}
}

FFrameTime FSequencerTimeSliderController::SnapSequencerTime(FFrameTime InTime) const
{
	using namespace UE::Sequencer;
	
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return InTime;
	}

	if (Sequencer->GetSequencerSettings()->GetForceWholeFrames())
	{
		if (Sequencer->GetScrubStyle() == ESequencerScrubberStyle::FrameBlock)
		{
			// Floor to the display frame
			InTime = ConvertFrameTime(ConvertFrameTime(InTime, GetTickResolution(), GetDisplayRate()).FloorToFrame(), GetDisplayRate(), GetTickResolution());
		}
		else
		{
			// Snap (round) to display rate
			InTime = FFrameRate::Snap(InTime, GetTickResolution(), GetDisplayRate());
		}
	}
	if (Sequencer->GetSequencerSettings()->GetIsSnapEnabled())
	{
		TViewModelPtr<IClockExtension> Clock = Sequencer->GetViewModel()->GetRootSequenceModel().ImplicitCast();
		if (Clock.IsValid() && Clock->SupportsSnapping() && Clock->ShouldSnapFrameTime())
		{
			InTime = Clock->SnapFrameTime(InTime);
		}
	}

	return InTime;
}

FFrameTime FSequencerTimeSliderController::ComputeScrubTimeFromMouse(const FGeometry& Geometry, const FPointerEvent& MouseEvent, FScrubRangeToScreen RangeToScreen) const
{
	FVector2D           ScreenSpacePosition = MouseEvent.GetScreenSpacePosition();
	FVector2D           CursorPos     = Geometry.AbsoluteToLocal( ScreenSpacePosition );
	double              MouseSeconds  = RangeToScreen.LocalXToInput( CursorPos.X );
	FFrameTime          ScrubTime     = MouseSeconds * GetTickResolution();

	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return ScrubTime;
	}
	
	ScrubTime = SnapSequencerTime(ScrubTime);

	if (Sequencer->GetSequencerSettings()->GetIsSnapEnabled() || MouseEvent.IsShiftDown())
	{
		// SnapTimeToNearestKey will return ScrubTime unmodified if there is no key within range.
		ScrubTime = SnapTimeToNearestKey(MouseEvent, RangeToScreen, CursorPos.X, ScrubTime);
	}

	return ScrubTime;
}

FFrameTime FSequencerTimeSliderController::ComputeFrameTimeFromMouse(const FGeometry& Geometry, FVector2D ScreenSpacePosition, FScrubRangeToScreen RangeToScreen, bool CheckSnapping) const
{
	FVector2D CursorPos  = Geometry.AbsoluteToLocal( ScreenSpacePosition );
	double    MouseValue = RangeToScreen.LocalXToInput( CursorPos.X );

	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return MouseValue * GetTickResolution();
	}

	if (CheckSnapping && Sequencer->GetSequencerSettings()->GetForceWholeFrames())
	{
		FFrameNumber        SnappedFrameNumber = (MouseValue * GetDisplayRate()).FloorToFrame();
		FQualifiedFrameTime RoundedPlayFrame   = FQualifiedFrameTime(SnappedFrameNumber, GetDisplayRate());
		return RoundedPlayFrame.ConvertTo(GetTickResolution());
	}
	else
	{
		return MouseValue * GetTickResolution();
	}
}

FSequencerTimeSliderController::FScrubberMetrics FSequencerTimeSliderController::GetHitTestScrubPixelMetrics(const FScrubRangeToScreen& RangeToScreen) const
{
	static const float DragToleranceSlateUnits = 2.f, MouseTolerance = 2.f;
	return GetScrubPixelMetrics(FQualifiedFrameTime(TimeSliderArgs.ScrubPosition.Get(), GetTickResolution()), RangeToScreen, DragToleranceSlateUnits + MouseTolerance);
}

FSequencerTimeSliderController::FScrubberMetrics FSequencerTimeSliderController::GetScrubPixelMetrics(const FQualifiedFrameTime& ScrubTime, const FScrubRangeToScreen& RangeToScreen, float DilationPixels) const
{
	static constexpr float MinScrubSize = 14.f;
	static constexpr float HalfMinScrubSize = MinScrubSize * .5f;

	const FFrameRate DisplayRate = GetDisplayRate();
	const FFrameNumber Frame = ScrubTime.ConvertTo(DisplayRate).FloorToFrame();

	float FrameStartPixel = RangeToScreen.InputToLocalX(Frame / DisplayRate);
	float FrameEndPixel = RangeToScreen.InputToLocalX((Frame + 1) / DisplayRate) - 1.f;
	{
		float RoundedStartPixel = FMath::RoundToInt(FrameStartPixel);
		FrameEndPixel -= (FrameStartPixel - RoundedStartPixel);

		FrameStartPixel = RoundedStartPixel;
		FrameEndPixel   = FMath::Max(FrameEndPixel, FrameStartPixel + 1);
	}

	// Store off the pixel width of the frame
	FScrubberMetrics Metrics;
	Metrics.FrameExtentsPx = TRange<float>(
		FrameStartPixel - DilationPixels,
		FrameEndPixel + DilationPixels
	);

	// Set the style of the scrub handle
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	Metrics.Style = Sequencer.IsValid()
		? Sequencer->GetScrubStyle()
		: ESequencerScrubberStyle::Vanilla;

	// Always draw the extents on the section area for frame block styles
	Metrics.bDrawExtents = Metrics.Style == ESequencerScrubberStyle::FrameBlock;

	// If it's vanilla style or too small to show the frame width, set that up
	if (Metrics.Style == ESequencerScrubberStyle::Vanilla
		|| FrameEndPixel - FrameStartPixel < MinScrubSize)
	{
		Metrics.Style = ESequencerScrubberStyle::Vanilla;

		const float ScrubPixel = RangeToScreen.InputToLocalX(ScrubTime.AsSeconds());
		Metrics.HandleRangePx = TRange<float>(
			ScrubPixel - HalfMinScrubSize - DilationPixels,
			ScrubPixel + HalfMinScrubSize + DilationPixels
		);
	}
	else
	{
		Metrics.HandleRangePx = Metrics.FrameExtentsPx;
	}

	return Metrics;
}

void FSequencerTimeSliderController::DrawTicks(FSlateWindowElementList& OutDrawElements, const TRange<double>& ViewRange, const FScrubRangeToScreen& RangeToScreen, FDrawTickArgs& InArgs) const
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	TViewModelPtr<IClockExtension> Clock = Sequencer->GetViewModel()->GetRootSequenceModel().ImplicitCast();

	if (Clock && Clock->DrawTicks(Sequencer, OutDrawElements, ViewRange, RangeToScreen, InArgs))
	{
		return;
	}

	DrawLinearTicks(OutDrawElements, ViewRange, RangeToScreen, InArgs);
}

void FSequencerTimeSliderController::DrawLinearTicks(FSlateWindowElementList& OutDrawElements, const TRange<double>& ViewRange, const FScrubRangeToScreen& RangeToScreen, FDrawTickArgs& InArgs) const
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	FFrameRate     TickResolution  = GetTickResolution();
	FFrameRate     DisplayRate     = GetDisplayRate();
	FPaintGeometry PaintGeometry   = InArgs.AllottedGeometry.ToPaintGeometry();
	FSlateFontInfo TickFrameFont = FCoreStyle::GetDefaultFontStyle("Regular", 8);

	double MajorGridStep  = 0.0;
	int32  MinorDivisions = 0;
	if (!Sequencer->GetGridMetrics(InArgs.AllottedGeometry.Size.X, ViewRange.GetLowerBoundValue(), ViewRange.GetUpperBoundValue(), MajorGridStep, MinorDivisions))
	{
		return;
	}

	if (InArgs.bOnlyDrawMajorTicks)
	{
		MinorDivisions = 0;
	}

	TArray<FVector2D> LinePoints;
	LinePoints.SetNumUninitialized(2);

	const bool bAntiAliasLines = false;

	const double FirstMajorLine = FMath::FloorToDouble(ViewRange.GetLowerBoundValue() / MajorGridStep) * MajorGridStep;
	const double LastMajorLine  = FMath::CeilToDouble(ViewRange.GetUpperBoundValue() / MajorGridStep) * MajorGridStep;

	const float  FlooredScrubPx  = RangeToScreen.InputToLocalX(ConvertFrameTime(TimeSliderArgs.ScrubPosition.Get(), TickResolution, GetDisplayRate()).FloorToFrame() / DisplayRate);

	TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface = Sequencer->GetNumericTypeInterface();

	for (double CurrentMajorLine = FirstMajorLine; CurrentMajorLine < LastMajorLine; CurrentMajorLine += MajorGridStep)
	{
		float MajorLinePx = RangeToScreen.InputToLocalX( CurrentMajorLine );

		LinePoints[0] = FVector2D( MajorLinePx, InArgs.TickOffset );
		LinePoints[1] = FVector2D( MajorLinePx, InArgs.TickOffset + InArgs.MajorTickHeight );

		// Draw each tick mark
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			InArgs.StartLayer,
			PaintGeometry,
			LinePoints,
			InArgs.DrawEffects,
			InArgs.TickColor,
			bAntiAliasLines
			);

		if (!InArgs.bOnlyDrawMajorTicks && !FMath::IsNearlyEqual(MajorLinePx, FlooredScrubPx, 3.f))
		{
			FString FrameString = NumericTypeInterface->ToString((CurrentMajorLine * TickResolution).RoundToFrame().Value);

			// Space the text between the tick mark but slightly above
			FVector2D TextOffset( MajorLinePx + 5.f, InArgs.bMirrorLabels ? 1.f : FMath::Abs( InArgs.AllottedGeometry.Size.Y - (InArgs.MajorTickHeight+3.f) ) );
			FSlateDrawElement::MakeText(
				OutDrawElements,
				InArgs.StartLayer+1, 
				InArgs.AllottedGeometry.ToPaintGeometry( InArgs.AllottedGeometry.Size, FSlateLayoutTransform(TextOffset) ), 
				FrameString, 
				TickFrameFont,
				InArgs.DrawEffects,
				InArgs.TickColor*0.65f 
			);
		}

		for (int32 Step = 1; Step < MinorDivisions; ++Step)
		{
			// Compute the size of each tick mark.  If we are half way between to visible values display a slightly larger tick mark
			const float MinorTickHeight = ( (MinorDivisions % 2 == 0) && (Step % (MinorDivisions/2)) == 0 ) ? 6.0f : 2.0f;
			const float MinorLinePx = RangeToScreen.InputToLocalX( CurrentMajorLine + Step*MajorGridStep/MinorDivisions );

			LinePoints[0] = FVector2D(MinorLinePx, InArgs.bMirrorLabels ? 0.0f : FMath::Abs( InArgs.AllottedGeometry.Size.Y - MinorTickHeight ) );
			LinePoints[1] = FVector2D(MinorLinePx, LinePoints[0].Y + MinorTickHeight);

			// Draw each sub mark
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				InArgs.StartLayer,
				PaintGeometry,
				LinePoints,
				InArgs.DrawEffects,
				InArgs.TickColor,
				bAntiAliasLines
			);
		}
	}
}

void FSequencerTimeSliderController::DrawHorizontalTrackSeparatorLines(FSlateWindowElementList& OutDrawElements, FDrawTickArgs& InArgs) const
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<SSequencer> SequencerWidget = Sequencer->GetUnderlyingSequencerWidget();
	if (!SequencerWidget.IsValid())
	{
		return;
	}

	const TSharedPtr<SOutlinerView> TreeViewWidget = SequencerWidget->GetTreeView();
	if (!TreeViewWidget.IsValid())
	{
		return;
	}

	const TSharedPtr<SOutlinerView> OutlinerViewWidget = Sequencer->GetOutlinerViewWidget();
	if (!OutlinerViewWidget.IsValid())
	{
		return;
	}

	TArray<TViewModelPtr<IOutlinerExtension>> VisibleItems;
	OutlinerViewWidget->GetVisibleItems(VisibleItems);

	for (const TViewModelPtr<IOutlinerExtension>& Item : VisibleItems)
	{
		FViewModelPtr ItemModel = Item.AsModel();

		if (!ItemModel.IsValid() || ItemModel->IsA<FOutlinerSpacer>())
		{
			continue;
		}

		// Get the vertical position and height of this item
		const TViewModelPtr<IGeometryExtension> GeometryExtension = ItemModel->CastThisShared<IGeometryExtension>();
		if (!GeometryExtension.IsValid())
		{
			continue;
		}

		if (const TSharedPtr<FViewModel> NextSibling = ItemModel->GetNextSibling())
		{
			const TViewModelPtr<IGeometryExtension> NextSiblingGeometry = NextSibling->CastThisShared<IGeometryExtension>();
			if (!NextSiblingGeometry.IsValid())
			{
				continue;
			}
		}

		// Convert virtual geometry to physical screen space
		const FVirtualGeometry ItemGeometry = GeometryExtension->GetVirtualGeometry();
		const float ItemBottom = TreeViewWidget->VirtualToPhysical(ItemGeometry.GetTop() + ItemGeometry.GetNestedHeight());

		// Draw a horizontal line at the bottom of this item
		TArray<FVector2D> LinePoints;
		LinePoints.AddUninitialized(2);
		LinePoints[0] = FVector2D(0.f, ItemBottom);
		LinePoints[1] = FVector2D(InArgs.AllottedGeometry.Size.X, ItemBottom);

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			InArgs.StartLayer,
			InArgs.AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			InArgs.DrawEffects,
			InArgs.TickColor,
			false
		);
	}
}

int32 FSequencerTimeSliderController::DrawScalingAnchors(const FGeometry& AllottedGeometry, const FScrubRangeToScreen& RangeToScreen, FSlateWindowElementList& OutDrawElements, int32 LayerId, const ESlateDrawEffect& DrawEffects) const
{
	UMovieSceneScalingAnchors* Anchors = WeakSequencer.Pin()->GetFocusedMovieSceneSequence()->GetMovieScene()->FindDecoration<UMovieSceneScalingAnchors>();
	if (!Anchors || !WeakSequencer.Pin()->GetSequencerSettings()->GetShowScalingAnchors())
	{
		return LayerId;
	}

	TArray<FVector2f> LinePoints;
	for (const TPair<FGuid, FMovieSceneScalingAnchor>& Pair : Anchors->GetCurrentAnchors())
	{
		const double StartSeconds = Pair.Value.Position / GetTickResolution();
		const float  StartLinePos = RangeToScreen.InputToLocalX(StartSeconds);

		LinePoints.Reset(2);
		LinePoints.Emplace(FVector2f(StartLinePos, 0.0f));
		LinePoints.Emplace(FVector2f(StartLinePos, FMath::FloorToFloat(AllottedGeometry.Size.Y)));

		FSlateDrawElement::MakeDashedLines(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			MoveTemp(LinePoints),
			DrawEffects,
			FLinearColor::Yellow.CopyWithNewOpacity(0.25f)
		);

		if (Pair.Value.Duration > 0)
		{
			const double EndSeconds = (Pair.Value.Position + Pair.Value.Duration) / GetTickResolution();
			const float  EndLinePos = RangeToScreen.InputToLocalX(EndSeconds);

			LinePoints.Reset(2);
			LinePoints.Emplace(FVector2f(EndLinePos, 0.0f));
			LinePoints.Emplace(FVector2f(EndLinePos, FMath::FloorToFloat(AllottedGeometry.Size.Y)));

			FSlateDrawElement::MakeDashedLines(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				MoveTemp(LinePoints),
				DrawEffects,
				FLinearColor::Yellow.CopyWithNewOpacity(0.25f)
			);
		}
	}

	return LayerId + 1;
}

int32 FSequencerTimeSliderController::DrawMarkedFrames(const FGeometry& InGeometry,
	const FScrubRangeToScreen& InRangeToScreen,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	const ESlateDrawEffect& InDrawEffects,
	bool bInDrawLabels) const
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return LayerId;
	}

	const USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
    if (!SequencerSettings)
    {
    	return LayerId;
    }

	if (!SequencerSettings->GetShowMarkedFrames())
	{
		return LayerId;
	}

	using namespace UE::Sequencer;

	const TArray<FMovieSceneMarkedFrame>& MarkedFrames = TimeSliderArgs.MarkedFrames.Get();
	const TArray<FMovieSceneMarkedFrame>& GlobalMarkedFrames = TimeSliderArgs.GlobalMarkedFrames.Get();
	if (MarkedFrames.Num() < 1 && GlobalMarkedFrames.Num() < 1)
	{
		return LayerId;
	}

	const FMarkedFrameSelection& SelectedMarkedFrames = Sequencer->GetViewModel()->GetSelection()->MarkedFrames;
	const FLinearColor DefaultMarkedFrameColor = SequencerSettings->GetMarkedFrameColor();

	auto DrawFrameMarkers = ([=, this](const TArray<FMovieSceneMarkedFrame>& InMarkedFrames,
		FSlateWindowElementList& DrawElements, bool bIsGlobal)
	{
		for (int32 MarkIndex = 0; MarkIndex < InMarkedFrames.Num(); ++MarkIndex)
		{
			const FMovieSceneMarkedFrame& MarkedFrame(InMarkedFrames[MarkIndex]);
			double Seconds = MarkedFrame.FrameNumber / GetTickResolution();

			const bool bIsHovered = (!bIsGlobal && HoverMarkIndex == MarkIndex);
			const bool bIsSelected = (!bIsGlobal && SelectedMarkedFrames.IsSelected(MarkIndex));

			// Get a selected color that's the marked frame color but at full opacity, full brightness, and a bit desaturated if it's
			// already bright to begin with.
			const FLinearColor MarkedFrameColor = MarkedFrame.bUseCustomColor ? MarkedFrame.CustomColor : DefaultMarkedFrameColor;
			const FLinearColor MarkedFrameColorHSV = MarkedFrameColor.CopyWithNewOpacity(1.f).LinearRGBToHSV();
			const FLinearColor SelectedColor = FLinearColor::LerpUsingHSV(
				MarkedFrameColorHSV, 
				FLinearColor(
					MarkedFrameColorHSV.R, 
					MarkedFrameColorHSV.G * (1.f - MarkedFrameColorHSV.B),
					1.f),
				0.5f)
				.HSVToLinearRGB();

			const FLinearColor DrawColor = bIsGlobal
				? MarkedFrameColor.Desaturate(0.25f)
				: (bIsSelected ? SelectedColor : MarkedFrameColor);
			const float LinePos = InRangeToScreen.InputToLocalX(Seconds);

			TArray<FVector2D> LinePoints;
			LinePoints.AddUninitialized(2);
			LinePoints[0] = FVector2D(LinePos, 0.0f);
			LinePoints[1] = FVector2D(LinePos, FMath::FloorToFloat(InGeometry.Size.Y));

			FSlateDrawElement::MakeLines(
				DrawElements,
				LayerId + 1,
				InGeometry.ToPaintGeometry(),
				LinePoints,
				InDrawEffects,
				DrawColor,
				false,
				(bIsHovered || bIsSelected) ? 2.f : 1.f
			);

			FString LabelString = MarkedFrame.Label;
			if (bInDrawLabels && !LabelString.IsEmpty())
			{
				// Draw the label next to the marked frame line
				bool bDrawLeft;
				FVector2D TextPosition, TextSize;
				GetMarkLabelGeometry(InGeometry, InRangeToScreen, MarkedFrame, TextPosition, TextSize, bDrawLeft);

				const FSlateBrush* LabelBrush = bDrawLeft
					? FAppStyle::GetBrush(TEXT("Sequencer.MarkedFrame.LabelLeft"))
					: FAppStyle::GetBrush(TEXT("Sequencer.MarkedFrame.LabelRight"));

				if (bIsHovered || bIsSelected)
				{
					FSlateDrawElement::MakeBox(
						DrawElements,
						LayerId + 1,
						InGeometry.ToPaintGeometry(
							TextSize + ScrubConstants::MarkLabelBoxMargin + ScrubConstants::MarkLabelBoxWideMargin,
							FSlateLayoutTransform(FVector2D(LinePos, 0.f))),
						LabelBrush,
						InDrawEffects,
						DrawColor.CopyWithNewOpacity(0.3f)
					);
				}

				FSlateDrawElement::MakeText(
					DrawElements,
					LayerId + 1,
					InGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextPosition)),
					LabelString,
					(bIsHovered || bIsSelected) ? SmallBoldLayoutFont : SmallLayoutFont,
					InDrawEffects,
					DrawColor
				);
			}
		}
	});
	
	DrawFrameMarkers(GlobalMarkedFrames, OutDrawElements, true);
	DrawFrameMarkers(MarkedFrames, OutDrawElements, false);
	
	return LayerId + 1;
}

int32 FSequencerTimeSliderController::DrawVerticalFrames(const FGeometry& AllottedGeometry, const FScrubRangeToScreen& RangeToScreen, FSlateWindowElementList& OutDrawElements, int32 LayerId, const ESlateDrawEffect& DrawEffects) const
{
	TSet<FFrameNumber> VerticalFrames = TimeSliderArgs.VerticalFrames.Get();
	if (VerticalFrames.Num() < 1)
	{
		return LayerId;
	}

	for (FFrameNumber TickFrame : VerticalFrames)
	{
		double Seconds = TickFrame / GetTickResolution();

		const float  LinePos = RangeToScreen.InputToLocalX(Seconds);
		TArray<FVector2D> LinePoints;
		LinePoints.AddUninitialized(2);
		LinePoints[0] = FVector2D(LinePos, 0.0f);
		LinePoints[1] = FVector2D(LinePos, FMath::FloorToFloat(AllottedGeometry.Size.Y));

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			DrawEffects,
			FLinearColor(0.7f, 0.7f, 0.f, 0.4f),
			false
		);
	}

	return LayerId + 1;
}

int32 FSequencerTimeSliderController::OnPaintTimeSlider( bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return LayerId;
	}

	const bool bEnabled = bParentEnabled;
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	TRange<double> LocalViewRange      = GetViewRange();
	const float    LocalViewRangeMin   = LocalViewRange.GetLowerBoundValue();
	const float    LocalViewRangeMax   = LocalViewRange.GetUpperBoundValue();
	const float    LocalSequenceLength = LocalViewRangeMax-LocalViewRangeMin;
	
	FVector2D Scale = FVector2D(1.0f,1.0f);
	if ( LocalSequenceLength > 0)
	{
		FScrubRangeToScreen RangeToScreen( LocalViewRange, AllottedGeometry.Size );

		// draw tick marks
		float MajorTickHeight = 9.0f;

		TViewModelPtr<IClockExtension> Clock = Sequencer->GetViewModel()->GetRootSequenceModel().ImplicitCast();
		if (Clock)
		{
			MajorTickHeight = Clock->GetCustomMajorTickSize(Sequencer);
		}

		FDrawTickArgs Args;
		{
			Args.AllottedGeometry = AllottedGeometry;
			Args.bMirrorLabels = bMirrorLabels;
			Args.bOnlyDrawMajorTicks = false;
			Args.TickColor = FLinearColor::White;
			Args.CullingRect = MyCullingRect;
			Args.DrawEffects = DrawEffects;
			Args.StartLayer = LayerId;
			Args.TickOffset = bMirrorLabels ? 0.0f : FMath::Abs( AllottedGeometry.Size.Y - MajorTickHeight );
			Args.MajorTickHeight = MajorTickHeight;
		}

		DrawTicks( OutDrawElements, LocalViewRange, RangeToScreen, Args );

		// draw playback & selection range
		FPaintPlaybackRangeArgs PlaybackRangeArgs(
			bMirrorLabels ? FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Bottom_L") : FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Top_L"),
			bMirrorLabels ? FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Bottom_R") : FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Top_R"),
			6.f
		);

		LayerId = DrawPlaybackRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PlaybackRangeArgs);
		LayerId = DrawSubSequenceRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PlaybackRangeArgs);

		PlaybackRangeArgs.SolidFillOpacity = 0.05f;
		LayerId = DrawSelectionRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PlaybackRangeArgs);

		// Draw the scrub handle
		FQualifiedFrameTime ScrubPosition = FQualifiedFrameTime(TimeSliderArgs.ScrubPosition.Get(), GetTickResolution());
		FScrubberMetrics    ScrubMetrics  = GetScrubPixelMetrics(ScrubPosition, RangeToScreen);
		const float         HandleStart   = ScrubMetrics.HandleRangePx.GetLowerBoundValue();
		const float         HandleEnd     = ScrubMetrics.HandleRangePx.GetUpperBoundValue();

		const int32 ArrowLayer = LayerId + 2;
		FPaintGeometry MyGeometry =	AllottedGeometry.ToPaintGeometry(
			FVector2f( HandleEnd - HandleStart, AllottedGeometry.Size.Y-1.f ),
			FSlateLayoutTransform(FVector2f( HandleStart, 1.f )) );

		FTimeWarpTrackExtension*   TimeWarpExtension   = Sequencer->GetViewModel()->GetRootSequenceModel()->GetSharedData()->CastDynamic<FTimeWarpTrackExtension>();
		const FTimeWarpTrackModel* ActiveTimeWarpTrack = TimeWarpExtension ? TimeWarpExtension->GetActiveTimeWarpTrack() : nullptr;

		const bool bIsWarped = (Sequencer->GetSequencerSettings()->GetTimeWarpDisplayMode() == ESequencerTimeWarpDisplay::WarpedTime && ActiveTimeWarpTrack != nullptr);
		FLinearColor ScrubColor = bIsWarped
			? FStyleColors::AccentOrange.GetSpecifiedColor()
			: FStyleColors::AccentRed.GetSpecifiedColor();

		if (!bIsEvaluating)
		{
			ScrubColor = FStyleColors::AccentYellow.GetSpecifiedColor().CopyWithNewOpacity(.7f);
		}

		const FSlateBrush* Brush = ScrubMetrics.Style == ESequencerScrubberStyle::Vanilla
			? VanillaScrubHandleDownBrush
			: FrameBlockScrubHandleDownBrush;

		if (bMirrorLabels)
		{
			FSlateRenderTransform FlipTransform(FScale2f(1.f, -1.f), FVector2f(0.f, AllottedGeometry.Size.Y));
			MyGeometry.SetRenderTransform(
				Concatenate(
					FlipTransform,
					MyGeometry.GetAccumulatedRenderTransform()
				)
			);
		}
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			ArrowLayer,
			MyGeometry,
			Brush,
			DrawEffects,
			ScrubColor
		);

		LayerId = DrawMarkedFrames(AllottedGeometry, RangeToScreen, OutDrawElements, LayerId, DrawEffects, true);

		{
			// Draw the scrub position string and the local frame string as ScrubPosition : LocalFrame where the ScrubPosition might be time in a parent sequence space
			TArray<FMovieSceneSequenceID> ParentChain = TimeSliderArgs.ScrubPositionParentChain.Get();
			const bool bValidScrubPosition = TimeSliderArgs.ScrubPositionParent.Get() != MovieSceneSequenceID::Invalid && ParentChain.Contains(TimeSliderArgs.ScrubPositionParent.Get()) && TimeSliderArgs.ScrubPositionParent.Get() != Sequencer->GetFocusedTemplateID();
			const FString ScrubPositionString = bValidScrubPosition ? FString::Format(TEXT("{0} : "), { TimeSliderArgs.ScrubPositionText.Get() }) : FString();
			const FString LocalFrameString = Sequencer->GetNumericTypeInterface()->ToString(ScrubPosition.Time.GetFrame().Value);
			const FString FullString = !ScrubPositionString.IsEmpty() ? FString::Join(TArray<FString>() = { ScrubPositionString, LocalFrameString }, TEXT(" ")) : LocalFrameString;

			const FVector2D ScrubPositionTextSize = FontMeasureService->Measure(ScrubPositionString, SmallLayoutFont);
			const FVector2D LocalFrameTextSize = FontMeasureService->Measure(LocalFrameString, SmallLayoutFont);
			const FVector2D FullTextSize = FontMeasureService->Measure(FullString, SmallLayoutFont);

			// Flip the text position if getting near the end of the view range
			static const float TextOffsetPx = 2.f;
			const bool bDrawLeft = (AllottedGeometry.Size.X - HandleEnd) < (FullTextSize.X + 14.f) - TextOffsetPx;

			float TextPosition = bDrawLeft ? HandleStart - FullTextSize.X - TextOffsetPx : HandleEnd + TextOffsetPx;
			FVector2D TextOffset = FVector2D(TextPosition, Args.bMirrorLabels ? Args.AllottedGeometry.Size.Y - FullTextSize.Y : 0.f);
			if (!ScrubPositionString.IsEmpty())
			{
				FLinearColor TextColor = FLinearColor::Yellow; // customizable

				FSlateDrawElement::MakeText(
					OutDrawElements,
					Args.StartLayer + 1,
					Args.AllottedGeometry.ToPaintGeometry(FullTextSize, FSlateLayoutTransform(TextOffset)),
					ScrubPositionString,
					SmallLayoutFont,
					Args.DrawEffects,
					TextColor
				);

				TextPosition = TextPosition + ScrubPositionTextSize.X;
				TextOffset = FVector2D(TextPosition, Args.bMirrorLabels ? Args.AllottedGeometry.Size.Y - FullTextSize.Y : 0.f);
			}

			FSlateDrawElement::MakeText(
				OutDrawElements,
				Args.StartLayer+1, 
				Args.AllottedGeometry.ToPaintGeometry(LocalFrameTextSize, FSlateLayoutTransform(TextOffset) ),
				LocalFrameString, 
				SmallLayoutFont,
				Args.DrawEffects,
				Args.TickColor
			);
		}
		
		if (MouseDragType == DRAG_SETTING_RANGE && MouseDownPosition.IsSet() && LastMousePosition.IsSet())
		{
			FFrameRate Resolution = GetTickResolution();
			FFrameTime MouseDownTime[2];

			FScrubRangeToScreen MouseDownRange(GetViewRange(), MouseDownGeometry.Size);
			MouseDownTime[0] = ComputeFrameTimeFromMouse(MouseDownGeometry, MouseDownPosition.GetValue(), MouseDownRange);
			MouseDownTime[1] = ComputeFrameTimeFromMouse(MouseDownGeometry, LastMousePosition.GetValue(), MouseDownRange);

			float      MouseStartPosX = RangeToScreen.InputToLocalX(MouseDownTime[0] / Resolution);
			float      MouseEndPosX   = RangeToScreen.InputToLocalX(MouseDownTime[1] / Resolution);

			float RangePosX = MouseStartPosX < MouseEndPosX ? MouseStartPosX : MouseEndPosX;
			float RangeSizeX = FMath::Abs(MouseStartPosX - MouseEndPosX);

			FPaintGeometry PaintGeom = AllottedGeometry.ToPaintGeometry( FVector2f(RangeSizeX, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(RangePosX, 0.f)) );

			if (bMirrorLabels)
			{
				FSlateRenderTransform FlipTransform(FScale2f(1.f, -1.f), FVector2f(0.f, AllottedGeometry.Size.Y));
				PaintGeom.SetRenderTransform(
					Concatenate(
						FlipTransform,
						PaintGeom.GetAccumulatedRenderTransform()
					)
				);
			}

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId+1,
				PaintGeom,
				VanillaScrubHandleDownBrush,
				DrawEffects,
				MouseStartPosX < MouseEndPosX ? FLinearColor(0.5f, 0.5f, 0.5f) : FLinearColor(0.25f, 0.3f, 0.3f)
			);
		}

		return ArrowLayer;
	}

	return LayerId;
}

int32 FSequencerTimeSliderController::DrawSelectionRange(const FGeometry& InGeometry,
	const FSlateRect& InCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	const FScrubRangeToScreen& InRangeToScreen,
	const FPaintPlaybackRangeArgs& InRangeArgs) const
{
	const TRange<double> SelectionRange = TimeSliderArgs.SelectionRange.Get() / GetTickResolution();
	if (SelectionRange.IsEmpty())
	{
		return LayerId;
	}

	const float SelectionRangeL = InRangeToScreen.InputToLocalX(SelectionRange.GetLowerBoundValue());
	const float SelectionRangeR = InRangeToScreen.InputToLocalX(SelectionRange.GetUpperBoundValue()) - 1.f;

	const FLinearColor DrawColor = FAppStyle::GetSlateColor(TEXT("SelectionColor")).GetColor(FWidgetStyle());
	const float SelectionWidth = FMath::Max(0.f, SelectionRangeR - SelectionRangeL);

	if (InRangeArgs.SolidFillOpacity > 0.f && SelectionWidth > 0.f)
	{
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			InGeometry.ToPaintGeometry(
				FVector2f(SelectionWidth, InGeometry.Size.Y),
				FSlateLayoutTransform(FVector2f(SelectionRangeL, 0.f))
			),
			FAppStyle::GetBrush(TEXT("WhiteBrush")),
			ESlateDrawEffect::None,
			DrawColor.CopyWithNewOpacity(InRangeArgs.SolidFillOpacity)
		);
	}

	const float BrushWidth = FMath::Max(1.f, InRangeArgs.BrushWidth);
	const float EndBrushX = FMath::Max(SelectionRangeL, SelectionRangeR - BrushWidth);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId + 1,
		InGeometry.ToPaintGeometry(
			FVector2f(BrushWidth, InGeometry.Size.Y),
			FSlateLayoutTransform(FVector2f(SelectionRangeL, 0.f))
		),
		InRangeArgs.StartBrush,
		ESlateDrawEffect::None,
		DrawColor
	);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId + 1,
		InGeometry.ToPaintGeometry(
			FVector2f(BrushWidth, InGeometry.Size.Y),
			FSlateLayoutTransform(FVector2f(EndBrushX, 0.f))
		),
		InRangeArgs.EndBrush,
		ESlateDrawEffect::None,
		DrawColor
	);

	return LayerId + 1;
}

int32 FSequencerTimeSliderController::DrawPlaybackRange(const FGeometry& InGeometry,
	const FSlateRect& InCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	const FScrubRangeToScreen& InRangeToScreen,
	const FPaintPlaybackRangeArgs& InRangeArgs) const
{
	const int32 NewLayerId = DrawPlaybackRangeMarkers(InGeometry, InCullingRect, OutDrawElements, LayerId, InRangeToScreen, InRangeArgs);
	DrawPlaybackRangeExcludedRegions(InGeometry, InCullingRect, OutDrawElements, LayerId, InRangeToScreen, InRangeArgs);
	return NewLayerId;
}

int32 FSequencerTimeSliderController::DrawPlaybackRangeMarkers(const FGeometry& InGeometry,
	const FSlateRect& InCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	const FScrubRangeToScreen& InRangeToScreen,
	const FPaintPlaybackRangeArgs& InRangeArgs) const
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return LayerId;
	}

	const USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!SequencerSettings)
	{
		return LayerId;
	}

	if (!TimeSliderArgs.PlaybackRange.IsSet())
	{
		return LayerId;
	}

	const TRange<FFrameNumber> PlaybackRange = TimeSliderArgs.PlaybackRange.Get();
	const FFrameRate TickResolution = GetTickResolution();

	float PlaybackRangeL = InRangeToScreen.InputToLocalX(PlaybackRange.GetLowerBoundValue() / TickResolution);
	float PlaybackRangeR = InRangeToScreen.InputToLocalX(PlaybackRange.GetUpperBoundValue() / TickResolution) - 1.f;

	const float BrushWidth = FMath::Max(1.f, InRangeArgs.BrushWidth);
	const float EndBrushX = FMath::Max(PlaybackRangeL, PlaybackRangeR - BrushWidth);

	const FLinearColor PlaybackRangeStartColor = SequencerSettings->GetPlaybackRangeStartColor();
	const FLinearColor PlaybackRangeEndColor = SequencerSettings->GetPlaybackRangeEndColor();
	const uint8 OpacityBlend = TimeSliderArgs.SubSequenceRange.Get().IsSet() ? 128 : 255;

	// Only draw start marker if it's within the visible area
	if (PlaybackRangeL >= 0.f && PlaybackRangeL <= InGeometry.Size.X)
	{
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			InGeometry.ToPaintGeometry(
				FVector2f(BrushWidth, InGeometry.Size.Y),
				FSlateLayoutTransform(FVector2f(PlaybackRangeL, 0.f))
			),
			InRangeArgs.StartBrush,
			ESlateDrawEffect::None,
			PlaybackRangeStartColor.CopyWithNewOpacity(OpacityBlend)
		);
	}

	// Only draw end marker if it's within the visible area
	if (EndBrushX >= 0.f && EndBrushX <= InGeometry.Size.X)
	{
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			InGeometry.ToPaintGeometry(
				FVector2f(BrushWidth, InGeometry.Size.Y),
				FSlateLayoutTransform(FVector2f(EndBrushX, 0.f))
			),
			InRangeArgs.EndBrush,
			ESlateDrawEffect::None,
			PlaybackRangeEndColor.CopyWithNewOpacity(OpacityBlend)
		);
	}

	return LayerId + 1;
}

int32 FSequencerTimeSliderController::DrawPlaybackRangeExcludedRegions(const FGeometry& InGeometry,
	const FSlateRect& InCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	const FScrubRangeToScreen& InRangeToScreen,
	const FPaintPlaybackRangeArgs& InRangeArgs) const
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return LayerId;
	}

	const USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!SequencerSettings)
	{
		return LayerId;
	}

	if (!TimeSliderArgs.PlaybackRange.IsSet())
	{
		return LayerId;
	}

	const TRange<FFrameNumber> PlaybackRange = TimeSliderArgs.PlaybackRange.Get();
	const FFrameRate TickResolution = GetTickResolution();

	const float PlaybackRangeL = InRangeToScreen.InputToLocalX(PlaybackRange.GetLowerBoundValue() / TickResolution);
	const float PlaybackRangeR = InRangeToScreen.InputToLocalX(PlaybackRange.GetUpperBoundValue() / TickResolution) - 1.f;
	const uint8 OpacityBlend = TimeSliderArgs.SubSequenceRange.Get().IsSet() ? 128 : 255;

	// Black tint for excluded regions
	const float LeftExcludedRange = FMath::Max(0.f, PlaybackRangeL);
	const float RightExcludedRange = FMath::Max(0.f, InGeometry.Size.X - PlaybackRangeR);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId + 1,
		InGeometry.ToPaintGeometry(
			FVector2f(LeftExcludedRange, InGeometry.Size.Y),
			FSlateLayoutTransform()
		),
		FAppStyle::GetBrush(TEXT("WhiteBrush")),
		ESlateDrawEffect::None,
		FLinearColor::Black.CopyWithNewOpacity(.3f * OpacityBlend / 255.f)
	);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId + 1,
		InGeometry.ToPaintGeometry(
			FVector2f(RightExcludedRange, InGeometry.Size.Y),
			FSlateLayoutTransform(FVector2f(PlaybackRangeR, 0.f))
		),
		FAppStyle::GetBrush(TEXT("WhiteBrush")),
		ESlateDrawEffect::None,
		FLinearColor::Black.CopyWithNewOpacity(.3f * OpacityBlend / 255.f)
	);

	return LayerId + 1;
}

int32 FSequencerTimeSliderController::DrawSubSequenceRange(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const
{
	const int32 NewLayerId = DrawSubSequenceRangeMarkers(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, Args);
	DrawSubSequenceRangeExcludedRegions(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, Args);
	DrawSubSequenceRangeHashMarks(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, Args);
	return NewLayerId;
}

int32 FSequencerTimeSliderController::DrawSubSequenceRangeMarkers(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return LayerId;
	}

	TOptional<TRange<FFrameNumber>> RangeValue;
	RangeValue = TimeSliderArgs.SubSequenceRange.Get(RangeValue);

	if (!RangeValue.IsSet() || RangeValue->IsEmpty())
	{
		return LayerId;
	}

	const FFrameRate   Resolution = GetTickResolution();
	const FFrameNumber LowerFrame = RangeValue.GetValue().GetLowerBoundValue();
	const FFrameNumber UpperFrame = RangeValue.GetValue().GetUpperBoundValue();

	const float SubSequenceRangeL = RangeToScreen.InputToLocalX(LowerFrame / Resolution) - 1;
	const float SubSequenceRangeR = RangeToScreen.InputToLocalX(UpperFrame / Resolution) + 1;

	static const FSlateBrush* LineBrushL(FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_L"));
	static const FSlateBrush* LineBrushR(FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_R"));

	const FLinearColor PlaybackRangeStartColor = Sequencer->GetSequencerSettings()->GetPlaybackRangeStartColor();
	const FLinearColor PlaybackRangeEndColor = Sequencer->GetSequencerSettings()->GetPlaybackRangeEndColor();

	FColor GreenTint(32, 128, 32);	// 120, 75, 50 (HSV)
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId+1,
		AllottedGeometry.ToPaintGeometry(FVector2f(Args.BrushWidth, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(SubSequenceRangeL, 0.f))),
		LineBrushL,
		ESlateDrawEffect::None,
		PlaybackRangeStartColor
	);

	FColor RedTint(192, 48, 48);	// 0, 75, 75 (HSV)
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId+1,
		AllottedGeometry.ToPaintGeometry(FVector2f(Args.BrushWidth, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(SubSequenceRangeR - Args.BrushWidth, 0.f))),
		LineBrushR,
		ESlateDrawEffect::None,
		PlaybackRangeEndColor
	);

	return LayerId + 1;
}

int32 FSequencerTimeSliderController::DrawSubSequenceRangeExcludedRegions(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return LayerId;
	}

	TOptional<TRange<FFrameNumber>> RangeValue;
	RangeValue = TimeSliderArgs.SubSequenceRange.Get(RangeValue);

	if (!RangeValue.IsSet() || RangeValue->IsEmpty())
	{
		return LayerId;
	}

	const FFrameRate   Resolution = GetTickResolution();
	const FFrameNumber LowerFrame = RangeValue.GetValue().GetLowerBoundValue();
	const FFrameNumber UpperFrame = RangeValue.GetValue().GetUpperBoundValue();

	const float SubSequenceRangeL = RangeToScreen.InputToLocalX(LowerFrame / Resolution) - 1;
	const float SubSequenceRangeR = RangeToScreen.InputToLocalX(UpperFrame / Resolution) + 1;

	// Black tint for excluded regions
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId+1,
		AllottedGeometry.ToPaintGeometry(FVector2f(SubSequenceRangeL, AllottedGeometry.Size.Y), FSlateLayoutTransform()),
		FAppStyle::GetBrush("WhiteBrush"),
		ESlateDrawEffect::None,
		FLinearColor::Black.CopyWithNewOpacity(0.3f)
	);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId+1,
		AllottedGeometry.ToPaintGeometry(FVector2f(AllottedGeometry.Size.X - SubSequenceRangeR, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(SubSequenceRangeR, 0.f))),
		FAppStyle::GetBrush("WhiteBrush"),
		ESlateDrawEffect::None,
		FLinearColor::Black.CopyWithNewOpacity(0.3f)
	);

	return LayerId + 1;
}

int32 FSequencerTimeSliderController::DrawSubSequenceRangeHashMarks(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return LayerId;
	}

	TOptional<TRange<FFrameNumber>> RangeValue;
	RangeValue = TimeSliderArgs.SubSequenceRange.Get(RangeValue);

	if (!RangeValue.IsSet() || RangeValue->IsEmpty())
	{
		return LayerId;
	}

	const FFrameRate   Resolution = GetTickResolution();
	const FFrameNumber LowerFrame = RangeValue.GetValue().GetLowerBoundValue();
	const FFrameNumber UpperFrame = RangeValue.GetValue().GetUpperBoundValue();

	const float SubSequenceRangeL = RangeToScreen.InputToLocalX(LowerFrame / Resolution) - 1;
	const float SubSequenceRangeR = RangeToScreen.InputToLocalX(UpperFrame / Resolution) + 1;

	const FLinearColor PlaybackRangeStartColor = Sequencer->GetSequencerSettings()->GetPlaybackRangeStartColor();
	const FLinearColor PlaybackRangeEndColor = Sequencer->GetSequencerSettings()->GetPlaybackRangeEndColor();

	// Hash applied to the left and right of the sequence bounds
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId+1,
		AllottedGeometry.ToPaintGeometry(FVector2f(16.f, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(SubSequenceRangeL - 16.f, 0.f))),
		FAppStyle::GetBrush("Sequencer.Timeline.SubSequenceRangeHashL"),
		ESlateDrawEffect::None,
		PlaybackRangeStartColor
	);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId+1,
		AllottedGeometry.ToPaintGeometry(FVector2f(16.f, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(SubSequenceRangeR, 0.f))),
		FAppStyle::GetBrush("Sequencer.Timeline.SubSequenceRangeHashR"),
		ESlateDrawEffect::None,
		PlaybackRangeEndColor
	);

	return LayerId + 1;
}

void FSequencerTimeSliderController::EndAnyInProgressTransactionalDrag()
{
	// Only the drag types whose Begin delegate opens a GEditor transaction
	// (OnPlaybackRangeBeginDrag / OnSelectionRangeBeginDrag / OnMarkBeginDrag).
	// DRAG_SCRUBBING_TIME is deliberately excluded: scrubbing isn't transactional
	// and its end path in OnMouseButtonUp_MouseDragType calls CommitScrubPosition
	// with mouse coordinates we don't have here.
	switch (MouseDragType)
	{
	case DRAG_PLAYBACK_START:
	case DRAG_PLAYBACK_END:
		TimeSliderArgs.OnPlaybackRangeEndDrag.ExecuteIfBound();
		break;
	case DRAG_SELECTION_START:
	case DRAG_SELECTION_END:
		TimeSliderArgs.OnSelectionRangeEndDrag.ExecuteIfBound();
		break;
	case DRAG_MARK:
		TimeSliderArgs.OnMarkEndDrag.ExecuteIfBound();
		break;
	default:
		return;
	}

	// Reset so the scrub-on-click branch in OnMouseButtonUp_MouseDragType doesn't
	// fall through and treat the interrupted drag as a click.
	MouseDragType = DRAG_NONE;
	bMouseDownInRegion = false;
}

void FSequencerTimeSliderController::ResetMouseDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	// If a transactional drag is already in progress (e.g. user clicked RMB while
	// LMB-dragging a playback range marker), fire its End delegate now so the
	// transaction the BeginDrag opened doesn't get orphaned by the clobber below.
	EndAnyInProgressTransactionalDrag();

	const FVector2D PointerPosition = InPointerEvent.GetScreenSpacePosition();

	MouseDragType = DRAG_NONE;
	DistanceDragged = 0;
	MouseDownPlaybackRange = TimeSliderArgs.PlaybackRange.Get();
	MouseDownSelectionRange = TimeSliderArgs.SelectionRange.Get();
	MouseDownPosition = PointerPosition;
	LastMousePosition = PointerPosition;
	MouseDownGeometry = InGeometry;

	const FVector2D LocalPointerPosition = MouseDownGeometry.AbsoluteToLocal(PointerPosition);
	bMouseDownInRegion = LocalPointerPosition.Y >= 0
		&& LocalPointerPosition.Y < MouseDownGeometry.GetLocalSize().Y;
}

FReply FSequencerTimeSliderController::OnMouseButtonDown(SWidget& WidgetOwner, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	ResetMouseDown(InGeometry, InPointerEvent);

	return FReply::Unhandled();
}

FReply FSequencerTimeSliderController::OnMouseButtonUp_ContextMenu(SWidget& WidgetOwner, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	const FKey EffectingButton = InPointerEvent.GetEffectingButton();
	if (EffectingButton != EKeys::RightMouseButton)
	{
		return FReply::Unhandled();
	}

	if (!bPanning)
	{
		// Open a context menu if allowed
		if (ContextMenuSuppression == 0 && DistanceDragged == 0.f)
		{
			if (const TSharedPtr<SWidget> MenuContent = OpenContextMenu(InGeometry, InPointerEvent))
			{
				const TSharedRef<SWidget> MenuContentRef = MenuContent.ToSharedRef();
				const FVector2D PointerPosition = InPointerEvent.GetScreenSpacePosition();

				FSlateApplication::Get().PushMenu(
					WidgetOwner.AsShared(),
					InPointerEvent.GetEventPath() ? *InPointerEvent.GetEventPath() : FWidgetPath(),
					MenuContentRef,
					PointerPosition,
					FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
				);

				return FReply::Handled()
					.SetUserFocus(MenuContentRef, EFocusCause::SetDirectly)
					.ReleaseMouseCapture();
			}
		}

		// Return unhandled in case our parent wants to use our right mouse button to open a context menu
		return FReply::Unhandled().ReleaseMouseCapture();
	}

	ResetMouseInput();

	return FReply::Handled().ReleaseMouseCapture();
}

FReply FSequencerTimeSliderController::OnMouseButtonUp_MouseDragType(SWidget& WidgetOwner
	, const FGeometry& InGeometry
	, const FPointerEvent& InPointerEvent
	, const FScrubRangeToScreen& InRangeToScreen
	, const FFrameTime& InMouseTime)
{
	// Transactional drag End delegates (Playback / Selection / Mark) are fired by
	// EndAnyInProgressTransactionalDrag at the top of OnMouseButtonUp so they
	// reliably balance their BeginDrag regardless of how the mouse-up arrives.
	if (MouseDragType == DRAG_SETTING_RANGE && MouseDownPosition.IsSet())
	{
		// Zooming
		const FFrameTime MouseDownStart = ComputeFrameTimeFromMouse(InGeometry, MouseDownPosition.GetValue(), InRangeToScreen);

		const bool bCanZoomIn  = InMouseTime > MouseDownStart;
		const bool bCanZoomOut = ViewRangeStack.Num() > 0;
		if (bCanZoomIn || bCanZoomOut)
		{
			TRange<double> ViewRange = GetViewRange();
			if (!bCanZoomIn)
			{
				ViewRange = ViewRangeStack.Pop();
			}

			if (bCanZoomIn)
			{
				// push the current value onto the stack
				ViewRangeStack.Add(ViewRange);

				const FFrameRate TickResolution = GetTickResolution();
				ViewRange = TRange<double>(MouseDownStart.FrameNumber / TickResolution, InMouseTime.FrameNumber / TickResolution);
			}
			
			TimeSliderArgs.OnViewRangeChanged.ExecuteIfBound(ViewRange, EViewRangeInterpolation::Immediate);
			if (!TimeSliderArgs.ViewRange.IsBound())
			{
				// The output is not bound to a delegate so we'll manage the value ourselves
				TimeSliderArgs.ViewRange.Set(ViewRange);
			}
		}
	}
	else if (bMouseDownInRegion
		&& MouseDragType != DRAG_PLAYBACK_START
		&& MouseDragType != DRAG_PLAYBACK_END
		&& MouseDragType != DRAG_SELECTION_START
		&& MouseDragType != DRAG_SELECTION_END
		&& MouseDragType != DRAG_MARK
		&& MouseDragType != DRAG_SETTING_RANGE)
	{
		const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
		const bool bHandleMiddleMouseButton = InPointerEvent.GetEffectingButton() == EKeys::MiddleMouseButton && WidgetOwner.HasMouseCapture();

		if (HoverMarkIndex == INDEX_NONE)
		{
			FFrameTime ScrubTime = InMouseTime;

			if (MouseDragType == DRAG_SCRUBBING_TIME)
			{
				ScrubTime = ComputeScrubTimeFromMouse(InGeometry, InPointerEvent, InRangeToScreen);
			}
			else
			{
				const FVector2D PointerPosition  = InPointerEvent.GetScreenSpacePosition();
				ScrubTime = SnapTimeToNearestKey(InPointerEvent, InRangeToScreen, PointerPosition.X, ScrubTime);
			}

			if (Sequencer.IsValid())
			{
				Sequencer->GetViewModel()->GetSelection()->MarkedFrames.Empty();
			}

			// If middle mouse button down we don't evaluate on the time change
			CommitScrubPosition(ScrubTime, /*bIsScrubbing=*/false, /*bEvaluate*/!bHandleMiddleMouseButton);

			// Teleport the playhead to the clicked time.
			TimeSliderArgs.OnEndScrubberMovement.ExecuteIfBound();
		}
		else if (Sequencer.IsValid())
		{
			// Select the clicked marker, and optionally teleport the playhead.
			HandleDragMark_NoDragType(HoverMarkIndex);

			const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
			const bool bSnapPlayTimeToMark = ModifierKeys.AreModifersDown(EModifierKey::Shift);
			if (bSnapPlayTimeToMark || Sequencer->GetSequencerSettings()->GetSnapPlayTimeToPressedKey())
			{
				const TArray<FMovieSceneMarkedFrame>& MarkedFrames = TimeSliderArgs.MarkedFrames.Get();
				const FFrameNumber MarkTime = MarkedFrames[HoverMarkIndex].FrameNumber;

				// If middle mouse button down we don't evaluate on the time change
				CommitScrubPosition(MarkTime, /*bIsScrubbing=*/false, /*bEvaluate*/!bHandleMiddleMouseButton);
			}
		}
	}

	MouseDragType = DRAG_NONE;
	DistanceDragged = 0.f;
	ResetMouseInput();

	return FReply::Handled().ReleaseMouseCapture();
}

FReply FSequencerTimeSliderController::OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	const bool bHandleLeftMouseButton  = InPointerEvent.GetEffectingButton() == EKeys::LeftMouseButton  && OwnerWidget.HasMouseCapture();
	const bool bHandleRightMouseButton = InPointerEvent.GetEffectingButton() == EKeys::RightMouseButton && OwnerWidget.HasMouseCapture() && TimeSliderArgs.AllowZoom;
	const bool bHandleMiddleMouseButton = InPointerEvent.GetEffectingButton() == EKeys::MiddleMouseButton && OwnerWidget.HasMouseCapture();

	const FScrubRangeToScreen RangeToScreen = FScrubRangeToScreen(GetViewRange(), InGeometry.Size);
	const FFrameTime MouseTime = ComputeFrameTimeFromMouse(InGeometry, InPointerEvent.GetScreenSpacePosition(), RangeToScreen);

	// End any in-progress transactional drag up front. Catches two leak paths:
	//   - RMB up where OnMouseButtonUp_ContextMenu releases mouse capture before the
	//     LMB up reaches OnMouseButtonUp_MouseDragType
	//   - LMB up of a normal transactional drag, replacing the duplicate End-delegate
	//     cascade that used to live at the top of OnMouseButtonUp_MouseDragType
	// Idempotent: no-op when no transactional drag is in progress. See UE-381007.
	EndAnyInProgressTransactionalDrag();

	FReply OutReply = OnMouseButtonUp_ContextMenu(OwnerWidget, InGeometry, InPointerEvent);
	if (OutReply.IsEventHandled())
	{
		return OutReply;
	}

	if (bHandleLeftMouseButton || bHandleMiddleMouseButton)
	{
		OutReply = OnMouseButtonUp_MouseDragType(OwnerWidget, InGeometry, InPointerEvent, RangeToScreen, MouseTime);
		if (OutReply.IsEventHandled())
		{
			return OutReply;
		}
	}

	ResetMouseInput();

	return FReply::Unhandled();
}

FReply FSequencerTimeSliderController::OnMouseButtonDoubleClick(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	MouseDragType = DRAG_NONE;
	DistanceDragged = 0.f;

	if (HoverMarkIndex != INDEX_NONE)
	{
		// Teleport the playhead to the double-clicked marker.
		const bool bHandleMiddleMouseButton = InPointerEvent.GetEffectingButton() == EKeys::MiddleMouseButton && OwnerWidget.HasMouseCapture();
		const TArray<FMovieSceneMarkedFrame>& MarkedFrames = TimeSliderArgs.MarkedFrames.Get();
		const FFrameNumber MarkTime = MarkedFrames[HoverMarkIndex].FrameNumber;
		CommitScrubPosition(MarkTime, /*bIsScrubbing=*/false, /*bEvaluate*/!bHandleMiddleMouseButton);
	}

	return FReply::Unhandled();
}

double FSequencerTimeSliderController::ComputeHeight() const
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		TViewModelPtr<IClockExtension> Clock = Sequencer->GetViewModel()->GetRootSequenceModel().ImplicitCast();
		if (Clock)
		{
			return Clock->GetDesiredTimeSliderHeight(Sequencer);
		}
	}
	return 22.0;
}

FReply FSequencerTimeSliderController::OnMouseMove(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	return OnMouseMoveImpl(OwnerWidget, InGeometry, InPointerEvent, false);
}

FReply FSequencerTimeSliderController::OnTimeSliderMouseMove(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	return OnMouseMoveImpl(OwnerWidget, InGeometry, InPointerEvent, true);
}

FReply FSequencerTimeSliderController::OnMouseMoveImpl(SWidget& WidgetOwner
	, const FGeometry& InGeometry
	, const FPointerEvent& InPointerEvent
	, const bool bInFromTimeSlider)
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return FReply::Unhandled();
	}

	const bool bHandleLeftMouseButton = InPointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
	const bool bHandleRightMouseButton = InPointerEvent.IsMouseButtonDown(EKeys::RightMouseButton) && TimeSliderArgs.AllowZoom;
	const bool bHandleMiddleMouseButton = InPointerEvent.IsMouseButtonDown(EKeys::MiddleMouseButton);

	const bool bLockedMarkedFrames = TimeSliderArgs.AreMarkedFramesLocked.Get();

	HoverMarkIndex = INDEX_NONE;
	int32 DragMarkIndex = INDEX_NONE;

	if (bHandleRightMouseButton)
	{
		OnMouseMove_RightMouseDown(InGeometry, InPointerEvent, *Sequencer->GetSequencerSettings());
	}
	else if ((bHandleLeftMouseButton || bHandleMiddleMouseButton) && MouseDownPosition.IsSet())
	{
		const TRange<double> LocalViewRange = GetViewRange();
		const FScrubRangeToScreen RangeToScreen(LocalViewRange, InGeometry.Size);
		OnMouseMove_LeftAndMiddleMouseDown(InGeometry, InPointerEvent, bInFromTimeSlider, RangeToScreen, *Sequencer->GetSequencerSettings());
	}
	else if (bInFromTimeSlider && !bLockedMarkedFrames && DragMarkIndex == INDEX_NONE)
	{
		// Update hover state of marked frames.
		const TRange<double> LocalViewRange = GetViewRange();
		const FScrubRangeToScreen RangeToScreen(LocalViewRange, InGeometry.Size);
		const FVector2D LocalMousePosition = InGeometry.AbsoluteToLocal(InPointerEvent.GetScreenSpacePosition());
		HitTestMark(InGeometry, RangeToScreen, LocalMousePosition.X, bInFromTimeSlider, &HoverMarkIndex);
	}

	if (DistanceDragged != 0.f && (bHandleLeftMouseButton || bHandleRightMouseButton))
	{
		return FReply::Handled().CaptureMouse(WidgetOwner.AsShared());
	}

	return FReply::Handled();
}

FReply FSequencerTimeSliderController::OnMouseMove_LeftAndMiddleMouseDown(const FGeometry& InGeometry
	, const FPointerEvent& InPointerEvent
	, const bool bInFromTimeSlider
	, const FScrubRangeToScreen& InRangeToScreen
	, const USequencerSettings& InSequencerSettings)
{
	const bool bHandleLeftMouseButton = InPointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
	const bool bHandleRightMouseButton = InPointerEvent.IsMouseButtonDown(EKeys::RightMouseButton) && TimeSliderArgs.AllowZoom;
	const bool bHandleMiddleMouseButton = InPointerEvent.IsMouseButtonDown(EKeys::MiddleMouseButton);

	DistanceDragged += FMath::Abs(InPointerEvent.GetCursorDelta().X);

	// 1) Determine the drag mode once the drag threshold has been crossed
	if (MouseDragType == DRAG_NONE
		&& DistanceDragged > 0.f/*FSlateApplication::Get().GetDragTriggerDistance()*/
		&& (bHandleLeftMouseButton || bHandleMiddleMouseButton)
		&& UE::CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport))
	{
		MouseDragType = DRAG_SCRUBBING_TIME;
		TimeSliderArgs.OnBeginScrubberMovement.ExecuteIfBound();
	}

	if (MouseDragType == DRAG_NONE
		&& DistanceDragged > 0.f/*FSlateApplication::Get().GetDragTriggerDistance()*/)
	{
		const FFrameTime MouseDownFree = ComputeFrameTimeFromMouse(InGeometry
			, MouseDownPosition.GetValue(), InRangeToScreen, false);

		//const bool       bReadOnly          = Sequencer->IsReadOnly();
		const FFrameRate TickResolution     = GetTickResolution();
		const bool       bLockedPlayRange   = TimeSliderArgs.IsPlaybackRangeLocked.Get();
		const float      MouseDownPixel     = InRangeToScreen.InputToLocalX(MouseDownFree / TickResolution);
		const bool       bHitScrubber       = GetHitTestScrubPixelMetrics(InRangeToScreen).HandleRangePx.Contains(MouseDownPixel);

		const TRange<double> SelectionRange = TimeSliderArgs.SelectionRange.Get() / TickResolution;
		const TRange<double> PlaybackRange = TimeSliderArgs.PlaybackRange.Get() / TickResolution;

		int32 HitMarkIndex = INDEX_NONE;

		// Disable selection range test if it's empty so that the playback range scrubbing gets priority
		if (!bHandleMiddleMouseButton
			&& !bHitScrubber
			&& !SelectionRange.IsEmpty()
			&& HitTestRangeEnd(InRangeToScreen, SelectionRange, MouseDownPixel))
		{
			// selection range end scrubber
			MouseDragType = DRAG_SELECTION_END;
			TimeSliderArgs.OnSelectionRangeBeginDrag.ExecuteIfBound();
		}
		else if (!bHandleMiddleMouseButton
			&& !bHitScrubber
			&& !SelectionRange.IsEmpty()
			&& HitTestRangeStart(InRangeToScreen, SelectionRange, MouseDownPixel))
		{
			// selection range start scrubber
			MouseDragType = DRAG_SELECTION_START;
			TimeSliderArgs.OnSelectionRangeBeginDrag.ExecuteIfBound();
		}
		else if (!bHandleMiddleMouseButton
			&& !bHitScrubber
			&& !bLockedPlayRange
			&& HitTestRangeEnd(InRangeToScreen, PlaybackRange, MouseDownPixel))
		{
			// playback range end scrubber
			MouseDragType = DRAG_PLAYBACK_END;
			TimeSliderArgs.OnPlaybackRangeBeginDrag.ExecuteIfBound();
		}
		else if (!bHandleMiddleMouseButton
			&& !bHitScrubber
			&& !bLockedPlayRange
			&& HitTestRangeStart(InRangeToScreen, PlaybackRange, MouseDownPixel))
		{
			// playback range start scrubber
			MouseDragType = DRAG_PLAYBACK_START;
			TimeSliderArgs.OnPlaybackRangeBeginDrag.ExecuteIfBound();
		}
		else if (!bHandleMiddleMouseButton
			&& !bHitScrubber
			&& !TimeSliderArgs.AreMarkedFramesLocked.Get()
			&& HitTestMark(InGeometry, InRangeToScreen, MouseDownPixel, bInFromTimeSlider, &HitMarkIndex))
		{
			MouseDragType = DRAG_MARK;

			HandleDragMark_NoDragType(HitMarkIndex);

			TimeSliderArgs.OnMarkBeginDrag.ExecuteIfBound();
		}
		else if (FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Control) && bHandleMiddleMouseButton == false)
		{
			MouseDragType = DRAG_SETTING_RANGE;
		}
		else if (bMouseDownInRegion)
		{
			MouseDragType = DRAG_SCRUBBING_TIME;
			TimeSliderArgs.OnBeginScrubberMovement.ExecuteIfBound();
		}
	}

	// 2) Do the drag update in the same mouse-move event after MouseDragType is chosen
	if (MouseDragType != DRAG_NONE)
	{
		const FVector2D ReportedMousePosition = InPointerEvent.GetScreenSpacePosition();
		const FVector2D PreviousMousePosition = LastMousePosition.Get(ReportedMousePosition);
		const FVector2D CursorDelta = InPointerEvent.GetCursorDelta();

		FVector2D CurrentMousePosition = ReportedMousePosition;
		if (!CursorDelta.IsNearlyZero()
			&& (ReportedMousePosition - PreviousMousePosition).IsNearlyZero())
		{
			CurrentMousePosition = PreviousMousePosition + CursorDelta;
		}

		const FFrameTime MouseEndTime = ComputeFrameTimeFromMouse(InGeometry, CurrentMousePosition, InRangeToScreen);
		const FFrameTime MouseStartTime = ComputeFrameTimeFromMouse(InGeometry, MouseDownPosition.GetValue(), InRangeToScreen);
		const FFrameNumber DiffFrame = MouseEndTime.FrameNumber - MouseStartTime.FrameNumber;

		// Set the start playback range time?
		if (MouseDragType == DRAG_PLAYBACK_START)
		{
			if (InPointerEvent.IsShiftDown())
			{
				SetPlaybackRangeStart(MouseDownPlaybackRange.GetLowerBoundValue() + DiffFrame);
				SetPlaybackRangeEnd(MouseDownPlaybackRange.GetUpperBoundValue() + DiffFrame);
			}
			else
			{
				SetPlaybackRangeStart(MouseEndTime.FrameNumber);
			}
		}
		// Set the end playback range time?
		else if (MouseDragType == DRAG_PLAYBACK_END)
		{
			if (InPointerEvent.IsShiftDown())
			{
				SetPlaybackRangeStart(MouseDownPlaybackRange.GetLowerBoundValue() + DiffFrame);
				SetPlaybackRangeEnd(MouseDownPlaybackRange.GetUpperBoundValue() + DiffFrame);
			}
			else
			{		
				SetPlaybackRangeEnd(MouseEndTime.FrameNumber);
			}
		}
		// Set the start selection range time?
		else if (MouseDragType == DRAG_SELECTION_START)
		{
			if (InPointerEvent.IsShiftDown())
			{
				SetSelectionRangeStart(MouseDownSelectionRange.GetLowerBoundValue() + DiffFrame);
				SetSelectionRangeEnd(MouseDownSelectionRange.GetUpperBoundValue() + DiffFrame);
			}
			else
			{
				SetSelectionRangeStart(MouseEndTime.FrameNumber);
			}
		}
		// Set the end selection range time?
		else if (MouseDragType == DRAG_SELECTION_END)
		{
			if (InPointerEvent.IsShiftDown())
			{
				SetSelectionRangeStart(MouseDownSelectionRange.GetLowerBoundValue() + DiffFrame);
				SetSelectionRangeEnd(MouseDownSelectionRange.GetUpperBoundValue() + DiffFrame);
			}
			else 
			{
				SetSelectionRangeEnd(MouseEndTime.FrameNumber);
			}
		}
		else if (MouseDragType == DRAG_MARK)
		{
			const FFrameTime LastMouseTime = ComputeFrameTimeFromMouse(InGeometry, PreviousMousePosition, InRangeToScreen);
			const FFrameNumber LastDiffFrame = MouseEndTime.FrameNumber - LastMouseTime.FrameNumber;
			HandleDragMark(LastDiffFrame);
		}
		else if (MouseDragType == DRAG_SCRUBBING_TIME)
		{
			const FFrameTime ScrubTime = ComputeScrubTimeFromMouse(InGeometry, InPointerEvent, InRangeToScreen);
			// Delegate responsibility for clamping to the current view range to the client.
			// If middle mouse button down we don't evaluate on the time change.
			CommitScrubPosition(ScrubTime, /*bIsScrubbing=*/true, /*bEvaluate=*/!bHandleMiddleMouseButton);
		}

		LastMousePosition = CurrentMousePosition;
	}

	return FReply::Unhandled();
}

FReply FSequencerTimeSliderController::OnMouseMove_RightMouseDown(const FGeometry& InGeometry
	, const FPointerEvent& InPointerEvent
	, const USequencerSettings& InSequencerSettings)
{
	DistanceDragged += FMath::Abs(InPointerEvent.GetCursorDelta().X);

	if (!bPanning)
	{
		if (DistanceDragged > 0.f/*FSlateApplication::Get().GetDragTriggerDistance()*/)
		{
			bPanning = true;
		}
	}
	else if (InPointerEvent.IsShiftDown() && InPointerEvent.IsAltDown())
	{
		const FVector2D PointerPosition = InPointerEvent.GetScreenSpacePosition();
		const FFrameRate TickResolution = GetTickResolution();

		float MouseFractionX = InGeometry.AbsoluteToLocal(PointerPosition).X / InGeometry.GetLocalSize().X;

		// If zooming on the current time, adjust mouse fractionX
		if (InSequencerSettings.GetZoomPosition() == ESequencerZoomPosition::SZP_CurrentTime)
		{
			const double ScrubPosition = TimeSliderArgs.ScrubPosition.Get() / TickResolution;
			if (GetViewRange().Contains(ScrubPosition))
			{
				FScrubRangeToScreen RangeToScreen(GetViewRange(), InGeometry.Size);
				const float TimePosition = RangeToScreen.InputToLocalX(ScrubPosition);
				MouseFractionX = TimePosition / InGeometry.GetLocalSize().X;
			}
		}

		const float ZoomDelta = -0.01f * InPointerEvent.GetCursorDelta().X;
		ZoomByDelta(ZoomDelta, MouseFractionX);
	}
	else
	{
		const TRange<double> LocalViewRange = GetViewRange();
		const double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
		const double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();

		const FScrubRangeToScreen ScaleInfo(LocalViewRange, InGeometry.Size);
		const FVector2D ScreenDelta = InPointerEvent.GetCursorDelta();
		const FVector2D InputDelta = FVector2D(ScreenDelta.X / ScaleInfo.PixelsPerInput, 0.);

		double NewViewOutputMin = LocalViewRangeMin - InputDelta.X;
		double NewViewOutputMax = LocalViewRangeMax - InputDelta.X;

		ClampViewRange(NewViewOutputMin, NewViewOutputMax);
		SetViewRange(NewViewOutputMin, NewViewOutputMax, EViewRangeInterpolation::Immediate);
	}

	return FReply::Handled();
}

void FSequencerTimeSliderController::CommitScrubPosition( FFrameTime NewValue, bool bIsScrubbing, bool bEvaluate)
{
	using namespace UE::Sequencer;

	bIsEvaluating = bEvaluate;

	// The user can scrub past the viewing range of the time slider controller, so we clamp it to the view range.
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if(!Sequencer.IsValid())
	{
		return;
	}

	const bool bIsWarped = Sequencer->GetSequencerSettings()->GetTimeWarpDisplayMode() == ESequencerTimeWarpDisplay::WarpedTime;
	FTimeDomainOverride TimeDomainOverride = Sequencer->OverrideTimeDomain(bIsWarped ? ETimeDomain::Warped : ETimeDomain::Unwarped);

	Sequencer->SnapSequencerTime(NewValue);

	if (bIsScrubbing)
	{
		FAnimatedRange ViewRange = GetViewRange();
	
		FFrameRate DisplayRate = Sequencer->GetFocusedDisplayRate();
		FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();

		FFrameTime LowerBound = (ViewRange.GetLowerBoundValue() * TickResolution).CeilToFrame();
		FFrameTime UpperBound = (ViewRange.GetUpperBoundValue() * TickResolution).FloorToFrame();

		if (Sequencer->GetSequencerSettings()->GetForceWholeFrames())
		{
			LowerBound = FFrameRate::Snap(LowerBound, TickResolution, DisplayRate);
			UpperBound = FFrameRate::Snap(UpperBound, TickResolution, DisplayRate);
		}

		NewValue = FMath::Clamp(NewValue, LowerBound, UpperBound);
	}

	// Manage the scrub position ourselves if its not bound to a delegate
	if ( !TimeSliderArgs.ScrubPosition.IsBound() )
	{
		TimeSliderArgs.ScrubPosition.Set( NewValue );
	}

	TimeSliderArgs.OnScrubPositionChanged.ExecuteIfBound( NewValue, bIsScrubbing, bEvaluate);
}

FReply FSequencerTimeSliderController::OnMouseWheel(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (TimeSliderArgs.AllowZoom && InPointerEvent.IsControlDown())
	{
		float MouseFractionX = InGeometry.AbsoluteToLocal(InPointerEvent.GetScreenSpacePosition()).X / InGeometry.GetLocalSize().X;

		// If zooming on the current time, adjust mouse fractionX
		const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
		if (Sequencer.IsValid() && Sequencer->GetSequencerSettings()->GetZoomPosition() == ESequencerZoomPosition::SZP_CurrentTime)
		{
			const double ScrubPosition = TimeSliderArgs.ScrubPosition.Get() / GetTickResolution();
			if (GetViewRange().Contains(ScrubPosition))
			{
				FScrubRangeToScreen RangeToScreen(GetViewRange(), InGeometry.Size);
				float TimePosition = RangeToScreen.InputToLocalX(ScrubPosition);
				MouseFractionX = TimePosition / InGeometry.GetLocalSize().X;
			}
		}

		const float ZoomDelta = -0.2f * InPointerEvent.GetWheelDelta();
		if (ZoomByDelta(ZoomDelta, MouseFractionX))
		{
			return FReply::Handled();
		}
	}
	else if (InPointerEvent.IsShiftDown())
	{
		PanByDelta(-InPointerEvent.GetWheelDelta());
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

FCursorReply FSequencerTimeSliderController::OnCursorQuery( const SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return FCursorReply::Unhandled();
	}

	FScrubRangeToScreen RangeToScreen(GetViewRange(), MyGeometry.Size);

	const bool       bReadOnly        = Sequencer->IsReadOnly();
	const FFrameRate TickResolution   = GetTickResolution();
	const bool       bLockedPlayRange = TimeSliderArgs.IsPlaybackRangeLocked.Get();
	const bool       bLockedMarkedFrames = TimeSliderArgs.AreMarkedFramesLocked.Get();
	const float      HitTestPixel     = MyGeometry.AbsoluteToLocal(CursorEvent.GetScreenSpacePosition()).X;
	const bool       bHitScrubber     = GetHitTestScrubPixelMetrics(RangeToScreen).HandleRangePx.Contains(HitTestPixel);

	TRange<double>   SelectionRange   = TimeSliderArgs.SelectionRange.Get() / TickResolution;
	TRange<double>   PlaybackRange    = TimeSliderArgs.PlaybackRange.Get()  / TickResolution;

	if (MouseDragType == DRAG_SCRUBBING_TIME
		|| UE::CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport))
	{
		return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
	}

	// Use L/R resize cursor if we're dragging or hovering a playback range bound
	if ((MouseDragType == DRAG_PLAYBACK_END) ||
		(MouseDragType == DRAG_PLAYBACK_START) ||
		(MouseDragType == DRAG_SELECTION_START) ||
		(MouseDragType == DRAG_SELECTION_END) ||
		(!bLockedPlayRange         && !bHitScrubber && HitTestRangeStart(RangeToScreen, PlaybackRange,  HitTestPixel)) ||
		(!bLockedPlayRange         && !bHitScrubber && HitTestRangeEnd(  RangeToScreen, PlaybackRange,  HitTestPixel)) ||
		(!SelectionRange.IsEmpty() && !bHitScrubber && HitTestRangeStart(RangeToScreen, SelectionRange, HitTestPixel)) ||
		(!SelectionRange.IsEmpty() && !bHitScrubber && HitTestRangeEnd(  RangeToScreen, SelectionRange, HitTestPixel)))
	{
		return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
	}

	const bool bFromTimeSlider = true;
	if (MouseDragType == DRAG_MARK || (!bLockedMarkedFrames && !bHitScrubber && HitTestMark(MyGeometry, RangeToScreen, HitTestPixel, bFromTimeSlider)))
	{
		return FCursorReply::Cursor(EMouseCursor::CardinalCross);
	}

	return FCursorReply::Unhandled();
}

int32 FSequencerTimeSliderController::OnPaintViewArea( const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, bool bEnabled, const FPaintViewAreaArgs& Args ) const
{
	using namespace UE::MovieScene;
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return LayerId;
	}

	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	TRange<double> LocalViewRange = GetViewRange();
	FScrubRangeToScreen RangeToScreen( LocalViewRange, AllottedGeometry.Size );

	if (Args.PlaybackRangeArgs.IsSet())
	{
		FPaintPlaybackRangeArgs PaintArgs = Args.PlaybackRangeArgs.GetValue();
		LayerId = DrawPlaybackRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PaintArgs);
		LayerId = DrawSubSequenceRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PaintArgs);
		PaintArgs.SolidFillOpacity = 0.f;
		LayerId = DrawSelectionRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PaintArgs);
	}

	if( Args.bDisplayTickLines )
	{
		static FLinearColor TickColor(0.f, 0.f, 0.f, 0.3f);

		// Draw major tick lines in the section area
		FDrawTickArgs DrawTickArgs;
		{
			DrawTickArgs.AllottedGeometry = AllottedGeometry;
			DrawTickArgs.bMirrorLabels = false;
			DrawTickArgs.bOnlyDrawMajorTicks = true;
			DrawTickArgs.TickColor = TickColor;
			DrawTickArgs.CullingRect = MyCullingRect;
			DrawTickArgs.DrawEffects = DrawEffects;
			// Draw major ticks under sections
			DrawTickArgs.StartLayer = LayerId-1;
			// Draw the tick the entire height of the section area
			DrawTickArgs.TickOffset = 0.0f;
			DrawTickArgs.MajorTickHeight = AllottedGeometry.Size.Y;
		}

		DrawTicks( OutDrawElements, LocalViewRange, RangeToScreen, DrawTickArgs );

		const USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
		if (SequencerSettings && SequencerSettings->ShouldDrawHorizontalTrackSeparatorLines())
		{
			DrawHorizontalTrackSeparatorLines(OutDrawElements, DrawTickArgs);
		}
	}

	if (Args.bDisplayMarkedFrames)
	{
		LayerId = DrawMarkedFrames(AllottedGeometry, RangeToScreen, OutDrawElements, LayerId, DrawEffects, false);
	}

	LayerId = DrawScalingAnchors(AllottedGeometry, RangeToScreen, OutDrawElements, LayerId, DrawEffects);

	LayerId = DrawVerticalFrames(AllottedGeometry, RangeToScreen, OutDrawElements, LayerId, DrawEffects);

	FTimeWarpTrackExtension*   TimeWarpExtension   = Sequencer->GetViewModel()->GetRootSequenceModel()->GetSharedData()->CastDynamic<FTimeWarpTrackExtension>();
	const FTimeWarpTrackModel* ActiveTimeWarpTrack = TimeWarpExtension ? TimeWarpExtension->GetActiveTimeWarpTrack() : nullptr;

	FLinearColor TimeWarpColor = FStyleColors::AccentOrange.GetSpecifiedColor();

	// If we have no active timewarp track, just draw the single scrub position
	if (ActiveTimeWarpTrack == nullptr || Sequencer->GetSequencerSettings()->GetTimeWarpDisplayMode() != ESequencerTimeWarpDisplay::Both)
	{
		const bool bIsWarped = (Sequencer->GetSequencerSettings()->GetTimeWarpDisplayMode() == ESequencerTimeWarpDisplay::WarpedTime && ActiveTimeWarpTrack != nullptr);
		FLinearColor ScrubColor = bIsWarped ? TimeWarpColor : FLinearColor::White;

		FQualifiedFrameTime ScrubPosition = FQualifiedFrameTime(TimeSliderArgs.ScrubPosition.Get(), GetTickResolution());
		FScrubberMetrics    ScrubMetrics  = GetScrubPixelMetrics(ScrubPosition, RangeToScreen);

		if (ScrubMetrics.bDrawExtents)
		{
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId + 1,
				AllottedGeometry.ToPaintGeometry(
					FVector2f(ScrubMetrics.FrameExtentsPx.Size<float>(), AllottedGeometry.Size.Y),
					FSlateLayoutTransform(FVector2f(ScrubMetrics.FrameExtentsPx.GetLowerBoundValue(), 0.f))),
				ScrubFillBrush,
				DrawEffects,
				ScrubColor.CopyWithNewOpacity(0.5f)
			);
		}

		if (Args.bDisplayScrubPosition)
		{
			const float ScrubPixelX = RangeToScreen.InputToLocalX(ScrubPosition.AsSeconds());

			TArray<FVector2f> LinePoints;
			LinePoints.AddUninitialized(2);
			LinePoints[0] = FVector2f( ScrubPixelX, 0.f );
			LinePoints[1] = FVector2f( ScrubPixelX, AllottedGeometry.Size.Y );

			// Draw a white line for the unwarped scrub position
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				LayerId+1,
				AllottedGeometry.ToPaintGeometry(),
				LinePoints,
				DrawEffects,
				ScrubColor,
				false
			);
		}
	}
	else
	{
		// Handle timewarp by drawing the white (unwarped) position up to the bottom of the active timewarp track,
		//     then draw the orange (warped) time from there
		FVirtualGeometry Geometry = ActiveTimeWarpTrack->GetVirtualGeometry();
		const float UnwarpedScrubVerticalClip = Sequencer->GetUnderlyingSequencerWidget()->GetPinnedTreeView()->VirtualToPhysical(Geometry.GetTop() + Geometry.GetNestedHeight());

		FQualifiedFrameTime UnwarpedScrubPosition = Sequencer->GetUnwarpedLocalTime();

		FQualifiedFrameTime WarpedScrubPosition   = Sequencer->GetLocalTime();
		FScrubberMetrics    UnwarpedScrubMetrics  = GetScrubPixelMetrics(UnwarpedScrubPosition, RangeToScreen);
		FScrubberMetrics    WarpedScrubMetrics    = GetScrubPixelMetrics(WarpedScrubPosition, RangeToScreen);

		if (UnwarpedScrubMetrics.bDrawExtents)
		{
			// Draw a box for the unwarped scrub position
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId+1,
				AllottedGeometry.ToPaintGeometry(
					FVector2f(UnwarpedScrubMetrics.FrameExtentsPx.Size<float>(), UnwarpedScrubVerticalClip - 1.f),
					FSlateLayoutTransform(FVector2f(UnwarpedScrubMetrics.FrameExtentsPx.GetLowerBoundValue(), 0.f))),
				ScrubFillBrush,
				DrawEffects,
				FLinearColor::White.CopyWithNewOpacity(0.5f)
			);

			// Draw a box for the warped scrub position
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId + 1,
				AllottedGeometry.ToPaintGeometry(
					FVector2f(WarpedScrubMetrics.FrameExtentsPx.Size<float>(), AllottedGeometry.Size.Y - UnwarpedScrubVerticalClip),
					FSlateLayoutTransform(FVector2f(WarpedScrubMetrics.FrameExtentsPx.GetLowerBoundValue(), UnwarpedScrubVerticalClip))),
				ScrubFillBrush,
				DrawEffects,
				TimeWarpColor.CopyWithNewOpacity(0.75f)
			);
		}

		if (Args.bDisplayScrubPosition)
		{
			const float UnwarpedScrubPixelX = RangeToScreen.InputToLocalX(UnwarpedScrubPosition.AsSeconds());
			const float WarpedScrubPixelX   = RangeToScreen.InputToLocalX(WarpedScrubPosition.AsSeconds());
			const float OriginPixelX        = RangeToScreen.InputToLocalX(0.f);

			TArray<FVector2f> LinePoints;
			LinePoints.AddUninitialized(2);
			LinePoints[0] = FVector2f( UnwarpedScrubPixelX, 0.f );
			LinePoints[1] = FVector2f( UnwarpedScrubPixelX, UnwarpedScrubVerticalClip-1.f );

			// Draw a white line for the unwarped scrub position
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				LayerId+1,
				AllottedGeometry.ToPaintGeometry(),
				LinePoints,
				DrawEffects,
				FLinearColor(1.f, 1.f, 1.f, .5f),
				false
			);

			LinePoints[0] = FVector2f( WarpedScrubPixelX, UnwarpedScrubVerticalClip );
			LinePoints[1] = FVector2f( WarpedScrubPixelX, AllottedGeometry.Size.Y );

			// Draw a line for the warped scrub position
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				LayerId+1,
				AllottedGeometry.ToPaintGeometry(),
				LinePoints,
				DrawEffects,
				TimeWarpColor,
				false
			);

			// Optional cvar for drawing the dashed line
			if (GSequencerShowTimeWarpScrubberLink)
			{
				const float StartLine = WarpedScrubPixelX < UnwarpedScrubPixelX ? WarpedScrubPixelX : UnwarpedScrubPixelX;
				const float EndLine   = WarpedScrubPixelX < UnwarpedScrubPixelX ? UnwarpedScrubPixelX : WarpedScrubPixelX;

				const float HalfPos = (EndLine + StartLine)*.5f;

				// Draw a dashed line to connect them
				LinePoints[0] = FVector2f( StartLine,   UnwarpedScrubVerticalClip-1.f );
				LinePoints[1] = FVector2f( FMath::Max(HalfPos - 10.f, StartLine), UnwarpedScrubVerticalClip-1.f );

				FSlateDrawElement::MakeDashedLines(
					OutDrawElements,
					LayerId++,
					AllottedGeometry.ToPaintGeometry(),
					CopyTemp(LinePoints),
					DrawEffects,
					FColor(212, 147, 20),
					2.f,
					5.f,
					StartLine - OriginPixelX
				);

				LinePoints[0] = FVector2f( FMath::Min(HalfPos + 10.f, EndLine),   UnwarpedScrubVerticalClip-1.f );
				LinePoints[1] = FVector2f( EndLine, UnwarpedScrubVerticalClip-1.f );

				FSlateDrawElement::MakeDashedLines(
					OutDrawElements,
					LayerId++,
					AllottedGeometry.ToPaintGeometry(),
					CopyTemp(LinePoints),
					DrawEffects,
					FColor(212, 147, 20),
					2.f,
					5.f,
					StartLine - OriginPixelX
				);

				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId + 1,
					AllottedGeometry.ToPaintGeometry(
						FVector2f(12.f, 12.f),
						FSlateLayoutTransform(FVector2f( HalfPos-6.f, UnwarpedScrubVerticalClip-6.f ))
					),
					FAppStyle::GetBrush( TEXT( "Sequencer.Tracks.TimeWarp" ) ),
					DrawEffects,
					TimeWarpColor.CopyWithNewOpacity(0.75f)
				);
			}
		}
	}

	return LayerId;
}

void FSequencerTimeSliderController::SetStoppedPosition(FFrameTime InTime)
{
	if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
	{
		SequencerPtr->SetLocalTimeDirectly(InTime);
	}
}

void FSequencerTimeSliderController::SetPlaybackStatus(ETimeSliderPlaybackStatus InStatus)
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	switch (InStatus)
	{
		case ETimeSliderPlaybackStatus::Jumping:
		{
			Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Jumping);
			break;
		}
		case ETimeSliderPlaybackStatus::Paused:
		{
			Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Paused);
			break;
		}
		case ETimeSliderPlaybackStatus::Playing:
		{
			Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Playing);
			break;
		}
		case ETimeSliderPlaybackStatus::Scrubbing:
		{
			Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Scrubbing);
			break;
		}
		case ETimeSliderPlaybackStatus::Stepping:
		{
			Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Stepping);
			break;
		}
		default:
		case ETimeSliderPlaybackStatus::Stopped:
		{
			//Call Pause() instead of SetPlaybackStatus(EMovieScenePlayerStatus::Stopped)
			//so the correct delegates/evals can happen so all systems like audio are probably stopped (Pause set's Stopped status).
			Sequencer->Pause();
			break;
		}
	}
}

ETimeSliderPlaybackStatus FSequencerTimeSliderController::GetPlaybackStatus() const
{
	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return ETimeSliderPlaybackStatus::Stopped;
	}

	const EMovieScenePlayerStatus::Type Status = Sequencer->GetPlaybackStatus();
	switch (Status)
	{
		case EMovieScenePlayerStatus::Jumping:
			return ETimeSliderPlaybackStatus::Jumping;
		case EMovieScenePlayerStatus::Paused:
			return ETimeSliderPlaybackStatus::Paused;
		case EMovieScenePlayerStatus::Playing:
			return ETimeSliderPlaybackStatus::Playing;
		case EMovieScenePlayerStatus::Scrubbing:
			return ETimeSliderPlaybackStatus::Scrubbing;
		case EMovieScenePlayerStatus::Stepping:
			return ETimeSliderPlaybackStatus::Stepping;
		case EMovieScenePlayerStatus::Stopped:
			return ETimeSliderPlaybackStatus::Stopped;
	}

	return ETimeSliderPlaybackStatus::Stopped;
}

TSharedPtr<SWidget> FSequencerTimeSliderController::OpenContextMenu(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	using namespace UE::Sequencer;
	
	class SMarkedFramePropertyWidget : public SCompoundWidget, public FNotifyHook
	{
	public:
		UMovieScene* MovieSceneToModify;
		TSharedPtr<IStructureDetailsView> DetailsView;
		TWeakPtr<FSequencer> WeakSequencer;

		SLATE_BEGIN_ARGS(SMarkedFramePropertyWidget){}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, UMovieScene* InMovieScene, int32 InMarkedFrameIndex, TWeakPtr<FSequencer> InWeakSequencer)
		{
			MovieSceneToModify = InMovieScene;
			WeakSequencer = InWeakSequencer;

			FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
			FDetailsViewArgs DetailsViewArgs;
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.bShowScrollBar = false;
			DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
			DetailsViewArgs.NotifyHook = this;

			FStructureDetailsViewArgs StructureDetailsViewArgs;
			StructureDetailsViewArgs.bShowObjects = true;
			StructureDetailsViewArgs.bShowAssets = true;
			StructureDetailsViewArgs.bShowClasses = true;
			StructureDetailsViewArgs.bShowInterfaces = true;

			TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(FMovieSceneMarkedFrame::StaticStruct(), (uint8 *)&InMovieScene->GetMarkedFrames()[InMarkedFrameIndex]);

			DetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureDetailsViewArgs, nullptr);
			DetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout("FrameNumber", 
				FOnGetPropertyTypeCustomizationInstance::CreateSP(WeakSequencer.Pin().ToSharedRef(), &FSequencer::MakeFrameNumberDetailsCustomization));
			DetailsView->SetStructureData(StructOnScope);

			ChildSlot
			[
				DetailsView->GetWidget().ToSharedRef()
			];
		}

		virtual void NotifyPreChange( FProperty* PropertyAboutToChange ) override
		{
			MovieSceneToModify->Modify();
		}

		virtual void NotifyPreChange( class FEditPropertyChain* PropertyAboutToChange ) override
		{
			MovieSceneToModify->Modify();
		}
	};

	static const FName MenuName("Sequencer.TimelineMenu");
	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		UToolMenu* TimelineMenu = UToolMenus::Get()->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu);
		TimelineMenu->bSearchable = false;
		TimelineMenu->bShouldCloseWindowAfterMenuSelection = true;
		
		TimelineMenu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* const InMenu)
		{
			USequencerTimeSliderControllerMenuContext* Context = InMenu->FindContext<USequencerTimeSliderControllerMenuContext>();
			if (!Context)
			{
				return;
			}
			
			TSharedPtr<FSequencerTimeSliderController> TimeSliderController = Context->WeakTimeSliderController.Pin();
            if (!TimeSliderController.IsValid())
            {
            	return;
            }

            TSharedPtr<FSequencer> Sequencer = TimeSliderController->WeakSequencer.Pin();
            if (!Sequencer.IsValid())
            {
            	return;
            }

			const bool bReadOnly = Sequencer->IsReadOnly();
			const FGeometry& MyGeometry = Context->Geometry;
			const FPointerEvent& MouseEvent = Context->PointerEvent;
			const FFrameNumber& FrameNumber = Context->FrameNumber;
				
			FScrubRangeToScreen RangeToScreen = FScrubRangeToScreen(TimeSliderController->GetViewRange(), MyGeometry.Size);
			const float MousePixel = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).X;
			
			FText CurrentTimeText = FText::FromString(Sequencer->GetNumericTypeInterface()->ToString(FrameNumber.Value));
			// SequencerPlaybackRangeMenu
			{
				FToolMenuSection& Section = InMenu->AddSection("SequencerPlaybackRangeMenu", FText::Format(LOCTEXT("PlaybackRangeTextFormat", "Playback Range ({0}):"), CurrentTimeText));
			
				TRange<FFrameNumber> PlaybackRange = TimeSliderController->TimeSliderArgs.PlaybackRange.Get();
				TOptional<TRange<FFrameNumber>> SubSequenceRange = TimeSliderController->TimeSliderArgs.SubSequenceRange.Get();

				Section.AddMenuEntry(
					"SetPlaybackStart",
					LOCTEXT("SetPlaybackStart", "Set Start Time"),
					FText(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([TimeSliderController, FrameNumber]
						{
							TimeSliderController->SetPlaybackRangeStart(FrameNumber);
						}),
						FCanExecuteAction::CreateLambda([TimeSliderController, FrameNumber, PlaybackRange]
						{
							return !TimeSliderController->TimeSliderArgs.IsPlaybackRangeLocked.Get() && FrameNumber < UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange);
						})
					)
				);
						
				Section.AddMenuEntry(
					"SetPlaybackEnd",
					LOCTEXT("SetPlaybackEnd", "Set End Time"),
					FText(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([TimeSliderController, FrameNumber]
						{
							TimeSliderController->SetPlaybackRangeEnd(FrameNumber);
						}),
						FCanExecuteAction::CreateLambda([TimeSliderController, FrameNumber, PlaybackRange]
						{
							return !TimeSliderController->TimeSliderArgs.IsPlaybackRangeLocked.Get() && FrameNumber >= UE::MovieScene::DiscreteInclusiveLower(PlaybackRange);
						})
					)
				);

				Section.AddMenuEntry(
					"ConformToSubsequenceRange",
					LOCTEXT("ConformToSubsequenceRange", "Conform to Range"),
					LOCTEXT("ConformToSubsequenceRangeTooltip", "Conform the start and end time to the extents of the subsequence range"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([TimeSliderController, SubSequenceRange]
						{
							TimeSliderController->SetPlaybackRangeStart(SubSequenceRange.GetValue().GetLowerBoundValue());
							TimeSliderController->SetPlaybackRangeEnd(SubSequenceRange.GetValue().GetUpperBoundValue());
						}),
						FCanExecuteAction::CreateLambda([TimeSliderController, SubSequenceRange]
						{
							return !TimeSliderController->TimeSliderArgs.IsPlaybackRangeLocked.Get() && SubSequenceRange.IsSet();
						})
					)
				);

				Section.AddMenuEntry(
					"ToggleLocked",
						LOCTEXT("ToggleLocked", "Locked"),
						LOCTEXT("ToggleLockedTooltip", "Lock/Unlock the playback range"),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([TimeSliderController]
							{
								TimeSliderController->TimeSliderArgs.OnTogglePlaybackRangeLocked.ExecuteIfBound();
							}),
							FCanExecuteAction::CreateLambda([bReadOnly]{ return !bReadOnly; }),
							FIsActionChecked::CreateLambda([TimeSliderController]
							{
								return TimeSliderController->TimeSliderArgs.IsPlaybackRangeLocked.Get();
							})
						),
						EUserInterfaceActionType::ToggleButton
					);
			}

			// SequencerSelectionRangeMenu
			{
				TRange<FFrameNumber> SelectionRange = TimeSliderController->TimeSliderArgs.SelectionRange.Get();
				FToolMenuSection& Section = InMenu->AddSection("SequencerSelectionRangeMenu", FText::Format(LOCTEXT("SelectionRangeTextFormat", "Selection Range ({0}):"), CurrentTimeText));

				Section.AddMenuEntry(
					"SetSelectionStart",
					LOCTEXT("SetSelectionStart", "Set Selection Start"),
					FText(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([TimeSliderController, FrameNumber]
						{
							TimeSliderController->SetSelectionRangeStart(FrameNumber);
						}),
						FCanExecuteAction::CreateLambda([SelectionRange, FrameNumber]
						{
							return SelectionRange.IsEmpty() || FrameNumber < UE::MovieScene::DiscreteExclusiveUpper(SelectionRange);
						})
					)
				);

				Section.AddMenuEntry(
					"SetSelectionEnd",
					LOCTEXT("SetSelectionEnd", "Set Selection End"),
					FText(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([TimeSliderController, FrameNumber]
						{
							TimeSliderController->SetSelectionRangeEnd(FrameNumber);
						}),
						FCanExecuteAction::CreateLambda([SelectionRange, FrameNumber]
						{
							return SelectionRange.IsEmpty() || FrameNumber >= UE::MovieScene::DiscreteInclusiveLower(SelectionRange);
						})
					)
				);

				Section.AddMenuEntry(
					"ClearSelectionRange",
					LOCTEXT("ClearSelectionRange", "Clear Selection Range"),
					FText(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([TimeSliderController]
						{
							TimeSliderController->TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>::Empty());
						}),
						FCanExecuteAction::CreateLambda([SelectionRange]
						{
							return !SelectionRange.IsEmpty();
						})
					)
				);
			}

			// SequencerParentChainMenu
			{
				UMovieSceneCompiledDataManager* CompiledDataManager = Sequencer->GetEvaluationTemplate().GetCompiledDataManager();
				const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(Sequencer->GetEvaluationTemplate().GetCompiledDataID());

				if (Hierarchy)
				{
					FToolMenuSection& Section = InMenu->AddSection("SequencerParentChainMenu", LOCTEXT("TimeSpace", "Time Space"));

					TArray<FMovieSceneSequenceID> ParentChain = TimeSliderController->TimeSliderArgs.ScrubPositionParentChain.Get();
					for (FMovieSceneSequenceID ParentID : ParentChain)
					{
						FText ParentText = Sequencer->GetRootMovieSceneSequence()->GetDisplayName();

						for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
						{
							if (Pair.Key == ParentID && Pair.Value.GetSequence())
							{
								ParentText = Pair.Value.GetSequence()->GetDisplayName();
								break;
							}
						}

						Section.AddMenuEntry(
							FName(*ParentText.ToString()),
							ParentText,
							FText::Format(LOCTEXT("DisplayTimeSpace", "Display time in the space of {0}"), ParentText),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([TimeSliderController, ParentID]
								{
									TimeSliderController->TimeSliderArgs.OnScrubPositionParentChanged.ExecuteIfBound(ParentID);
								}),
								FCanExecuteAction(),
								FIsActionChecked::CreateLambda([TimeSliderController, ParentID]
								{
									return TimeSliderController->TimeSliderArgs.ScrubPositionParent.Get() == MovieSceneSequenceID::Invalid
									? ParentID == TimeSliderController->TimeSliderArgs.ScrubPositionParentChain.Get().Last()
									: TimeSliderController->TimeSliderArgs.ScrubPositionParent.Get() == ParentID;
								})
							),
							EUserInterfaceActionType::RadioButton
						);
					}
				}
			}

			// SequencerMarkMenu
			{
				FToolMenuSection& Section = InMenu->AddSection("SequencerMarkMenu", FText::Format(LOCTEXT("MarkTextFormat", "Mark ({0}):"), CurrentTimeText));

				UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
				bool bHasMarks = MovieScene->GetMarkedFrames().Num() > 0;

				int32 MarkedIndex = INDEX_NONE;
				const bool bTestLabelBox = true;
				TimeSliderController->HitTestMark(MyGeometry, RangeToScreen, MousePixel, bTestLabelBox, &MarkedIndex);

				FSequencerSelection& SequencerSelection = Sequencer->GetSelection();
				TWeakPtr<FSequencer> WeakSequencer = Sequencer.ToWeakPtr();

				if (MarkedIndex != INDEX_NONE)
				{
					const bool bIsSelected = SequencerSelection.MarkedFrames.IsSelected(MarkedIndex);
					if (!bIsSelected)
					{
						SequencerSelection.MarkedFrames.Empty();
						SequencerSelection.MarkedFrames.Select(MarkedIndex);
					}

					Section.AddSubMenu(TEXT("MarkProperties"), LOCTEXT("MarkPropertiesLabel", "Properties"), FText::GetEmpty(),
						FNewToolMenuDelegate::CreateLambda([TimeSliderController, MovieScene, MarkedIndex, WeakSequencer](UToolMenu* InSubMenu)
						{
							const bool bLockedMarkedFrames = TimeSliderController->TimeSliderArgs.AreMarkedFramesLocked.Get();
							TSharedRef<SMarkedFramePropertyWidget> Widget = SNew(SMarkedFramePropertyWidget, MovieScene, MarkedIndex, WeakSequencer);
							Widget->SetEnabled(!bLockedMarkedFrames);
							InSubMenu->AddMenuEntry("MarkPropertiesSection", FToolMenuEntry::InitWidget(NAME_None, Widget, FText::GetEmpty(), false));
						})
					);
				}

				Section.AddMenuEntry(
					"AddMark",
					LOCTEXT("AddMarkAtPlayhead", "Add Mark at Playhead"),
					FText(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda( [TimeSliderController]
						{
							TimeSliderController->AddMarkAtFrame(TimeSliderController->TimeSliderArgs.ScrubPosition.Get().FloorToFrame());
						}),
						FCanExecuteAction::CreateLambda([TimeSliderController, MarkedIndex]
						{
							return !TimeSliderController->TimeSliderArgs.AreMarkedFramesLocked.Get() && MarkedIndex == INDEX_NONE;
						}))
				);

				Section.AddMenuEntry(
					"DeleteMark",
					LOCTEXT("DeleteSelectedMarks", "Delete Selected Marks"),
					FText(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([TimeSliderController]
						{
							TimeSliderController->DeleteSelectedMarks();
						}),
						FCanExecuteAction::CreateLambda([TimeSliderController, WeakSequencer]
						{
							TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();

							return !TimeSliderController->TimeSliderArgs.AreMarkedFramesLocked.Get()
								&& Sequencer && Sequencer->GetSelection().MarkedFrames.Num() > 0;
						}))
				);

				Section.AddMenuEntry(
					"DeleteAllMarks",
					LOCTEXT("DeleteAllMarks", "Delete All Marks"),
					FText(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([TimeSliderController]
						{
							TimeSliderController->DeleteAllMarks();
						}),
						FCanExecuteAction::CreateLambda([TimeSliderController, bHasMarks]
						{
							return !TimeSliderController->TimeSliderArgs.AreMarkedFramesLocked.Get() && bHasMarks;
						}))
					);

				Section.AddMenuEntry(
					"ToggleLockedMarks",
					LOCTEXT("ToggleLockedMarks", "Locked"),
					LOCTEXT("ToggleLockedMarksTooltip", "Lock/Unlock all marked frames"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([TimeSliderController]
						{
							TimeSliderController->TimeSliderArgs.OnToggleMarkedFramesLocked.ExecuteIfBound();
						}),
						FCanExecuteAction::CreateLambda([bReadOnly]{ return !bReadOnly; }),
						FIsActionChecked::CreateLambda([TimeSliderController]
						{
							return TimeSliderController->TimeSliderArgs.AreMarkedFramesLocked.Get();
						})),
					EUserInterfaceActionType::ToggleButton
					);
			}
		}));
	}
	
	USequencerTimeSliderControllerMenuContext* TimeSliderContext = NewObject<USequencerTimeSliderControllerMenuContext>();
	TimeSliderContext->WeakTimeSliderController = AsWeak();
	TimeSliderContext->Geometry = InGeometry;
	TimeSliderContext->PointerEvent = InPointerEvent;
	TimeSliderContext->FrameNumber = ComputeFrameTimeFromMouse(
		InGeometry,
		InPointerEvent.GetScreenSpacePosition(),
		FScrubRangeToScreen(GetViewRange(), InGeometry.Size)
	).FrameNumber;

	USequencerToolMenuContext* SequencerContext = NewObject<USequencerToolMenuContext>();
	SequencerContext->WeakSequencer = WeakSequencer;

	FToolMenuContext MenuContext;
	MenuContext.AddObject(TimeSliderContext);
	MenuContext.AddObject(SequencerContext);
	return UToolMenus::Get()->GenerateWidget(MenuName, MenuContext);
}

void FSequencerTimeSliderController::ClampViewRange(double& NewRangeMin, double& NewRangeMax)
{
	bool bNeedsClampSet = false;
	double NewClampRangeMin = TimeSliderArgs.ClampRange.Get().GetLowerBoundValue();
	if ( NewRangeMin < TimeSliderArgs.ClampRange.Get().GetLowerBoundValue() )
	{
		NewClampRangeMin = NewRangeMin;
		bNeedsClampSet = true;
	}

	double NewClampRangeMax = TimeSliderArgs.ClampRange.Get().GetUpperBoundValue();
	if ( NewRangeMax > TimeSliderArgs.ClampRange.Get().GetUpperBoundValue() )
	{
		NewClampRangeMax = NewRangeMax;
		bNeedsClampSet = true;
	}

	if (bNeedsClampSet)
	{
		SetClampRange(NewClampRangeMin, NewClampRangeMax);
	}
}

void FSequencerTimeSliderController::SetViewRange( double NewRangeMin, double NewRangeMax, EViewRangeInterpolation Interpolation )
{
	// It's possible to quickly drag the outliner / track area splitter and trigger 0 value calls to
	// this function when the parent widget is sized smaller than the outliner. Guard against this 
	// "infinite" zoom to avoid SlateVertex buffer overflow ensure being triggered.
	if (NewRangeMin == 0.f || NewRangeMax == 0.f)
	{
		return;
	}

	// Clamp to a minimum size to avoid zero-sized or negative visible ranges
	double MinVisibleTimeRange = FFrameNumber(1) / GetTickResolution();
	TRange<double> ExistingViewRange  = GetViewRange();
	TRange<double> ExistingClampRange = TimeSliderArgs.ClampRange.Get();

	if (NewRangeMax == ExistingViewRange.GetUpperBoundValue())
	{
		if (NewRangeMin > NewRangeMax - MinVisibleTimeRange)
		{
			NewRangeMin = NewRangeMax - MinVisibleTimeRange;
		}
	}
	else if (NewRangeMax < NewRangeMin + MinVisibleTimeRange)
	{
		NewRangeMax = NewRangeMin + MinVisibleTimeRange;
	}

	// Clamp to the clamp range
	const TRange<double> NewRange = TRange<double>::Intersection(TRange<double>(NewRangeMin, NewRangeMax), ExistingClampRange);
	TimeSliderArgs.OnViewRangeChanged.ExecuteIfBound( NewRange, Interpolation );

	if( !TimeSliderArgs.ViewRange.IsBound() )
	{
		// The  output is not bound to a delegate so we'll manage the value ourselves (no animation)
		TimeSliderArgs.ViewRange.Set( NewRange );
	}
}

void FSequencerTimeSliderController::SetClampRange( double NewRangeMin, double NewRangeMax )
{
	const TRange<double> NewRange(NewRangeMin, NewRangeMax);

	TimeSliderArgs.OnClampRangeChanged.ExecuteIfBound(NewRange);

	if( !TimeSliderArgs.ClampRange.IsBound() )
	{	
		// The  output is not bound to a delegate so we'll manage the value ourselves (no animation)
		TimeSliderArgs.ClampRange.Set(NewRange);
	}
}

void FSequencerTimeSliderController::SetPlayRange( FFrameNumber RangeStart, int32 RangeDuration )
{
	check(RangeDuration >= 0);

	const TRange<FFrameNumber> NewRange(RangeStart, RangeStart + RangeDuration);

	TimeSliderArgs.OnPlaybackRangeChanged.ExecuteIfBound(NewRange);

	if( !TimeSliderArgs.PlaybackRange.IsBound() )
	{
		// The  output is not bound to a delegate so we'll manage the value ourselves (no animation)
		TimeSliderArgs.PlaybackRange.Set(NewRange);
	}
}

void FSequencerTimeSliderController::SetSelectionRange(const TRange<FFrameNumber>& NewRange)
{
	TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(NewRange);
}

bool FSequencerTimeSliderController::ZoomByDelta( float InDelta, float MousePositionFraction )
{
	TRange<double> LocalViewRange = GetViewRange().GetAnimationTarget();
	double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();
	double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
	const double OutputViewSize = LocalViewRangeMax - LocalViewRangeMin;
	const double OutputChange = OutputViewSize * InDelta;

	double NewViewOutputMin = LocalViewRangeMin - (OutputChange * MousePositionFraction);
	double NewViewOutputMax = LocalViewRangeMax + (OutputChange * (1.f - MousePositionFraction));

	if( NewViewOutputMin < NewViewOutputMax )
	{
		ClampViewRange(NewViewOutputMin, NewViewOutputMax);
		SetViewRange(NewViewOutputMin, NewViewOutputMax, EViewRangeInterpolation::Animated);
		return true;
	}

	return false;
}

void FSequencerTimeSliderController::PanByDelta( float InDelta )
{
	TRange<double> LocalViewRange = GetViewRange().GetAnimationTarget();

	double CurrentMin = LocalViewRange.GetLowerBoundValue();
	double CurrentMax = LocalViewRange.GetUpperBoundValue();

	// Adjust the delta to be a percentage of the current range
	InDelta *= ScrubConstants::ScrollPanFraction * (CurrentMax - CurrentMin);

	double NewViewOutputMin = CurrentMin + InDelta;
	double NewViewOutputMax = CurrentMax + InDelta;

	ClampViewRange(NewViewOutputMin, NewViewOutputMax);
	SetViewRange(NewViewOutputMin, NewViewOutputMax, EViewRangeInterpolation::Animated);
}


bool FSequencerTimeSliderController::HitTestRangeStart(const FScrubRangeToScreen& RangeToScreen, const TRange<double>& Range, float HitPixel) const
{
	static float BrushSizeInStateUnits = 6.f, DragToleranceSlateUnits = 2.f, MouseTolerance = 2.f;
	const float  RangeStartPixel = RangeToScreen.InputToLocalX(Range.GetLowerBoundValue());

	// Hit test against the brush region to the right of the playback start position, +/- DragToleranceSlateUnits
	return HitPixel >= RangeStartPixel - MouseTolerance - DragToleranceSlateUnits &&
		HitPixel <= RangeStartPixel + MouseTolerance + BrushSizeInStateUnits + DragToleranceSlateUnits;
}

bool FSequencerTimeSliderController::HitTestRangeEnd(const FScrubRangeToScreen& RangeToScreen, const TRange<double>& Range, float HitPixel) const
{
	static float BrushSizeInStateUnits = 6.f, DragToleranceSlateUnits = 2.f, MouseTolerance = 2.f;
	const float  RangeEndPixel = RangeToScreen.InputToLocalX(Range.GetUpperBoundValue());

	// Hit test against the brush region to the left of the playback end position, +/- DragToleranceSlateUnits
	return HitPixel >= RangeEndPixel - MouseTolerance - BrushSizeInStateUnits - DragToleranceSlateUnits &&
		HitPixel <= RangeEndPixel + MouseTolerance + DragToleranceSlateUnits;
}

bool FSequencerTimeSliderController::HitTestMark(const FGeometry& AllottedGeometry, const FScrubRangeToScreen& RangeToScreen, float HitPixel, bool bTestLabelBox, int32* OutMarkIndex, FFrameNumber* OutMarkFrameNumber) const
{
	if (OutMarkIndex)
	{
		*OutMarkIndex = INDEX_NONE;
	}
	if (OutMarkFrameNumber)
	{
		*OutMarkFrameNumber = 0;
	}

	if (!WeakSequencer.Pin()->GetSequencerSettings()->GetShowMarkedFrames())
	{
		return false;
	}

	const TArray<FMovieSceneMarkedFrame> & MarkedFrames = TimeSliderArgs.MarkedFrames.Get();
	if (MarkedFrames.Num() < 1)
	{
		return false;
	}

	static float BrushSizeInStateUnits = 3.f, DragToleranceSlateUnits = 2.f, MouseTolerance = 2.f;

	for (int32 MarkIndex = 0; MarkIndex < MarkedFrames.Num(); ++MarkIndex)
	{
		double Seconds = MarkedFrames[MarkIndex].FrameNumber / GetTickResolution();

		float MarkPixel = RangeToScreen.InputToLocalX(Seconds);

		// Hit test against the brush region to the left/right of the mark position, +/- DragToleranceSlateUnits
		if ((HitPixel >= MarkPixel - MouseTolerance - DragToleranceSlateUnits &&
			 HitPixel <= MarkPixel + MouseTolerance + BrushSizeInStateUnits + DragToleranceSlateUnits) ||
			(HitPixel >= MarkPixel - MouseTolerance - BrushSizeInStateUnits - DragToleranceSlateUnits &&
			 HitPixel <= MarkPixel + MouseTolerance + DragToleranceSlateUnits))
		{
			if (OutMarkIndex)
			{
				*OutMarkIndex = MarkIndex;
			}
			if (OutMarkFrameNumber)
			{
				*OutMarkFrameNumber = MarkedFrames[MarkIndex].FrameNumber;
			}
			return true;
		}

		// Hit test against the label box, which is the text size offset and grown by the margin.
		if (bTestLabelBox)
		{
			FVector2D TextPosition, TextSize;
			bool bDrawLeft;
			GetMarkLabelGeometry(AllottedGeometry, RangeToScreen, MarkedFrames[MarkIndex], TextPosition, TextSize, bDrawLeft);
			FVector2D TextBoxSize = TextSize + ScrubConstants::MarkLabelBoxMargin + ScrubConstants::MarkLabelBoxWideMargin;
			if (HitPixel >= MarkPixel && HitPixel <= MarkPixel + TextBoxSize.X)
			{
				if (OutMarkIndex)
				{
					*OutMarkIndex = MarkIndex;
				}
				if (OutMarkFrameNumber)
				{
					*OutMarkFrameNumber = MarkedFrames[MarkIndex].FrameNumber;
				}
				return true;
			}
		}
	}

	return false;
}

void FSequencerTimeSliderController::GetMarkLabelGeometry(const FGeometry& AllottedGeometry, const FScrubRangeToScreen& RangeToScreen, const FMovieSceneMarkedFrame& MarkedFrame, FVector2D& OutPosition, FVector2D& OutSize, bool& bIsDrawLeft) const
{
	double Seconds = MarkedFrame.FrameNumber / GetTickResolution();
	float MarkPixel = RangeToScreen.InputToLocalX(Seconds);

	const FString& LabelString = MarkedFrame.Label;
	FVector2D TextSize = FontMeasureService->Measure(LabelString, SmallLayoutFont);

	// Flip the text position if getting near the end of the view range
	bool bDrawLeft = (AllottedGeometry.Size.X - MarkPixel) < (TextSize.X + ScrubConstants::MarkLabelBoxMargin + ScrubConstants::MarkLabelBoxWideMargin);
	float TextPosition = bDrawLeft ? 
		MarkPixel - TextSize.X - ScrubConstants::MarkLabelBoxMargin : 
		MarkPixel + ScrubConstants::MarkLabelBoxMargin;

	OutPosition = FVector2D(TextPosition, 0);
	OutSize = TextSize;
	bIsDrawLeft = bDrawLeft;
}

FFrameTime FSequencerTimeSliderController::SnapTimeToNearestKey(const FPointerEvent& MouseEvent, const FScrubRangeToScreen& RangeToScreen, float CursorPos, FFrameTime InTime) const
{
	using namespace UE::Sequencer;

	if (!WeakSequencer.IsValid())
	{
		return InTime;
	}

	if (TimeSliderArgs.OnGetNearestKey.IsBound())
	{
		ENearestKeyOption NearestKeyOption = ENearestKeyOption::NKO_None;

		if (WeakSequencer.Pin()->GetSequencerSettings()->GetSnapPlayTimeToKeys() || MouseEvent.IsShiftDown())
		{
			EnumAddFlags(NearestKeyOption, ENearestKeyOption::NKO_SearchKeys);
		}

		if (WeakSequencer.Pin()->GetSequencerSettings()->GetSnapPlayTimeToSections() || MouseEvent.IsShiftDown())
		{
			EnumAddFlags(NearestKeyOption, ENearestKeyOption::NKO_SearchSections);
		}

		if (WeakSequencer.Pin()->GetSequencerSettings()->GetSnapPlayTimeToMarkers() || MouseEvent.IsShiftDown())
		{
			EnumAddFlags(NearestKeyOption, ENearestKeyOption::NKO_SearchMarkers);
		}

		FFrameNumber NearestKey = TimeSliderArgs.OnGetNearestKey.Execute(InTime, NearestKeyOption);

		float LocalKeyPos = RangeToScreen.InputToLocalX( NearestKey / GetTickResolution() );
		static float MouseTolerance = 20.f;

		if (FMath::IsNearlyEqual(LocalKeyPos, CursorPos, MouseTolerance))
		{
			return NearestKey;
		}
	}

	return InTime;
}

void FSequencerTimeSliderController::SetPlaybackRangeStart(FFrameNumber NewStart)
{
	TRange<FFrameNumber> PlaybackRange = TimeSliderArgs.PlaybackRange.Get();

	NewStart = SnapSequencerTime(NewStart).FrameNumber;

	if (NewStart <= UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange))
	{
		TimeSliderArgs.OnPlaybackRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(NewStart, PlaybackRange.GetUpperBound()));
	}
}

void FSequencerTimeSliderController::SetPlaybackRangeEnd(FFrameNumber NewEnd)
{
	TRange<FFrameNumber> PlaybackRange = TimeSliderArgs.PlaybackRange.Get();

	NewEnd = SnapSequencerTime(NewEnd).FrameNumber;

	if (NewEnd >= UE::MovieScene::DiscreteInclusiveLower(PlaybackRange))
	{
		TimeSliderArgs.OnPlaybackRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(PlaybackRange.GetLowerBound(), TRangeBound<FFrameNumber>::Exclusive(NewEnd)));
	}
}

void FSequencerTimeSliderController::SetSelectionRangeStart(FFrameNumber NewStart)
{
	TRange<FFrameNumber> SelectionRange = TimeSliderArgs.SelectionRange.Get();

	NewStart = SnapSequencerTime(NewStart).FrameNumber;

	if (SelectionRange.IsEmpty())
	{
		TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(NewStart, NewStart + 1));
	}
	else if (NewStart <= UE::MovieScene::DiscreteExclusiveUpper(SelectionRange))
	{
		TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(NewStart, SelectionRange.GetUpperBound()));
	}
}

void FSequencerTimeSliderController::SetSelectionRangeEnd(FFrameNumber NewEnd)
{
	TRange<FFrameNumber> SelectionRange = TimeSliderArgs.SelectionRange.Get();

	NewEnd = SnapSequencerTime(NewEnd).FrameNumber;

	if (SelectionRange.IsEmpty())
	{
		TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(NewEnd - 1, NewEnd));
	}
	else if (NewEnd >= UE::MovieScene::DiscreteInclusiveLower(SelectionRange))
	{
		TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(SelectionRange.GetLowerBound(), TRangeBound<FFrameNumber>::Exclusive(NewEnd)));
	}
}

void FSequencerTimeSliderController::HandleDragMark_NoDragType(const int32 InMarkIndex)
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer->GetSequencerSettings()->GetShowMarkedFrames())
	{
		return;
	}

	FSequencerSelection& SequencerSelection = Sequencer->GetSelection();

	const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
	const bool bToggleSelection = ModifierKeys.AreModifersDown(EModifierKey::Control)
		|| ModifierKeys.AreModifersDown(EModifierKey::Shift);

	if (!bToggleSelection)
	{
		if (!SequencerSelection.MarkedFrames.IsSelected(InMarkIndex))
		{
			SequencerSelection.Empty();
			SequencerSelection.MarkedFrames.Select(InMarkIndex);
		}
	}
	else if (!SequencerSelection.MarkedFrames.IsSelected(InMarkIndex))
	{
		SequencerSelection.MarkedFrames.Select(InMarkIndex);
	}
	else
	{
		SequencerSelection.MarkedFrames.Deselect(InMarkIndex);
	}
}

void FSequencerTimeSliderController::HandleDragMark(const FFrameNumber& InLastDiffFrame)
{
	using namespace UE::Sequencer;

	if (InLastDiffFrame == 0)
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	FSequencerSelection& SequencerSelection = Sequencer->GetSelection();
	const TArray<FMovieSceneMarkedFrame>& MarkedFrames = TimeSliderArgs.MarkedFrames.Get();

	for (int32 MarkIndex = 0; MarkIndex < MarkedFrames.Num(); ++MarkIndex)
	{
		if (SequencerSelection.MarkedFrames.IsSelected(MarkIndex))
		{
			SetMarkAtFrame(MarkIndex, MarkedFrames[MarkIndex].FrameNumber + InLastDiffFrame);
		}
	}

	TSet<FSequencerSelectedKey> SelectedKeys;
	FSequencerSelectedKey::AppendKeySelection(SelectedKeys, SequencerSelection.KeySelection);
	if (!SelectedKeys.IsEmpty())
	{
		KeyHelperUtils::TransformKeySelection(*Sequencer, SelectedKeys, InLastDiffFrame, 1.f, false);
	}
}

void FSequencerTimeSliderController::ClearMarkSelection()
{
	if (const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin())
	{
		Sequencer->GetSelection().MarkedFrames.Empty();
	}
}

TOptional<FFrameNumber> FSequencerTimeSliderController::GetMarkFrameNumber(const int32 InMarkIndex) const
{
	const TArray<FMovieSceneMarkedFrame>& MarkedFrames = TimeSliderArgs.MarkedFrames.Get();
	return MarkedFrames.IsValidIndex(InMarkIndex)
		? TOptional<FFrameNumber>(MarkedFrames[InMarkIndex].FrameNumber)
		: TOptional<FFrameNumber>();
}

void FSequencerTimeSliderController::AddMarkAtFrame(const FFrameNumber InFrameNumber)
{
	TimeSliderArgs.OnAddMarkedFrame.ExecuteIfBound(InFrameNumber);
}

void FSequencerTimeSliderController::SetMarkAtFrame(const int32 InMarkIndex, const FFrameNumber InFrameNumber)
{
	TimeSliderArgs.OnSetMarkedFrame.ExecuteIfBound(InMarkIndex, InFrameNumber);
}

void FSequencerTimeSliderController::DeleteSelectedMarks()
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UMovieSceneSequence* FocusedMovieSequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!FocusedMovieSequence)
	{
		return;
	}

	UMovieScene* FocusedMovieScene = FocusedMovieSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	FMarkedFrameSelection& MarkedFrameSelection = Sequencer->GetSelection().MarkedFrames;
	if (MarkedFrameSelection.Num() == 0)
	{
		return;
	}

	// Sort descending so deleting at higher indices doesn't shift the lower ones.
	TArray<int32> SortedIndices = MarkedFrameSelection.GetSelected().Array();
	SortedIndices.Sort([](int32 A, int32 B) { return A > B; });

	FScopedTransaction DeleteSelectedMarkedFramesTransaction(NSLOCTEXT("FSequencerTimeSliderController", "DeleteSelectedMarkedFrames_Transaction", "Delete Selected Marked Frames"));

	FocusedMovieScene->Modify();
	for (int32 Index : SortedIndices)
	{
		FocusedMovieScene->DeleteMarkedFrame(Index);
	}

	MarkedFrameSelection.Empty();

	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

void FSequencerTimeSliderController::DeleteAllMarks()
{
	TimeSliderArgs.OnDeleteAllMarkedFrames.ExecuteIfBound();
}

TSharedPtr<ISequencer> FSequencerTimeSliderController::GetSequencer() const
{
	return WeakSequencer.Pin();
}

void FSequencerTimeSliderController::ResetMouseInput()
{
	bPanning = false;
	bMouseDownInRegion = false;
	MouseDownPosition.Reset();
	LastMousePosition.Reset();
}

#undef LOCTEXT_NAMESPACE
