// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerUtilities.h"
#include "ActorForWorldTransforms.h"
#include "AnimatedRange.h"
#include "BakingAnimationKeySettings.h"
#include "CineCameraActor.h"
#include "CameraRig_Rail.h"
#include "CameraRig_Crane.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/SplineComponent.h"
#include "Containers/ArrayBuilder.h"
#include "Editor/EditorEngine.h"
#include "Engine/Selection.h"
#include "EngineUtils.h"
#include "Exporters/Exporter.h"
#include "Factories.h"
#include "Misc/Attribute.h"
#include "Misc/Paths.h"
#include "Layout/Margin.h"
#include "LevelEditorViewport.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SButton.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneCopyableBinding.h"
#include "MovieSceneCopyableTrack.h"
#include "EntitySystem/MovieSceneBlenderSystem.h"
#include "EntitySystem/IMovieSceneBlenderSystemSupport.h"
#include "MovieSceneFolder.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneSection.h"
#include "MovieSceneSpawnRegister.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneToolHelpers.h"
#include "MovieSceneTrack.h"
#include "Sections/MovieScene3DConstraintSection.h"
#include "Tracks/MovieScene3DAttachTrack.h"
#include "Tracks/MovieScene3DConstraintTrack.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "Tracks/MovieSceneCameraShakeTrack.h"
#include "Tracks/MovieSceneCommonAnimationTrack.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/SlateIconFinder.h"
#include "ISequencerTrackEditor.h"
#include "ISequencer.h"
#include "Sequencer.h"
#include "SequencerLog.h"
#include "SequencerNodeTree.h"
#include "MovieSceneBindingProxy.h"
#include "ScopedTransaction.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "UnrealEdGlobals.h"
#include "UnrealExporter.h"
#include "UObject/Package.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ISequencerObjectSchema.h"
#include "FileHelpers.h"
#include "HAL/PlatformApplicationMisc.h"
#include "LevelSequence.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/Views/ViewUtilities.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Bindings/MovieSceneCustomBinding.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "Bindings/MovieSceneReplaceableBinding.h"
#include "Bindings/MovieSceneSpawnableActorBinding.h"
#include "Bindings/MovieSceneReplaceableActorBinding.h"
#include "ActorFactories/ActorFactory.h"
#include "Tracks/MovieSceneBindingLifetimeTrack.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "ClassIconFinder.h"
#include "Styling/SlateIconFinder.h"
#include "Framework/Application/SlateApplication.h"
#include "SequencerCommands.h"
#include "MVVM/Selection/Selection.h"
#include "UnrealEdGlobals.h"
#include "Misc/FeedbackContext.h"
#include "Variants/MovieSceneTimeWarpVariant.h"
#include "Variants/MovieSceneTimeWarpGetter.h"
#include "UObject/UObjectIterator.h"
#include "ObjectTools.h"
#include "MVVM/Extensions/ISectionOwnerExtension.h"
#include "Decorations/IMovieSceneSectionProviderDecoration.h"
#include "Decorations/MovieSceneDecorationContainer.h"
#include "IMovieSceneModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequencerUtilities)

#define LOCTEXT_NAMESPACE "FSequencerUtilities"

void FMovieScenePasteSectionsParams::AddTarget(
	UMovieSceneTrack* Track,
	int32 RowIndex,
	UMovieSceneDecorationContainerObject* DecorationContainer,
	UObject* PreferredDecoration)
{
	Tracks.Add(Track);
	TrackRowIndices.Add(RowIndex);

	if (DecorationContainer || PreferredDecoration)
	{
		DecorationTargets.Add({DecorationContainer, PreferredDecoration});
	}
}

static void ResetCopiedTracksFlags(UMovieSceneTrack* Track)
{
	Track->ClearFlags(RF_Transient);

	ForEachObjectWithOuter(Track, [](UObject* InObject) {

		if (!InObject->GetClass() || !InObject->GetClass()->HasAnyClassFlags(CLASS_Transient))
		{
			InObject->ClearFlags(RF_Transient);
		}
	});

	for (UMovieSceneSection* Section : Track->GetAllSections())
	{
		Section->PostPaste();
	}
}

namespace UE::Sequencer
{
	namespace Private
	{
		struct FCreateBindingParentOverrideKey
		{
			const ISequencer* Sequencer = nullptr;
			const UObject* Object = nullptr;

			FCreateBindingParentOverrideKey(const ISequencer& InSequencer, const UObject& InObject)
				: Sequencer(&InSequencer)
				, Object(&InObject)
			{}

			friend bool operator==(const FCreateBindingParentOverrideKey& InA, const FCreateBindingParentOverrideKey& InB)
			{
				return InA.Sequencer == InB.Sequencer && InA.Object == InB.Object;
			}

			friend uint32 GetTypeHash(const FCreateBindingParentOverrideKey& InKey)
			{
				return HashCombine(GetTypeHash(InKey.Sequencer), GetTypeHash(InKey.Object));
			}
		};

		static TMap<FCreateBindingParentOverrideKey, FGuid>& GetCreateBindingParentOverrides()
		{
			static TMap<FCreateBindingParentOverrideKey, FGuid> ParentOverrides;
			return ParentOverrides;
		}

		SEQUENCER_API void SetCreateBindingParentOverride(const ISequencer& InSequencer, const UObject& InObject, const FGuid& InParentBinding)
		{
			if (InParentBinding.IsValid())
			{
				GetCreateBindingParentOverrides().Add(FCreateBindingParentOverrideKey(InSequencer, InObject), InParentBinding);
			}
		}

		FGuid TakeCreateBindingParentOverride(const ISequencer& InSequencer, const UObject& InObject)
		{
			const FCreateBindingParentOverrideKey Key(InSequencer, InObject);
			FGuid OutParentBinding;
			if (FGuid* ParentBinding = GetCreateBindingParentOverrides().Find(Key))
			{
				OutParentBinding = *ParentBinding;
				GetCreateBindingParentOverrides().Remove(Key);
			}
			return OutParentBinding;
		}
	}

	bool IteratePathToObject(UObject* CurrentObject, TFunctionRef<void(UObject*, const IObjectSchema&)> InCallback)
	{
		static ISequencerModule& Module = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
		if (!CurrentObject)
		{
			return false;
		}

		TSharedPtr<IObjectSchema> Schema = Module.FindObjectSchema(CurrentObject);
		if (Schema)
		{
			CurrentObject = Schema->GetRedirectedBoundObject(CurrentObject);

			// Add the parent first
			UObject* ParentObject = Schema->GetParentObject(CurrentObject);
			if (ParentObject && !IteratePathToObject(ParentObject, InCallback))
			{
				return false;
			}

			InCallback(CurrentObject, *Schema);
			return true;
		}
		return false;
	}


	bool GetPathToObject(UObject* InObject, TArray<UObject*>& OutObjectPath)
	{
		return IteratePathToObject(InObject, [&](UObject* CurrentObject, const IObjectSchema& Schema)
		{
			OutObjectPath.Add(CurrentObject);
		});
	}

} // namespace UE::Sequencer

TSharedRef<SWidget> FSequencerUtilities::MakeAddButton(FText HoverText, FOnGetContent MenuContent, const TAttribute<bool>& HoverState, TWeakPtr<ISequencer> InSequencer)
{
	TAttribute<bool> IsEnabled = MakeAttributeLambda([InSequencer]() -> bool { return InSequencer.IsValid() ? !InSequencer.Pin()->IsReadOnly() : false; });
	return UE::Sequencer::MakeAddButton(HoverText, MenuContent, HoverState, IsEnabled);
}

TSharedRef<SWidget> FSequencerUtilities::MakeAddButton(FText HoverText, FOnClicked OnClicked, const TAttribute<bool>& HoverState, TWeakPtr<ISequencer> InSequencer)
{
	TAttribute<bool> IsEnabled = MakeAttributeLambda([InSequencer]() -> bool { return InSequencer.IsValid() ? !InSequencer.Pin()->IsReadOnly() : false; });
	return UE::Sequencer::MakeAddButton(HoverText, OnClicked, HoverState, IsEnabled);
}

void FSequencerUtilities::MakeTimeWarpMenuEntry(FMenuBuilder& MenuBuilder, UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::FViewModel> WeakViewModel)
{
	using namespace UE::Sequencer;

	TViewModelPtr<ISectionOwnerExtension> SectionOwner = WeakViewModel.ImplicitPin();
	if (!SectionOwner)
	{
		return;
	}

	TOptional<UClass*> CommonClass;
	for (const TViewModelPtr<FSectionModel>& SectionModel : SectionOwner->GetSectionModels().IterateSubList<FSectionModel>())
	{
		UMovieSceneSection* Section = SectionModel->GetSection();
		if (!Section)
		{
			continue;
		}

		FMovieSceneTimeWarpVariant* Variant = Section->GetTimeWarp();
		UMovieSceneTimeWarpGetter*  Getter  = Variant && Variant->GetType() == EMovieSceneTimeWarpType::Custom 
			? Variant->AsCustom()
			: nullptr;

		if (Getter)
		{
			if (!CommonClass)
			{
				CommonClass = Getter->GetClass();
			}
			else if (CommonClass.GetValue() != Getter->GetClass())
			{
				CommonClass = nullptr;
			}
		}
	}

	FText TimeWarpLabel = CommonClass.IsSet()
		? LOCTEXT("ReplaceTimeWarp_Label", "Replace Time Warp")
		: LOCTEXT("AddTimeWarp_Label", "Add Time Warp");
	FText TimeWarpToolTip = CommonClass.IsSet()
		? LOCTEXT("ReplaceTimeWarp_ToolTip", "Replaces the Time Warp implementation with a different kind")
		: LOCTEXT("AddTimeWarp_ToolTip", "Add Time Warp");

	MenuBuilder.AddSubMenu(
		TimeWarpLabel,
		TimeWarpToolTip,
		FNewMenuDelegate::CreateStatic(PopulateTimeWarpChannelSubMenu, SectionOwner.AsWeak())
	);
}

void FSequencerUtilities::PopulateTimeWarpSubMenu(FMenuBuilder& MenuBuilder, TFunction<void(TSubclassOf<UMovieSceneTimeWarpGetter>)> OnTimeWarpPicked)
{
	using namespace UE::Sequencer;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	TSet<FTopLevelAssetPath> AllTimeWarpClasses;
	{
		FTopLevelAssetPath TargetClassPath(UMovieSceneTimeWarpGetter::StaticClass());
		AssetRegistryModule.Get().GetDerivedClassNames({ TargetClassPath }, TSet<FTopLevelAssetPath>(), AllTimeWarpClasses);
		AllTimeWarpClasses.Remove(TargetClassPath);
	}

	if (AllTimeWarpClasses.Num() == 0)
	{
		MenuBuilder.AddWidget(SNew(STextBlock).Text(LOCTEXT("NoTimeWarpTypesError", "No Time Warp implementations found")), FText(), true);
		return;
	}

	auto HandleTimeWarpSelection = [OnTimeWarpPicked](FTopLevelAssetPath ClassPath)
	{
		UClass* Class = FSoftClassPath(ClassPath.ToString()).TryLoadClass<UMovieSceneTimeWarpGetter>();
		if (Class)
		{
			OnTimeWarpPicked(Class);
		}
	};

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("TimeWarpCategoryLabel", "Time Warp Types:"));

	for (const FTopLevelAssetPath& ClassPath : AllTimeWarpClasses)
	{
		FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ClassPath.ToString()));

		const UClass* IconClass = FClassIconFinder::GetIconClassForAssetData(AssetData);
		const UClass* Class     = Cast<const UClass>(AssetData.FastGetAsset());

		if (!Class->HasMetaData("Hidden"))
		{
			MenuBuilder.AddMenuEntry(
				Class ? Class->GetDisplayNameText() : FText::FromName(ClassPath.GetAssetName()),
				Class ? Class->GetToolTipText()     : FText(),
				FSlateIconFinder::FindIconForClass(IconClass),
				FUIAction(FExecuteAction::CreateLambda(HandleTimeWarpSelection, ClassPath))
			);
		}
	}

	MenuBuilder.EndSection();
}

void FSequencerUtilities::PopulateTimeWarpChannelSubMenu(FMenuBuilder& MenuBuilder, UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::ITrackExtension> WeakTrackModel)
{
	using namespace UE::Sequencer;
	TViewModelPtr<ISectionOwnerExtension> SectionOwner = WeakTrackModel.ImplicitPin();
	return PopulateTimeWarpChannelSubMenu(MenuBuilder, SectionOwner.AsWeak());
}

void FSequencerUtilities::PopulateTimeWarpChannelSubMenu(FMenuBuilder& MenuBuilder, UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::ISectionOwnerExtension> WeakSectionOwner)
{
	using namespace UE::Sequencer;

	auto HandleTimeWarpSelection = [WeakSectionOwner](TSubclassOf<UMovieSceneTimeWarpGetter> Class)
	{
		TViewModelPtr<ISectionOwnerExtension> SectionOwner = WeakSectionOwner.Pin();
		if (!SectionOwner)
		{
			return;
		}

		// Snapshot the section list: the mutation loop below may re-enter the section owner's
		// layout and re-parent these models, which must not happen while iterating them.
		TArray<TSharedPtr<FSectionModel>> SectionsToModify;
		for (const TViewModelPtr<FSectionModel>& SectionModel : SectionOwner->GetSectionModels().IterateSubList<FSectionModel>())
		{
			SectionsToModify.Add(SectionModel);
		}

		FScopedTransaction Transaction(LOCTEXT("ChangeTimeWarpType", "Changed Time Warp type"));

		for (const TSharedPtr<FSectionModel>& SectionModel : SectionsToModify)
		{
			UMovieSceneSection*         Section = SectionModel ? SectionModel->GetSection() : nullptr;
			FMovieSceneTimeWarpVariant* Variant = Section ? Section->GetTimeWarp() : nullptr;

			if (Variant)
			{
				Section->Modify();

				UObject* GetterOuter = Section->GetTypedOuter<UMovieSceneSequence>();
				UMovieSceneTimeWarpGetter* Getter = NewObject<UMovieSceneTimeWarpGetter>(GetterOuter ? GetterOuter : Section, Class.Get(), NAME_None, RF_Transactional);
				Getter->InitializeDefaults(Section);

				Variant->Set(Getter);

				Section->InvalidateChannelProxy();

				TViewModelPtr<IOutlinerExtension> Outliner = SectionOwner.ImplicitCast();
				if (Outliner && !Outliner->IsExpanded())
				{
					Outliner->SetExpansion(true);
				}
			}
		}
	};

	PopulateTimeWarpSubMenu(MenuBuilder, HandleTimeWarpSelection);
}

void FSequencerUtilities::CreateNewSection(UMovieSceneTrack* InTrack, TWeakPtr<ISequencer> InSequencer, int32 InRowIndex, EMovieSceneBlendType InBlendType)
{
	TSharedPtr<ISequencer> Sequencer = InSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddSectionTransactionText", "Add Section"));
	if (UMovieSceneSection* NewSection = InTrack->CreateNewSection())
	{
		int32 OverlapPriority = 0;
		for (UMovieSceneSection* Section : InTrack->GetAllSections())
		{
			OverlapPriority = FMath::Max(Section->GetOverlapPriority() + 1, OverlapPriority);

			// Move existing sections on the same row or beyond so that they don't overlap with the new section
			if (Section != NewSection && Section->GetRowIndex() >= InRowIndex)
			{
				Section->SetRowIndex(Section->GetRowIndex() + 1);
			}
		}

		InTrack->Modify();

		if (Sequencer->GetInfiniteKeyAreas())
		{
			NewSection->SetRange(TRange<FFrameNumber>::All());
		}

		NewSection->SetOverlapPriority(OverlapPriority);
		NewSection->SetRowIndex(InRowIndex);
		NewSection->SetBlendType(InBlendType);

		InTrack->AddSection(*NewSection);
		InTrack->UpdateEasing();

		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
		Sequencer->EmptySelection();
		Sequencer->SelectSection(NewSection);
		Sequencer->ThrobSectionSelection();
	}
	else
	{
		Transaction.Cancel();
	}
}

void FSequencerUtilities::PopulateMenu_CreateNewSection(FMenuBuilder& MenuBuilder, int32 RowIndex, UMovieSceneTrack* Track, TWeakPtr<ISequencer> InSequencer)
{
	if (!Track)
	{
		return;
	}
	
	auto CreateNewSection = [Track, InSequencer, RowIndex](EMovieSceneBlendType BlendType)
	{
		TSharedPtr<ISequencer> Sequencer = InSequencer.IsValid() ? InSequencer.Pin() : nullptr;
		if (!Sequencer)
		{
			return;
		}

		FQualifiedFrameTime CurrentTime = Sequencer->GetLocalTime();
		FFrameNumber PlaybackEnd = UE::MovieScene::DiscreteExclusiveUpper(Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange());

		int32 SpecifiedRowIndex = RowIndex;

		FScopedTransaction Transaction(LOCTEXT("AddSectionTransactionText", "Add Section"));
		if (UMovieSceneSection* NewSection = Track->CreateNewSection())
		{
			int32 OverlapPriority = 0;
			TMap<int32, int32> NewToOldRowIndices;
			//if creating with an override force the row index to be last
			if (Track->GetSupportedBlendTypes().Contains(EMovieSceneBlendType::Override))
			{
				SpecifiedRowIndex = Track->GetMaxRowIndex() + 1;
			}
			for (UMovieSceneSection* Section : Track->GetAllSections())
			{
				OverlapPriority = FMath::Max(Section->GetOverlapPriority() + 1, OverlapPriority);				

				// Move existing sections on the same row or beyond so that they don't overlap with the new section
				if (Section != NewSection && Section->GetRowIndex() >= SpecifiedRowIndex)
				{
					int32 OldRowIndex = Section->GetRowIndex();
					int32 NewRowIndex = Section->GetRowIndex() + 1;
					NewToOldRowIndices.FindOrAdd(NewRowIndex, OldRowIndex);
					Section->Modify();
					Section->SetRowIndex(NewRowIndex);
				}
			}

			Track->Modify();

			Track->OnRowIndicesChanged(NewToOldRowIndices);

			if (Sequencer->GetInfiniteKeyAreas() && NewSection->GetSupportsInfiniteRange())
			{
				NewSection->SetRange(TRange<FFrameNumber>::All());
			}
			else
			{
				FFrameNumber NewSectionRangeEnd = PlaybackEnd;
				if (PlaybackEnd <= CurrentTime.Time.FrameNumber)
				{
					const FAnimatedRange ViewRange = Sequencer->GetViewRange();
					const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
					NewSectionRangeEnd = (ViewRange.GetUpperBoundValue() * TickResolution).FloorToFrame();
				}

				NewSection->SetRange(TRange<FFrameNumber>(CurrentTime.Time.FrameNumber, NewSectionRangeEnd));
			}

			NewSection->SetOverlapPriority(OverlapPriority);
			NewSection->SetRowIndex(SpecifiedRowIndex);
			NewSection->SetBlendType(BlendType);

			Track->AddSection(*NewSection);
			Track->UpdateEasing();

			if (UMovieSceneNameableTrack* NameableTrack = Cast<UMovieSceneNameableTrack>(Track))
			{
				NameableTrack->SetTrackRowDisplayName(FText::GetEmpty(), SpecifiedRowIndex);
			}

			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
		}
		else
		{
			Transaction.Cancel();
		}
	};

	FText NameOverride		= Track->GetSupportedBlendTypes().Num() == 1 ? LOCTEXT("AddSectionText", "Add New Section") : FText();
	FText TooltipOverride	= Track->GetSupportedBlendTypes().Num() == 1 ? LOCTEXT("AddSectionToolTip", "Adds a new section") : FText();

	const UEnum* MovieSceneBlendType = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/MovieScene.EMovieSceneBlendType"));
	for (EMovieSceneBlendType BlendType : Track->GetSupportedBlendTypes())
	{
		FText DisplayName = MovieSceneBlendType->GetDisplayNameTextByValue((int64)BlendType);
		FName EnumValueName = MovieSceneBlendType->GetNameByValue((int64)BlendType);
		MenuBuilder.AddMenuEntry(
			NameOverride.IsEmpty() ? DisplayName : NameOverride,
			TooltipOverride.IsEmpty() ? FText::Format(LOCTEXT("AddSectionFormatToolTip", "Adds a new {0} section"), DisplayName) : TooltipOverride,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), EnumValueName),
			FUIAction(
				FExecuteAction::CreateLambda(CreateNewSection, BlendType),
				FCanExecuteAction::CreateLambda([InSequencer] { return InSequencer.IsValid() && !InSequencer.Pin()->IsReadOnly(); })
			)
		);
	}
}

void FSequencerUtilities::PopulateMenu_BlenderSubMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track, TWeakPtr<ISequencer> InSequencer)
{
	IMovieSceneBlenderSystemSupport* BlenderSystemSupport = Cast<IMovieSceneBlenderSystemSupport>(Track);
	// Shouldn't have been called with a track that does not implement this interface
	check(BlenderSystemSupport);

	TArray<TSubclassOf<UMovieSceneBlenderSystem>> BlenderTypes;
	BlenderSystemSupport->GetSupportedBlenderSystems(BlenderTypes);

	// Ensure no nulls
	BlenderTypes.Remove(TSubclassOf<UMovieSceneBlenderSystem>());

	// Sort alphabetically
	Algo::Sort(BlenderTypes,
		[](TSubclassOf<UMovieSceneBlenderSystem> A, TSubclassOf<UMovieSceneBlenderSystem> B)
		{
			return A->GetDisplayNameText().CompareTo(B->GetDisplayNameText()) < 0;
		}
	);

	MenuBuilder.BeginSection(TEXT("Blending"), LOCTEXT("BlendingMenuSection", "Blending"));

	for (TSubclassOf<UMovieSceneBlenderSystem> SystemClass : BlenderTypes)
	{
		MenuBuilder.AddMenuEntry(
			SystemClass->GetDisplayNameText(),
			SystemClass->GetToolTipText(),
			FSlateIconFinder::FindIconForClass(SystemClass.Get()),
			FUIAction(
				FExecuteAction::CreateLambda([Track, BlenderSystemSupport, SystemClass]{
					FScopedTransaction Transaction(FText::Format(LOCTEXT("ChangeBlenderType", "Change blender to '{0}'"), SystemClass.Get()->GetDisplayNameText()));

					Track->Modify();
					BlenderSystemSupport->SetBlenderSystem(SystemClass);
				}),
				FCanExecuteAction::CreateLambda([InSequencer] { return InSequencer.IsValid() && !InSequencer.Pin()->IsReadOnly(); }),
				FIsActionChecked::CreateLambda([BlenderSystemSupport, SystemClass]
				{
					return BlenderSystemSupport->GetBlenderSystem() == SystemClass;
				})),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	
	MenuBuilder.EndSection();
}

void FSequencerUtilities::PopulateMenu_SetBlendType(FMenuBuilder& MenuBuilder, UMovieSceneSection* Section, TWeakPtr<ISequencer> InSequencer)
{
	PopulateMenu_SetBlendType(MenuBuilder, TArray<TWeakObjectPtr<UMovieSceneSection>>({ Section }), InSequencer);
}

void FSequencerUtilities::PopulateMenu_SetBlendType(FMenuBuilder& MenuBuilder, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InSections, TWeakPtr<ISequencer> InSequencer)
{
	using namespace UE::MovieScene;
	using namespace UE::Sequencer;

	auto Execute = [InSections, InSequencer](EMovieSceneBlendType BlendType)
	{
		FScopedTransaction Transaction(LOCTEXT("SetBlendType", "Set Blend Type"));
		for (TWeakObjectPtr<UMovieSceneSection> WeakSection : InSections)
		{
			if (UMovieSceneSection* Section = WeakSection.Get())
			{
				Section->Modify();
				Section->SetBlendType(BlendType);
			}
		}
			
		TSharedPtr<FSequencer> Sequencer = StaticCastSharedPtr<FSequencer>(InSequencer.Pin());
		if (Sequencer.IsValid())
		{
			// If the blend type is changed to additive or relative, restore the state of the objects boud to this section before evaluating again. 
			// This allows the additive or relative to evaluate based on the initial values of the object, rather than the current animated values.
			if (BlendType == EMovieSceneBlendType::Additive || BlendType == EMovieSceneBlendType::Relative)
			{
				TSet<UObject*> ObjectsToRestore;
				TSharedRef<FSequencerNodeTree> SequencerNodeTree = Sequencer->GetNodeTree();
				for (TWeakObjectPtr<UMovieSceneSection> WeakSection : InSections)
				{
					if (UMovieSceneSection* Section = WeakSection.Get())
					{
						TSharedPtr<FSectionModel> SectionHandle = SequencerNodeTree->GetSectionModel(Section);
						if (!SectionHandle)
						{
							continue;
						}

						TSharedPtr<IObjectBindingExtension> ParentObjectBindingNode = SectionHandle->FindAncestorOfType<IObjectBindingExtension>();
						if (!ParentObjectBindingNode.IsValid())
						{
							continue;
						}

						for (TWeakObjectPtr<> BoundObject : Sequencer->FindObjectsInCurrentSequence(ParentObjectBindingNode->GetObjectGuid()))
						{
							if (AActor* BoundActor = Cast<AActor>(BoundObject))
							{
								for (UActorComponent* Component : TInlineComponentArray<UActorComponent*>(BoundActor))
								{
									if (Component)
									{
										ObjectsToRestore.Add(Component);
									}
								}
							}

							ObjectsToRestore.Add(BoundObject.Get());
						}
					}
				}

				for (UObject* ObjectToRestore : ObjectsToRestore)
				{
					Sequencer->PreAnimatedState.RestorePreAnimatedState(*ObjectToRestore);
				}
			}

			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		}
	};

	const UEnum* MovieSceneBlendType = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/MovieScene.EMovieSceneBlendType"));
	for (int32 NameIndex = 0; NameIndex < MovieSceneBlendType->NumEnums() - 1; ++NameIndex)
	{
		EMovieSceneBlendType BlendType = (EMovieSceneBlendType)MovieSceneBlendType->GetValueByIndex(NameIndex);

		// Include this if any section supports it
		bool bAnySupported = false;
		for (TWeakObjectPtr<UMovieSceneSection> WeakSection : InSections)
		{
			UMovieSceneSection* Section = WeakSection.Get();
			if (Section && Section->GetSupportedBlendTypes().Contains(BlendType))
			{
				bAnySupported = true;
				break;
			}
		}

		if (!bAnySupported)
		{
			continue;
		}

		FName EnumValueName = MovieSceneBlendType->GetNameByIndex(NameIndex);
		MenuBuilder.AddMenuEntry(
			MovieSceneBlendType->GetDisplayNameTextByIndex(NameIndex),
			MovieSceneBlendType->GetToolTipTextByIndex(NameIndex),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), EnumValueName),
			FUIAction(
				FExecuteAction::CreateLambda(Execute, BlendType),
				FCanExecuteAction::CreateLambda([InSequencer] { return InSequencer.IsValid() && !InSequencer.Pin()->IsReadOnly(); }),
				FIsActionChecked::CreateLambda([InSections, BlendType]
				{
					int32 NumActiveBlendTypes = 0;
					for (TWeakObjectPtr<UMovieSceneSection> WeakSection : InSections)
					{
						UMovieSceneSection* Section = WeakSection.Get();
						if (Section && Section->GetBlendType() == BlendType)
						{
							++NumActiveBlendTypes;
						}
					}
					return NumActiveBlendTypes == InSections.Num();
				})),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
}

FName FSequencerUtilities::GetUniqueName( FName CandidateName, const TArray<FName>& ExistingNames )
{
	if (!ExistingNames.Contains(CandidateName))
	{
		return CandidateName;
	}

	FString CandidateNameString = CandidateName.ToString();
	FString BaseNameString = CandidateNameString;
	if ( CandidateNameString.Len() >= 3 && CandidateNameString.Right(3).IsNumeric() )
	{
		BaseNameString = CandidateNameString.Left( CandidateNameString.Len() - 3 );
	}

	FName UniqueName = FName(*BaseNameString);
	int32 NameIndex = 1;
	while ( ExistingNames.Contains( UniqueName ) )
	{
		UniqueName = FName( *FString::Printf(TEXT("%s%i"), *BaseNameString, NameIndex ) );
		NameIndex++;
	}

	return UniqueName;
}

TArray<FString> FSequencerUtilities::GetAssociatedLevelSequenceMapPackages(const ULevelSequence* InSequence)
{
	if (!InSequence)
	{
		return TArray<FString>();
	}

	const FName LSMapPathName = *InSequence->GetOutermost()->GetPathName();
	return GetAssociatedLevelSequenceMapPackages(LSMapPathName);
}

TArray<FString> FSequencerUtilities::GetAssociatedLevelSequenceMapPackages(FName LevelSequencePackageName)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	TArray<FString> AssociatedMaps;
	TArray<FAssetIdentifier> AssociatedAssets;

	// This makes the assumption these functions will append the array, and not clear it.
	AssetRegistryModule.Get().GetReferencers(LevelSequencePackageName, AssociatedAssets);
	AssetRegistryModule.Get().GetDependencies(LevelSequencePackageName, AssociatedAssets);

	for (FAssetIdentifier& AssociatedMap : AssociatedAssets)
	{
		FString MapFilePath;
		FString LevelPath = AssociatedMap.PackageName.ToString();
		if (FEditorFileUtils::IsMapPackageAsset(LevelPath, MapFilePath))
		{
			AssociatedMaps.AddUnique(LevelPath);
		}
	}

	AssociatedMaps.Sort([](const FString& One, const FString& Two) { return FPaths::GetBaseFilename(One) < FPaths::GetBaseFilename(Two); });
	return AssociatedMaps;
}

/** Recurses through a folder to replace converted GUID with new GUID */
bool UpdateFolderBindingID(UMovieSceneFolder* Folder, FGuid OldGuid, FGuid NewGuid)
{
	for (FGuid ChildGuid : Folder->GetChildObjectBindings())
	{
		if (ChildGuid == OldGuid)
		{
			Folder->AddChildObjectBinding(NewGuid);
			Folder->RemoveChildObjectBinding(OldGuid);
			return true;
		}
	}

	for (UMovieSceneFolder* ChildFolder : Folder->GetChildFolders())
	{
		if (UpdateFolderBindingID(ChildFolder, OldGuid, NewGuid))
		{
			return true;
		}
	}

	return false;
}

/** Expands Possessables with multiple bindings into individual Possessables for each binding */
TArray<FGuid> ExpandMultiplePossessableBindings(TSharedRef<ISequencer> Sequencer, FGuid PossessableGuid)
{
	TArray<FGuid> NewPossessableGuids;

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return NewPossessableGuids;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return NewPossessableGuids;
	}

	// Create a copy of the TArrayView of bound objects, as the underlying array will get destroyed
	TArray<TWeakObjectPtr<>> FoundObjects;
	for (TWeakObjectPtr<> BoundObject : Sequencer->FindBoundObjects(PossessableGuid, Sequencer->GetFocusedTemplateID()))
	{
		FoundObjects.Insert(BoundObject, 0);
	}

	if (FoundObjects.Num() < 2)
	{
		// If less than two objects, nothing to do, return the same Guid
		NewPossessableGuids.Add(PossessableGuid);
		return NewPossessableGuids;
	}

	Sequence->Modify();
	MovieScene->Modify();

	FMovieSceneBinding* PossessableBinding = MovieScene->FindBinding(PossessableGuid);

	// First gather the children
	TArray<FGuid> ChildPossessableGuids;
	for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
	{
		FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);
		if (Possessable.GetParent() == PossessableGuid)
		{
			ChildPossessableGuids.Add(Possessable.GetGuid());
		}
	}

	TArray<UMovieSceneTrack* > Tracks = PossessableBinding->StealTracks(MovieScene);

	// Remove binding to stop any children from claiming the old guid as their parent
	if (MovieScene->RemovePossessable(PossessableGuid))
	{
		Sequence->UnbindPossessableObjects(PossessableGuid);
	}

	for (TWeakObjectPtr<> FoundObjectPtr : FoundObjects)
	{
		UObject* FoundObject = FoundObjectPtr.Get();
		if (!FoundObject)
		{
			continue;
		}

		FoundObject->Modify();

		UObject* BindingContext = Sequencer->GetPlaybackContext();

		// Find this object's parent object, if it has one.
		UObject* ParentObject = Sequence->GetParentObject(FoundObject);
		if (ParentObject)
		{
			BindingContext = ParentObject;
		}

		// Create a new Possessable for this object
		AActor* PossessedActor = Cast<AActor>(FoundObject);
		const FGuid NewPossessableGuid = MovieScene->AddPossessable(PossessedActor != nullptr ? PossessedActor->GetActorLabel() : FoundObject->GetName(), FoundObject->GetClass());
		FMovieScenePossessable* NewPossessable = MovieScene->FindPossessable(NewPossessableGuid);
		if (NewPossessable)
		{
			FMovieSceneBinding* NewPossessableBinding = MovieScene->FindBinding(NewPossessableGuid);

			if (ParentObject)
			{
				FGuid ParentGuid = Sequencer->FindObjectId(*ParentObject, Sequencer->GetFocusedTemplateID());
				NewPossessable->SetParent(ParentGuid, MovieScene);
			}

			if (!NewPossessable->BindSpawnableObject(Sequencer->GetFocusedTemplateID(), FoundObject, Sequencer->GetSharedPlaybackState()))
			{
				Sequence->BindPossessableObject(NewPossessableGuid, *FoundObject, BindingContext);
				NewPossessable->FixupPossessedObjectClass(Sequence, BindingContext);
			}

			NewPossessableGuids.Add(NewPossessableGuid);

			// Create copies of the tracks
			for (UMovieSceneTrack* Track : Tracks)
			{
				UMovieSceneTrack* DuplicatedTrack = Cast<UMovieSceneTrack>(StaticDuplicateObject(Track, MovieScene));
				NewPossessableBinding->AddTrack(*DuplicatedTrack, MovieScene);
			}
		}
	}

	// Finally, recurse in to any children
	for (FGuid ChildPossessableGuid : ChildPossessableGuids)
	{
		ExpandMultiplePossessableBindings(Sequencer, ChildPossessableGuid);
	}

	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);

	return NewPossessableGuids;
}

void NewCameraAdded(TSharedRef<ISequencer> Sequencer, ACameraActor* NewCamera, FGuid CameraGuid)
{
	if (Sequencer->OnCameraAddedToSequencer().IsBound() && !Sequencer->OnCameraAddedToSequencer().Execute(NewCamera, CameraGuid))
	{
		return;
	}

	MovieSceneToolHelpers::LockCameraActorToViewport(Sequencer, NewCamera);

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (Sequence && Sequence->IsTrackSupported(UMovieSceneCameraCutTrack::StaticClass()) == ETrackSupport::Supported)
	{
		MovieSceneToolHelpers::CreateCameraCutSectionForCamera(Sequence->GetMovieScene(), CameraGuid, Sequencer->GetLocalTime().Time.FloorToFrame());
	}
}

FGuid AddSpawnable(TSharedRef<ISequencer> Sequencer, UObject& Object, UActorFactory* ActorFactory = nullptr, FName SpawnableName = NAME_None)
{
	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence->AllowsSpawnableObjects())
	{
		return FGuid();
	}

	// Grab the MovieScene that is currently focused.  We'll add our Blueprint as an inner of the
	// MovieScene asset.
	UMovieScene* OwnerMovieScene = Sequence->GetMovieScene();

	TValueOrError<FNewSpawnable, FText> Result = Sequencer->GetSpawnRegister().CreateNewSpawnableType(Object, *OwnerMovieScene, ActorFactory);
	if (!Result.IsValid())
	{
		FNotificationInfo Info(Result.GetError());
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
		return FGuid();
	}

	FNewSpawnable& NewSpawnable = Result.GetValue();

	if (SpawnableName == NAME_None)
	{
		NewSpawnable.Name = MovieSceneHelpers::MakeUniqueSpawnableName(OwnerMovieScene, NewSpawnable.Name);
	}
	else
	{
		NewSpawnable.Name = SpawnableName.ToString();
	}

	FGuid NewGuid = OwnerMovieScene->AddSpawnable(NewSpawnable.Name, *NewSpawnable.ObjectTemplate);

	Sequencer->ForceEvaluate();

	return NewGuid;
}

FGuid FSequencerUtilities::MakeNewSpawnable(TSharedRef<ISequencer> Sequencer, UObject& Object, UActorFactory* ActorFactory, bool bSetupDefaults, FName SpawnableName)
{
	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return FGuid();
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return FGuid();
	}

	if (MovieScene->IsReadOnly())
	{
		ShowReadOnlyError();
		return FGuid();
	}
	
	if (!Sequence->AllowsSpawnableObjects())
	{
		ShowSpawnableNotAllowedError();
		return FGuid();
	}

	FGuid NewGuid = AddSpawnable(Sequencer, Object, ActorFactory, SpawnableName);
	if (!NewGuid.IsValid())
	{
		return FGuid();
	}

	// Spawn the object so we can position it correctly, it's going to get spawned anyway since things default to spawned.
	UObject* SpawnedObject = Sequencer->GetSpawnRegister().SpawnObject(NewGuid, *MovieScene, Sequencer->GetFocusedTemplateID(), Sequencer.Get());

	if (bSetupDefaults)
	{
		FTransformData TransformData;
		Sequencer->GetSpawnRegister().SetupDefaultsForSpawnable(SpawnedObject, NewGuid, TransformData, Sequencer, Sequencer->GetSequencerSettings());
	}

	if (ACameraActor* NewCamera = Cast<ACameraActor>(SpawnedObject))
	{
		NewCameraAdded(Sequencer, NewCamera, NewGuid);
	}

	return NewGuid;
}

FGuid FSequencerUtilities::CreateCamera(TSharedRef<ISequencer> Sequencer, const bool bSpawnable, ACineCameraActor*& OutActor)
{
	FGuid CameraGuid;

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return CameraGuid;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return CameraGuid;
	}

	if (MovieScene->IsReadOnly())
	{
		ShowReadOnlyError();
		return CameraGuid;
	}

	UObject* PlaybackContext = Sequencer->GetPlaybackContext();
	UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
	if (!World)
	{
		return CameraGuid;
	}

	const FScopedTransaction Transaction(LOCTEXT("CreateCamera", "Create Camera"));

	FActorSpawnParameters SpawnParams;
	if (bSpawnable)
	{
		// Don't bother transacting this object if we're creating a spawnable since it's temporary
		SpawnParams.ObjectFlags &= ~RF_Transactional;
	}

	// Set new camera to match viewport
	OutActor = World->SpawnActor<ACineCameraActor>(SpawnParams);
	if (!OutActor)
	{
		return CameraGuid;
	}

	OutActor->SetActorLocation(GCurrentLevelEditingViewportClient->GetViewLocation(), false);
	OutActor->SetActorRotation(GCurrentLevelEditingViewportClient->GetViewRotation());
	//OutActor->CameraComponent->FieldOfView = ViewportClient->ViewFOV; //@todo set the focal length from this field of view

	FActorLabelUtilities::SetActorLabelUnique(OutActor, ACineCameraActor::StaticClass()->GetName());

	CameraGuid = CreateBinding(Sequencer, *OutActor);

	TSubclassOf<UMovieSceneCustomBinding> CustomBindingClass = bSpawnable ? UMovieSceneSpawnableActorBinding::StaticClass() : UMovieSceneReplaceableActorBinding::StaticClass();

	const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences();

	if (BindingReferences)
	{
		for (const FMovieSceneBindingReference& Reference : BindingReferences->GetReferences(CameraGuid))
		{
			for (const TSubclassOf<UMovieSceneCustomBinding>& SupportedCustomBindingType : Sequencer->GetSupportedCustomBindingTypes())
			{
				if (SupportedCustomBindingType && SupportedCustomBindingType->IsChildOf(CustomBindingClass) &&
					SupportedCustomBindingType->GetDefaultObject<UMovieSceneCustomBinding>()->SupportsConversionFromBinding(Reference, OutActor))
				{
					FMovieScenePossessable* NewPossessable = FSequencerUtilities::ConvertToCustomBinding(Sequencer->AsShared(), CameraGuid, CustomBindingClass);

					if (NewPossessable)
					{
						for (TWeakObjectPtr<> WeakObject : Sequencer->FindBoundObjects(NewPossessable->GetGuid(), Sequencer->GetFocusedTemplateID()))
						{
							ACineCameraActor* SpawnedActor = Cast<ACineCameraActor>(WeakObject.Get());
							if (SpawnedActor)
							{
								OutActor = SpawnedActor;
							}
						}

						CameraGuid = NewPossessable->GetGuid();
					}
					break;
				}
			}
		}
	}

	if (!CameraGuid.IsValid())
	{
		return CameraGuid;
	}

	NewCameraAdded(Sequencer, OutActor, CameraGuid);

	return CameraGuid;
}

FGuid FSequencerUtilities::CreateCameraWithRig(TSharedRef<ISequencer> Sequencer, AActor* Actor, const bool bSpawnable, ACineCameraActor*& OutActor)
{
	FGuid CameraGuid;

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return CameraGuid;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return CameraGuid;
	}

	if (MovieScene->IsReadOnly())
	{
		ShowReadOnlyError();
		return CameraGuid;
	}

	const FScopedTransaction Transaction(LOCTEXT("CreateCameraWithRig", "Create Camera with Rig"));

	ACameraRig_Rail* RailActor = nullptr;
	if (Actor->GetClass() == ACameraRig_Rail::StaticClass())
	{
		RailActor = Cast<ACameraRig_Rail>(Actor);
	}

	// Create a cine camera actor
	UObject* PlaybackContext = Sequencer->GetPlaybackContext();
	UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
	if (!World)
	{
		return CameraGuid;
	}

	OutActor = World->SpawnActor<ACineCameraActor>();

	FString NewCameraName = MovieSceneHelpers::MakeUniqueSpawnableName(MovieScene, FName::NameToDisplayString(ACineCameraActor::StaticClass()->GetFName().ToString(), false));
	UE::Sequencer::FCreateBindingParams CreateBindingParams;
	CreateBindingParams.BindingNameOverride = NewCameraName;
	CreateBindingParams.bSpawnable = bSpawnable;

	CameraGuid = CreateBinding(Sequencer, *OutActor, CreateBindingParams);

	if (RailActor)
	{
		OutActor->SetActorRotation(FRotator(0.f, -90.f, 0.f));
	}

	TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();

	if (bSpawnable)
	{
		for (TWeakObjectPtr<> WeakObject : Sequencer->FindBoundObjects(CameraGuid, Sequencer->GetFocusedTemplateID()))
		{
			OutActor = Cast<ACineCameraActor>(WeakObject.Get());
			if (OutActor)
			{
				break;
			}
		}

		OutActor->SetActorLabel(NewCameraName, false);

		// Create an attach track
		UMovieScene3DAttachTrack* AttachTrack = Cast<UMovieScene3DAttachTrack>(MovieScene->AddTrack(UMovieScene3DAttachTrack::StaticClass(), CameraGuid));

		FGuid NewGuid = Sequencer->FindObjectId(*Actor, Sequencer->GetFocusedTemplateID());
		FMovieSceneObjectBindingID AttachBindingID = UE::MovieScene::FRelativeObjectBindingID(NewGuid);
		FFrameNumber StartTime = UE::MovieScene::DiscreteInclusiveLower(PlaybackRange);
		FFrameNumber Duration = UE::MovieScene::DiscreteSize(PlaybackRange);

		AttachTrack->AddConstraint(StartTime, Duration.Value, NAME_None, NAME_None, AttachBindingID);
	}
	else
	{
		FActorLabelUtilities::SetActorLabelUnique(OutActor, ACineCameraActor::StaticClass()->GetName());

		// Parent it
		OutActor->AttachToActor(Actor, FAttachmentTransformRules::KeepRelativeTransform);
	}

	if (RailActor)
	{
		// Extend the rail a bit
		if (RailActor->GetRailSplineComponent()->GetNumberOfSplinePoints() == 2)
		{
			FVector SplinePoint1 = RailActor->GetRailSplineComponent()->GetLocationAtSplinePoint(0, ESplineCoordinateSpace::Local);
			FVector SplinePoint2 = RailActor->GetRailSplineComponent()->GetLocationAtSplinePoint(1, ESplineCoordinateSpace::Local);
			FVector SplineDirection = SplinePoint2 - SplinePoint1;
			SplineDirection.Normalize();

			float DefaultRailDistance = 650.f;
			SplinePoint2 = SplinePoint1 + SplineDirection * DefaultRailDistance;
			RailActor->GetRailSplineComponent()->SetLocationAtSplinePoint(1, SplinePoint2, ESplineCoordinateSpace::Local);
			RailActor->GetRailSplineComponent()->bSplineHasBeenEdited = true;
		}

		// Create a track for the CurrentPositionOnRail
		FPropertyPath PropertyPath;
		PropertyPath.AddProperty(FPropertyInfo(RailActor->GetClass()->FindPropertyByName(TEXT("CurrentPositionOnRail"))));

		FKeyPropertyParams KeyPropertyParams(TArrayBuilder<UObject*>().Add(RailActor), PropertyPath, ESequencerKeyMode::ManualKeyForced);

		FFrameTime OriginalTime = Sequencer->GetLocalTime().Time;

		Sequencer->SetLocalTimeDirectly(UE::MovieScene::DiscreteInclusiveLower(PlaybackRange));
		RailActor->CurrentPositionOnRail = 0.f;
		Sequencer->KeyProperty(KeyPropertyParams);

		Sequencer->SetLocalTimeDirectly(UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange) - 1);
		RailActor->CurrentPositionOnRail = 1.f;
		Sequencer->KeyProperty(KeyPropertyParams);

		Sequencer->SetLocalTimeDirectly(OriginalTime);
	}

	NewCameraAdded(Sequencer, OutActor, CameraGuid);

	return CameraGuid;
}

TArray<FGuid> FSequencerUtilities::AddActors(TSharedRef<ISequencer> Sequencer, const TArray<TWeakObjectPtr<AActor> >& InActors)
{
	TArray<FGuid> PossessableGuids;

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return PossessableGuids;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return PossessableGuids;
	}

	if (MovieScene->IsReadOnly())
	{
		ShowReadOnlyError();
		return PossessableGuids;
	}
	
	const FScopedTransaction Transaction(LOCTEXT("AddActors", "Add Actors"));
	Sequence->Modify();

	for (TWeakObjectPtr<AActor> WeakActor : InActors)
	{
		if (AActor* Actor = WeakActor.Get())
		{
			FGuid ExistingGuid = Sequencer->FindObjectId(*Actor, Sequencer->GetFocusedTemplateID());
			if (!ExistingGuid.IsValid())
			{
				FGuid PossessableGuid = CreateBinding(Sequencer, *Actor);
				PossessableGuids.Add(PossessableGuid);

				if (ACameraActor* CameraActor = Cast<ACameraActor>(Actor))
				{
					NewCameraAdded(Sequencer, CameraActor, PossessableGuid);
				}
			}
		}
	}

	return PossessableGuids;
}

TArray<FMovieSceneSpawnable*> FSequencerUtilities::ConvertToSpawnable(TSharedRef<ISequencer> Sequencer, FGuid PossessableGuid)
{
	TArray<FMovieSceneSpawnable*> CreatedSpawnables;

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return CreatedSpawnables;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return CreatedSpawnables;
	}

	if (MovieScene->IsReadOnly() || !Sequence->AllowsSpawnableObjects())
	{
		ShowReadOnlyError();
		return CreatedSpawnables;
	}

	TArrayView<TWeakObjectPtr<>> FoundObjects = Sequencer->FindBoundObjects(PossessableGuid, Sequencer->GetFocusedTemplateID());

	if (FoundObjects.Num() == 0)
	{
		FMovieScenePossessable* Possessable = MovieScene->FindPossessable(PossessableGuid);

		UE_LOGF(LogSequencer, Error, "Failed to convert %ls to spawnable because there are no objects bound to it", Possessable ? *Possessable->GetName() : TEXT(""));
	}
	else if (FoundObjects.Num() > 1)
	{
		// Expand to individual possessables for each bound object, then convert each one individually
		TArray<FGuid> ExpandedPossessableGuids = ExpandMultiplePossessableBindings(Sequencer, PossessableGuid);
		for (FGuid NewPossessableGuid : ExpandedPossessableGuids)
		{
			CreatedSpawnables.Append(ConvertToSpawnable(Sequencer, NewPossessableGuid));
		}

		Sequencer->ForceEvaluate();
	}
	else
	{
		UObject* FoundObject = FoundObjects[0].Get();
		if (!FoundObject)
		{
			return CreatedSpawnables;
		}

		Sequence->Modify();
		MovieScene->Modify();

		// Locate the folder containing the original possessable
		UMovieSceneFolder* ParentFolder = nullptr;
		for (UMovieSceneFolder* Folder : MovieScene->GetRootFolders())
		{
			ParentFolder = Folder->FindFolderContaining(PossessableGuid);
			if (ParentFolder != nullptr)
			{
				break;
			}
		}

		FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(AddSpawnable(Sequencer, *FoundObject));
		if (Spawnable)
		{
			FGuid SpawnableGuid = Spawnable->GetGuid();
			CreatedSpawnables.Add(Spawnable);

			// Remap all the spawnable's tracks and child bindings onto the new possessable
			MovieScene->MoveBindingContents(PossessableGuid, SpawnableGuid);

			FMovieSceneBinding* PossessableBinding = MovieScene->FindBinding(PossessableGuid);
			check(PossessableBinding);

			for (UMovieSceneFolder* Folder : MovieScene->GetRootFolders())
			{
				if (UpdateFolderBindingID(Folder, PossessableGuid, SpawnableGuid))
				{
					break;
				}
			}

			int32 SortingOrder = PossessableBinding->GetSortingOrder();

			if (MovieScene->RemovePossessable(PossessableGuid))
			{
				Sequence->UnbindPossessableObjects(PossessableGuid);

				FMovieSceneBinding* SpawnableBinding = MovieScene->FindBinding(SpawnableGuid);
				check(SpawnableBinding);

				SpawnableBinding->SetSortingOrder(SortingOrder);
			}

			TOptional<FTransformData> TransformData;
			Sequencer->GetSpawnRegister().HandleConvertPossessableToSpawnable(FoundObject, *Sequencer, TransformData);
			Sequencer->GetSpawnRegister().SetupDefaultsForSpawnable(nullptr, Spawnable->GetGuid(), TransformData, Sequencer, Sequencer->GetSequencerSettings());

			UpdateBindingIDs(Sequencer, PossessableGuid, Spawnable->GetGuid());

			Sequencer->ForceEvaluate();
		}
	}

	return CreatedSpawnables;
}

FMovieScenePossessable* FSequencerUtilities::ConvertToPossessable(TSharedRef<ISequencer> Sequencer, FGuid BindingGuid, int32 BindingIndex/*=0*/)
{
	FMovieScenePossessable* CreatedPossessable = nullptr;

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return CreatedPossessable;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return CreatedPossessable;
	}

	if (MovieScene->IsReadOnly())
	{
		ShowReadOnlyError();
		return CreatedPossessable;
	}

	const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences();

	FMovieScenePossessable* ExistingPossessable = MovieScene->FindPossessable(BindingGuid);
	if (ExistingPossessable && BindingReferences)
	{
		if (const FMovieSceneBindingReference* ExistingReference = BindingReferences->GetReference(BindingGuid, BindingIndex))
		{
			if (!ExistingReference->CustomBinding)
			{
				// Already a possessable, just return
				return ExistingPossessable;
			}
		}
	}

	// For now, just get the first bound object to convert
	TArray<UObject*> BoundObjects = MovieSceneHelpers::GetBoundObjects(Sequencer->GetFocusedTemplateID(), BindingGuid, Sequencer->GetSharedPlaybackState(), BindingIndex);
	UObject* ObjectToConvert = BoundObjects.Num() > 0 ? BoundObjects[0] : nullptr;

	// If we have an old-style spawnable, use the template as the object to convert instead.
	bool bConvertFromSpawnable = MovieSceneHelpers::IsBoundToSpawnable(Sequence, BindingGuid, Sequencer->GetSharedPlaybackState(), BindingIndex);
	if (bConvertFromSpawnable && MovieSceneHelpers::SupportsObjectTemplate(Sequence, BindingGuid, Sequencer->GetSharedPlaybackState(), BindingIndex))
	{
		ObjectToConvert = MovieSceneHelpers::GetObjectTemplate(Sequence, BindingGuid, Sequencer->GetSharedPlaybackState(), BindingIndex);
	}

	AActor* SpawnableActorTemplate = Cast<AActor>(ObjectToConvert);

	TMap<TWeakObjectPtr<AActor>, FTransform> AttachedChildTransforms;
	FTransform DefaultTransform = SpawnableActorTemplate ? SpawnableActorTemplate->GetActorTransform() : FTransform();
	// Prefer the transform at the current time over the spawnable actor template's transform because that's most likely 0. 
	// This makes it so that the object will return to the current position on restore state.
	AActor* Actor = Cast<AActor>(ObjectToConvert);
	if (Actor)
	{
		if (Actor->GetRootComponent())
		{
			DefaultTransform = Actor->GetRootComponent()->GetRelativeTransform();
		}

		// Removing a parent will compensate the children at their world transform. We don't want that since we'll be replacing that parent right away.
		// To negate that, we store the relative transform of these children and reset it after the parent is replaced with the new possessable.
		TArray<AActor*> AttachedActors;
		Actor->GetAttachedActors(AttachedActors);
		for (AActor* ChildActor : AttachedActors)
		{
			if (ChildActor && ChildActor->GetRootComponent())
			{
				// Only do this for child actors that Sequencer is controlling
				FGuid ExistingID = Sequencer->FindObjectId(*ChildActor, Sequencer->GetFocusedTemplateID());
				if (ExistingID.IsValid())
				{
					AttachedChildTransforms.Add(ChildActor);
					AttachedChildTransforms[ChildActor] = ChildActor->GetRootComponent()->GetRelativeTransform();
				}
			}
		}
	}

	FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(BindingGuid);
	// TODO: How to convert to possessable of non-actor type? Presumably we need to generalize the 'creation' step here for now.
	UObject* PossessedObject = nullptr;
	if (Actor)
	{
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.bDeferConstruction = true;
		SpawnInfo.Template = SpawnableActorTemplate;

		UObject* PlaybackContext = Sequencer->GetPlaybackContext();
		UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
		if (World)
		{
			AActor* PossessedActor = World->SpawnActor(Actor->GetClass(), &DefaultTransform, SpawnInfo);

			if (!PossessedActor)
			{
				return nullptr;
			}

			FString ActorLabel = Actor->GetActorLabel();
			if (Spawnable)
			{
				ActorLabel = Spawnable->GetName();
			}
			else if (ExistingPossessable)
			{
				if (BindingReferences)
				{
					// If we don't have multiple bound objects, use the Possessable name instead of the template label
					if (BindingReferences->GetReferences(BindingGuid).Num() == 1)
					{
						ActorLabel = ExistingPossessable->GetName();
					}
				}
			}

			PossessedActor->SetActorLabel(ActorLabel);

			const bool bIsDefaultTransform = true;
			PossessedActor->FinishSpawning(DefaultTransform, bIsDefaultTransform);

			// The transform needs to be set again for deferred construction and dynamic root components. Until the fix for: UE-67537
			PossessedActor->SetActorTransform(DefaultTransform);

			PossessedObject = PossessedActor;
		}
	}

	Sequence->Modify();
	MovieScene->Modify();

	UE::Sequencer::FCreateBindingParams CreateBindingParams;
	CreateBindingParams.ReplacementGuid = BindingGuid;
	CreateBindingParams.BindingIndex = BindingIndex;
	CreateBindingParams.bAllowCustomBinding = false;
	CreateBindingParams.bAllowEmptyBinding = PossessedObject == nullptr;

	// Create or replace the binding
	FGuid NewPossessableGuid = CreateOrReplaceBinding(Sequencer, Sequence, PossessedObject, CreateBindingParams);

	TArrayView<const FMovieSceneBindingReference> References = BindingReferences ? BindingReferences->GetReferences(BindingGuid) : TArrayView<const FMovieSceneBindingReference>();
	bool bAnySpawnablesLeft = Algo::AnyOf(References, [](const FMovieSceneBindingReference& BindingReference) { return BindingReference.CustomBinding && BindingReference.CustomBinding->IsA<UMovieSceneSpawnableBindingBase>(); });

	// If we're converting from a spawnable and none of the other bindings on the guid are spawnable, we'll need to remove the spawn track
	if (bConvertFromSpawnable && !bAnySpawnablesLeft)
	{
		// Delete the spawn track
		UMovieSceneSpawnTrack* SpawnTrack = Cast<UMovieSceneSpawnTrack>(MovieScene->FindTrack(UMovieSceneSpawnTrack::StaticClass(), BindingGuid, NAME_None));
		if (SpawnTrack)
		{
			MovieScene->RemoveTrack(*SpawnTrack);
		}
	}


	FMovieScenePossessable* Possessable = MovieScene->FindPossessable(NewPossessableGuid);
	if (Spawnable)
	{
		// Remap all the spawnable's tracks and child bindings onto the new possessable
		MovieScene->MoveBindingContents(BindingGuid, NewPossessableGuid);

		FMovieSceneBinding* SpawnableBinding = MovieScene->FindBinding(BindingGuid);
		check(SpawnableBinding);

		for (UMovieSceneFolder* Folder : MovieScene->GetRootFolders())
		{
			if (UpdateFolderBindingID(Folder, Spawnable->GetGuid(), Possessable->GetGuid()))
			{
				break;
			}
		}
		int32 SortingOrder = SpawnableBinding->GetSortingOrder();

		// Remove the spawnable and all it's sub tracks
		if (MovieScene->RemoveSpawnable(BindingGuid))
		{
			UpdateBindingIDs(Sequencer, BindingGuid, NewPossessableGuid);

			FMovieSceneBinding* PossessableBinding = MovieScene->FindBinding(NewPossessableGuid);
			check(PossessableBinding);

			PossessableBinding->SetSortingOrder(SortingOrder);
		}
	}

	// If we previously had an old-style spawnable or a spawnable custom binding, destroy the old spawned object
	if (bConvertFromSpawnable)
	{
		Sequencer->GetSpawnRegister().DestroySpawnedObject(BindingGuid, Sequencer->GetFocusedTemplateID(), Sequencer->GetSharedPlaybackState(), BindingIndex);
	}

	if (AActor* PossessedActor = Cast<AActor>(PossessedObject))
	{
		static const FName SequencerActorTag(TEXT("SequencerActor"));
		static const FName SequencerPreviewActorTag(TEXT("SequencerPreviewActor"));
		PossessedActor->Tags.Remove(SequencerActorTag);
		PossessedActor->Tags.Remove(SequencerPreviewActorTag);

		GEditor->SelectActor(PossessedActor, false, true);

		for (TPair<TWeakObjectPtr<AActor>, FTransform> AttachedChildTransform : AttachedChildTransforms)
		{
			if (AActor* AttachedChild = AttachedChildTransform.Key.Get())
			{
				if (AttachedChild->GetRootComponent())
				{
					AttachedChild->GetRootComponent()->SetRelativeTransform(AttachedChildTransform.Value);
				}
			}
		}
	}

	Sequencer->ForceEvaluate();

	return Possessable;
}

FMovieScenePossessable* FSequencerUtilities::ConvertToCustomBinding(TSharedRef<ISequencer> Sequencer, FGuid BindingGuid, TSubclassOf<UMovieSceneCustomBinding> CustomBindingType, int32 BindingIndex/*=0*/)
{
	FMovieScenePossessable* CreatedPossessable = nullptr;

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return CreatedPossessable;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return CreatedPossessable;
	}

	if (MovieScene->IsReadOnly())
	{
		ShowReadOnlyError();
		return CreatedPossessable;
	}

	const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences();
	if (!BindingReferences)
	{
		// Not supported with this sequence type- show an error?
		return CreatedPossessable;
	}

	if (!CustomBindingType)
	{
		return CreatedPossessable;
	}

	TArray<UObject*> BoundObjects = MovieSceneHelpers::GetBoundObjects(Sequencer->GetFocusedTemplateID(), BindingGuid, Sequencer->GetSharedPlaybackState(), BindingIndex);
	UObject* ObjectToConvert = BoundObjects.Num() > 0 ? BoundObjects[0] : nullptr;

	bool bConvertFromSpawnable = false;

	// If we have an old-style spawnable, use the template as the object to convert instead.
	FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(BindingGuid);
	const FMovieSceneBindingReference* PreviousBindingReference = BindingReferences->GetReference(BindingGuid, BindingIndex);
	const UMovieSceneCustomBinding* PreviousCustomBinding = nullptr;
	if (Spawnable)
	{
		ObjectToConvert = Spawnable->GetObjectTemplate();
		bConvertFromSpawnable = true;
	}
	else if (PreviousBindingReference)
	{
		if (const UMovieSceneCustomBinding* CustomBinding = PreviousBindingReference->CustomBinding)
		{
			PreviousCustomBinding = CustomBinding;
			bConvertFromSpawnable = PreviousCustomBinding->WillSpawnObject(Sequencer->GetSharedPlaybackState());
		}
	}

	bool bConvertFromPossessable = !bConvertFromSpawnable && !BindingReferences->GetCustomBinding(BindingGuid, BindingIndex);

	UMovieSceneCustomBinding* NewCustomBinding = nullptr;
	if (PreviousBindingReference)
	{
		NewCustomBinding = CustomBindingType->GetDefaultObject<UMovieSceneCustomBinding>()->CreateCustomBindingFromBinding(*PreviousBindingReference, ObjectToConvert, *MovieScene);
	}
	else
	{
		NewCustomBinding = CustomBindingType->GetDefaultObject<UMovieSceneCustomBinding>()->CreateNewCustomBinding(ObjectToConvert, *MovieScene);
	}

	if (!NewCustomBinding)
	{
		return CreatedPossessable;
	}

	Sequence->Modify();
	MovieScene->Modify();


	UE::Sequencer::FCreateBindingParams CreateBindingParams;
	CreateBindingParams.ReplacementGuid = BindingGuid;
	CreateBindingParams.BindingIndex = BindingIndex;
	CreateBindingParams.BindingNameOverride = NewCustomBinding->GetDesiredBindingName();
	CreateBindingParams.CustomBinding = NewCustomBinding;
	CreateBindingParams.bSetupDefaults = false;
	CreateBindingParams.bAllowEmptyBinding = !ObjectToConvert;

	// Create or replace the binding
	FGuid NewPossessableGuid = CreateOrReplaceBinding(Sequencer, ObjectToConvert, CreateBindingParams);

	TArrayView<const FMovieSceneBindingReference> BindingReferencesForGuid = BindingReferences->GetReferences(BindingGuid);
	bool bAnySpawnablesLeft = Algo::AnyOf(BindingReferencesForGuid, [](const FMovieSceneBindingReference& BindingReference) { return BindingReference.CustomBinding && BindingReference.CustomBinding->IsA<UMovieSceneSpawnableBindingBase>(); });
	bool bAnyReplaceablesLeft = Algo::AnyOf(BindingReferencesForGuid, [](const FMovieSceneBindingReference& BindingReference) { return BindingReference.CustomBinding && BindingReference.CustomBinding->IsA<UMovieSceneReplaceableBindingBase>(); });

	// If we're converting from a spawnable and the new custom binding isn't a spawnable, remove the spawn track
	if (PreviousCustomBinding && PreviousCustomBinding->IsA<UMovieSceneSpawnableBindingBase>() && !bAnySpawnablesLeft)
	{
		// Delete the spawn track
		UMovieSceneSpawnTrack* SpawnTrack = Cast<UMovieSceneSpawnTrack>(MovieScene->FindTrack(UMovieSceneSpawnTrack::StaticClass(), BindingGuid, NAME_None));
		if (SpawnTrack)
		{
			MovieScene->RemoveTrack(*SpawnTrack);
		}
	}
	else if (PreviousCustomBinding && PreviousCustomBinding->IsA<UMovieSceneReplaceableBindingBase>() && !bAnyReplaceablesLeft)
	{
		// Delete the binding lifetime track
		UMovieSceneBindingLifetimeTrack* BindingLifetimeTrack = Cast<UMovieSceneBindingLifetimeTrack>(MovieScene->FindTrack(UMovieSceneBindingLifetimeTrack::StaticClass(), BindingGuid, NAME_None));
		if (BindingLifetimeTrack)
		{
			MovieScene->RemoveTrack(*BindingLifetimeTrack);
		}
	}

	CreatedPossessable = MovieScene->FindPossessable(NewPossessableGuid);

	// If we previously had an old-style spawnable, we need to move over bindings
	if (Spawnable)
	{
		// Remap all the spawnable's tracks and child bindings onto the new possessable
		MovieScene->MoveBindingContents(BindingGuid, NewPossessableGuid);

		FMovieSceneBinding* SpawnableBinding = MovieScene->FindBinding(BindingGuid);
		check(SpawnableBinding);

		for (UMovieSceneFolder* Folder : MovieScene->GetRootFolders())
		{
			if (UpdateFolderBindingID(Folder, Spawnable->GetGuid(), NewPossessableGuid))
			{
				break;
			}
		}

		int32 SortingOrder = SpawnableBinding->GetSortingOrder();

		// Remove the spawnable and all its' sub tracks
		if (MovieScene->RemoveSpawnable(BindingGuid))
		{
			FMovieSceneBinding* PossessableBinding = MovieScene->FindBinding(NewPossessableGuid);
			check(PossessableBinding);

			PossessableBinding->SetSortingOrder(SortingOrder);
		}

		UpdateBindingIDs(Sequencer, BindingGuid, NewPossessableGuid);

		Sequencer->ForceEvaluate();
	}

	TOptional<FTransformData> TransformData;
	if (bConvertFromSpawnable)
	{
		Sequencer->GetSpawnRegister().DestroySpawnedObject(BindingGuid, Sequencer->GetFocusedTemplateID(), Sequencer->GetSharedPlaybackState(), BindingIndex);
	}
	else if (bConvertFromPossessable && NewCustomBinding->WillSpawnObject(Sequencer->GetSharedPlaybackState()))
	{
		// We have an old possessable to destroy
		Sequencer->GetSpawnRegister().HandleConvertPossessableToSpawnable(ObjectToConvert, *Sequencer, TransformData);
	}

	// If this is a new spawnable or replaceable binding, we need to set up some defaults
	if (NewCustomBinding->WillSpawnObject(Sequencer->GetSharedPlaybackState()))
	{
		// We purposefully pass in nullptr to SetupDefaultsForSpawnable below. 
		// This will prevent a section of code in it from calling OnActorAddedToSequencer, which should not be called in the case of binding conversion,
		// as it may cause some default tracks to get added for a second time.

		// Allow the binding to set up any necessary defaults
		NewCustomBinding->SetupDefaults(nullptr, NewPossessableGuid, *MovieScene);

		Sequencer->GetSpawnRegister().SetupDefaultsForSpawnable(nullptr, NewPossessableGuid, TransformData, Sequencer, Sequencer->GetSequencerSettings());
	}

	//Sequencer->State.Invalidate(NewPossessableGuid, Sequencer->GetFocusedTemplateID());

	return CreatedPossessable;
}

void ExportObjectsToText(const TArray<UObject*>& ObjectsToExport, FString& ExportedText)
{
	if (ObjectsToExport.Num() == 0)
	{
		return;
	}

	// Clear the mark state for saving.
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	FStringOutputDevice Archive;
	const FExportObjectInnerContext Context;

	// Export each of the selected nodes
	UObject* LastOuter = nullptr;

	for (UObject* ObjectToExport : ObjectsToExport)
	{
		if (ObjectToExport)
		{
			// The nodes should all be from the same scope
			UObject* ThisOuter = ObjectToExport->GetOuter();
			if (LastOuter != nullptr && ThisOuter != LastOuter)
			{
				UE_LOGF(LogSequencer, Error, "Cannot copy objects from different outers. Only copying from %ls", *LastOuter->GetName());
				continue;
			}
			LastOuter = ThisOuter;

			UExporter::ExportToOutputDevice(&Context, ObjectToExport, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, ThisOuter);
		}
	}

	ExportedText = Archive;
}

/**
 *
 * Copy/paste folders
 *
 */

void GatherChildFolders(UMovieSceneFolder* ParentFolder, TArray<UObject*>& Objects)
{
	for (UMovieSceneFolder* ChildFolder : ParentFolder->GetChildFolders())
	{
		if (ChildFolder)
		{
			Objects.AddUnique(ChildFolder);

			GatherChildFolders(ChildFolder, Objects);
		}
	}
}

void FSequencerUtilities::CopyFolders(const TArray<UMovieSceneFolder*>& Folders, FString& ExportedText)
{
	TArray<UObject*> Objects;
	for (UMovieSceneFolder* Folder : Folders)
	{
		Objects.AddUnique(Folder);

		GatherChildFolders(Folder, Objects);
	}

	ExportObjectsToText(Objects, /*out*/ ExportedText);
}

void GatherChildBindings(UMovieSceneSequence* Sequence, const FGuid& ObjectBinding, TArray<FMovieSceneBindingProxy>& Bindings)
{
	UMovieScene* MovieScene = Sequence->GetMovieScene();

	for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
	{
		const FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);
		if (Possessable.GetParent() == ObjectBinding)
		{
			Bindings.AddUnique(FMovieSceneBindingProxy(Possessable.GetGuid(), Sequence));

			GatherChildBindings(Sequence, Possessable.GetGuid(), Bindings);
		}
	}
}

void GatherFolderContents(UMovieSceneFolder* Folder, TArray<UMovieSceneFolder*>& Folders, TArray<UMovieSceneTrack*>& Tracks, TArray<FMovieSceneBindingProxy>& Bindings)
{
	if (!Folder)
	{
		return;
	}

	Folders.AddUnique(Folder);

	UMovieScene* MovieScene = CastChecked<UMovieScene>(Folder->GetOuter());

	UMovieSceneSequence* Sequence = CastChecked<UMovieSceneSequence>(MovieScene->GetOuter());

	for (const FGuid& ObjectBinding : Folder->GetChildObjectBindings())
	{
		Bindings.AddUnique(FMovieSceneBindingProxy(ObjectBinding, Sequence));

		GatherChildBindings(Sequence, ObjectBinding, Bindings);
	}

	for (UMovieSceneTrack* ChildTrack : Folder->GetChildTracks())
	{
		Tracks.AddUnique(ChildTrack);
	}

	for (UMovieSceneFolder* ChildFolder : Folder->GetChildFolders())
	{
		if (ChildFolder)
		{
			GatherFolderContents(ChildFolder, Folders, Tracks, Bindings);
		}
	}
}

void FSequencerUtilities::CopyFolders(TSharedRef<ISequencer> Sequencer, const TArray<UMovieSceneFolder*>& InFolders, FString& FoldersExportedText, FString& TracksExportedText, FString& ObjectsExportedText)
{
	TArray<UMovieSceneFolder*> Folders;
	TArray<UMovieSceneTrack*> Tracks;
	TArray<FMovieSceneBindingProxy> Bindings;

	for (UMovieSceneFolder* Folder : InFolders)
	{
		GatherFolderContents(Folder, Folders, Tracks, Bindings);
	}

	CopyTracks(Tracks, Folders, TracksExportedText);
	CopyBindings(Sequencer, Bindings, Folders, ObjectsExportedText);

	TArray<UObject*> Objects;
	for (UMovieSceneFolder* Folder : Folders)
	{
		Objects.Add(Folder);
	}

	ExportObjectsToText(Objects, /*out*/ FoldersExportedText);
}

class FFolderObjectTextFactory : public FCustomizableTextObjectFactory
{
public:
	FFolderObjectTextFactory()
		: FCustomizableTextObjectFactory(GWarn)
	{
	}

	// FCustomizableTextObjectFactory implementation
	virtual bool CanCreateClass(UClass* InObjectClass, bool& bOmitSubObjs) const override
	{
		if (InObjectClass->IsChildOf(UMovieSceneFolder::StaticClass()))
		{
			return true;
		}
		return false;
	}


	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		check(NewObject);

		NewFolders.Add(Cast<UMovieSceneFolder>(NewObject));
	}

public:
	TArray<UMovieSceneFolder*> NewFolders;
};

void ImportFoldersFromText(const FString& TextToImport, /*out*/ TArray<UMovieSceneFolder*>& ImportedFolders)
{
	UPackage* TempPackage = NewObject<UPackage>(nullptr, TEXT("/Engine/Sequencer/Editor/Transient"), RF_Transient);
	TempPackage->AddToRoot();

	// Turn the text buffer into objects
	FFolderObjectTextFactory Factory;
	Factory.ProcessBuffer(TempPackage, RF_Transactional, TextToImport);

	ImportedFolders = Factory.NewFolders;

	// Remove the temp package from the root now that it has served its purpose
	TempPackage->RemoveFromRoot();
}

bool FSequencerUtilities::PasteFolders(const FString& TextToImport, FMovieScenePasteFoldersParams PasteFoldersParams, TArray<UMovieSceneFolder*>& OutFolders, TArray<FNotificationInfo>& OutErrors)
{
	if (!PasteFoldersParams.Sequence)
	{
		return false;
	}

	UMovieScene* MovieScene = PasteFoldersParams.Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return false;
	}

	TArray<UMovieSceneFolder*> ImportedFolders;
	ImportFoldersFromText(TextToImport, ImportedFolders);

	if (ImportedFolders.Num() == 0)
	{
		return false;
	}

	const FScopedTransaction Transaction(LOCTEXT("PasteFolders", "Paste Folders"));

	MovieScene->Modify();

	for (UMovieSceneFolder* CopiedFolder : ImportedFolders)
	{
		CopiedFolder->Rename(nullptr, MovieScene);

		OutFolders.Add(CopiedFolder);

		// Clear the folder contents, those relationships will be made when the tracks are pasted
		CopiedFolder->ClearChildTracks();
		CopiedFolder->ClearChildObjectBindings();

		bool bHasParent = false;
		for (UMovieSceneFolder* ImportedParentFolder : ImportedFolders)
		{
			if (ImportedParentFolder != CopiedFolder)
			{
				if (ImportedParentFolder->GetChildFolders().Contains(CopiedFolder))
				{
					bHasParent = true;
					break;
				}
			}
		}

		if (!bHasParent)
		{
			if (PasteFoldersParams.ParentFolder)
			{
				PasteFoldersParams.ParentFolder->AddChildFolder(CopiedFolder);
			}
			else
			{
				MovieScene->AddRootFolder(CopiedFolder);
			}
		}
	}

	return true;
}

bool FSequencerUtilities::CanPasteFolders(const FString& TextToImport)
{
	FFolderObjectTextFactory FolderFactory;
	return FolderFactory.CanCreateObjectsFromText(TextToImport);
}

/**
 *
 * Copy/paste tracks
 *
 */

void FSequencerUtilities::CopyTracks(const TArray<UMovieSceneTrack*>& Tracks, const TArray<UMovieSceneFolder*>& Folders, FString& ExportedText)
{
	TArray<UObject*> Objects;
	TMap<UObject*, UObject*> SourceToDuplicate;
	TArray<UMovieSceneTrack*> DuplicatesToRemap;
	for (UMovieSceneTrack* Track : Tracks)
	{
		UMovieScene* MovieScene = Track->GetTypedOuter<UMovieScene>();

		UMovieSceneCopyableTrack* CopyableTrack = NewObject<UMovieSceneCopyableTrack>(GetTransientPackage(), UMovieSceneCopyableTrack::StaticClass(), NAME_None, RF_Transient);
		Objects.Add(CopyableTrack);

		UMovieSceneTrack* DuplicatedTrack = Cast<UMovieSceneTrack>(StaticDuplicateObject(Track, CopyableTrack));
		CopyableTrack->Track = DuplicatedTrack;
		CopyableTrack->bIsRootTrack = MovieScene->ContainsTrack(*Track);
		CopyableTrack->bIsCameraCutTrack = Track->IsA<UMovieSceneCameraCutTrack>();

		SourceToDuplicate.Add(Track, DuplicatedTrack);
		DuplicatesToRemap.Add(DuplicatedTrack);

		// A track's child tracks (e.g. an Animation Mixer track's Control Rig tracks) are
		// re-outered to the MovieScene as siblings rather than owned as subobjects, so they
		// are not part of the copied selection and StaticDuplicateObject leaves the parent's
		// references aimed at them. Duplicate them alongside the parent - parented to the
		// parent copy so they travel with the clipboard text - so paste can re-register them.
		if (UMovieSceneCommonAnimationTrack* CommonTrack = Cast<UMovieSceneCommonAnimationTrack>(Track))
		{
			TArray<TObjectPtr<UMovieSceneTrack>> ChildTracks;
			CommonTrack->GetAllChildTracks(ChildTracks);
			for (UMovieSceneTrack* ChildTrack : ChildTracks)
			{
				if (ChildTrack)
				{
					UMovieSceneTrack* DuplicatedChild = Cast<UMovieSceneTrack>(StaticDuplicateObject(ChildTrack, DuplicatedTrack));
					SourceToDuplicate.Add(ChildTrack, DuplicatedChild);
					DuplicatesToRemap.Add(DuplicatedChild);
				}
			}
		}

		UMovieSceneFolder* Folder = nullptr;
		for (UMovieSceneFolder* RootFolder : MovieScene->GetRootFolders())
		{
			Folder = RootFolder->FindFolderContaining(Track);
			if (Folder && Folders.Contains(Folder))
			{
				UMovieSceneFolder::CalculateFolderPath(Folder, Folders, CopyableTrack->FolderPath);
				break;
			}
		}
	}

	// Remap every duplicate's references (including those held by subobjects such as mixer
	// layers) from the originals onto their duplicates so the copied tracks are self-contained.
	for (UMovieSceneTrack* Duplicate : DuplicatesToRemap)
	{
		FArchiveReplaceObjectRef<UObject> ReplaceTrackRefs(Duplicate, SourceToDuplicate);
	}

	ExportObjectsToText(Objects, /*out*/ ExportedText);
}

class FTrackObjectTextFactory : public FCustomizableTextObjectFactory
{
public:
	FTrackObjectTextFactory()
		: FCustomizableTextObjectFactory(GWarn)
	{
	}

	// FCustomizableTextObjectFactory implementation
	virtual bool CanCreateClass(UClass* InObjectClass, bool& bOmitSubObjs) const override
	{
		if (InObjectClass->IsChildOf(UMovieSceneCopyableTrack::StaticClass()))
		{
			return true;
		}
		return false;
	}


	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		check(NewObject);

		NewTracks.Add(Cast<UMovieSceneCopyableTrack>(NewObject));
	}

public:
	TArray<UMovieSceneCopyableTrack*> NewTracks;
};

void ImportTracksFromText(const FString& TextToImport, /*out*/ TArray<UMovieSceneCopyableTrack*>& ImportedTracks)
{
	UPackage* TempPackage = NewObject<UPackage>(nullptr, TEXT("/Engine/Sequencer/Editor/Transient"), RF_Transient);
	TempPackage->AddToRoot();

	// Turn the text buffer into objects
	FTrackObjectTextFactory Factory;
	Factory.ProcessBuffer(TempPackage, RF_Transactional, TextToImport);

	ImportedTracks = Factory.NewTracks;

	// Remove the temp package from the root now that it has served its purpose
	TempPackage->RemoveFromRoot();
}

bool FSequencerUtilities::PasteTracks(const FString& TextToImport, FMovieScenePasteTracksParams PasteTracksParams, TArray<UMovieSceneTrack*>& OutTracks, TArray<FNotificationInfo>& OutErrors)
{
	TArray<UMovieSceneCopyableTrack*> ImportedTracks;
	ImportTracksFromText(TextToImport, ImportedTracks);

	if (ImportedTracks.Num() == 0)
	{
		return false;
	}

	const FScopedTransaction Transaction(LOCTEXT("PasteTracks", "Paste Tracks"));

	int32 NumRootOrCameraCutTracks = 0;
	int32 NumTracks = 0;

	for (UMovieSceneCopyableTrack* CopyableTrack : ImportedTracks)
	{
		if (CopyableTrack->bIsRootTrack || CopyableTrack->bIsCameraCutTrack)
		{
			++NumRootOrCameraCutTracks;
		}
		else
		{
			++NumTracks;
		}
	}

	int32 NumTracksPasted = 0;
	int32 NumRootOrCameraCutTracksPasted = 0;

	for (const FMovieSceneBindingProxy& ObjectBinding : PasteTracksParams.Bindings)
	{
		TArray<UMovieSceneCopyableTrack*> NewTracks;
		ImportTracksFromText(TextToImport, NewTracks);

		UMovieScene* MovieScene = ObjectBinding.GetMovieScene();
		if (!MovieScene)
		{
			continue;
		}

		for (UMovieSceneCopyableTrack* CopyableTrack : NewTracks)
		{
			if (!CopyableTrack->bIsRootTrack && !CopyableTrack->bIsCameraCutTrack)
			{
				UMovieSceneTrack* NewTrack = CopyableTrack->Track;
				ResetCopiedTracksFlags(NewTrack);

				// Remove tracks with the same name before adding
				if (const FMovieSceneBinding* Binding = MovieScene->FindBinding(ObjectBinding.BindingID))
				{
					for (UMovieSceneTrack* Track : Binding->GetTracks())
					{
						if (Track->GetClass() == NewTrack->GetClass() && Track->GetTrackName() == NewTrack->GetTrackName() && Track->GetDisplayName().IdenticalTo(NewTrack->GetDisplayName()))
						{
							// If a track of the same class and name exists, remove it so the new track replaces it
							MovieScene->RemoveTrack(*Track);
							break;
						}
					}
				}

				if (!MovieScene->AddGivenTrack(NewTrack, ObjectBinding.BindingID))
				{
					continue;
				}
				else
				{
					OutTracks.Add(NewTrack);
					++NumTracksPasted;

					// Child tracks travel as subobjects of the parent copy; re-register them on
					// the same binding so they become MovieScene tracks again. The parent already
					// references these duplicates via its child-track bookkeeping, so registering
					// them keeps the pasted track independent of the source.
					if (UMovieSceneCommonAnimationTrack* CommonTrack = Cast<UMovieSceneCommonAnimationTrack>(NewTrack))
					{
						TArray<TObjectPtr<UMovieSceneTrack>> ChildTracks;
						CommonTrack->GetAllChildTracks(ChildTracks);
						for (UMovieSceneTrack* ChildTrack : ChildTracks)
						{
							if (ChildTrack)
							{
								ResetCopiedTracksFlags(ChildTrack);
								MovieScene->AddGivenTrack(ChildTrack, ObjectBinding.BindingID);
							}
						}
					}
				}
			}
		}
	}

	UMovieScene* MovieScene = PasteTracksParams.Sequence ? PasteTracksParams.Sequence->GetMovieScene() : nullptr;
	if (MovieScene)
	{
		// Add as root track or set camera cut track
		for (UMovieSceneCopyableTrack* CopyableTrack : ImportedTracks)
		{
			if (CopyableTrack->bIsRootTrack || CopyableTrack->bIsCameraCutTrack)
			{
				UMovieSceneTrack* NewTrack = CopyableTrack->Track;
				ResetCopiedTracksFlags(NewTrack);

				UMovieSceneFolder* ParentFolder = PasteTracksParams.ParentFolder;

				if (CopyableTrack->FolderPath.Num() > 0)
				{
					ParentFolder = UMovieSceneFolder::GetFolderWithPath(CopyableTrack->FolderPath, PasteTracksParams.Folders, ParentFolder ? ParentFolder->GetChildFolders() : MovieScene->GetRootFolders());
				}

				if (NewTrack->IsA(UMovieSceneCameraCutTrack::StaticClass()))
				{
					MovieScene->SetCameraCutTrack(NewTrack);
					if (ParentFolder != nullptr)
					{
						ParentFolder->AddChildTrack(NewTrack);
					}

					++NumRootOrCameraCutTracksPasted;
				}
				else
				{
					if (MovieScene->AddGivenTrack(NewTrack))
					{
						if (ParentFolder != nullptr)
						{
							ParentFolder->AddChildTrack(NewTrack);
						}
					}

					++NumRootOrCameraCutTracksPasted;
				}

				OutTracks.Add(NewTrack);
			}
		}
	}
	
	if (NumRootOrCameraCutTracksPasted < NumRootOrCameraCutTracks)
	{
		FNotificationInfo Info(LOCTEXT("PasteTracks_NoTracks", "Can't paste track. Root track could not be pasted"));
		OutErrors.Add(Info);
	}

	if (NumTracksPasted < NumTracks)
	{
		FNotificationInfo Info(LOCTEXT("PasteTracks_NoSelectedObjects", "Can't paste track. No selected objects to paste tracks onto"));
		OutErrors.Add(Info);
	}

	return (NumRootOrCameraCutTracksPasted + NumTracksPasted) > 0;
}

bool FSequencerUtilities::CanPasteTracks(const FString& TextToImport)
{
	FTrackObjectTextFactory TrackFactory;
	return TrackFactory.CanCreateObjectsFromText(TextToImport);
}

/**
 *
 * Copy/paste sections
 *
 */

void FSequencerUtilities::CopySections(const TArray<UMovieSceneSection*>& Sections, FString& ExportedText)
{
	TArray<UObject*> Objects;
	for (UMovieSceneSection* Section : Sections)
	{
		Objects.Add(Section);
	}

	ExportObjectsToText(Objects, /*out*/ ExportedText);
}

class FSectionObjectTextFactory : public FCustomizableTextObjectFactory
{
public:
	FSectionObjectTextFactory()
		: FCustomizableTextObjectFactory(GWarn)
	{
	}

	// FCustomizableTextObjectFactory implementation
	virtual bool CanCreateClass(UClass* InObjectClass, bool& bOmitSubObjs) const override
	{
		if (InObjectClass->IsChildOf(UMovieSceneSection::StaticClass()))
		{
			return true;
		}
		return false;
	}


	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		check(NewObject);

		NewSections.Add(Cast<UMovieSceneSection>(NewObject));
	}

public:
	TArray<UMovieSceneSection*> NewSections;
};

void ImportSectionsFromText(const FString& TextToImport, /*out*/ TArray<UMovieSceneSection*>& ImportedSections)
{
	UPackage* TempPackage = NewObject<UPackage>(nullptr, TEXT("/Engine/Sequencer/Editor/Transient"), RF_Transient);
	TempPackage->AddToRoot();

	// Turn the text buffer into objects
	FSectionObjectTextFactory Factory;
	Factory.ProcessBuffer(TempPackage, RF_Transactional, TextToImport);

	ImportedSections = Factory.NewSections;

	// Remove the temp package from the root now that it has served its purpose
	TempPackage->RemoveFromRoot();
}

bool FSequencerUtilities::PasteSections(const FString& TextToImport, FMovieScenePasteSectionsParams PasteSectionsParams, TArray<UMovieSceneSection*>& OutSections, TArray<FNotificationInfo>& OutErrors)
{
	// First import as a track and extract sections to allow for copying track contents to another track
	TArray<UMovieSceneCopyableTrack*> ImportedTracks;
	ImportTracksFromText(TextToImport, ImportedTracks);

	TArray<UMovieSceneSection*> ImportedSections;
	for (UMovieSceneCopyableTrack* CopyableTrack : ImportedTracks)
	{
		for (UMovieSceneSection* CopyableSection : CopyableTrack->Track->GetAllSections())
		{
			ImportedSections.Add(CopyableSection);
		}
	}

	// Otherwise, import as sections
	if (ImportedSections.Num() == 0)
	{
		ImportSectionsFromText(TextToImport, ImportedSections);
	}

	if (ImportedSections.Num() == 0)
	{
		return false;
	}

	const FScopedTransaction Transaction(LOCTEXT("PasteSections", "Paste Sections"));

	TOptional<FFrameNumber> FirstFrame;
	for (UMovieSceneSection* Section : ImportedSections)
	{
		if (Section->HasStartFrame())
		{
			if (FirstFrame.IsSet())
			{
				if (FirstFrame.GetValue() > Section->GetInclusiveStartFrame())
				{
					FirstFrame = Section->GetInclusiveStartFrame();
				}
			}
			else
			{
				FirstFrame = Section->GetInclusiveStartFrame();
			}
		}
	}

	TArray<int32> SectionIndicesImported;

	// Route decoration-hosted sections (e.g. mask sections inside
	// UMovieSceneAnimationMaskDecoration) to their host decoration. Resolution:
	//   A. A user-selected decoration on a target wins.
	//   B. Otherwise, auto-create from a registered-compatible decoration class
	//      on a target container. Decorations not registered for the container
	//      class are never created.
	// Hosted sections that find no compatible host are dropped, never falling
	// through to the per-track loop.

	struct FDecorationHostInfo
	{
		UClass* DecorationClass = nullptr;
		TSubclassOf<UMovieSceneSection> HostedSectionClass;
	};
	TArray<FDecorationHostInfo> AllDecorationHosts;
	{
		TSet<UClass*> RegisteredDecorationClasses;
		IMovieSceneModule::Get().GetAllRegisteredDecorationClasses(RegisteredDecorationClasses);
		for (UClass* TestClass : RegisteredDecorationClasses)
		{
			if (!TestClass || !TestClass->ImplementsInterface(UMovieSceneSectionProviderDecoration::StaticClass()))
			{
				continue;
			}
			IMovieSceneSectionProviderDecoration* Provider = Cast<IMovieSceneSectionProviderDecoration>(TestClass->GetDefaultObject());
			if (!Provider || !Provider->SupportsMultipleSections())
			{
				continue;
			}
			TSubclassOf<UMovieSceneSection> Hosted = Provider->GetHostedSectionClass();
			if (!*Hosted)
			{
				continue;
			}
			AllDecorationHosts.Add({TestClass, Hosted});
		}
	}

	auto IsDecorationHostedSectionClass = [&](UClass* SectionClass) -> bool
	{
		for (const FDecorationHostInfo& Host : AllDecorationHosts)
		{
			if (SectionClass->IsChildOf(*Host.HostedSectionClass))
			{
				return true;
			}
		}
		return false;
	};

	// Resolve per-track containers up-front so callers that pass plain Tracks
	// without explicit DecorationTargets still benefit (e.g. anim mixer track
	// returns its layer for a row).
	TArray<FMovieScenePasteDecorationTarget> ResolvedDecorationTargets = PasteSectionsParams.DecorationTargets;
	for (int32 Index = 0; Index < PasteSectionsParams.Tracks.Num(); ++Index)
	{
		UMovieSceneTrack* Track = PasteSectionsParams.Tracks[Index];
		if (!Track)
		{
			continue;
		}

		int32 TargetRowIndex = INDEX_NONE;
		if (PasteSectionsParams.TrackRowIndices.Num() == PasteSectionsParams.Tracks.Num())
		{
			TargetRowIndex = PasteSectionsParams.TrackRowIndices[Index];
		}
		else if (PasteSectionsParams.TrackRowIndices.Num() == 1)
		{
			TargetRowIndex = PasteSectionsParams.TrackRowIndices[0];
		}

		if (UMovieSceneDecorationContainerObject* RowContainer = Track->GetDecorationContainerForRow(TargetRowIndex))
		{
			const bool bAlreadyPresent = ResolvedDecorationTargets.ContainsByPredicate(
				[RowContainer](const FMovieScenePasteDecorationTarget& Target) { return Target.Container == RowContainer; });
			if (!bAlreadyPresent)
			{
				ResolvedDecorationTargets.Add({RowContainer, nullptr});
			}
		}
	}

	TArray<UMovieSceneSection*> RoutedToDecoration;
	TArray<UMovieSceneSection*> HostedButOrphaned;
	for (UMovieSceneSection* Section : ImportedSections)
	{
		UClass* SectionClass = Section->GetClass();
		if (!IsDecorationHostedSectionClass(SectionClass))
		{
			continue;
		}

		UObject* HostDecoration = nullptr;
		UMovieSceneDecorationContainerObject* HostContainer = nullptr;
		UClass* MatchedDecorationClass = nullptr;

		// User-selected decoration wins over auto-create.
		for (const FMovieScenePasteDecorationTarget& Target : ResolvedDecorationTargets)
		{
			if (!Target.PreferredDecoration)
			{
				continue;
			}
			IMovieSceneSectionProviderDecoration* Provider = Cast<IMovieSceneSectionProviderDecoration>(Target.PreferredDecoration.Get());
			if (!Provider)
			{
				continue;
			}
			TSubclassOf<UMovieSceneSection> Hosted = Provider->GetHostedSectionClass();
			if (!*Hosted)
			{
				continue;
			}
			if (SectionClass->IsChildOf(*Hosted))
			{
				HostDecoration = Target.PreferredDecoration.Get();
				HostContainer = Target.Container.Get();
				MatchedDecorationClass = Target.PreferredDecoration->GetClass();
				break;
			}
		}

		// Auto-create on a registered-compatible target container.
		if (!HostDecoration)
		{
			for (const FMovieScenePasteDecorationTarget& Target : ResolvedDecorationTargets)
			{
				if (!Target.Container)
				{
					continue;
				}

				TSet<UClass*> CompatibleDecorations;
				IMovieSceneModule::Get().GetCompatibleDecorationsForContainer(Target.Container.Get(), CompatibleDecorations);
				for (UClass* DecorationClass : CompatibleDecorations)
				{
					if (!DecorationClass || !DecorationClass->ImplementsInterface(UMovieSceneSectionProviderDecoration::StaticClass()))
					{
						continue;
					}
					IMovieSceneSectionProviderDecoration* CDO = Cast<IMovieSceneSectionProviderDecoration>(DecorationClass->GetDefaultObject());
					if (!CDO || !CDO->SupportsMultipleSections())
					{
						continue;
					}
					TSubclassOf<UMovieSceneSection> Hosted = CDO->GetHostedSectionClass();
					if (!*Hosted)
					{
						continue;
					}
					if (SectionClass->IsChildOf(*Hosted))
					{
						HostContainer = Target.Container.Get();
						MatchedDecorationClass = DecorationClass;
						break;
					}
				}

				if (HostContainer)
				{
					break;
				}
			}
		}

		if (!HostDecoration && !HostContainer)
		{
			HostedButOrphaned.Add(Section);
			continue;
		}

		if (!HostDecoration)
		{
			check(HostContainer && MatchedDecorationClass);
			HostDecoration = HostContainer->FindDecoration(MatchedDecorationClass);
			if (!HostDecoration)
			{
				HostContainer->Modify();
				HostDecoration = HostContainer->GetOrCreateDecoration(MatchedDecorationClass);
			}
		}

		IMovieSceneSectionProviderDecoration* HostProvider = Cast<IMovieSceneSectionProviderDecoration>(HostDecoration);
		if (!HostProvider)
		{
			HostedButOrphaned.Add(Section);
			continue;
		}

		Section->ClearFlags(RF_Transient);
		Section->PostPaste();
		if (!Section->Rename(nullptr, HostDecoration, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch))
		{
			HostedButOrphaned.Add(Section);
			continue;
		}

		// Move before AddSection so any overlap fixup the decoration runs sees
		// the destination range, not the source range.
		if (Section->HasStartFrame() && FirstFrame.IsSet())
		{
			FFrameNumber NewStartFrame = PasteSectionsParams.Time.FrameNumber + (Section->GetInclusiveStartFrame() - FirstFrame.GetValue());
			FFrameNumber Delta = NewStartFrame - Section->GetInclusiveStartFrame();
			if (Delta != 0)
			{
				Section->MoveSection(Delta);
			}
		}

		HostProvider->AddSection(Section);

		RoutedToDecoration.Add(Section);
		OutSections.Add(Section);
	}

	for (UMovieSceneSection* Section : RoutedToDecoration)
	{
		ImportedSections.Remove(Section);
	}
	for (UMovieSceneSection* Section : HostedButOrphaned)
	{
		UE_LOGF(LogSequencer, Error, "Could not paste section of type %ls: no compatible decoration host on the paste target.",
			*Section->GetClass()->GetName());
		ImportedSections.Remove(Section);
	}

	for (int32 Index = 0; Index < PasteSectionsParams.Tracks.Num(); ++Index)
	{
		UMovieSceneTrack* Track = PasteSectionsParams.Tracks[Index];
		if (!Track)
		{
			continue;
		}

		const bool bAllowOverlap = Track->SupportsMultipleRows() || Track->GetSupportedBlendTypes().Num() > 0;

		for (int32 SectionIndex = 0; SectionIndex < ImportedSections.Num(); ++SectionIndex)
		{
			UMovieSceneSection* Section = ImportedSections[SectionIndex];
			if (!Track->SupportsType(Section->GetClass()))
			{
				continue;
			}

			int32 RowIndex = Section->GetRowIndex();

			// If there is only 1 track row index, paste all sections onto that row.
			// INDEX_NONE means "no override / no filter" -- keep section's source row.
			if (PasteSectionsParams.TrackRowIndices.Num() == 1)
			{
				if (PasteSectionsParams.TrackRowIndices[0] != INDEX_NONE)
				{
					RowIndex = PasteSectionsParams.TrackRowIndices[0];
				}
			}
			else if (Index < PasteSectionsParams.TrackRowIndices.Num())
			{
				const int32 OverrideRowIndex = PasteSectionsParams.TrackRowIndices[Index];
				if (OverrideRowIndex != INDEX_NONE && OverrideRowIndex != Section->GetRowIndex())
				{
					continue;
				}
			}

			Track->Modify();

			Section->ClearFlags(RF_Transient);
			Section->PostPaste();
			Section->Rename(nullptr, Track);

			if (Track->SupportsMultipleRows())
			{
				Section->SetRowIndex(RowIndex);
			}
			else if (!Section->HasStartFrame() && !Section->HasEndFrame())
			{
				// If the track doesn't support multiple rows and the pasted section is infinite, it should win out over existing sections
				Track->RemoveAllAnimationData();
			}

			Track->AddSection(*Section);
			if (Section->HasStartFrame())
			{
				FFrameNumber NewStartFrame = PasteSectionsParams.Time.FrameNumber + (Section->GetInclusiveStartFrame() - FirstFrame.GetValue());
				FFrameNumber Delta = NewStartFrame - Section->GetInclusiveStartFrame();
				if (Delta != 0)
				{
					Section->MoveSection(Delta);
					FMovieSceneSectionMovedParams MovedParams(EPropertyChangeType::ValueSet);
					Track->OnSectionMoved(*Section, MovedParams);
				}
			}

			if (!bAllowOverlap)
			{
				if (Section->OverlapsWithSections(Track->GetAllSections()))
				{
					Track->RemoveSection(*Section);
					UE_LOGF(LogSequencer, Error, "Could not paste section because it overlaps with existing sections and this track type does not allow overlaps");
					continue;
				}
			}

			SectionIndicesImported.AddUnique(SectionIndex);
			OutSections.Add(Section);
		}

		// Fix up rows after sections are in place
		Track->FixRowIndices();

		// Remove sections that were pasted so that they aren't pasted again to another track
		for (UMovieSceneSection* OutSection : OutSections)
		{
			ImportedSections.Remove(OutSection);
		}
	}

	for (int32 SectionIndex = 0; SectionIndex < ImportedSections.Num(); ++SectionIndex)
	{
		if (!SectionIndicesImported.Contains(SectionIndex))
		{
			UE_LOGF(LogSequencer, Error, "Could not paste section of type %ls", *ImportedSections[SectionIndex]->GetClass()->GetName());
		}
	}

	if (SectionIndicesImported.Num() == 0 && RoutedToDecoration.Num() == 0)
	{
		FNotificationInfo Info(LOCTEXT("PasteSections_NothingPasted", "Can't paste section. No matching section types found."));
		OutErrors.Add(Info);
		return false;
	}

	return true;
}

bool FSequencerUtilities::CanPasteSections(const FString& TextToImport)
{
	FSectionObjectTextFactory SectionFactory;
	return SectionFactory.CanCreateObjectsFromText(TextToImport);
}

/**
 *
 * Copy/paste decorations
 *
 */

void FSequencerUtilities::CopyDecorations(const TArray<UObject*>& Decorations, FString& ExportedText)
{
	TArray<UObject*> Objects;
	for (UObject* Decoration : Decorations)
	{
		if (Decoration)
		{
			Objects.Add(Decoration);
		}
	}

	ExportObjectsToText(Objects, /*out*/ ExportedText);
}

class FDecorationObjectTextFactory : public FCustomizableTextObjectFactory
{
public:
	FDecorationObjectTextFactory()
		: FCustomizableTextObjectFactory(GWarn)
	{
	}

	virtual bool CanCreateClass(UClass* InObjectClass, bool& bOmitSubObjs) const override
	{
		// Section-provider decorations only. Their inner section sub-objects
		// must remain nested under the decoration's Begin Object block during
		// import; accepting UMovieSceneSection here would hoist them to
		// top-level and leave the decoration's section array dangling.
		return InObjectClass->ImplementsInterface(UMovieSceneSectionProviderDecoration::StaticClass());
	}

	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		check(NewObject);
		NewDecorations.Add(NewObject);
	}

public:
	TArray<UObject*> NewDecorations;
};

static void ImportDecorationsFromText(const FString& TextToImport, /*out*/ TArray<UObject*>& ImportedDecorations)
{
	UPackage* TempPackage = NewObject<UPackage>(nullptr, TEXT("/Engine/Sequencer/Editor/Transient"), RF_Transient);
	TempPackage->AddToRoot();

	FDecorationObjectTextFactory Factory;
	Factory.ProcessBuffer(TempPackage, RF_Transactional, TextToImport);

	ImportedDecorations = Factory.NewDecorations;

	TempPackage->RemoveFromRoot();
}

bool FSequencerUtilities::PasteDecorations(const FString& TextToImport, FMovieScenePasteDecorationsParams PasteDecorationsParams, TArray<UObject*>& OutDecorations, TArray<FNotificationInfo>& OutErrors)
{
	TArray<UObject*> SourceDecorations;
	ImportDecorationsFromText(TextToImport, SourceDecorations);
	if (SourceDecorations.Num() == 0)
	{
		return false;
	}

	if (PasteDecorationsParams.Containers.Num() == 0)
	{
		FNotificationInfo Info(LOCTEXT("PasteDecorations_NoTargets", "Can't paste decoration. No selected target to paste onto."));
		OutErrors.Add(Info);
		return false;
	}

	const FScopedTransaction Transaction(LOCTEXT("PasteDecorations", "Paste Decorations"));

	int32 NumPasted = 0;
	int32 NumIncompatible = 0;
	int32 NumDuplicates = 0;

	for (UMovieSceneDecorationContainerObject* Container : PasteDecorationsParams.Containers)
	{
		if (!Container)
		{
			continue;
		}

		TSet<UClass*> CompatibleDecorations;
		IMovieSceneModule::Get().GetCompatibleDecorationsForContainer(Container, CompatibleDecorations);

		for (UObject* SourceDecoration : SourceDecorations)
		{
			if (!SourceDecoration)
			{
				continue;
			}

			UClass* DecorationClass = SourceDecoration->GetClass();
			if (!CompatibleDecorations.Contains(DecorationClass))
			{
				++NumIncompatible;
				continue;
			}

			// Decorations are singletons today (FindDecoration / GetOrCreateDecoration are class-keyed).
			if (Container->FindDecoration(DecorationClass) != nullptr)
			{
				++NumDuplicates;
				continue;
			}

			Container->Modify();
			UObject* NewDecoration = StaticDuplicateObject(SourceDecoration, Container);
			NewDecoration->ClearFlags(RF_Transient);
			ForEachObjectWithOuter(NewDecoration, [](UObject* InObject) {
				if (!InObject->GetClass() || !InObject->GetClass()->HasAnyClassFlags(CLASS_Transient))
				{
					InObject->ClearFlags(RF_Transient);
				}
			});

			if (IMovieSceneSectionProviderDecoration* Provider = Cast<IMovieSceneSectionProviderDecoration>(NewDecoration))
			{
				for (UMovieSceneSection* Section : Provider->GetSections())
				{
					if (Section)
					{
						Section->PostPaste();
					}
				}
			}

			Container->AddDecoration(NewDecoration);

			OutDecorations.Add(NewDecoration);
			++NumPasted;
		}
	}

	if (NumPasted == 0)
	{
		FText Reason;
		if (NumDuplicates > 0 && NumIncompatible == 0)
		{
			Reason = LOCTEXT("PasteDecorations_AlreadyExists", "Can't paste decoration. Target already has a decoration of that type.");
		}
		else
		{
			Reason = LOCTEXT("PasteDecorations_Incompatible", "Can't paste decoration. Target does not support that decoration type.");
		}
		FNotificationInfo Info(Reason);
		OutErrors.Add(Info);
		return false;
	}

	return true;
}

bool FSequencerUtilities::CanPasteDecorations(const FString& TextToImport)
{
	FDecorationObjectTextFactory Factory;
	return Factory.CanCreateObjectsFromText(TextToImport);
}

/**
 *
 * Copy/paste object bindings
 *
 */

void ExportObjectBindingsToText(const TArray<UMovieSceneCopyableBinding*>& ObjectsToExport, FOutputDevice& Archive, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	// Clear the mark state for saving.
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	const FExportObjectInnerContext Context;

	// Export each of the selected nodes
	UObject* LastOuter = nullptr;

	for (UMovieSceneCopyableBinding* ObjectToExport : ObjectsToExport)
	{
		if (!ObjectToExport)
		{
			continue;
		}

		// The nodes should all be from the same scope
		UObject* ThisOuter = ObjectToExport->GetOuter();
		check((LastOuter == ThisOuter) || (LastOuter == nullptr));
		LastOuter = ThisOuter;

		// We can't use TextExportTransient on USTRUCTS (which our object contains) so we're going to manually null out some references before serializing them. These references are
		// serialized manually into the archive, as the auto-serialization will only store a reference (to a privately owned object) which creates issues on deserialization. Attempting 
		// to deserialize these private objects throws a superflous error in the console that makes it look like things went wrong when they're actually OK and expected.
		TArray<UMovieSceneTrack*> OldTracks = ObjectToExport->Binding.StealTracks(nullptr);

		TArray<UObject*, TInlineAllocator<1>> OldObjectTemplates;
		TArray<UMovieSceneCustomBinding*, TInlineAllocator<1>> OldCustomBindings;
		TMap<int32, UMovieSceneSpawnableBindingBase*> OldPreviewSpawnables;
		if (ObjectToExport->Spawnable.GetGuid().IsValid())
		{
			OldObjectTemplates.Add(ObjectToExport->Spawnable.GetObjectTemplate());
			ObjectToExport->Spawnable.SetObjectTemplate(nullptr);
		}
		else
		{
			
			for (int32 CustomBindingIndex = 0; CustomBindingIndex < ObjectToExport->CustomBindings.Num(); ++CustomBindingIndex)
			{
				UMovieSceneCustomBinding* CustomBinding = ObjectToExport->CustomBindings[CustomBindingIndex];
				if (UMovieSceneSpawnableBindingBase* SpawnableBinding = Cast<UMovieSceneSpawnableBindingBase>(CustomBinding))
				{
					if (SpawnableBinding->SupportsObjectTemplates())
					{
						OldObjectTemplates.Add(SpawnableBinding->GetObjectTemplate());
						SpawnableBinding->SetObjectTemplate(nullptr);
					}
				}
				else if (UMovieSceneReplaceableBindingBase * ReplaceableBinding = Cast<UMovieSceneReplaceableBindingBase>(CustomBinding))
				{
					// Prevent inner references here during export
					if (ReplaceableBinding->PreviewSpawnable)
					{ 
						OldPreviewSpawnables.Add(CustomBindingIndex, ReplaceableBinding->PreviewSpawnable);
						// The Preview Spawnable is next in the CustomBindings list
						ObjectToExport->PreviewSpawnableBindings.Add(CustomBindingIndex+1);
						ReplaceableBinding->PreviewSpawnable = nullptr;
					}
				}
				OldCustomBindings.Add(CustomBinding);
			}
			ObjectToExport->CustomBindings.Empty();
		}

		ObjectToExport->NumCustomBindings = OldCustomBindings.Num();
		ObjectToExport->NumSpawnableObjectTemplates = OldObjectTemplates.Num();

		UExporter::ExportToOutputDevice(&Context, ObjectToExport, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, ThisOuter);

		// Restore the references (as we don't want to modify the original in the event of a copy operation!)
		ObjectToExport->Binding.SetTracks(MoveTemp(OldTracks), nullptr);

		ObjectToExport->CustomBindings.Append(OldCustomBindings);

		for (UMovieSceneCustomBinding* CustomBinding : ObjectToExport->CustomBindings)
		{
			if (CustomBinding)
			{
				UExporter::ExportToOutputDevice(&Context, CustomBinding, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited);
			}
		}

		// Restore replaceable references now that we've exported them
		for (const TPair<int32, UMovieSceneSpawnableBindingBase*>& PreviewSpawnable : OldPreviewSpawnables)
		{
			if (UMovieSceneReplaceableBindingBase* ReplaceableBinding = Cast<UMovieSceneReplaceableBindingBase>(ObjectToExport->CustomBindings[PreviewSpawnable.Key].Get()))
			{
				ReplaceableBinding->PreviewSpawnable = PreviewSpawnable.Value;
			}
		}

		int32 ObjectTemplateIndex = 0;
		if (ObjectToExport->Spawnable.GetGuid().IsValid())
		{
			ObjectToExport->Spawnable.SetObjectTemplate(OldObjectTemplates[ObjectTemplateIndex++]);
		}
		else
		{
			for (UMovieSceneCustomBinding* CustomBinding : ObjectToExport->CustomBindings)
			{
				if (UMovieSceneSpawnableBindingBase* SpawnableBinding = CustomBinding->AsSpawnable(SharedPlaybackState))
				{
					// Ignore bindings with their template already set, which is possible for Replaceables since they'll show up twice in the list.
					if (SpawnableBinding->SupportsObjectTemplates() && !SpawnableBinding->GetObjectTemplate())
					{
						SpawnableBinding->SetObjectTemplate(OldObjectTemplates[ObjectTemplateIndex++]);
					}
				}
			}
		}

		// We manually export the object templates for the same private-ownership reason as above. Templates need to be re-created anyways as each Spawnable contains its own copy of the template.
		for (UObject* ObjectTemplate : ObjectToExport->SpawnableObjectTemplates)
		{
			if (ObjectTemplate)
			{
				UExporter::ExportToOutputDevice(&Context, ObjectTemplate, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited);
			}
		}
	}
}

void FSequencerUtilities::CopyBindings(TSharedRef<ISequencer> Sequencer, const TArray<FMovieSceneBindingProxy>& Bindings, const TArray<UMovieSceneFolder*>& InFolders, FString& ExportedText)
{
	FStringOutputDevice Archive;
	CopyBindings(Sequencer, Bindings, InFolders, (FOutputDevice&)Archive);
	ExportedText = Archive;
}

void FSequencerUtilities::CopyBindings(TSharedRef<ISequencer> Sequencer, const TArray<FMovieSceneBindingProxy>& Bindings, const TArray<UMovieSceneFolder*>& InFolders, FOutputDevice& Ar)
{
	UObject* PlaybackContext = Sequencer->GetPlaybackContext();
	UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

	TArray<UMovieSceneCopyableBinding*> Objects;
	for (const FMovieSceneBindingProxy& ObjectBinding : Bindings)
	{
		UMovieSceneCopyableBinding* CopyableBinding = NewObject<UMovieSceneCopyableBinding>(GetTransientPackage(), UMovieSceneCopyableBinding::StaticClass(), NAME_None, RF_Transient);
		Objects.Add(CopyableBinding);

		UMovieScene* MovieScene = ObjectBinding.GetMovieScene();
		if (!MovieScene)
		{
			continue;
		}

		FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectBinding.BindingID);
		if (Possessable)
		{
			CopyableBinding->Possessable = *Possessable;

			// Store any custom bindings
			if (FMovieSceneBindingReferences* BindingReferences = ObjectBinding.Sequence->GetBindingReferences())
			{
				int32 BindingIndex = 0;
				for (const FMovieSceneBindingReference& BindingReference : BindingReferences->GetReferences(ObjectBinding.BindingID))
				{
					if (UMovieSceneCustomBinding* CustomBinding = BindingReference.CustomBinding)
					{
						CopyableBinding->CustomBindings.Add(CustomBinding);

						if (UMovieSceneSpawnableBindingBase* SpawnableBinding = CustomBinding->AsSpawnable(Sequencer->GetSharedPlaybackState()))
						{
							if (SpawnableBinding->SupportsObjectTemplates())
							{
								// We manually serialize the spawnable object template so that it's not a reference to a privately owned object. Spawnables all have unique copies of their template objects anyways.
								// Object Templates are re-created on paste (based on these templates) with the correct ownership set up.
								CopyableBinding->SpawnableObjectTemplates.Add(SpawnableBinding->GetObjectTemplate());
							}

							// This is the inner spawnable of a replaceable and is always placed after the replaceable in the list
							if (SpawnableBinding != CustomBinding)
							{
								CopyableBinding->CustomBindings.Add(SpawnableBinding);
							}
						}
					}
					else
					{
						TArray<UObject*> BoundObjects = MovieSceneHelpers::GetBoundObjects(Sequencer->GetFocusedTemplateID(), ObjectBinding.BindingID, Sequencer->GetSharedPlaybackState(), BindingIndex);
						if (BoundObjects.Num() > 0)
						{
							CopyableBinding->BoundObjectNames.Add(BoundObjects[0]->GetPathName(World));
						}

					}
					BindingIndex++;
				}
			}
			else
			{
				// Store the names of the bound objects so that they can be found on paste

				for (TWeakObjectPtr<> RuntimeObject : Sequencer->FindBoundObjects(CopyableBinding->Possessable.GetGuid(), Sequencer->GetFocusedTemplateID()))
				{
					CopyableBinding->BoundObjectNames.Add(RuntimeObject->GetPathName(World));
				}
			}
		}
		else
		{
			FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(ObjectBinding.BindingID);
			if (Spawnable)
			{
				CopyableBinding->Spawnable = *Spawnable;

				// We manually serialize the spawnable object template so that it's not a reference to a privately owned object. Spawnables all have unique copies of their template objects anyways.
				// Object Templates are re-created on paste (based on these templates) with the correct ownership set up.
				CopyableBinding->SpawnableObjectTemplates.Add(Spawnable->GetObjectTemplate());
			}
		}

		const FMovieSceneBinding* Binding = MovieScene->FindBinding(ObjectBinding.BindingID);
		if (Binding)
		{
			CopyableBinding->Binding = *Binding;

			TMap<UObject*, UObject*> SourceToDuplicate;
			for (UMovieSceneTrack* Track : Binding->GetTracks())
			{
				// Tracks suffer from the same issues as Spawnable's Object Templates (reference to a privately owned object). We'll manually serialize the tracks to copy them,
				// and then restore them on paste.
				UMovieSceneTrack* DuplicatedTrack = Cast<UMovieSceneTrack>(StaticDuplicateObject(Track, CopyableBinding));

				SourceToDuplicate.Add(Track, DuplicatedTrack);
				CopyableBinding->Tracks.Add(DuplicatedTrack);
			}

			// A track may reference sibling tracks in the same binding by pointer rather than
			// owning them as subobjects (e.g. an Animation Mixer track and the Control Rig child
			// tracks that AddGivenTrack re-outers to the MovieScene). Duplicating each track in
			// isolation leaves those cross-track pointers aimed at the originals, so remap every
			// duplicate's references - including those held by its subobjects - onto the duplicates.
			for (const TPair<UObject*, UObject*>& Pair : SourceToDuplicate)
			{
				FArchiveReplaceObjectRef<UObject> ReplaceTrackRefs(Pair.Value, SourceToDuplicate);
			}
		}

		UMovieSceneFolder* Folder = nullptr;
		for (UMovieSceneFolder* RootFolder : MovieScene->GetRootFolders())
		{
			Folder = RootFolder->FindFolderContaining(ObjectBinding.BindingID);
			if (Folder && InFolders.Contains(Folder))
			{
				UMovieSceneFolder::CalculateFolderPath(Folder, InFolders, CopyableBinding->FolderPath);
				break;
			}
		}

		for (TPair<FName, FMovieSceneObjectBindingIDs> TaggedBinding : Sequencer->GetRootMovieSceneSequence()->GetMovieScene()->AllTaggedBindings())
		{
			if (TaggedBinding.Value.IDs.Contains(FMovieSceneObjectBindingID(UE::MovieScene::FFixedObjectBindingID(ObjectBinding.BindingID, Sequencer->GetFocusedTemplateID()))))
			{
				CopyableBinding->Tags.Add(TaggedBinding.Key);
			}
		}
	}

	ExportObjectBindingsToText(Objects, /*out*/ Ar, Sequencer->GetSharedPlaybackState());
}

class FObjectBindingTextFactory : public FCustomizableTextObjectFactory
{
public:
	FObjectBindingTextFactory(ISequencer& InSequencer)
		: FCustomizableTextObjectFactory(GWarn)
		, Sequencer(&InSequencer)
	{
	}

	// FCustomizableTextObjectFactory implementation
	virtual bool CanCreateClass(UClass* InObjectClass, bool& bOmitSubObjs) const override
	{
		if (InObjectClass->IsChildOf<UMovieSceneCopyableBinding>())
		{
			return true;
		}
		else if (InObjectClass->IsChildOf<UMovieSceneCustomBinding>())
		{
			return true;
		}
		else
		{
			return Sequencer->GetSpawnRegister().CanSpawnObject(InObjectClass);
		}
	}


	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		check(NewObject);

		if (NewObject->IsA<UMovieSceneCopyableBinding>())
		{
			UMovieSceneCopyableBinding* CopyableBinding = Cast<UMovieSceneCopyableBinding>(NewObject);
			NewCopyableBindings.Add(CopyableBinding);
		}
		else if (NewObject->IsA<UMovieSceneCustomBinding>())
		{
			NewCustomBindings.Add(Cast<UMovieSceneCustomBinding>(NewObject));
		}
		else
		{
			NewSpawnableObjectTemplates.Add(NewObject);
		}
	}

public:
	TArray<UMovieSceneCopyableBinding*> NewCopyableBindings;
	TArray<UObject*> NewSpawnableObjectTemplates;
	TArray<UMovieSceneCustomBinding*> NewCustomBindings;

private:
	ISequencer* Sequencer;
};

void ImportObjectBindingsFromText(ISequencer& InSequencer, const FString& TextToImport, /*out*/ TArray<UMovieSceneCopyableBinding*>& ImportedObjects)
{
	UPackage* TempPackage = NewObject<UPackage>(nullptr, TEXT("/Engine/Sequencer/Editor/Transient"), RF_Transient);
	TempPackage->AddToRoot();

	// Turn the text buffer into objects
	FObjectBindingTextFactory Factory(InSequencer);
	Factory.ProcessBuffer(TempPackage, RF_Transactional, TextToImport);
	ImportedObjects = Factory.NewCopyableBindings;

	// We had to explicitly serialize object templates due to them being a reference to a privately owned object. We now deserialize these object template copies
	// and match them up with their MovieSceneCopyableBinding again.

	int32 SpawnableObjectTemplateIndex = 0;
	int32 CustomBindingIndex = 0;
	for (auto ImportedObject : ImportedObjects)
	{
		if (ImportedObject->Spawnable.GetGuid().IsValid())
		{
			// This Spawnable Object Template is owned by our transient package, so you'll need to change the owner if you want to keep it later.
			ImportedObject->SpawnableObjectTemplates.Add(Factory.NewSpawnableObjectTemplates[SpawnableObjectTemplateIndex++]);
		}
		else if (CustomBindingIndex < Factory.NewCustomBindings.Num())
		{
			for(int32 Index = 0; Index < ImportedObject->NumCustomBindings; ++Index)
			{
				ImportedObject->CustomBindings.Add(Factory.NewCustomBindings[CustomBindingIndex++]);
			}

			if (ImportedObject->CustomBindings.Num() > 0 && SpawnableObjectTemplateIndex < Factory.NewSpawnableObjectTemplates.Num())
			{
				for (UMovieSceneCustomBinding* CustomBinding : ImportedObject->CustomBindings)
				{
					if (UMovieSceneSpawnableBindingBase* SpawnableBinding = Cast<UMovieSceneSpawnableBindingBase>(CustomBinding))
					{
						if (SpawnableBinding->SupportsObjectTemplates())
						{
							ImportedObject->SpawnableObjectTemplates.Add(Factory.NewSpawnableObjectTemplates[SpawnableObjectTemplateIndex++]);
						}
					}
				}
			}
		}
	}

	// Remove the temp package from the root now that it has served its purpose
	TempPackage->RemoveFromRoot();
}

FGuid TryCreateCustomBinding(TSharedPtr<ISequencer> Sequencer, UObject* CustomBindingObject, AActor* FactoryCreatedActor, FMovieSceneBindingReferences* BindingReferences, const UE::Sequencer::FCreateBindingParams& InParams, UMovieScene* OwnerMovieScene, bool bSpawnable, bool bReplaceable)
{
	UMovieSceneCustomBinding * NewCustomBinding = nullptr;

	// If the passed in object is a UClass, and we have an actor factory created instance, prioritize that, otherwise let the binding choose
	if (CustomBindingObject && FactoryCreatedActor && CustomBindingObject->IsA<UClass>())
	{
		CustomBindingObject = FactoryCreatedActor;
		FactoryCreatedActor = nullptr;
	}

	if (InParams.CustomBinding)
	{
		const FMovieSceneBindingReference* PreviousBindingReference = BindingReferences ? BindingReferences->GetReference(InParams.ReplacementGuid, InParams.BindingIndex) : nullptr;
		// We've been provided a custom binding pre-created. Ensure it supports the object given
		if (CustomBindingObject == nullptr 
			|| InParams.CustomBinding->SupportsBindingCreationFromObject(CustomBindingObject)
			|| (PreviousBindingReference && InParams.CustomBinding->SupportsConversionFromBinding(*PreviousBindingReference, CustomBindingObject)))
		{
			NewCustomBinding = InParams.CustomBinding;
		}
	}
	else
	{
		TArrayView<const TSubclassOf<UMovieSceneCustomBinding>> PrioritySortedCustomBindingTypes;
		if (Sequencer)
		{
			PrioritySortedCustomBindingTypes = Sequencer->GetSupportedCustomBindingTypes();
		}
		else
		{
			static TArray<const TSubclassOf<UMovieSceneCustomBinding>> CachedCustomBindingTypes;
			static bool CustomBindingTypesCached = false;
			if (!CustomBindingTypesCached)
			{
				CustomBindingTypesCached = true;
				MovieSceneHelpers::GetPrioritySortedCustomBindingTypes(CachedCustomBindingTypes);
			}
			PrioritySortedCustomBindingTypes = CachedCustomBindingTypes;
		}

		for (const TSubclassOf<UMovieSceneCustomBinding>& CustomBindingType : PrioritySortedCustomBindingTypes)
		{
			// If 'spawnable' has been passed in, we can use children of UMovieSceneSpawnableBindingBase
			// If 'replaceable' has been passed in, we can use children of UMovieSceneReplaceableBindingBase
			// Otherwise if neither has been passed in, we only want to use bindings that aren't children of either.
			const bool bIsCustomSpawnableBinding = CustomBindingType->IsChildOf<UMovieSceneSpawnableBindingBase>();
			const bool bIsCustomReplaceableBinding = CustomBindingType->IsChildOf<UMovieSceneReplaceableBindingBase>();
			if ((bSpawnable && bIsCustomSpawnableBinding) ||
				(bReplaceable && bIsCustomReplaceableBinding) ||
				(!bSpawnable && !bReplaceable && !bIsCustomSpawnableBinding && !bIsCustomReplaceableBinding))
			{
				if (UMovieSceneCustomBinding* CustomBindingCDO = CustomBindingType ? CustomBindingType->GetDefaultObject<UMovieSceneCustomBinding>() : nullptr)
				{
					if (CustomBindingObject && CustomBindingCDO->SupportsBindingCreationFromObject(CustomBindingObject))
					{
						// Create a custom binding from this Object
						NewCustomBinding = CustomBindingCDO->CreateNewCustomBinding(CustomBindingObject, *OwnerMovieScene);
						if (NewCustomBinding)
						{
							break;
						}
					}

					if (!NewCustomBinding && FactoryCreatedActor && CustomBindingCDO->SupportsBindingCreationFromObject(FactoryCreatedActor))
					{
						// Create a custom binding from the factory created actor
						NewCustomBinding = CustomBindingCDO->CreateNewCustomBinding(FactoryCreatedActor, *OwnerMovieScene);
						if (NewCustomBinding)
						{
							break;
						}
					}
				}
			}
		}
	}

	if (NewCustomBinding)
	{
		FString DesiredBindingName = NewCustomBinding->GetDesiredBindingName();
		FString CurrentName = DesiredBindingName.IsEmpty() ? 
			(InParams.BindingNameOverride.IsEmpty() && CustomBindingObject ? FName::NameToDisplayString(CustomBindingObject->GetName(), false) : InParams.BindingNameOverride)
			: DesiredBindingName;
		CurrentName = MovieSceneHelpers::MakeUniqueBindingName(OwnerMovieScene, CurrentName);

		FMovieScenePossessable* NewPossessable = nullptr;
		FGuid NewID;
		if (InParams.ReplacementGuid.IsValid())
		{
			NewID = InParams.ReplacementGuid;
			NewPossessable = OwnerMovieScene->FindPossessable(InParams.ReplacementGuid);
		}
		if (!NewPossessable)
		{
			// Add a possessable binding track- we will use these even if the custom binding is a 'spawnable' one
			NewID = OwnerMovieScene->AddPossessable(CurrentName, NewCustomBinding->GetBoundObjectClass());
			NewPossessable = OwnerMovieScene->FindPossessable(NewID);
		}

		// Add the custom binding
		if (BindingReferences)
		{
			BindingReferences->AddOrReplaceBinding(NewID, NewCustomBinding, InParams.BindingIndex);
		}

		UObject* SpawnedObject = nullptr;

		// If this is a spawnable or replaceable binding, we need to set up some defaults
		if (Sequencer)
		{
			if (NewCustomBinding->WillSpawnObject(Sequencer->GetSharedPlaybackState()))
			{
				// Spawn the object so we can position it correctly, it's going to get spawned anyway since things default to spawned.
				SpawnedObject = Sequencer->GetSpawnRegister().SpawnObject(NewID, *OwnerMovieScene, Sequencer->GetFocusedTemplateID(), Sequencer->GetSharedPlaybackState(), 0);
			}
		}

		// Allow the binding to set up any necessary defaults
		NewCustomBinding->SetupDefaults(SpawnedObject, NewID, *OwnerMovieScene);

		if (Sequencer)
		{
			if (InParams.bSetupDefaults)
			{
				FTransformData TransformData;
				Sequencer->GetSpawnRegister().SetupDefaultsForSpawnable(SpawnedObject, NewID, TransformData, Sequencer.ToSharedRef(), Sequencer->GetSequencerSettings());
			}
			Sequencer->GetEvaluationState()->Invalidate(NewID, Sequencer->GetFocusedTemplateID());
			Sequencer->ForceEvaluate();

			// We don't call these events in the case bSetupDefaults is false because they may add tracks.
			if (InParams.bSetupDefaults)
			{
				if (AActor* Actor = Cast<AActor>(SpawnedObject))
				{
					Sequencer->OnActorAddedToSequencer().Broadcast(Actor, NewID);
				}

				Sequencer->OnAddBinding(NewID, OwnerMovieScene);
			}
		}

		return NewID;
	}
	return FGuid();
}

FGuid CreateGenericBinding(TSharedPtr<ISequencer> Sequencer, UMovieSceneSequence* OwnerSequence, UObject* InObject, FMovieSceneBindingReferences* BindingReferences, const UE::Sequencer::FCreateBindingParams& InParams)
{
	if (!OwnerSequence)
	{
		return FGuid();
	}

	using namespace UE::Sequencer;

	UMovieScene* OwnerMovieScene = OwnerSequence->GetMovieScene();
	const FGuid ParentBindingOverride = (Sequencer && InObject)
		? Private::TakeCreateBindingParentOverride(*Sequencer, *InObject) : FGuid();

	ISequencerModule& Module = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	bool bSpawnable = InParams.bSpawnable && OwnerSequence->AllowsSpawnableObjects();
	bool bAllowCustom = InParams.bAllowCustomBinding && OwnerSequence->AllowsCustomBindings();
	bool bReplaceable = InParams.bReplaceable && bAllowCustom;
	FGuid NewBindingID;
	// First see if any custom bindings support creation from this object type directly. 

	if (bAllowCustom)
	{
		// In addition to the raw object, we also try spawning an actor from an actor factory if relevant, to give the custom binding an option to create from that as well
		AActor* FactoryCreatedActorInstance = nullptr;

		UWorld* World = nullptr;
		if (Sequencer)
		{
			UObject* PlaybackContext = Sequencer->GetPlaybackContext();
			World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
		}
		else
		{
			World = GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr;
		}

		if (InObject && !InObject->IsA<AActor>())
		{
			// Workaround for a bug in UActorFactoryBlueprint- the actor factory will claim it can create an actor for a blueprint generated class, but then fail to do so
			// This pattern of redirecting to the UBlueprint asset is present also in FAssetData constructor.
			const UClass* InClass = Cast<UClass>(InObject);
			if (InClass && InClass->ClassGeneratedBy)
			{
				InObject = InClass->ClassGeneratedBy;
			}

			// If the passed in object is not an actor, see if we can create an Actor from it, and if so, if that Actor type has a custom binding that supports it
			UActorFactory* FactoryToUse = InParams.ActorFactory ? InParams.ActorFactory.Get() : FActorFactoryAssetProxy::GetFactoryForAssetObject(InObject);

			if (FactoryToUse)
			{
				FText ErrorMessage;
				if (FactoryToUse->CanCreateActorFrom(FAssetData(InObject), ErrorMessage))
				{
					if (World)
					{
						const FName ActorName = MakeUniqueObjectName(World->PersistentLevel, FactoryToUse->NewActorClass->StaticClass(), *InParams.BindingNameOverride);

						FActorSpawnParameters SpawnParams;
						SpawnParams.ObjectFlags = RF_Transient | RF_Transactional;
						SpawnParams.Name = ActorName;

						FactoryCreatedActorInstance = FactoryToUse->CreateActor(InObject, World->PersistentLevel, FTransform(), SpawnParams);
						if (FactoryCreatedActorInstance)
						{
							FactoryCreatedActorInstance->SetActorLabel(MovieSceneHelpers::MakeUniqueBindingName(OwnerMovieScene, FName::NameToDisplayString(InObject->GetName(), false)));
							FactoryCreatedActorInstance->bIsEditorPreviewActor = false;
						}
					}
				}
			}
		}
		NewBindingID = TryCreateCustomBinding(Sequencer, InObject, FactoryCreatedActorInstance, BindingReferences, InParams, OwnerMovieScene, bSpawnable, bReplaceable);
		if (FactoryCreatedActorInstance && World)
		{
			const bool bNetForce = false;
			const bool bShouldModifyLevel = false;
			World->DestroyActor(FactoryCreatedActorInstance, bNetForce, bShouldModifyLevel);
			FactoryCreatedActorInstance = nullptr;
		}
		if (NewBindingID.IsValid())
		{
			return NewBindingID;
		}
	}

	if (!InObject && !InParams.bAllowEmptyBinding)
	{
		return FGuid();
	}


	// If no custom bindings support this object type, but InParams.bSpawnable is true, attempt to make an old-style spawnable.
	if (!BindingReferences)
	{
		if (InObject && bSpawnable && Sequencer)
		{
			NewBindingID = FSequencerUtilities::MakeNewSpawnable(Sequencer.ToSharedRef(), *InObject, InParams.ActorFactory);
			if (NewBindingID.IsValid())
			{
				return NewBindingID;
			}
		}
	}
	
	// Otherwise, create a possessable.




	TArray<TPair<UObject*, const IObjectSchema*>> ObjectsToPossess;
	auto Iterator = [&ObjectsToPossess, &InParams](UObject* CurrentObject, const IObjectSchema& Schema)
	{
		ObjectsToPossess.Add(MakeTuple(CurrentObject, &Schema));
	};

	if (!InObject)
	{
		ObjectsToPossess.Add(MakeTuple(InObject, nullptr));
	}
	else if (!IteratePathToObject(InObject, Iterator))
	{
		return FGuid();
	}


	// Nothing to possess?
	if (ObjectsToPossess.Num() == 0)
	{
		// We've failed to find a custom binding type 

		return FGuid();
	}

	const bool bParentContextsAreSignificant = OwnerSequence->AreParentContextsSignificant();


	UObject* Context = Sequencer ? Sequencer->GetPlaybackContext() : nullptr;

	FGuid ParentID;
	int32 StartObjectIndex = 0;
	bool bUsingParentBinding = false;

	if (ParentBindingOverride.IsValid() && ObjectsToPossess.Num() > 0)
	{
		const bool bParentBindingExists = OwnerMovieScene->FindPossessable(ParentBindingOverride) != nullptr
			|| OwnerMovieScene->FindSpawnable(ParentBindingOverride) != nullptr;
		if (bParentBindingExists)
		{
			ParentID = ParentBindingOverride;
			StartObjectIndex = ObjectsToPossess.Num() - 1;
			bUsingParentBinding = true;

			if (bParentContextsAreSignificant && StartObjectIndex > 0)
			{
				Context = ObjectsToPossess[StartObjectIndex - 1].Key;
			}
		}
	}

	for (int32 Index = StartObjectIndex; Index < ObjectsToPossess.Num(); ++Index)
	{
		UObject*             CurrentObject = ObjectsToPossess[Index].Key;
		const IObjectSchema* Schema        = ObjectsToPossess[Index].Value;

		// If we're not purposefully replacing a binding, then check to see if we already have one, and use that
		if (Sequencer && !bUsingParentBinding && !InParams.ReplacementGuid.IsValid() && CurrentObject)
		{
			FGuid    ObjectGuid = Sequencer->FindCachedObjectId(*CurrentObject, Sequencer->GetFocusedTemplateID());

			// If the object already has a binding, use that and move on
			if (ObjectGuid.IsValid())
			{
				ParentID = ObjectGuid;
				if (bParentContextsAreSignificant)
				{
					Context = CurrentObject;
				}
				continue;
			}
		}


		// Create a new binding for this object
		FMovieScenePossessable* NewPossessable = nullptr;
		FGuid NewID;
		if (InParams.ReplacementGuid.IsValid() && !ParentID.IsValid())
		{
			NewID = InParams.ReplacementGuid;
			NewPossessable = OwnerMovieScene->FindPossessable(InParams.ReplacementGuid);
		}
		if (!NewPossessable)
		{
			FString CurrentName = (Index == ObjectsToPossess.Num()-1 && InParams.BindingNameOverride.Len() != 0)
				? InParams.BindingNameOverride
				: CurrentObject ? Schema->GetPrettyName(CurrentObject).ToString() : TEXT("EmptyBinding");
			UClass* CurrentClass = CurrentObject ? CurrentObject->GetClass() : nullptr;

			NewID = OwnerMovieScene->AddPossessable(CurrentName, CurrentClass);
			NewPossessable = OwnerMovieScene->FindPossessable(NewID);
		}

		// If we're not trying to replace a binding, and the object is a spawnable, try and bind to that first
		if (InParams.ReplacementGuid.IsValid()
			|| (Sequencer && (!CurrentObject || !NewPossessable->BindSpawnableObject(Sequencer->GetFocusedTemplateID(), CurrentObject, Sequencer->GetSharedPlaybackState()))))
		{
			FUniversalObjectLocator Locator;
			if (CurrentObject && (!OwnerSequence->MakeLocatorForObject(CurrentObject, Context, Locator) || Locator.IsEmpty()))
			{
				// Unable to possess this object
				return FGuid();
			}

			if (InParams.ReplacementGuid.IsValid() && !ParentID.IsValid())
			{
				if (BindingReferences)
				{
					BindingReferences->AddOrReplaceBinding(NewID, MoveTemp(Locator), InParams.BindingIndex);
				}
				if (Sequencer)
				{
					Sequencer->GetEvaluationState()->Invalidate(NewID, Sequencer->GetFocusedTemplateID());
				}
			}
			else
			{
				if (BindingReferences)
				{
					BindingReferences->AddBinding(NewID, MoveTemp(Locator));
				}
			}

			// Fixup the possessed class in case the bound object is actually different than the one we tried to possess
			NewPossessable->FixupPossessedObjectClass(OwnerSequence, Context);
		}

		if (ParentID.IsValid())
		{
			NewPossessable->SetParent(ParentID, OwnerMovieScene);

			FMovieSceneSpawnable* ParentSpawnable = OwnerMovieScene->FindSpawnable(ParentID);
			if (ParentSpawnable)
			{
				ParentSpawnable->AddChildPossessable(NewID);
			}
		}

		ParentID = NewID;
		if (Sequencer)
		{
			if (AActor* Actor = Cast<AActor>(CurrentObject))
			{
				Sequencer->OnActorAddedToSequencer().Broadcast(Actor, NewID);
			}
		}

		// If this is the last one
		if (Index == ObjectsToPossess.Num()-1)
		{
			if (Sequencer)
			{
				Sequencer->OnAddBinding(NewID, OwnerMovieScene);
			}
			return NewID;
		}

		if (bParentContextsAreSignificant)
		{
			Context = CurrentObject;
		}
	}

	// Should never get here - we should always hit the Index == ObjectsToPossess.Num()-1 condition inside the loop
	return FGuid();
}

bool FSequencerUtilities::PasteBindings(const FString& TextToImport, TSharedRef<ISequencer> Sequencer, FMovieScenePasteBindingsParams PasteBindingsParams, TArray<FMovieSceneBindingProxy>& OutBindings, TArray<FNotificationInfo>& OutErrors)
{
	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return false;
	}

	FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences();

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return false;
	}

	UMovieScene* RootMovieScene = Sequencer->GetRootMovieSceneSequence()->GetMovieScene();

	UObject* PlaybackContext = Sequencer->GetPlaybackContext();
	UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

	const FScopedTransaction Transaction(LOCTEXT("PasteBindings", "Paste Bindings"));

	TMap<FGuid, FGuid> OldToNewGuidMap;
	TArray<FGuid> PossessableGuids;
	TArray<FGuid> SpawnableGuids;
	TMap<FGuid, UMovieSceneFolder*> GuidToFolderMap;

	TArray<FMovieSceneBinding> BindingsPasted;

	const int NumTargets = FMath::Max(1, PasteBindingsParams.Bindings.Num());

	for (int32 TargetIndex = 0; TargetIndex < NumTargets; ++TargetIndex)
	{
		TArray<UMovieSceneCopyableBinding*> ImportedBindings;
		ImportObjectBindingsFromText(Sequencer.Get(), TextToImport, ImportedBindings);

		if (ImportedBindings.Num() == 0)
		{
			return false;
		}

		TArray<UObject*> SectionSubObjects;
		for (UMovieSceneCopyableBinding* CopyableBinding : ImportedBindings)
		{
			// Clear transient flags on the imported tracks
			for (UMovieSceneTrack* CopiedTrack : CopyableBinding->Tracks)
			{
				ResetCopiedTracksFlags(CopiedTrack);
			}

			UMovieSceneFolder* ParentFolder = PasteBindingsParams.ParentFolder;

			if (CopyableBinding->FolderPath.Num() > 0)
			{
				ParentFolder = UMovieSceneFolder::GetFolderWithPath(CopyableBinding->FolderPath, PasteBindingsParams.Folders, ParentFolder ? ParentFolder->GetChildFolders() : MovieScene->GetRootFolders());
			}

			if (CopyableBinding->Possessable.GetGuid().IsValid())
			{
				// TODO: We likely need additional work here for possessable bindings using locators other than actor locators.
				// For now, we'll at least handle the custom binding case

				// If we have a custom binding, we need to let the sequence create it, especially since it could have a spawnable template.
				// However, making a new custom spawnable also creates the binding for us - this is a problem
				// because we need to use our binding (which has tracks associated with it). To solve this, we let it create
				// an object template based off of our (transient package owned) template, then find the newly created binding
				// and update it.

				FGuid NewGuid;
				if (!CopyableBinding->CustomBindings.IsEmpty())
				{
					if (BindingReferences)
					{
						int32 SpawnableBindingIndex = 0;
						UMovieSceneCustomBinding* PreviousBinding = nullptr;
						for(int32 BindingIndex = 0; BindingIndex < CopyableBinding->CustomBindings.Num(); ++BindingIndex)
						{
							UMovieSceneCustomBinding* CustomBinding = CopyableBinding->CustomBindings[BindingIndex];
							if (CustomBinding)
							{
								UMovieSceneCustomBinding* NewCustomBinding = Cast<UMovieSceneCustomBinding>(StaticDuplicateObject(CustomBinding, MovieScene));

								// Need to re-copy the object template to avoid private object issues
								if (UMovieSceneSpawnableBindingBase* SpawnableBinding = Cast<UMovieSceneSpawnableBindingBase>(NewCustomBinding))
								{
									if (CopyableBinding->SpawnableObjectTemplates.IsValidIndex(SpawnableBindingIndex))
									{
										UObject* SpawnableObjectTemplate = CopyableBinding->SpawnableObjectTemplates[SpawnableBindingIndex++];
										if (SpawnableObjectTemplate)
										{
											UObject* NewObjectTemplate = StaticDuplicateObject(SpawnableObjectTemplate, MovieScene);
											SpawnableBinding->SetObjectTemplate(NewObjectTemplate);
										}
									}

									// If this is a preview spawnable, find the just added replaceable and link with that rather than creating a new binding
									if (CopyableBinding->PreviewSpawnableBindings.Contains(BindingIndex))
									{
										if (UMovieSceneReplaceableBindingBase* PreviousReplaceableBinding = Cast<UMovieSceneReplaceableBindingBase>(PreviousBinding))
										{
											PreviousReplaceableBinding->PreviewSpawnable = SpawnableBinding;
											PreviousBinding = NewCustomBinding;
											continue;
										}
									}
								}

								// This will either add a brand new possessable and binding (if one doesn't exist for that guid), or just add a new binding to that same possessable
								UE::Sequencer::FCreateBindingParams CreateBindingParams;
								CreateBindingParams.ReplacementGuid = NewGuid;
								CreateBindingParams.BindingIndex = BindingIndex;
								CreateBindingParams.bAllowCustomBinding = true;
								CreateBindingParams.CustomBinding = NewCustomBinding;
								CreateBindingParams.bSetupDefaults = false;
								CreateBindingParams.BindingNameOverride = CopyableBinding->Possessable.GetName();
								NewGuid = CreateGenericBinding(Sequencer, Sequence, nullptr, BindingReferences, CreateBindingParams);

								PreviousBinding = NewCustomBinding;
							}
						}
					}
				}
				else
				{
					NewGuid = MovieScene->AddPossessable(CopyableBinding->Possessable.GetName(), (UClass*)CopyableBinding->Possessable.GetLoadedPossessedObjectClass());
					FMovieScenePossessable* Possessable = MovieScene->FindPossessable(NewGuid);
					Possessable->SetParent(CopyableBinding->Possessable.GetParent(), MovieScene);
				}

				FMovieSceneBinding* NewBinding = MovieScene->FindBinding(NewGuid);
				NewBinding->SetTracks(TArray<UMovieSceneTrack*>(CopyableBinding->Tracks), nullptr);
				FMovieScenePossessable* Possessable = MovieScene->FindPossessable(NewGuid);

				// Clear the transient flags on the copyable binding before assigning to the new possessable
				for (UMovieSceneTrack* Track : NewBinding->GetTracks())
				{
					ResetCopiedTracksFlags(Track);
				}

				// Replace the auto-generated binding with our deserialized bindings (which has our tracks)
				MovieScene->ReplaceBinding(NewGuid, *NewBinding);

				OldToNewGuidMap.Add(CopyableBinding->Possessable.GetGuid(), NewGuid);

				BindingsPasted.Add(*NewBinding);

				PossessableGuids.Add(NewGuid);

				if (ParentFolder)
				{
					GuidToFolderMap.Add(NewGuid, ParentFolder);
				}

				if (CopyableBinding->Tags.Num() > 0)
				{
					RootMovieScene->Modify();

					for (const FName& Tag : CopyableBinding->Tags)
					{
						RootMovieScene->TagBinding(Tag, UE::MovieScene::FFixedObjectBindingID(NewGuid, Sequencer->GetFocusedTemplateID()));
					}
				}

				// Find the objects that this pasted binding should bind to
				TArray<UObject*> ObjectsToBind;

				UObject* ResolutionContext = FindResolutionContext(Sequencer
					, *MovieScene->GetTypedOuter<UMovieSceneSequence>()
					, *MovieScene
					, Possessable->GetParent()
					, Sequencer->GetPlaybackContext());

				if (World)
				{
					// If PastedActors are specified, replace the existing bound object name with the matching pasted actor path name
					for (const TPair<FName, TObjectPtr<AActor>>& PastedActorPair : PasteBindingsParams.PastedActors)
					{
						for (FString& BoundObjectName : CopyableBinding->BoundObjectNames)
						{
							if (BoundObjectName == PastedActorPair.Key)
							{
								BoundObjectName = PastedActorPair.Value->GetPathName(World);
							}
						}
					}

					for (TActorIterator<AActor> ActorItr(World); ActorItr; ++ActorItr)
					{
						AActor* Actor = *ActorItr;
						if (Actor && CopyableBinding->BoundObjectNames.Contains(Actor->GetPathName(World)))
						{
							// If this actor is already bound and we're not duplicating actors, don't bind to anything
							if (!PasteBindingsParams.bDuplicateExistingActors && Sequencer->FindObjectId(*Actor, Sequencer->GetFocusedTemplateID()).IsValid())
							{
								continue;
							}

							ObjectsToBind.Add(Actor);
							CopyableBinding->BoundObjectNames.Remove(Actor->GetPathName(World));
						}
					}
				}

				bool bSetParent = false;
				if (const UClass* PossessedObjectClass = CopyableBinding->Possessable.GetPossessedObjectClass())
				{
					if (!PossessedObjectClass->IsChildOf(AActor::StaticClass()))
					{
						// Attempt to set the parent to be the paste target only if the possessed object class is not an actor
						bSetParent = true;
					}
				}
				else if (ObjectsToBind.IsEmpty())
				{
					// Attempt to set the parent to be the paste target only if the binding does not resolve to an actor in the world.
					bSetParent = true;
				}
				
				if (bSetParent)
				{
					if (TargetIndex < PasteBindingsParams.Bindings.Num())
					{
						Possessable->SetParent(PasteBindingsParams.Bindings[TargetIndex].BindingID, MovieScene);
					}
				}
				
				if (ObjectsToBind.Num() != 0)
				{
					if (PasteBindingsParams.bDuplicateExistingActors)
					{
						GEditor->SelectNone(false, true);
						TArray<UObject*> SelectedObjects;
						for (UObject* ObjectToBind : ObjectsToBind)
						{
							if (AActor* Actor = Cast<AActor>(ObjectToBind))
							{
								GEditor->SelectActor(Actor, true, false, false);
								SelectedObjects.Add(Actor);
							}
						}
							
						// Duplicate the bound actors
						GEditor->edactDuplicateSelected(World->GetCurrentLevel(), false);

						// Duplicating the bound actor through GEditor, edits the copy/paste clipboard. This is not desired from the user's 
						// point of view since the user didn't explicitly invoke the copy operation. Instead, restore the copied contents
						// of the clipboard after duplicating the actor
						FPlatformApplicationMisc::ClipboardCopy(*TextToImport);

						ObjectsToBind.RemoveAll([&SelectedObjects](UObject* Object) { return SelectedObjects.Contains(Object);});
						USelection* ActorSelection = GEditor->GetSelectedActors();
						for (FSelectionIterator Iter(*ActorSelection); Iter; ++Iter)
						{
							AActor* Actor = Cast<AActor>(*Iter);
							if (Actor)
							{
								ObjectsToBind.Add(Actor);

								CopyableBinding->BoundObjectNames.Add(Actor->GetPathName(ResolutionContext));
							}
						}
					}

					// Bind the actors
					if (ObjectsToBind.Num())
					{
						AddObjectsToBinding(Sequencer, ObjectsToBind, FMovieSceneBindingProxy(NewGuid, Sequence), ResolutionContext);
					}
				}
			}
			else if (CopyableBinding->Spawnable.GetGuid().IsValid() && !CopyableBinding->SpawnableObjectTemplates.IsEmpty())
			{
				// We need to let the sequence create the spawnable so that it has everything set up properly internally.
				// This is required to get spawnables with the correct references to object templates, object templates with
				// correct owners, etc. However, making a new spawnable also creates the binding for us - this is a problem
				// because we need to use our binding (which has tracks associated with it). To solve this, we let it create
				// an object template based off of our (transient package owned) template, then find the newly created binding
				// and update it.

				FGuid NewGuid = MakeNewSpawnable(Sequencer, *CopyableBinding->SpawnableObjectTemplates[0], nullptr, false, FName(*CopyableBinding->Spawnable.GetName()));
				FMovieSceneBinding* NewBinding = MovieScene->FindBinding(NewGuid);
				NewBinding->SetTracks(TArray<UMovieSceneTrack*>(CopyableBinding->Tracks), nullptr);
				FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(NewGuid);

				// Copy the name of the original spawnable too.
				Spawnable->SetName(CopyableBinding->Spawnable.GetName());

				// Clear the transient flags on the copyable binding before assigning to the new spawnable
				for (auto Track : NewBinding->GetTracks())
				{
					ResetCopiedTracksFlags(Track);
				}

				// Replace the auto-generated binding with our deserialized bindings (which has our tracks)
				MovieScene->ReplaceBinding(NewGuid, *NewBinding);

				OldToNewGuidMap.Add(CopyableBinding->Spawnable.GetGuid(), NewGuid);

				BindingsPasted.Add(*NewBinding);

				SpawnableGuids.Add(NewGuid);

				if (ParentFolder)
				{
					GuidToFolderMap.Add(NewGuid, ParentFolder);
				}

				if (CopyableBinding->Tags.Num() > 0)
				{
					RootMovieScene->Modify();

					for (const FName& Tag : CopyableBinding->Tags)
					{
						RootMovieScene->TagBinding(Tag, UE::MovieScene::FFixedObjectBindingID(NewGuid, Sequencer->GetFocusedTemplateID()));
					}
				}
			}
		}
	}

	// Fix up parent guids
	for (auto PossessableGuid : PossessableGuids)
	{
		FMovieScenePossessable* Possessable = MovieScene->FindPossessable(PossessableGuid);
		if (Possessable && OldToNewGuidMap.Contains(Possessable->GetParent()) && PossessableGuid != OldToNewGuidMap[Possessable->GetParent()])
		{
			Possessable->SetParent(OldToNewGuidMap[Possessable->GetParent()], MovieScene);
		}
	}

	// Set up folders
	for (auto PossessableGuid : PossessableGuids)
	{
		FMovieScenePossessable* Possessable = MovieScene->FindPossessable(PossessableGuid);
		if (Possessable && !Possessable->GetParent().IsValid())
		{
			if (GuidToFolderMap.Contains(PossessableGuid))
			{
				GuidToFolderMap[PossessableGuid]->AddChildObjectBinding(PossessableGuid);
			}
		}
	}
	for (auto SpawnableGuid : SpawnableGuids)
	{
		FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(SpawnableGuid);
		if (Spawnable)
		{
			if (GuidToFolderMap.Contains(SpawnableGuid))
			{
				GuidToFolderMap[SpawnableGuid]->AddChildObjectBinding(SpawnableGuid);
			}
		}
	}

	Sequencer->OnMovieSceneBindingsPasted().Broadcast(BindingsPasted);

	// Refresh all immediately so that spawned actors will be generated immediately
	Sequencer->ForceEvaluate();

	// Fix possessable subobject bindings
	for (int32 PossessableGuidIndex = 0; PossessableGuidIndex < PossessableGuids.Num(); ++PossessableGuidIndex)
	{
		FGuid PossessableGuid = PossessableGuids[PossessableGuidIndex];
		// If a possessable guid does not have any bound objects, they might be 
		// possessable components for spawnables, so they need to be remapped
		if (Sequencer->FindBoundObjects(PossessableGuid, Sequencer->GetFocusedTemplateID()).Num() == 0)
		{
			FMovieScenePossessable* Possessable = MovieScene->FindPossessable(PossessableGuid);
			if (Possessable)
			{
				FGuid ParentGuid = Possessable->GetParent();
				bool bBound = false;
				for (TWeakObjectPtr<> WeakObject : Sequencer->FindBoundObjects(ParentGuid, Sequencer->GetFocusedTemplateID()))
				{
					if (AActor* SpawnedActor = Cast<AActor>(WeakObject.Get()))
					{
						for (UActorComponent* Component : SpawnedActor->GetComponents())
						{
							if (Component->GetName() == Possessable->GetName())
							{
								Sequence->BindPossessableObject(PossessableGuid, *Component, SpawnedActor);
								bBound = true;
								break;
							}
						}
					}
				}

				// If the parent doesn't actually exist, clear it.
				FMovieScenePossessable* PossessableParent = MovieScene->FindPossessable(ParentGuid);
				FMovieSceneSpawnable* SpawnableParent = MovieScene->FindSpawnable(ParentGuid);
				if (!PossessableParent && !SpawnableParent)
				{
					Possessable->SetParent(FGuid(), MovieScene);
				}
				else if (SpawnableParent)
				{
					SpawnableParent->AddChildPossessable(PossessableGuid);
				}
			}
		}
	}

	// Find all the sections that have been added and only remap bindings in those sections
	TSet<UMovieSceneSection*> Sections;
	for (auto BindingPasted : BindingsPasted)
	{
		if (FMovieSceneBinding* Binding = MovieScene->FindBinding(BindingPasted.GetObjectGuid()))
		{
			for (UMovieSceneTrack* Track : Binding->GetTracks())
			{
				for (UMovieSceneSection* Section : Track->GetAllSections())
				{
					Sections.Add(Section);
				}
			}
		}
	}

	if (Sections.Num() != 0)
	{
		FMovieSceneSequenceIDRef FocusedGuid = Sequencer->GetFocusedTemplateID();

		TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID> OldFixedToNewFixedMap;

		TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = Sequencer->GetSharedPlaybackState();

		for (TPair<FGuid, FGuid> GuidPair : OldToNewGuidMap)
		{
			OldFixedToNewFixedMap.Add(UE::MovieScene::FFixedObjectBindingID(GuidPair.Key, FocusedGuid), UE::MovieScene::FFixedObjectBindingID(GuidPair.Value, FocusedGuid));
		}

		for (UMovieSceneSection* Section : Sections)
		{	
			Section->OnBindingIDsUpdated(OldFixedToNewFixedMap, FocusedGuid, SharedPlaybackState);
		}
	}

	for (auto BindingPasted : BindingsPasted)
	{
		OutBindings.Add(FMovieSceneBindingProxy(BindingPasted.GetObjectGuid(), Sequence));
		
		Sequencer->OnAddBinding(BindingPasted.GetObjectGuid(), MovieScene);
	}

	return true; 
}

bool FSequencerUtilities::CanPasteBindings(TSharedRef<ISequencer> Sequencer, const FString& TextToImport)
{
	FObjectBindingTextFactory ObjectBindingFactory(Sequencer.Get());
	return ObjectBindingFactory.CanCreateObjectsFromText(TextToImport);
}

TArray<FString> FSequencerUtilities::GetPasteBindingsObjectNames(TSharedRef<ISequencer> Sequencer, const FString& TextToImport)
{
	TArray<FString> ObjectNames;

	TArray<UMovieSceneCopyableBinding*> ImportedBindings;
	ImportObjectBindingsFromText(Sequencer.Get(), TextToImport, ImportedBindings);

	for (UMovieSceneCopyableBinding* CopyableBinding : ImportedBindings)
	{
		if (CopyableBinding)
		{
			for (const FString& BoundObjectName : CopyableBinding->BoundObjectNames)
			{
				ObjectNames.Add(BoundObjectName);
			}
		}
	}

	return ObjectNames;
}

FGuid CreateImplementationDefinedBinding(TSharedRef<ISequencer> Sequencer, UObject& InObject, const UE::Sequencer::FCreateBindingParams& InParams)
{
	UMovieSceneSequence* OwnerSequence   = Sequencer->GetFocusedMovieSceneSequence();
	UMovieScene*         OwnerMovieScene = OwnerSequence->GetMovieScene();

	AActor* Actor = Cast<AActor>(&InObject);

	FString Name = InParams.BindingNameOverride.Len() > 0
		? InParams.BindingNameOverride
		: (Actor != nullptr ? Actor->GetActorLabel() : InObject.GetName());

	FGuid PossessableGuid = OwnerMovieScene->AddPossessable(Name, InObject.GetClass());

	// Attempt to use the parent as a context if necessary
	UObject* ParentObject = OwnerSequence->GetParentObject(&InObject);
	UObject* BindingContext = Sequencer->GetPlaybackContext();

	AActor* ParentActorAdded = nullptr;
	FGuid ParentGuid;

	if (ParentObject)
	{
		// Ensure we have possessed the outer object, if necessary
		ParentGuid = Sequencer->GetHandleToObject(ParentObject, false);
		if (!ParentGuid.IsValid())
		{
			ParentGuid = Sequencer->GetHandleToObject(ParentObject);
			ParentActorAdded = Cast<AActor>(ParentObject);
		}

		if (OwnerSequence->AreParentContextsSignificant())
		{
			BindingContext = ParentObject;
		}

		// Set up parent/child guids for possessables within spawnables
		if (ParentGuid.IsValid())
		{
			FMovieScenePossessable* ChildPossessable = OwnerMovieScene->FindPossessable(PossessableGuid);
			if (ensure(ChildPossessable))
			{
				ChildPossessable->SetParent(ParentGuid, OwnerMovieScene);
			}

			FMovieSceneSpawnable* ParentSpawnable = OwnerMovieScene->FindSpawnable(ParentGuid);
			if (ParentSpawnable)
			{
				ParentSpawnable->AddChildPossessable(PossessableGuid);
			}
		}
	}

	FMovieScenePossessable* NewPossessable = OwnerMovieScene->FindPossessable(PossessableGuid);
	if (!NewPossessable->BindSpawnableObject(Sequencer->GetFocusedTemplateID(), &InObject, Sequencer->GetSharedPlaybackState()))
	{
		OwnerSequence->BindPossessableObject(PossessableGuid, InObject, BindingContext);
	}

	// Broadcast if a parent actor was added as a result of adding this object
	if (ParentActorAdded && ParentGuid.IsValid())
	{
		Sequencer->OnActorAddedToSequencer().Broadcast(ParentActorAdded, ParentGuid);
	}

	Sequencer->OnAddBinding(PossessableGuid, OwnerMovieScene);

	if (Actor)
	{
		Sequencer->OnActorAddedToSequencer().Broadcast(Actor, PossessableGuid);
	}

	return PossessableGuid;
}

UObject* FSequencerUtilities::FindResolutionContext(TSharedRef<ISequencer> InSequencer
	, UMovieSceneSequence& InSequence
	, UMovieScene& InMovieScene
	, const FGuid& InParentGuid
	, UObject* InPlaybackContext)
{
	if (!InPlaybackContext || !InParentGuid.IsValid() || !InSequence.AreParentContextsSignificant())
	{
		return InPlaybackContext;
	}

	UObject* ResolutionContext = nullptr;

	// Recursive call up the hierarchy
	if (FMovieScenePossessable* const ParentPossessable = InMovieScene.FindPossessable(InParentGuid))
	{
		ResolutionContext = FSequencerUtilities::FindResolutionContext(InSequencer
			, InSequence
			, InMovieScene
			, ParentPossessable->GetParent()
			, InPlaybackContext);
	}

	if (!ResolutionContext)
	{
		ResolutionContext = InPlaybackContext;
	}

	TArray<UObject*, TInlineAllocator<1>> FoundObjects;
	for (TWeakObjectPtr<> WeakObj : InSequencer->FindBoundObjects(InParentGuid, InSequencer->GetFocusedTemplateID()))
	{
		FoundObjects.Add(WeakObj.Get());
	}

	if (FoundObjects.IsEmpty())
	{
		return ResolutionContext;
	}

	return FoundObjects[0] ? FoundObjects[0] : ResolutionContext;
}

FGuid FSequencerUtilities::CreateBinding(TSharedRef<ISequencer> Sequencer, UObject& InObject, const UE::Sequencer::FCreateBindingParams& InParams)
{
	return CreateOrReplaceBinding(Sequencer.ToSharedPtr(), Sequencer->GetFocusedMovieSceneSequence(), &InObject, InParams);
}

FGuid FSequencerUtilities::CreateOrReplaceBinding(TSharedRef<ISequencer> Sequencer, UObject* InObject, const UE::Sequencer::FCreateBindingParams& InParams)
{
	return CreateOrReplaceBinding(Sequencer.ToSharedPtr(), Sequencer->GetFocusedMovieSceneSequence(), InObject, InParams);
}

FGuid FSequencerUtilities::CreateOrReplaceBinding(TSharedPtr<ISequencer> Sequencer, UMovieSceneSequence* OwnerSequence, UObject* InObject, const UE::Sequencer::FCreateBindingParams& InParams)
{
	if (!OwnerSequence)
		return FGuid();
	
	const FScopedTransaction Transaction(LOCTEXT("CreateBinding", "Create New Binding"));

	UMovieScene* OwnerMovieScene = OwnerSequence->GetMovieScene();

	OwnerSequence->Modify();
	OwnerMovieScene->Modify();

	FGuid BindingGuid;
	FMovieSceneBindingReferences* BindingReferences = OwnerSequence->GetBindingReferences();
	if (BindingReferences)
	{
		BindingGuid = CreateGenericBinding(Sequencer, OwnerSequence, InObject, BindingReferences, InParams);
	}
	else if (Sequencer && InParams.bSpawnable && InObject)
	{
		// Create an old-style spawnable
		BindingGuid = MakeNewSpawnable(Sequencer.ToSharedRef(), *InObject, InParams.ActorFactory.Get());
	}
	else if (Sequencer && InObject)
	{
		BindingGuid = CreateImplementationDefinedBinding(Sequencer.ToSharedRef(), *InObject, InParams);
	}

	if (!BindingGuid.IsValid())
	{
		return FGuid();
	}

	if (InParams.DesiredFolder != NAME_None)
	{
		// Find the outermost object and put it in a folder of the specified name.
		FGuid RootObjectGuid = BindingGuid;
		while (true)
		{
			// This only applies to possessables/custom bindings, as old-style spawnables will not have parents.
			FMovieScenePossessable* Possessable = OwnerMovieScene->FindPossessable(RootObjectGuid);
			if (!Possessable || !Possessable->GetParent().IsValid())
			{
				break;
			}
			RootObjectGuid = Possessable->GetParent();
		}

		UMovieSceneFolder* DestinationFolder = nullptr;
		for (UMovieSceneFolder* Folder : OwnerMovieScene->GetRootFolders())
		{
			if (Folder->GetFolderName() == InParams.DesiredFolder)
			{
				DestinationFolder = Folder;
				break;
			}
		}

		// If we didn't find a folder with the desired name then we create a new folder as a sibling of the existing folders.
		if (DestinationFolder == nullptr)
		{
			DestinationFolder = NewObject<UMovieSceneFolder>(OwnerMovieScene, NAME_None, RF_Transactional);
			DestinationFolder->SetFolderName(InParams.DesiredFolder);

			OwnerMovieScene->AddRootFolder(DestinationFolder);
			DestinationFolder->AddChildObjectBinding(RootObjectGuid);
		}
		else
		{
			DestinationFolder->AddChildObjectBinding(RootObjectGuid);
		}
	}

	if (Sequencer)
	{
		if (ACameraActor* NewCamera = Cast<ACameraActor>(InObject))
		{
			NewCameraAdded(Sequencer.ToSharedRef(), NewCamera, BindingGuid);
		}

		Sequencer->OnAddBinding(BindingGuid, OwnerMovieScene);
	}
	return BindingGuid;
}


void FSequencerUtilities::UpdateBindingIDs(TSharedRef<ISequencer> Sequencer, FGuid OldGuid, FGuid NewGuid)
{
	UMovieSceneCompiledDataManager* CompiledDataManager = FindObject<UMovieSceneCompiledDataManager>(GetTransientPackage(), TEXT("SequencerCompiledDataManager"));
	if (!CompiledDataManager)
	{
		CompiledDataManager = NewObject<UMovieSceneCompiledDataManager>(GetTransientPackage(), "SequencerCompiledDataManager");
	}

	if (!CompiledDataManager)
	{
		return;
	}

	const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(Sequencer->GetEvaluationTemplate().GetCompiledDataID());

	FMovieSceneSequenceIDRef FocusedGuid = Sequencer->GetFocusedTemplateID();

	TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID> OldFixedToNewFixedMap;
	OldFixedToNewFixedMap.Add(UE::MovieScene::FFixedObjectBindingID(OldGuid, FocusedGuid), UE::MovieScene::FFixedObjectBindingID(NewGuid, FocusedGuid));

	TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = Sequencer->GetSharedPlaybackState();

	if (UMovieScene* MovieScene = Sequencer->GetRootMovieSceneSequence()->GetMovieScene())
	{
		for (UMovieSceneSection* Section : MovieScene->GetAllSections())
		{
			if (Section)
			{
				Section->OnBindingIDsUpdated(OldFixedToNewFixedMap, Sequencer->GetRootTemplateID(), SharedPlaybackState);
			}
		}
	}

	if (Hierarchy)
	{
		for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
		{
			if (UMovieSceneSequence* Sequence = Pair.Value.GetSequence())
			{
				if (UMovieScene* MovieScene = Sequence->GetMovieScene())
				{
					for (UMovieSceneSection* Section : MovieScene->GetAllSections())
					{
						if (Section)
						{
							Section->OnBindingIDsUpdated(OldFixedToNewFixedMap, Pair.Key, SharedPlaybackState);
						}
					}
				}
			}
		}
	}
}

FGuid FSequencerUtilities::AssignActor(TSharedRef<ISequencer> Sequencer, AActor* Actor, FGuid InObjectBinding)
{
	if (Actor == nullptr)
	{
		return FGuid();
	}

	UMovieSceneSequence* OwnerSequence = Sequencer->GetFocusedMovieSceneSequence();
	UMovieScene* OwnerMovieScene = OwnerSequence->GetMovieScene();

	if (OwnerMovieScene->IsReadOnly())
	{
		ShowReadOnlyError();
		return FGuid();
	}

	FScopedTransaction AssignActor(LOCTEXT("AssignActor", "Assign Actor"));

	Actor->Modify();
	OwnerSequence->Modify();
	OwnerMovieScene->Modify();

	TArrayView<TWeakObjectPtr<>> RuntimeObjects = Sequencer->FindObjectsInCurrentSequence(InObjectBinding);

	UObject* RuntimeObject = RuntimeObjects.Num() ? RuntimeObjects[0].Get() : nullptr;

	// Replace the object itself
	FMovieScenePossessable NewPossessableActor;
	FGuid NewGuid;
	{
		// Get the object guid to assign, remove the binding if it already exists
		FGuid ParentGuid = Sequencer->FindObjectId(*Actor, Sequencer->GetFocusedTemplateID());
		FString NewActorLabel = Actor->GetActorLabel();
		if (ParentGuid.IsValid())
		{
			OwnerMovieScene->RemovePossessable(ParentGuid);
			OwnerSequence->UnbindPossessableObjects(ParentGuid);
		}

		// Add this object
		NewPossessableActor = FMovieScenePossessable(NewActorLabel, Actor->GetClass());
		NewGuid = NewPossessableActor.GetGuid();
		if (!NewPossessableActor.BindSpawnableObject(Sequencer->GetFocusedTemplateID(), Actor, Sequencer->GetSharedPlaybackState()))
		{
			OwnerSequence->BindPossessableObject(NewPossessableActor.GetGuid(), *Actor, Sequencer->GetPlaybackContext());
		}

		// Defer replacing this object until the components have been updated
	}

	auto UpdateComponent = [&](FGuid OldComponentGuid, UActorComponent* NewComponent, TArray<FGuid>& NewComponentGuids)
	{
		FMovieSceneSequenceIDRef FocusedGuid = Sequencer->GetFocusedTemplateID();

		// Get the object guid to assign, remove the binding if it already exists
		FGuid NewComponentGuid = Sequencer->FindObjectId(*NewComponent, FocusedGuid);
		if (NewComponentGuid.IsValid())
		{
			OwnerMovieScene->RemovePossessable(NewComponentGuid);
			OwnerSequence->UnbindPossessableObjects(NewComponentGuid);
		}

		// Add this object
		FMovieScenePossessable NewPossessable(NewComponent->GetName(), NewComponent->GetClass());
		OwnerSequence->BindPossessableObject(NewPossessable.GetGuid(), *NewComponent, Actor);

		// Replace
		OwnerMovieScene->ReplacePossessable(OldComponentGuid, NewPossessable);
		OwnerSequence->UnbindPossessableObjects(OldComponentGuid);
		
		FMovieSceneEvaluationState* State = Sequencer->GetEvaluationState();
		State->Invalidate(OldComponentGuid, FocusedGuid);
		State->Invalidate(NewPossessable.GetGuid(), FocusedGuid);

		NewComponentGuids.Add(NewPossessable.GetGuid());
	};

	TArray<FGuid> NewComponentGuids;

	// Handle components
	AActor* ActorToReplace = Cast<AActor>(RuntimeObject);
	if (ActorToReplace != nullptr && ActorToReplace->IsActorBeingDestroyed() == false)
	{
		for (UActorComponent* ComponentToReplace : ActorToReplace->GetComponents())
		{
			if (ComponentToReplace != nullptr)
			{
				FGuid ComponentGuid = Sequencer->FindObjectId(*ComponentToReplace, Sequencer->GetFocusedTemplateID());
				if (ComponentGuid.IsValid())
				{
					bool bComponentWasUpdated = false;
					for (UActorComponent* NewComponent : Actor->GetComponents())
					{
						if (NewComponent->GetFullName(Actor) == ComponentToReplace->GetFullName(ActorToReplace))
						{
							UpdateComponent(ComponentGuid, NewComponent, NewComponentGuids);
							bComponentWasUpdated = true;
						}
					}

					// Clear the parent guid since this possessable component doesn't match to any component on the new actor
					if (!bComponentWasUpdated)
					{
						FMovieScenePossessable* ThisPossessable = OwnerMovieScene->FindPossessable(ComponentGuid);
						ThisPossessable->SetParent(FGuid(), OwnerMovieScene);
					}
				}
			}
		}
	}
	else // If the actor didn't exist, try to find components who's parent guids were the previous actors guid.
	{
		TMap<FString, UActorComponent*> ComponentNameToComponent;
		for (UActorComponent* Component : Actor->GetComponents())
		{
			ComponentNameToComponent.Add(Component->GetName(), Component);
		}

		TMap<FGuid, UActorComponent*> ComponentsToUpdate;
		for (int32 i = 0; i < OwnerMovieScene->GetPossessableCount(); i++)
		{
			FMovieScenePossessable& OldPossessable = OwnerMovieScene->GetPossessable(i);
			if (OldPossessable.GetParent() == InObjectBinding)
			{
				UActorComponent** ComponentPtr = ComponentNameToComponent.Find(OldPossessable.GetName());
				if (ComponentPtr != nullptr)
				{
					ComponentsToUpdate.Add(OldPossessable.GetGuid(), *ComponentPtr);
				}
			}
		}

		for (TPair<FGuid, UActorComponent*> ComponentToUpdate : ComponentsToUpdate)
		{
			UpdateComponent(ComponentToUpdate.Key, ComponentToUpdate.Value, NewComponentGuids);
		}
	}

	// Replace the actor itself after components have been updated
	OwnerMovieScene->ReplacePossessable(InObjectBinding, NewPossessableActor);
	OwnerSequence->UnbindPossessableObjects(InObjectBinding);

	FMovieSceneEvaluationState* State = Sequencer->GetEvaluationState();
	State->Invalidate(InObjectBinding, Sequencer->GetFocusedTemplateID());
	State->Invalidate(NewPossessableActor.GetGuid(), Sequencer->GetFocusedTemplateID());

	for (const FGuid& NewComponentGuid : NewComponentGuids)
	{
		FMovieScenePossessable* ThisPossessable = OwnerMovieScene->FindPossessable(NewComponentGuid);
		if (ensure(ThisPossessable))
		{
			ThisPossessable->SetParent(NewGuid, OwnerMovieScene);
		}
	}

	// Try to fix up folders
	TArray<UMovieSceneFolder*> FoldersToCheck;
	for (UMovieSceneFolder* Folder : Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetRootFolders())
	{
		FoldersToCheck.Add(Folder);
	}
	bool bFolderFound = false;
	while (FoldersToCheck.Num() > 0 && bFolderFound == false)
	{
		UMovieSceneFolder* Folder = FoldersToCheck[0];
		FoldersToCheck.RemoveAt(0);
		if (Folder->GetChildObjectBindings().Contains(InObjectBinding))
		{
			Folder->RemoveChildObjectBinding(InObjectBinding);
			Folder->AddChildObjectBinding(NewGuid);
			bFolderFound = true;
		}

		for (UMovieSceneFolder* ChildFolder : Folder->GetChildFolders())
		{
			FoldersToCheck.Add(ChildFolder);
		}
	}

	Sequencer->RestorePreAnimatedState();

	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);

	return NewGuid;
}

void FSequencerUtilities::AddActorsToBinding(TSharedRef<ISequencer> Sequencer, const TArray<AActor*>& Actors, const FMovieSceneBindingProxy& ObjectBinding)
{
	FScopedTransaction AddActorsToBinding(LOCTEXT("AddActorsToBinding", "Add Actors to Binding"));

	TArray<UObject*> ObjectsToAdd;
	Algo::Transform(Actors, ObjectsToAdd, [](AActor* Actor) { return Actor;});
	AddObjectsToBinding(Sequencer, ObjectsToAdd, ObjectBinding, Sequencer->GetPlaybackContext());
}

void FSequencerUtilities::AddObjectsToBinding(TSharedRef<ISequencer> InSequencer, const TArray<UObject*>& InObjectsToAdd, const FMovieSceneBindingProxy& InObjectBinding, UObject* InResolutionContext)
{
	UMovieSceneSequence* Sequence = InObjectBinding.Sequence;
	if (!Sequence)
	{
		return;
	}

	UMovieScene* const MovieScene = Sequence->GetMovieScene();
	if (!MovieScene || InObjectsToAdd.IsEmpty())
	{
		return;
	}

	UClass* ObjectClass = nullptr;
	int32 ValidObjectCount = 0;

	FGuid Guid = InObjectBinding.BindingID;

	TArrayView<TWeakObjectPtr<>> ObjectsInCurrentSequence = InSequencer->FindObjectsInCurrentSequence(Guid);

	for (TWeakObjectPtr<> Ptr : ObjectsInCurrentSequence)
	{
		if (const UObject* Object = Cast<AActor>(Ptr.Get()))
		{
			ObjectClass = Object->GetClass();
			++ValidObjectCount;
		}
	}

	Sequence->Modify();
	MovieScene->Modify();

	TArray<UObject*> AddedObjects;
	AddedObjects.Reserve(InObjectsToAdd.Num());

	for (UObject* ObjectToAdd : InObjectsToAdd)
	{
		// Skip invalid objects or objects already in the sequence
		if (!ObjectToAdd || ObjectsInCurrentSequence.Contains(ObjectToAdd))
		{
			continue;
		}

		// Skip if the object has no common class with the objects already in the binding
		if (ObjectClass && !UClass::FindCommonBase(ObjectToAdd->GetClass(), ObjectClass))
		{
			continue;
		}

		// if no objects are in the binding, set the class to this object's
		if (!ObjectClass)
		{
			ObjectClass = ObjectToAdd->GetClass();
		}

		FMovieScenePossessable* Possessable = MovieScene->FindPossessable(Guid);
		if (!ensureAlways(Possessable))
		{
			continue;
		}

		ObjectToAdd->Modify();
		if (!Possessable->BindSpawnableObject(InSequencer->GetFocusedTemplateID(), ObjectToAdd, InSequencer->GetSharedPlaybackState()))
		{
			Sequence->BindPossessableObject(Guid, *ObjectToAdd, InResolutionContext);
		}

		// If the object was added successfully, continue
		FGuid AddedGuid = InSequencer->GetHandleToObject(ObjectToAdd, false);
		if (AddedGuid.IsValid())
		{
			AddedObjects.Add(ObjectToAdd);
			continue;
		}

		// Otherwise...
		if (ObjectClass == nullptr || UClass::FindCommonBase(ObjectToAdd->GetClass(), ObjectClass) != nullptr)
		{
			if (ObjectClass == nullptr)
			{
				ObjectClass = ObjectToAdd->GetClass();
			}

			ObjectToAdd->Modify();
			if (!MovieScene->FindPossessable(Guid)->BindSpawnableObject(InSequencer->GetFocusedTemplateID(), ObjectToAdd, InSequencer->GetSharedPlaybackState()))
			{
				Sequence->BindPossessableObject(Guid, *ObjectToAdd, InResolutionContext);
			}
			AddedObjects.Add(ObjectToAdd);
		}
		else
		{
			const FText NotificationText = FText::Format(LOCTEXT("UnableToAssignObject", "Cannot assign object {0}. Expected class {1}"), FText::FromString(ObjectToAdd->GetPathName()), FText::FromString(ObjectClass->GetName()));
			FNotificationInfo Info(NotificationText);
			Info.ExpireDuration = 3.f;
			Info.bUseLargeFont = false;
			FSlateNotificationManager::Get().AddNotification(Info);
		}
	}

	// Update Labels
	if (ValidObjectCount + AddedObjects.Num() > 0)
	{
		FMovieScenePossessable* const Possessable = MovieScene->FindPossessable(Guid);
		if (Possessable && ObjectClass)
		{
			// If there are multiple objects within the same possessable, name possessable as "ClassName (Count)"
			if (ValidObjectCount + AddedObjects.Num() > 1)
			{
				Possessable->SetName(FString::Printf(TEXT("%s (%d)")
					, *ObjectClass->GetName()
					, ValidObjectCount + AddedObjects.Num()));
			}
			else if (!AddedObjects.IsEmpty())
			{
				FString PossessableName = AddedObjects[0]->GetName();
				if (AActor* const Actor = Cast<AActor>(AddedObjects[0]))
				{
					PossessableName = Actor->GetActorLabel();
				}
				Possessable->SetName(PossessableName);
			}
			Possessable->SetPossessedObjectClass(ObjectClass);
		}
	}

	InSequencer->RestorePreAnimatedState();

	InSequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

void FSequencerUtilities::ReplaceBindingWithActors(TSharedRef<ISequencer> Sequencer, const TArray<AActor*>& Actors, const FMovieSceneBindingProxy& ObjectBinding)
{
	FScopedTransaction ReplaceBindingWithActors(LOCTEXT("ReplaceBindingWithActors", "Replace Binding with Actors"));

	FGuid Guid = ObjectBinding.BindingID;
	TArray<AActor*> ExistingActors;
	for (TWeakObjectPtr<> Ptr : Sequencer->FindObjectsInCurrentSequence(Guid))
	{
		if (AActor* Actor = Cast<AActor>(Ptr.Get()))
		{
			if (!Actors.Contains(Actor))
			{
				ExistingActors.Add(Actor);
			}
		}
	}

	RemoveActorsFromBinding(Sequencer, ExistingActors, ObjectBinding);

	TArray<AActor*> NewActors;
	for (AActor* NewActor : Actors)
	{
		if (!ExistingActors.Contains(NewActor))
		{
			NewActors.Add(NewActor);
		}
	}

	AddActorsToBinding(Sequencer, NewActors, ObjectBinding);
}

void FSequencerUtilities::RemoveActorsFromBinding(TSharedRef<ISequencer> Sequencer, const TArray<AActor*>& Actors, const FMovieSceneBindingProxy& ObjectBinding)
{
	if (!Actors.Num())
	{
		return;
	}

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	UClass* ActorClass = nullptr;
	int32 NumRuntimeObjects = 0;

	FGuid Guid = ObjectBinding.BindingID;
	for (TWeakObjectPtr<> Ptr : Sequencer->FindObjectsInCurrentSequence(Guid))
	{
		if (const AActor* Actor = Cast<AActor>(Ptr.Get()))
		{
			ActorClass = Actor->GetClass();
			++NumRuntimeObjects;
		}
	}

	FScopedTransaction RemoveSelectedFromBinding(LOCTEXT("RemoveSelectedFromBinding", "Remove Selected from Binding"));

	TArray<UObject*> ObjectsToRemove;
	for (AActor* ActorToRemove : Actors)
	{
		// Restore state on any components
		for (UActorComponent* Component : TInlineComponentArray<UActorComponent*>(ActorToRemove))
		{
			if (Component)
			{
				Sequencer->PreAnimatedState.RestorePreAnimatedState(*Component);
			}
		}

		// Restore state on the object itself
		Sequencer->PreAnimatedState.RestorePreAnimatedState(*ActorToRemove);

		ActorToRemove->Modify();

		ObjectsToRemove.Add(ActorToRemove);
	}

	Sequence->Modify();
	MovieScene->Modify();

	// Unbind objects
	Sequence->UnbindObjects(Guid, ObjectsToRemove, Sequencer->GetPlaybackContext());

	// Update label
	if (NumRuntimeObjects - ObjectsToRemove.Num() > 0)
	{
		FMovieScenePossessable* Possessable = MovieScene->FindPossessable(Guid);
		if (Possessable && ActorClass != nullptr)
		{
			if (NumRuntimeObjects - ObjectsToRemove.Num() > 1)
			{
				FString NewLabel = ActorClass->GetName() + FString::Printf(TEXT(" (%d)"), NumRuntimeObjects - ObjectsToRemove.Num());

				Possessable->SetName(NewLabel);
			}
			else if (ObjectsToRemove.Num() > 0 && Actors.Num() > 0)
			{
				Possessable->SetName(Actors[0]->GetActorLabel());
			}
		}
	}

	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

void FSequencerUtilities::RemoveAllBindings(TSharedRef<ISequencer> Sequencer, const FMovieSceneBindingProxy& ObjectBinding)
{
	UMovieSceneSequence* Sequence = ObjectBinding.Sequence;
	UMovieScene* MovieScene = ObjectBinding.GetMovieScene();
	if (!Sequence || !MovieScene)
	{
		return;
	}

	FScopedTransaction RemoveAllBindings(LOCTEXT("RemoveAllBindings", "Remove All Bound Objects"));

	Sequence->Modify();
	MovieScene->Modify();

	FGuid Guid = ObjectBinding.BindingID;
	Sequence->UnbindPossessableObjects(Guid);

	Sequencer->RestorePreAnimatedState();

	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

void FSequencerUtilities::RemoveInvalidBindings(TSharedRef<ISequencer> Sequencer, const FMovieSceneBindingProxy& ObjectBinding)
{
	UMovieSceneSequence* Sequence = ObjectBinding.Sequence;
	UMovieScene* MovieScene = ObjectBinding.GetMovieScene();
	if (!Sequence || !MovieScene)
	{
		return;
	}

	FScopedTransaction RemoveInvalidBindings(LOCTEXT("RemoveMissing", "Remove Missing Objects"));

	Sequence->Modify();
	MovieScene->Modify();

	FGuid Guid = ObjectBinding.BindingID;
	Sequence->UnbindInvalidObjects(Guid, Sequencer->GetPlaybackContext());

	// Update label
	UClass* ActorClass = nullptr;

	TArray<AActor*> ValidActors;
	for (TWeakObjectPtr<> Ptr : Sequencer->FindObjectsInCurrentSequence(Guid))
	{
		if (AActor* Actor = Cast<AActor>(Ptr.Get()))
		{
			ActorClass = Actor->GetClass();
			ValidActors.Add(Actor);
		}
	}

	FMovieScenePossessable* Possessable = MovieScene->FindPossessable(Guid);
	if (Possessable && ActorClass != nullptr && ValidActors.Num() != 0)
	{
		if (ValidActors.Num() > 1)
		{
			FString NewLabel = ActorClass->GetName() + FString::Printf(TEXT(" (%d)"), ValidActors.Num());

			Possessable->SetName(NewLabel);
		}
		else
		{
			Possessable->SetName(ValidActors[0]->GetActorLabel());
		}
	}

	Sequencer->RestorePreAnimatedState();

	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

struct FBakeData
{
	TArray<FVector> Locations;
	TArray<FRotator> Rotations;
	TArray<FVector> Scales;
	TSortedMap<FFrameNumber, FFrameNumber> KeyTimes;
};

void CalculateFramesPerGuid(TSharedRef<ISequencer> Sequencer, const FBakingAnimationKeySettings& InSettings, TMap<FGuid, FBakeData>& OutBakeDataMap, TSortedMap<FFrameNumber, FFrameNumber>& OutFrameMap)
{
	OutFrameMap.Reset();
	TArray<FFrameNumber> Frames;
	//we get all frames since we need to get the Actor PER FRAME in order to handle spanwables
	MovieSceneToolHelpers::CalculateFramesBetween(Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene(), InSettings.StartFrame, InSettings.EndFrame, InSettings.FrameIncrement, Frames);
	if (InSettings.BakingKeySettings == EBakingKeySettings::AllFrames)
	{
		for (FFrameNumber& Frame : Frames)
		{
			OutFrameMap.Add(Frame, Frame);
		}
		for (TPair<FGuid, FBakeData>& BakeData : OutBakeDataMap)
		{
			BakeData.Value.KeyTimes.Reset();
			BakeData.Value.KeyTimes = OutFrameMap;
		}
	}
	else
	{
		for (TPair<FGuid, FBakeData>& BakeData : OutBakeDataMap)
		{
			FActorForWorldTransforms ActorForWorldTransforms;
			FGuid Guid = BakeData.Key;

			for (TWeakObjectPtr<> RuntimeObject : Sequencer->FindObjectsInCurrentSequence(Guid))
			{
				UActorComponent* ActorComponent = nullptr;
				AActor* Actor = Cast<AActor>(RuntimeObject.Get());
				if (!Actor)
				{
					ActorComponent = Cast<UActorComponent>(RuntimeObject.Get());
					if (ActorComponent)
					{
						Actor = ActorComponent->GetOwner();
					}
				}
				ActorForWorldTransforms.Actor = Actor;
				ActorForWorldTransforms.Component = Cast<USceneComponent>(ActorComponent);
				BakeData.Value.KeyTimes.Reset();
				MovieSceneToolHelpers::GetActorsAndParentsKeyFrames(&Sequencer.Get(), ActorForWorldTransforms,
					InSettings.StartFrame, InSettings.EndFrame, BakeData.Value.KeyTimes);
				OutFrameMap.Append(BakeData.Value.KeyTimes);
			}
		}
	}
}

bool FSequencerUtilities::BakeTransform(TSharedRef<ISequencer> Sequencer, const TArray<FMovieSceneBindingProxy>& ObjectBindings, const FBakingAnimationKeySettings& InSettings)
{
	UMovieScene* FocusedMovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!FocusedMovieScene)
	{
		UE_LOGF(LogSequencer, Warning, "Bake Transform failed.");
		return false;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		UE_LOGF(LogSequencer, Warning, "Bake Transform failed.");
		FSequencerUtilities::ShowReadOnlyError();
		return false;
	}

	if (ObjectBindings.Num() == 0)
	{
		UE_LOGF(LogSequencer, Warning, "Bake Transform failed.");
		return false;
	}

	FScopedTransaction BakeTransform(LOCTEXT("BakeTransform", "Bake Transform"));

	FocusedMovieScene->Modify();

	FQualifiedFrameTime ResetTime = Sequencer->GetLocalTime();

	FFrameRate TickResolution = FocusedMovieScene->GetTickResolution();
	FFrameRate DisplayRate = FocusedMovieScene->GetDisplayRate();

	TSortedMap<FFrameNumber, FFrameNumber> TotalFrameMap;
	TMap<FGuid, FBakeData> BakeDataMap;
	for (const FMovieSceneBindingProxy& ObjectBinding : ObjectBindings)
	{
		BakeDataMap.Add(ObjectBinding.BindingID);
	}
	CalculateFramesPerGuid(Sequencer, InSettings, BakeDataMap, TotalFrameMap);

	FMovieSceneInverseSequenceTransform LocalToRootTransform = Sequencer->GetFocusedMovieSceneSequenceTransform().Inverse();

	TArray<FFrameNumber> AllFrames;
	TotalFrameMap.GenerateKeyArray(AllFrames);

	UWorld* PlaybackContext = Sequencer->GetPlaybackContext()->GetWorld();
	ensure(PlaybackContext);
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(PlaybackContext);

	TArray<FFrameNumber> RemappedTimes;
	for (FFrameNumber KeyTime : AllFrames)
	{
		TOptional<FFrameTime> NewGlobalTime = LocalToRootTransform.TryTransformTime(KeyTime);
		if (!NewGlobalTime)
		{
			continue;
		}

		Sequencer->SetGlobalTime(NewGlobalTime.GetValue());
		Controller.EvaluateAllConstraints();

		if (InSettings.bTimeWarp)
		{
			FQualifiedFrameTime RemappedTime = Sequencer->GetUnwarpedLocalTime();
			RemappedTimes.Add(RemappedTime.Time.FrameNumber);
		}
		else
		{
			RemappedTimes.Add(KeyTime);
		}

		for (const FMovieSceneBindingProxy& ObjectBinding : ObjectBindings)
		{
			FGuid Guid = ObjectBinding.BindingID;

			for (TWeakObjectPtr<> RuntimeObject : Sequencer->FindObjectsInCurrentSequence(Guid))
			{
				const FFrameNumber* Number = BakeDataMap[Guid].KeyTimes.FindKey(KeyTime);
				if (Number == nullptr)
				{
					continue;
				}
				AActor* Actor = Cast<AActor>(RuntimeObject.Get());
				if (!Actor)
				{
					UActorComponent* ActorComponent = Cast<UActorComponent>(RuntimeObject.Get());
					if (ActorComponent)
					{
						Actor = ActorComponent->GetOwner();
					}
				}

				if (!Actor)
				{
					continue;
				}

				UCameraComponent* CameraComponent = MovieSceneHelpers::CameraComponentFromRuntimeObject(RuntimeObject.Get());

				// Cache transforms
				USceneComponent* Parent = nullptr;
				if (CameraComponent)
				{
					Parent = CameraComponent->GetAttachParent();
				}
				else if (Actor->GetRootComponent())
				{
					Parent = Actor->GetRootComponent()->GetAttachParent();
				}

				// The CameraRig_rail updates the spline position tick, so it needs to be ticked manually while baking the frames
				while (Parent && Parent->GetOwner())
				{
					Parent->GetOwner()->Tick(0.03f);
					if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Parent))
					{
						SkeletalMeshComponent->TickAnimation(0.f, false);

						SkeletalMeshComponent->RefreshBoneTransforms();
						SkeletalMeshComponent->RefreshFollowerComponents();
						SkeletalMeshComponent->UpdateComponentToWorld();
						SkeletalMeshComponent->FinalizeBoneTransform();
						SkeletalMeshComponent->MarkRenderTransformDirty();
						SkeletalMeshComponent->MarkRenderDynamicDataDirty();
					}
					Parent = Parent->GetAttachParent();
				}

				if (CameraComponent)
				{
					FTransform AdditiveOffset;
					float AdditiveFOVOffset;
					CameraComponent->GetAdditiveOffset(AdditiveOffset, AdditiveFOVOffset);

					FTransform Transform(Actor->GetActorRotation(), Actor->GetActorLocation());
					FTransform TransformWithAdditiveOffset = AdditiveOffset * Transform;
					FVector LocalTranslation = TransformWithAdditiveOffset.GetTranslation();
					FRotator LocalRotation = TransformWithAdditiveOffset.GetRotation().Rotator();

					BakeDataMap[Guid].Locations.Add(LocalTranslation);
					BakeDataMap[Guid].Rotations.Add(LocalRotation);
					BakeDataMap[Guid].Scales.Add(FVector::OneVector);
				}
				else
				{
					BakeDataMap[Guid].Locations.Add(Actor->GetActorLocation());
					BakeDataMap[Guid].Rotations.Add(Actor->GetActorRotation());
					BakeDataMap[Guid].Scales.Add(Actor->GetActorScale());
				}

			}
		}
	}

	const bool bDisableSectionsAfterBaking = Sequencer->GetSequencerSettings()->GetDisableSectionsAfterBaking();

	for (TPair<FGuid, FBakeData>& BakeData : BakeDataMap)
	{
		FGuid Guid = BakeData.Key;
		TArray<FFrameNumber> KeyTimes = RemappedTimes;
		
		// Disable or delete any constraint (attach/path) tracks
		AActor* AttachParentActor = nullptr;
		for (UMovieSceneTrack* Track : FocusedMovieScene->FindTracks(UMovieScene3DConstraintTrack::StaticClass(), Guid))
		{
			if (UMovieScene3DConstraintTrack* ConstraintTrack = Cast<UMovieScene3DConstraintTrack>(Track))
			{
				for (UMovieSceneSection* ConstraintSection : ConstraintTrack->GetAllSections())
				{
					FMovieSceneObjectBindingID ConstraintBindingID = (Cast<UMovieScene3DConstraintSection>(ConstraintSection))->GetConstraintBindingID();
					if (auto BoundObjectsView = ConstraintBindingID.ResolveBoundObjects(Sequencer->GetFocusedTemplateID(), *Sequencer); BoundObjectsView.Num() > 0)
					{
						TWeakObjectPtr<> ParentObject = BoundObjectsView[0];
						AttachParentActor = Cast<AActor>(ParentObject.Get());
					}
				}

				if (bDisableSectionsAfterBaking)
				{
					MovieSceneHelpers::DisableTrack(ConstraintTrack);
				}
				else
				{
					FocusedMovieScene->RemoveTrack(*ConstraintTrack);
				}
			}
		}

		// Disable or delete any transform tracks
		for (UMovieSceneTrack* Track : FocusedMovieScene->FindTracks(UMovieScene3DTransformTrack::StaticClass(), Guid))
		{
			if (UMovieScene3DTransformTrack* TransformTrack = Cast<UMovieScene3DTransformTrack>(Track))
			{
				if (bDisableSectionsAfterBaking)
				{
					MovieSceneHelpers::DisableTrack(TransformTrack);
				}
				else
				{
					FocusedMovieScene->RemoveTrack(*TransformTrack);
				}
			}
		}

		// Disable or delete any camera shake tracks
		for (UMovieSceneTrack* Track : FocusedMovieScene->FindTracks(UMovieSceneCameraShakeTrack::StaticClass(), Guid))
		{
			if (UMovieSceneCameraShakeTrack* CameraShakeTrack = Cast<UMovieSceneCameraShakeTrack>(Track))
			{
				if (bDisableSectionsAfterBaking)
				{
					MovieSceneHelpers::DisableTrack(CameraShakeTrack);
				}
				else
				{
					FocusedMovieScene->RemoveTrack(*CameraShakeTrack);
				}
			}
		}

		// Reset position
		Sequencer->SetLocalTimeDirectly(ResetTime.Time);
		Sequencer->ForceEvaluate();

		FVector DefaultLocation = FVector::ZeroVector;
		FVector DefaultRotation = FVector::ZeroVector;
		FVector DefaultScale = FVector::OneVector;

		for (TWeakObjectPtr<> RuntimeObject : Sequencer->FindObjectsInCurrentSequence(Guid))
		{
			AActor* Actor = Cast<AActor>(RuntimeObject.Get());
			if (!Actor)
			{
				UActorComponent* ActorComponent = Cast<UActorComponent>(RuntimeObject.Get());
				if (ActorComponent)
				{
					Actor = ActorComponent->GetOwner();
				}
			}

			if (!Actor)
			{
				continue;
			}

			DefaultLocation = Actor->GetActorLocation();
			DefaultRotation = Actor->GetActorRotation().Euler();
			DefaultScale = Actor->GetActorScale();

			// Always detach from any existing parent
			Actor->DetachFromActor(FDetachmentTransformRules::KeepRelativeTransform);
		}

		// Create new transform track and section
		UMovieScene3DTransformTrack* TransformTrack = Cast<UMovieScene3DTransformTrack>(FocusedMovieScene->AddTrack(UMovieScene3DTransformTrack::StaticClass(), Guid));

		if (TransformTrack)
		{
			UMovieScene3DTransformSection* TransformSection = CastChecked<UMovieScene3DTransformSection>(TransformTrack->CreateNewSection());
			TransformTrack->AddSection(*TransformSection);

			TransformSection->SetRange(TRange<FFrameNumber>::All());

			TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = TransformSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
			DoubleChannels[0]->SetDefault(DefaultLocation.X);
			DoubleChannels[1]->SetDefault(DefaultLocation.Y);
			DoubleChannels[2]->SetDefault(DefaultLocation.Z);
			DoubleChannels[3]->SetDefault(DefaultRotation.X);
			DoubleChannels[4]->SetDefault(DefaultRotation.Y);
			DoubleChannels[5]->SetDefault(DefaultRotation.Z);
			DoubleChannels[6]->SetDefault(DefaultScale.X);
			DoubleChannels[7]->SetDefault(DefaultScale.Y);
			DoubleChannels[8]->SetDefault(DefaultScale.Z);

			TArray<FVector> LocalTranslations, LocalRotations, LocalScales;
			LocalTranslations.SetNum(KeyTimes.Num());
			LocalRotations.SetNum(KeyTimes.Num());
			LocalScales.SetNum(KeyTimes.Num());

			for (int32 Counter = 0; Counter < KeyTimes.Num(); ++Counter)
			{
				FVector LocalTranslation = DefaultLocation;
				FVector LocalScale = DefaultScale;
				FRotator LocalRotation = DefaultRotation.Rotation();

				if (Counter < BakeData.Value.Locations.Num())
				{
					LocalTranslation = BakeData.Value.Locations[Counter];
				}
				if (Counter < BakeData.Value.Rotations.Num())
				{
					LocalRotation = BakeData.Value.Rotations[Counter];
				}
				if (Counter < BakeData.Value.Scales.Num())
				{
					LocalScale = BakeData.Value.Scales[Counter];
				}

				FTransform LocalTransform(LocalRotation, LocalTranslation, LocalScale);
				LocalTranslations[Counter] = LocalTransform.GetTranslation();
				LocalRotations[Counter] = LocalTransform.GetRotation().Euler();
				LocalScales[Counter] = LocalTransform.GetScale3D();
			}

			// Euler filter
			for (int32 Counter = 0; Counter < LocalRotations.Num() - 1; ++Counter)
			{
				FMath::WindRelativeAnglesDegrees(LocalRotations[Counter].X, LocalRotations[Counter + 1].X);
				FMath::WindRelativeAnglesDegrees(LocalRotations[Counter].Y, LocalRotations[Counter + 1].Y);
				FMath::WindRelativeAnglesDegrees(LocalRotations[Counter].Z, LocalRotations[Counter + 1].Z);
			}
			if (InSettings.BakingKeySettings == EBakingKeySettings::KeysOnly)
			{
				const EMovieSceneKeyInterpolation KeyInterpolation = Sequencer->GetSequencerSettings()->GetKeyInterpolation();

				for (int32 Counter = 0; Counter < KeyTimes.Num(); ++Counter)
				{
					int ChannelIndex = 0;
					FFrameNumber KeyTime = KeyTimes[Counter];
					{
						TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[ChannelIndex++]->GetData();
						MovieSceneToolHelpers::SetOrAddKey(ChannelData, KeyTime, LocalTranslations[Counter].X);
					}
					{
						TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[ChannelIndex++]->GetData();
						MovieSceneToolHelpers::SetOrAddKey(ChannelData, KeyTime, LocalTranslations[Counter].Y);
					}
					{
						TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[ChannelIndex++]->GetData();
						MovieSceneToolHelpers::SetOrAddKey(ChannelData, KeyTime, LocalTranslations[Counter].Z);
					}
					{
						TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[ChannelIndex++]->GetData();
						MovieSceneToolHelpers::SetOrAddKey(ChannelData, KeyTime, LocalRotations[Counter].X);
					}
					{
						TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[ChannelIndex++]->GetData();
						MovieSceneToolHelpers::SetOrAddKey(ChannelData, KeyTime, LocalRotations[Counter].Y);

					}
					{
						TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[ChannelIndex++]->GetData();
						MovieSceneToolHelpers::SetOrAddKey(ChannelData, KeyTime, LocalRotations[Counter].Z);
					}
					{
						TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[ChannelIndex++]->GetData();
						MovieSceneToolHelpers::SetOrAddKey(ChannelData, KeyTime, LocalScales[Counter].X);
					}
					{
						TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[ChannelIndex++]->GetData();
						MovieSceneToolHelpers::SetOrAddKey(ChannelData, KeyTime, LocalScales[Counter].Y);
					}
					{
						TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = DoubleChannels[ChannelIndex++]->GetData();
						MovieSceneToolHelpers::SetOrAddKey(ChannelData, KeyTime, LocalScales[Counter].Z);
					}
				}
			}
			else
			{
				for (int32 Counter = 0; Counter < KeyTimes.Num(); ++Counter)
				{
					FFrameNumber KeyTime = KeyTimes[Counter];
					DoubleChannels[0]->AddLinearKey(KeyTime, LocalTranslations[Counter].X);
					DoubleChannels[1]->AddLinearKey(KeyTime, LocalTranslations[Counter].Y);
					DoubleChannels[2]->AddLinearKey(KeyTime, LocalTranslations[Counter].Z);
					DoubleChannels[3]->AddLinearKey(KeyTime, LocalRotations[Counter].X);
					DoubleChannels[4]->AddLinearKey(KeyTime, LocalRotations[Counter].Y);
					DoubleChannels[5]->AddLinearKey(KeyTime, LocalRotations[Counter].Z);
					DoubleChannels[6]->AddLinearKey(KeyTime, LocalScales[Counter].X);
					DoubleChannels[7]->AddLinearKey(KeyTime, LocalScales[Counter].Y);
					DoubleChannels[8]->AddLinearKey(KeyTime, LocalScales[Counter].Z);
				}
				if (InSettings.bReduceKeys == true)
				{
					FKeyDataOptimizationParams Param;
					Param.bAutoSetInterpolation = true;
					Param.Tolerance = InSettings.Tolerance;
					TRange<FFrameNumber> Range(InSettings.StartFrame, InSettings.EndFrame);
					Param.Range = Range;
					MovieSceneToolHelpers::OptimizeSection(Param, TransformSection);
				}
			}
		}
	}

	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	return true;
}

void FSequencerUtilities::ShowReadOnlyError()
{
	FNotificationInfo Info(LOCTEXT("SequenceReadOnly", "Sequence is read only."));
	Info.ExpireDuration = 5.0f;
	FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
}

void FSequencerUtilities::ShowSpawnableNotAllowedError()
{
	FNotificationInfo Info(LOCTEXT("SequenceSpawnableNotAllowed", "Spawnable object is not allowed for Sequence."));
	Info.ExpireDuration = 5.0f;
	FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);	
}

void FSequencerUtilities::SaveCurrentMovieSceneAs(TSharedRef<ISequencer> Sequencer)
{
	StaticCastSharedPtr<FSequencer>(Sequencer.ToSharedPtr())->SaveCurrentMovieSceneAs();
}

void FSequencerUtilities::SynchronizeExternalSelectionWithSequencerSelection (TSharedRef<ISequencer> Sequencer)
{
	StaticCastSharedPtr<FSequencer>(Sequencer.ToSharedPtr())->SynchronizeExternalSelectionWithSequencerSelection();
}

TRange<FFrameNumber> FSequencerUtilities::GetTimeBounds(TSharedRef<ISequencer> Sequencer)
{
	return StaticCastSharedPtr<FSequencer>(Sequencer.ToSharedPtr())->GetTimeBounds();
}

void FSequencerUtilities::AddChangeClassMenu(FMenuBuilder& MenuBuilder, TSharedRef<ISequencer> Sequencer, const TArray<FSequencerChangeBindingInfo>& Bindings, TFunction<void()> OnBindingChanged)
{
	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return;
	}
	UMovieScene* MovieScene = Sequence->GetMovieScene();

	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.bIsPlaceableOnly = true;

	for (const FSequencerChangeBindingInfo& Binding : Bindings)
	{
		FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(Binding.BindingID);
		if (Spawnable)
		{
			Options.bIsActorsOnly = true;
		}
		else if (const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
		{
			TArrayView<const FMovieSceneBindingReference> BindingReferencesList = BindingReferences->GetReferences(Binding.BindingID);
			if (BindingReferencesList.IsValidIndex(Binding.BindingIndex) && BindingReferencesList[Binding.BindingIndex].CustomBinding && BindingReferencesList[Binding.BindingIndex].CustomBinding->WillSpawnObject(Sequencer->GetSharedPlaybackState()))
			{
				// Class filter for the custom binding type
				class FCustomBindingClassFilter : public IClassViewerFilter
				{
				public:
					bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
					{
						return CustomBinding && InClass && CustomBinding->SupportsBindingCreationFromObject(InClass->GetDefaultObject());
					}

					virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
					{
						if (const UClass* ClassWithin = InClass->GetClassWithin())
						{
							return IsClassAllowed(InInitOptions, ClassWithin, InFilterFuncs);
						}
						return false;
					}

					TObjectPtr<UMovieSceneCustomBinding> CustomBinding;
				};

				TSharedRef<FCustomBindingClassFilter> ClassFilter = MakeShared<FCustomBindingClassFilter>();
				ClassFilter->CustomBinding = BindingReferencesList[0].CustomBinding;
				Options.ClassFilters.Add(ClassFilter);
			}
			else
			{
				return;
			}
		}
		else
		{
			return;
		}

		const UClass* ClassForObjectBinding = MovieSceneHelpers::GetBoundObjectClass(Sequence, Binding.BindingID, Binding.BindingIndex);
		if (ClassForObjectBinding)
		{
			Options.ViewerTitleString = FText::FromString(TEXT("Change from: ") + ClassForObjectBinding->GetFName().ToString());
		}
		else
		{
			Options.ViewerTitleString = FText::FromString(TEXT("Change from: (empty)"));
		}
	}

	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	MenuBuilder.AddWidget(
		SNew(SBox)
		.MinDesiredWidth(300.0f)
		.MaxDesiredHeight(400.0f)
		[
			ClassViewerModule.CreateClassViewer(Options, FOnClassPicked::CreateLambda([Sequencer, Bindings, OnBindingChanged](UClass* Class) { FSequencerUtilities::HandleTemplateActorClassPicked(Class, Sequencer, Bindings, OnBindingChanged); }))
		],
		FText(), true, false
	);
}

void UpdatePossessedClasses(UMovieScene* MovieScene, FMovieSceneSequenceIDRef SequenceID, const FMovieSceneSequenceHierarchy* Hierarchy, FGuid ObjectBindingID, UClass* ChosenClass)
{
	for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
	{
		FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);
		const bool bIsSelf = Possessable.GetGuid() == ObjectBindingID;
		const bool bTracksSpawnable = Possessable.GetSpawnableObjectBindingID().GetGuid() == ObjectBindingID;
		if ((bIsSelf || bTracksSpawnable) && Possessable.GetPossessedObjectClass() != ChosenClass)
		{
			MovieScene->Modify();

			Possessable.SetPossessedObjectClass(ChosenClass);
		}
	}

	if (const FMovieSceneSequenceHierarchyNode* Node = Hierarchy->FindNode(SequenceID))
	{
		for (FMovieSceneSequenceIDRef ChildID : Node->Children)
		{
			const FMovieSceneSubSequenceData* SubData = Hierarchy->FindSubData(ChildID);
			if (SubData)
			{
				UMovieSceneSequence* SubSequence = SubData->GetSequence();
				UMovieScene* SubMovieScene = SubSequence ? SubSequence->GetMovieScene() : nullptr;

				if (SubMovieScene)
				{
					UpdatePossessedClasses(SubMovieScene, ChildID, Hierarchy, ObjectBindingID, ChosenClass);
				}
			}
		}
	}
}

void FSequencerUtilities::HandleTemplateActorClassPicked(UClass* ChosenClass, TSharedRef<ISequencer> Sequencer, const TArray<FSequencerChangeBindingInfo>& Bindings, TFunction<void()> OnBindingChanged)
{
	UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();

	FScopedTransaction Transaction(LOCTEXT("ChangeClass", "Change Class"));

	MovieScene->Modify();

	TValueOrError<FNewSpawnable, FText> Result = Sequencer->GetSpawnRegister().CreateNewSpawnableType(*ChosenClass, *MovieScene, nullptr);
	if (Result.IsValid())
	{
		FMovieSceneRootEvaluationTemplateInstance& RootInstance = Sequencer->GetEvaluationTemplate();
		const FMovieSceneSequenceHierarchy* Hierarchy = RootInstance.GetCompiledDataManager()->FindHierarchy(RootInstance.GetCompiledDataID());

		for (const FSequencerChangeBindingInfo& Binding : Bindings)
		{
			UpdatePossessedClasses(Sequencer->GetRootMovieSceneSequence()->GetMovieScene(), MovieSceneSequenceID::Root, Hierarchy, Binding.BindingID, ChosenClass);

			MovieSceneHelpers::SetObjectTemplate(Sequencer->GetFocusedMovieSceneSequence(), Binding.BindingID, Result.GetValue().ObjectTemplate, Sequencer->GetSharedPlaybackState(), Binding.BindingIndex);

			Sequencer->GetSpawnRegister().DestroySpawnedObject(Binding.BindingID, Sequencer->GetFocusedTemplateID(), Sequencer->GetSharedPlaybackState(), Binding.BindingIndex);
		}
		Sequencer->ForceEvaluate();
	}

	if (OnBindingChanged)
	{
		OnBindingChanged();
	}
}

bool FSequencerUtilities::CanConvertToPossessable(TSharedRef<ISequencer> Sequencer, FGuid BindingGuid, int32 BindingIndex)
{
	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return false;
	}

	if (FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(BindingGuid))
	{
		return true;
	}
	else if (const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
	{
		TArrayView<const FMovieSceneBindingReference> BindingReferencesList = BindingReferences->GetReferences(BindingGuid);
		if (BindingReferencesList.IsValidIndex(BindingIndex) && BindingReferencesList[BindingIndex].CustomBinding != nullptr)
		{
			return true;
		}
	}
	return false;
}

bool FSequencerUtilities::CanConvertToCustomBinding(TSharedRef<ISequencer> Sequencer, FGuid BindingGuid, TSubclassOf<UMovieSceneCustomBinding> CustomBindingType, int32 BindingIndex)
{
	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return false;
	}
	if (FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(BindingGuid))
	{
		if (UObject* CurrentBoundObject = Spawnable->GetObjectTemplate())
		{
			return CustomBindingType && CustomBindingType->GetDefaultObject<UMovieSceneCustomBinding>()->SupportsBindingCreationFromObject(CurrentBoundObject);
		}
	}
	else if (FMovieScenePossessable* Possessable = MovieScene->FindPossessable(BindingGuid))
	{
		if (const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
		{
			UObject* ResolutionContext = MovieSceneHelpers::GetResolutionContext(Sequence, BindingGuid, Sequencer->GetFocusedTemplateID(), Sequencer->GetSharedPlaybackState());

			TArrayView<const FMovieSceneBindingReference> BindingReferencesList = Sequencer->GetFocusedMovieSceneSequence()->GetBindingReferences()->GetReferences(BindingGuid);

			if (const FMovieSceneBindingReference* CurrentBindingReference = BindingReferences->GetReference(BindingGuid, BindingIndex))
			{
				UE::UniversalObjectLocator::FResolveParams LocatorResolveParams(ResolutionContext);
				FMovieSceneBindingResolveParams BindingResolveParams{ Sequence, BindingGuid, Sequencer->GetFocusedTemplateID(), ResolutionContext };
				TArray<UObject*, TInlineAllocator<1>> CurrentBoundObjects;
				BindingReferences->ResolveSingleBinding(BindingResolveParams, BindingIndex, LocatorResolveParams, Sequencer->GetSharedPlaybackState(), CurrentBoundObjects);

				for (UObject* CurrentBoundObject : CurrentBoundObjects)
				{
					if (CustomBindingType
						&& (!CurrentBindingReference->CustomBinding
							|| CurrentBindingReference->CustomBinding->GetClass() != CustomBindingType)
						&& CustomBindingType->GetDefaultObject<UMovieSceneCustomBinding>()->SupportsConversionFromBinding(*CurrentBindingReference, CurrentBoundObject))
					{
						return true;
					}
				}
			}
		}
	}
	return false;
}

UMovieSceneSequence* FSequencerUtilities::GetMovieSceneSequence(TSharedPtr<ISequencer>& InSequencer, const FMovieSceneSequenceID& SequenceID)
{
	if (MovieSceneSequenceID::Root != SequenceID)
	{
		UMovieSceneSubSection* SubSection = InSequencer->FindSubSection(SequenceID);
		return SubSection ? SubSection->GetSequence() : nullptr;
	}
	return InSequencer->GetRootMovieSceneSequence();
}

void FSequencerUtilities::SnapSectionsToTimelineUsingSourceTimecode(TSharedRef<ISequencer> Sequencer, const TArray<UMovieSceneSection*>& Sections)
{
	const UMovieScene* FocusedMovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		ShowReadOnlyError();
		return;
	}

	FScopedTransaction SnapSectionsToTimelineUsingSourceTimecodeTransaction(LOCTEXT("SnapSectionsToTimelineUsingSourceTimecode_Transaction", "Snap Sections to Timeline using Source Timecode"));
	bool bAnythingChanged = false;

	const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
	const FFrameRate DisplayRate = Sequencer->GetFocusedDisplayRate();

	for (UMovieSceneSection* Section : Sections)
	{
		if (!Section || !Section->HasStartFrame())
		{
			continue;
		}

		const FTimecode SectionSourceTimecode = Section->TimecodeSource.Timecode;
		if (SectionSourceTimecode == FTimecode())
		{
			// Do not move sections with default values for source timecode.
			continue;
		}

		const FFrameNumber SectionSourceStartFrameNumber = SectionSourceTimecode.ToFrameNumber(DisplayRate);

		// Account for any trimming at the start of the section when computing the
		// target frame number to move this section to.
		const FFrameNumber SectionOffsetFrames = Section->GetOffsetTime().Get(FFrameTime()).FloorToFrame();
		const FFrameNumber TargetFrameNumber = SectionSourceStartFrameNumber + SectionOffsetFrames;

		const FFrameNumber SectionCurrentStartFrameNumber = Section->GetInclusiveStartFrame();

		const FFrameNumber Delta = -(SectionCurrentStartFrameNumber - ConvertFrameTime(TargetFrameNumber, DisplayRate, TickResolution).GetFrame().Value);

		Section->MoveSection(Delta);

		bAnythingChanged |= (Delta.Value != 0);
	}

	if (bAnythingChanged)
	{
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}
}

void FSequencerUtilities::SyncSectionsUsingSourceTimecode(TSharedRef<ISequencer> Sequencer, const TArray<UMovieSceneSection*>& Sections)
{
	const UMovieScene* FocusedMovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		ShowReadOnlyError();
		return;
	}

	// Pull out all of the valid sections that have a start frame and verify
	// we have at least two sections to sync.
	TArray<UMovieSceneSection*> SectionsToSync;
	for (UMovieSceneSection* Section : Sections)
	{
		if (Section && Section->HasStartFrame())
		{
			SectionsToSync.Add(Section);
		}
	}

	if (SectionsToSync.Num() < 2)
	{
		return;
	}

	FScopedTransaction SyncSectionsUsingSourceTimecodeTransaction(LOCTEXT("SyncSectionsUsingSourceTimecode_Transaction", "Sync Sections Using Source Timecode"));
	bool bAnythingChanged = false;

	const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
	const FFrameRate DisplayRate = Sequencer->GetFocusedDisplayRate();

	const UMovieSceneSection* FirstSection = SectionsToSync[0];
	const FFrameNumber FirstSectionSourceTimecode = FirstSection->TimecodeSource.Timecode.ToFrameNumber(DisplayRate);

	const FFrameNumber FirstSectionCurrentStartFrame = FirstSection->GetInclusiveStartFrame();
	const FFrameNumber FirstSectionOffsetFrames = FirstSection->GetOffsetTime().Get(FFrameTime()).FloorToFrame();
	SectionsToSync.RemoveAt(0);

	for (UMovieSceneSection* Section : SectionsToSync)
	{
		const FFrameNumber SectionSourceTimecode = Section->TimecodeSource.Timecode.ToFrameNumber(DisplayRate);
		const FFrameNumber SectionCurrentStartFrame = Section->GetInclusiveStartFrame();
		const FFrameNumber SectionOffsetFrames = Section->GetOffsetTime().Get(FFrameTime()).FloorToFrame();

		const FFrameNumber TimecodeDelta = ConvertFrameTime(SectionSourceTimecode - FirstSectionSourceTimecode, DisplayRate, TickResolution).GetFrame().Value;
		const FFrameNumber CurrentDelta = (SectionCurrentStartFrame - SectionOffsetFrames) - (FirstSectionCurrentStartFrame - FirstSectionOffsetFrames);
		const FFrameNumber Delta = -CurrentDelta + TimecodeDelta;

		Section->MoveSection(Delta);

		bAnythingChanged |= (Delta.Value != 0);
	}

	if (bAnythingChanged)
	{
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}
}

void FSequencerUtilities::FixActorReferences(TSharedRef<ISequencer> Sequencer)
{
	UObject* PlaybackContext = Sequencer->GetPlaybackContext();
	if (!PlaybackContext)
	{
		return;
	}

	UWorld* World = PlaybackContext->GetWorld();
	if (!World)
	{
		return;
	}

	UMovieSceneSequence* FocusedSequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return;
	}

	UMovieScene* FocusedMovieScene = FocusedSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}

	if (FocusedMovieScene->IsReadOnly())
	{
		ShowReadOnlyError();
		return;
	}

	FScopedTransaction FixActorReferencesTransaction(LOCTEXT("FixActorReferences", "Fix Actor References"));

	TMap<FString, AActor*> ActorNameToActorMap;

	for (TActorIterator<AActor> ActorItr(World); ActorItr; ++ActorItr)
	{
		// Same as with the Object Iterator, access the subclass instance with the * or -> operators.
		AActor* Actor = *ActorItr;
		ActorNameToActorMap.Add(Actor->GetActorLabel(), Actor);
	}

	// Cache the possessables to fix up first since the bindings will change as the fix ups happen.
	TArray<FMovieScenePossessable> ActorsPossessablesToFix;
	for (int32 i = 0; i < FocusedMovieScene->GetPossessableCount(); i++)
	{
		FMovieScenePossessable& Possessable = FocusedMovieScene->GetPossessable(i);
		// Possessables with parents are components so ignore them.
		if (Possessable.GetParent().IsValid() == false)
		{
			bool bAnyValidBindings = false;
			for (TWeakObjectPtr<> WeakObject : Sequencer->FindBoundObjects(Possessable.GetGuid(), Sequencer->GetFocusedTemplateID()))
			{
				if (WeakObject.IsValid())
				{
					bAnyValidBindings = true;
					break;
				}
			}

			if (!bAnyValidBindings)
			{
				ActorsPossessablesToFix.Add(Possessable);
			}
		}
	}

	// For the possessables to fix, look up the actors by name and reassign them if found.
	TMap<FGuid, FGuid> OldGuidToNewGuidMap;
	for (const FMovieScenePossessable& ActorPossessableToFix : ActorsPossessablesToFix)
	{
		AActor* ActorPtr = ActorNameToActorMap.FindRef(ActorPossessableToFix.GetName());
		if (ActorPtr != nullptr)
		{
			FGuid OldGuid = ActorPossessableToFix.GetGuid();

			// The actor might have an existing guid while the possessable with the same name might not. 
			// In that case, make sure we also replace the existing guid with the new guid 
			FGuid ExistingGuid = Sequencer->FindObjectId(*ActorPtr, Sequencer->GetFocusedTemplateID());

			FGuid NewGuid = AssignActor(Sequencer, ActorPtr, ActorPossessableToFix.GetGuid());

			OldGuidToNewGuidMap.Add(OldGuid, NewGuid);

			if (ExistingGuid.IsValid())
			{
				OldGuidToNewGuidMap.Add(ExistingGuid, NewGuid);
			}
		}
	}

	for (TPair<FGuid, FGuid> GuidPair : OldGuidToNewGuidMap)
	{
		UpdateBindingIDs(Sequencer, GuidPair.Key, GuidPair.Value);
	}

	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

void FOpenSequencerWatcher::DoStartup(TFunction<void()> StartupComplete)
{
	auto RegisterWatcher = [this, StartupCompleteFunc = MoveTemp(StartupComplete)]()
	{
		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
		SequencerModule.RegisterOnSequencerCreated(
			FOnSequencerCreated::FDelegate::CreateRaw(this, &FOpenSequencerWatcher::OnSequencerCreated));

		StartupCompleteFunc();
	};

	if (GEngine)
	{
		RegisterWatcher();
	}
	else
	{
		FCoreDelegates::OnFEngineLoopInitComplete.AddLambda(RegisterWatcher);
	}
}

void FOpenSequencerWatcher::OnSequencerCreated(TSharedRef<ISequencer> InSequencer)
{
	FOpenSequencerData OpenSequencer;
	OpenSequencer.WeakSequencer = TWeakPtr<ISequencer>(InSequencer);
	OpenSequencer.OnCloseEventHandle = InSequencer->OnCloseEvent().AddRaw(this, &FOpenSequencerWatcher::OnSequencerClosed);
	OpenSequencers.Add(MoveTemp(OpenSequencer));
}

void FOpenSequencerWatcher::OnSequencerClosed(TSharedRef<ISequencer> InSequencer)
{
	OpenSequencers.RemoveAll([SequencerObject=&InSequencer.Get()](const FOpenSequencerData& Data)
	{
		return Data.WeakSequencer.HasSameObject(SequencerObject);
	});
}

#undef LOCTEXT_NAMESPACE
