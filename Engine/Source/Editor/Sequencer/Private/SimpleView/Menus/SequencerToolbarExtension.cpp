// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerToolbarExtension.h"
#include "ISequencer.h"
#include "Menus/SequencerToolbarUtils.h"
#include "Sequencer.h"
#include "SimpleView/SimpleViewCommands.h"
#include "SimpleView/SimpleViewTimeline.h"
#include "SimpleView/SimpleViewUtils.h"
#include "SimpleView/Widgets/SValueStepper.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolableTimeline/Menus/ToolableTimelineMenuContext.h"
#include "ToolableTimeline/ToolableTimelineCommands.h"

#define LOCTEXT_NAMESPACE "SequencerToolbarExtension"

namespace UE::Sequencer::SimpleView
{

TSharedPtr<FSequencer> GetSequencerFromTimeline(const TWeakPtr<FSimpleViewTimeline>& InWeakTimeline)
{
	const TSharedPtr<FSimpleViewTimeline> Timeline = InWeakTimeline.Pin();
	return Timeline.IsValid() ? Timeline->GetSequencer() : nullptr;
}

TSharedPtr<FSimpleViewTimeline> GetTimelineFromContext(UToolableTimelineMenuContext* const InContext)
{
	return InContext ? StaticCastSharedPtr<FSimpleViewTimeline>(InContext->WeakTimeline.Pin()) : nullptr;
}

void ExecuteSimpleViewAction(const TWeakPtr<FSimpleViewTimeline> InWeakTimeline
	, const TSharedRef<FUICommandInfo> InAction)
{
	if (const TSharedPtr<FSimpleViewTimeline> Timeline = InWeakTimeline.Pin())
	{
		Timeline->GetCommandList()->ExecuteAction(InAction);
	}
}

bool CanExecuteSimpleViewAction(const TWeakPtr<FSimpleViewTimeline> InWeakTimeline
	, const TArray<TSharedPtr<FUICommandInfo>> InActions)
{
	const TSharedPtr<FSimpleViewTimeline> Timeline = InWeakTimeline.Pin();
	if (!Timeline.IsValid())
	{
		return false;
	}

	for (const TSharedPtr<FUICommandInfo>& Action : InActions)
	{
		if (Action.IsValid()
			&& Timeline->GetCommandList()->CanExecuteAction(Action.ToSharedRef()))
		{
			return true;
		}
	}

	return false;
}

FSlateIcon GetIconForTimelineKeyDisplay(const TWeakPtr<FSimpleViewTimeline> InWeakTimeline)
{
	const TSharedPtr<FSimpleViewTimeline> Timeline = InWeakTimeline.Pin();
	if (!Timeline.IsValid())
	{
		return FSlateIcon();
	}

	switch (Timeline->GetKeyDisplay())
	{
	case ETimelineKeyDisplay::SelectedAndPinned:
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.SimpleView.KeyDisplay.SelectedAndPinned"));
	case ETimelineKeyDisplay::Selected:
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.SimpleView.KeyDisplay.Selected"));
	case ETimelineKeyDisplay::All:
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.SimpleView.KeyDisplay.All"));
	default:
		return FSlateIcon();
	}
}

TSharedRef<SWidget> MakeKeyDisplayMenu(const FToolMenuContext& InContext)
{
	const TSharedPtr<FSimpleViewTimeline> Timeline = GetTimelineFromContext(InContext.FindContext<UToolableTimelineMenuContext>());
	if (!Timeline.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	if (!UToolMenus::Get()->IsMenuRegistered(FSequencerToolbarExtension::KeyDisplayMenuName))
	{
		UToolMenu* const NewMenu = UToolMenus::Get()->RegisterMenu(FSequencerToolbarExtension::KeyDisplayMenuName);
		check(NewMenu);

		FToolMenuSection& Section = NewMenu->FindOrAddSection(TEXT("KeyDisplay"), LOCTEXT("KeyDisplaySection", "Key Display"));

		const FToolableTimelineCommands& TimelineCommands = FToolableTimelineCommands::Get();

		Section.AddMenuEntry(
			TimelineCommands.SetKeyDisplay_SelectedAndPinned,
			TimelineCommands.SetKeyDisplay_SelectedAndPinned->GetLabel(),
			TimelineCommands.SetKeyDisplay_SelectedAndPinned->GetDescription(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.SimpleView.KeyDisplay.SelectedAndPinned"))
		);

		Section.AddMenuEntry(
			TimelineCommands.SetKeyDisplay_Selected,
			TimelineCommands.SetKeyDisplay_Selected->GetLabel(),
			TimelineCommands.SetKeyDisplay_Selected->GetDescription(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.SimpleView.KeyDisplay.Selected"))
		);

		Section.AddMenuEntry(
			TimelineCommands.SetKeyDisplay_All,
			TimelineCommands.SetKeyDisplay_All->GetLabel(),
			TimelineCommands.SetKeyDisplay_All->GetDescription(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.SimpleView.KeyDisplay.All"))
		);
	}

	const FToolMenuContext MenuContext(Timeline->GetCommandList());
	return UToolMenus::Get()->GenerateWidget(FSequencerToolbarExtension::KeyDisplayMenuName, MenuContext);
}

void CreateKeyDisplaySection(FToolMenuSection& InSection, const UToolableTimelineMenuContext& InContext)
{
	const TSharedPtr<FSimpleViewTimeline> Timeline = StaticCastSharedPtr<FSimpleViewTimeline>(InContext.WeakTimeline.Pin());
	if (!Timeline.IsValid())
	{
		return;
	}

	const TWeakPtr<FSimpleViewTimeline> WeakTimeline = Timeline;

	FToolMenuEntry ComboButtonEntry = FToolMenuEntry::InitComboButton(TEXT("KeyDisplayOptions")
		, FUIAction()
		, FNewToolMenuWidget::CreateStatic(&MakeKeyDisplayMenu)
		, LOCTEXT("KeyDisplaySection_Label", "Key Display")
		, LOCTEXT("KeyDisplaySection_Tooltip", "Options for what keys get displayed in the timeline")
		, TAttribute<FSlateIcon>::CreateStatic(&GetIconForTimelineKeyDisplay, WeakTimeline)
	);
	ComboButtonEntry.StyleNameOverride = GSequencerToolbarStyleName;
	ComboButtonEntry.InsertPosition = FToolMenuInsert(NAME_None, EToolMenuInsertType::First);
	ComboButtonEntry.StyleNameOverride = GSequencerToolbarStyleName;
	ComboButtonEntry.InsertPosition = FToolMenuInsert(NAME_None, EToolMenuInsertType::First);
	ComboButtonEntry.Visibility = TAttribute<EVisibility>::CreateStatic(&Utils::GetSimpleViewVisibility, WeakTimeline);
	InSection.AddEntry(ComboButtonEntry);
}

void CreateKeyTranslateSection(FToolMenuSection& InSection, const UToolableTimelineMenuContext& InContext)
{
	const TSharedPtr<FSimpleViewTimeline> Timeline = StaticCastSharedPtr<FSimpleViewTimeline>(InContext.WeakTimeline.Pin());
	if (!Timeline.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = Timeline->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TWeakPtr<FSimpleViewTimeline> WeakTimeline = Timeline;

	const FSimpleViewCommands& SimpleViewCommands = FSimpleViewCommands::Get();

	const TSharedRef<SValueStepper> ValueWidget = SNew(SValueStepper)
		.NumericTypeInterface(Sequencer->GetNumericTypeInterface())
		.MinFractionalDigits(0)
		.IsVisible_Lambda([WeakTimeline]()
			{
				const TSharedPtr<FSimpleViewTimeline> Timeline = WeakTimeline.Pin();
				return Timeline.IsValid() && Utils::IsInSimpleView(WeakTimeline);
			})
		.ValueTooltipText(LOCTEXT("KeyTranslate_Tooltip", "Frame offset for translating selected keys"))
		.LeftButtonIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.KeyTransform.TranslateLeft")))
		.LeftButtonTooltipText(SimpleViewCommands.TranslateKeyLeft->GetDescription())
		.OnLeftButtonClick(FSimpleDelegate::CreateStatic(&ExecuteSimpleViewAction
			, WeakTimeline, SimpleViewCommands.TranslateKeyLeft.ToSharedRef()))
		.RightButtonIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.KeyTransform.TranslateRight")))
		.RightButtonTooltipText(SimpleViewCommands.TranslateKeyRight->GetDescription())
		.OnRightButtonClick(FSimpleDelegate::CreateStatic(&ExecuteSimpleViewAction
			, WeakTimeline, SimpleViewCommands.TranslateKeyRight.ToSharedRef()))
		.InitialValue(Timeline->GetKeyTranslateDelta())
		.OnValueCommitted_Lambda([WeakTimeline](const double InValue)
			{
				if (const TSharedPtr<FSimpleViewTimeline> Timeline = WeakTimeline.Pin())
				{
					Timeline->SetKeyTranslateDelta(InValue);
				}
			});

	FToolMenuEntry ValueEntry = FToolMenuEntry::InitWidget(TEXT("TranslateDelta")
			, ValueWidget, LOCTEXT("KeyTranslate_Label", "Key Translate"));
	ValueEntry.StyleNameOverride = GSequencerToolbarStyleName;
	ValueEntry.InsertPosition = FToolMenuInsert(NAME_None, EToolMenuInsertType::First);
	ValueEntry.Visibility = TAttribute<EVisibility>::CreateStatic(&Utils::GetSimpleViewVisibility, WeakTimeline);
	ValueEntry.WidgetData.StyleParams.VerticalAlignment = VAlign_Center;
	ValueEntry.WidgetData.bNoPadding = true;
	InSection.AddEntry(ValueEntry);
}

void CreateKeyScaleSection(FToolMenuSection& InSection, const UToolableTimelineMenuContext& InContext)
{
	const TSharedPtr<FSimpleViewTimeline> Timeline = StaticCastSharedPtr<FSimpleViewTimeline>(InContext.WeakTimeline.Pin());
	if (!Timeline.IsValid())
	{
		return;
	}

	const TWeakPtr<FSimpleViewTimeline> WeakTimeline = Timeline;

	const FSimpleViewCommands& SimpleViewCommands = FSimpleViewCommands::Get();

	const TSharedRef<SValueStepper> ValueWidget = SNew(SValueStepper)
		.MinFractionalDigits(2)
		.IsEnabled_Lambda([WeakTimeline]()
			{
				const TSharedPtr<FSimpleViewTimeline> Timeline = WeakTimeline.Pin();
				return Timeline.IsValid() && Utils::IsSimpleViewToolMenuVisible(WeakTimeline);
			})
		.IsVisible_Lambda([WeakTimeline]()
			{
				const TSharedPtr<FSimpleViewTimeline> Timeline = WeakTimeline.Pin();
				return Timeline.IsValid() && Utils::IsInSimpleView(WeakTimeline);
			})
		.ValueTooltipText(LOCTEXT("KeyScaleValue_Tooltip", "Scale factor for scaling selected keys"))
		.LeftButtonIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.KeyTransform.ScaleMultiply")))
		.LeftButtonTooltipText(SimpleViewCommands.ScaleKeyMultiply->GetDescription())
		.OnLeftButtonClick(FSimpleDelegate::CreateStatic(&ExecuteSimpleViewAction
			, WeakTimeline, SimpleViewCommands.ScaleKeyMultiply.ToSharedRef()))
		.RightButtonIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.KeyTransform.ScaleDivide")))
		.RightButtonTooltipText(SimpleViewCommands.ScaleKeyDivide->GetDescription())
		.OnRightButtonClick(FSimpleDelegate::CreateStatic(&ExecuteSimpleViewAction
			, WeakTimeline, SimpleViewCommands.ScaleKeyDivide.ToSharedRef()))
		.InitialValue(Timeline->GetKeyScaleFactor())
		.OnValueCommitted_Lambda([WeakTimeline](const double InValue)
			{
				if (const TSharedPtr<FSimpleViewTimeline> Timeline = WeakTimeline.Pin())
				{
					Timeline->SetKeyScaleFactor(InValue);
				}
			});

	FToolMenuEntry ValueEntry = FToolMenuEntry::InitWidget(TEXT("ScaleFactor")
			, ValueWidget, LOCTEXT("KeyScale_Label", "Key Scale"));
	ValueEntry.StyleNameOverride = GSequencerToolbarStyleName;
	ValueEntry.InsertPosition = FToolMenuInsert(TEXT("TranslateDelta"), EToolMenuInsertType::After);
	ValueEntry.Visibility = TAttribute<EVisibility>::CreateStatic(&Utils::GetSimpleViewVisibility, WeakTimeline);
	ValueEntry.WidgetData.StyleParams.VerticalAlignment = VAlign_Center;
	ValueEntry.WidgetData.bNoPadding = true;
	InSection.AddEntry(ValueEntry);
}

void BuildSimpleViewSection(FToolMenuSection& InSection)
{
	UToolableTimelineMenuContext* const SimpleViewContext = InSection.FindContext<UToolableTimelineMenuContext>();
	if (!SimpleViewContext)
	{
		return;
	}

	CreateKeyDisplaySection(InSection, *SimpleViewContext);
	CreateKeyTranslateSection(InSection, *SimpleViewContext);
	CreateKeyScaleSection(InSection, *SimpleViewContext);
}

const FName FSequencerToolbarExtension::SequencerToolbarMenuName = TEXT("Sequencer.MainToolBar");
const FName FSequencerToolbarExtension::SimpleViewOwner = TEXT("SimpleViewSequencerToolbar");
const FName FSequencerToolbarExtension::SimpleViewSectionName = TEXT("SimpleView");
const FName FSequencerToolbarExtension::KeyDisplayMenuName = TEXT("SimpleView.Menu.KeyDisplay");

void FSequencerToolbarExtension::AddExtension()
{
	UToolMenus* const ToolMenus = Utils::GetToolMenusSafe();
	if (!ToolMenus)
	{
		return;
	}

	FToolMenuOwnerScoped ScopeOwner(SimpleViewOwner);

	UToolMenu* const ToolMenu = ToolMenus->ExtendMenu(SequencerToolbarMenuName);
	if (!ToolMenu)
	{
		return;
	}

	FToolMenuSection& Section = ToolMenu->FindOrAddSection(SimpleViewSectionName, LOCTEXT("SimpleViewSection", "Simple View"));

	Section.AddDynamicEntry(TEXT("SimpleViewToolbarEntries"),
		FNewToolMenuSectionDelegate::CreateStatic(&BuildSimpleViewSection));
}

void FSequencerToolbarExtension::RemoveExtension()
{
	UToolMenus* const ToolMenus = Utils::GetToolMenusSafe();
	if (!ToolMenus)
	{
		return;
	}

	ToolMenus->UnregisterOwnerByName(SimpleViewOwner);
}

} // namespace UE::Sequencer::SimpleView

#undef LOCTEXT_NAMESPACE
