// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::Sequencer::SimpleView
{

/** Extension for Sequencer toolbar extensions */
class FSequencerToolbarExtension
{
public:
	static const FName SequencerToolbarMenuName;
	static const FName SimpleViewOwner;
	static const FName SimpleViewSectionName;
	static const FName KeyDisplayMenuName;

	static void AddExtension();
	static void RemoveExtension();
};

} // namespace UE::Sequencer::SimpleView
