// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimelineMenuExtension.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Sequencer.h"
#include "SequencerCommands.h"
#include "SimpleView/SequencerSimpleViewSettings.h"
#include "SimpleView/SimpleViewTimeline.h"
#include "ToolableTimeline/Menus/ToolableTimelineMenu.h"
#include "ToolableTimeline/Menus/ToolableTimeSliderControllerMenuContext.h"
#include "ToolableTimeline/ToolableTimelineInstanceSettingsCustomization.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "SimpleView/SimpleViewCommands.h"
#include "SimpleView/SimpleViewUtils.h"

#define LOCTEXT_NAMESPACE "ToolableTimelineMenuExtension"

namespace UE::Sequencer::SimpleView
{

const FName FToolableTimelineMenuExtension::SimpleViewSelectionRangeMenuOwner = TEXT("SimpleViewMenuExtension");
const FName FToolableTimelineMenuExtension::ToggleSectionName = TEXT("ToggleSection");

FName FToolableTimelineMenuExtension::GetToolSelectionRangeMenuName()
{
	const FString MenuName = ToolableTimeline::FToolableTimelineMenu::MenuName.ToString() + TEXT(".SelectionRange");
	return *MenuName;
}

void FToolableTimelineMenuExtension::AddExtension(const TSharedRef<FSimpleViewTimeline>& InTimeline)
{
	UToolMenus* const ToolMenus = Utils::GetToolMenusSafe();
	if (!ToolMenus)
	{
		return;
	}

	FToolMenuOwnerScoped ScopeOwner(SimpleViewSelectionRangeMenuOwner);

	UToolMenu* const ToolMenu = ToolMenus->ExtendMenu(ToolableTimeline::FToolableTimelineMenu::MenuName);
	if (!ToolMenu)
	{
		return;
	}

	const TWeakPtr<FSimpleViewTimeline> WeakTimeline = InTimeline;

	auto GetSequencer = [WeakTimeline]() -> TSharedPtr<FSequencer>
	{
		const TSharedPtr<FSimpleViewTimeline> Timeline = WeakTimeline.Pin();
		return Timeline.IsValid() ? Timeline->GetSequencer() : nullptr;
	};

	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const FSequencerCommands& SequencerCommands = FSequencerCommands::Get();

	FToolMenuSection& MiscSection = ToolMenu->FindOrAddSection(ToggleSectionName
		, LOCTEXT("SimpleViewSection", "Sequencer / Timeline"));

	MiscSection.AddEntry(FToolMenuEntry::InitSubMenu(TEXT("SimpleViewSettings")
		, LOCTEXT("SimpleViewSettings_Label", "Simple View Settings")
		, LOCTEXT("SimpleViewSettings_Tooltip", "Settings for the Sequencer Simple View")
		, FNewToolMenuDelegate::CreateLambda([](UToolMenu* const InMenu)
			{
				UToolableTimeSliderControllerMenuContext* const MenuContext = InMenu->FindContext<UToolableTimeSliderControllerMenuContext>();
				if (!MenuContext)
				{
					return;
				}

				const TSharedPtr<ToolableTimeline::FToolableTimeline> Timeline = MenuContext->WeakTimeline.Pin();
				if (!Timeline.IsValid())
				{
					return;
				}

				const TSharedRef<SBox> SettingsWidget = SNew(SBox)
					.MinDesiredWidth(300)
					[
						CreateSettingsDetailsView(Timeline.ToSharedRef())
					];

				InMenu->AddSection(TEXT("SimpleViewSettings"), LOCTEXT("SimpleViewSettings", "Simple View Settings"));
				InMenu->AddMenuEntry(TEXT("SimpleViewSettings")
					, FToolMenuEntry::InitWidget(TEXT("SimpleViewSettings")
					, SettingsWidget
					, FText::GetEmpty()));
			})
		, FUIAction()
		, EUserInterfaceActionType::None
		, false
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Settings"))));

	MiscSection.AddMenuEntryWithCommandList(SequencerCommands.ToggleSimpleView
		, Sequencer->GetCommandBindings()
		, SequencerCommands.ToggleSimpleView->GetLabel()
		, SequencerCommands.ToggleSimpleView->GetDescription()
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.SimpleView.ToggleToFullView")));
}

void FToolableTimelineMenuExtension::RemoveExtension()
{
	UToolMenus* const ToolMenus = Utils::GetToolMenusSafe();
	if (!ToolMenus)
	{
		return;
	}

	ToolMenus->UnregisterOwnerByName(SimpleViewSelectionRangeMenuOwner);
}

TSharedRef<IDetailsView> FToolableTimelineMenuExtension::CreateSettingsDetailsView(const TSharedRef<ToolableTimeline::FToolableTimeline>& InTimeline)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.ColumnWidth = .3f;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;

	const TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->RegisterInstancedCustomPropertyTypeLayout(FToolableTimelineInstanceSettings::StaticStruct()->GetFName()
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&ToolableTimeline::FToolableTimelineInstanceSettingsCustomization::MakeInstance));

	USequencerSimpleViewSettings* const SimpleViewSettings = GetMutableDefault<USequencerSimpleViewSettings>();
	DetailsView->SetObject(SimpleViewSettings);

	return DetailsView;
}

} // namespace UE::Sequencer::SimpleView

#undef LOCTEXT_NAMESPACE
