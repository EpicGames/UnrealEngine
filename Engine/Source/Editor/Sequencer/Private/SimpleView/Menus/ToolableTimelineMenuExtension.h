// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IDetailsView;

namespace UE::Sequencer::ToolableTimeline
{
	class FToolableTimeline;
}

namespace UE::Sequencer::SimpleView
{

class FSimpleViewTimeline;

/** Extension for toolable timeline menu extensions */
class FToolableTimelineMenuExtension
{
public:
	static const FName SimpleViewSelectionRangeMenuOwner;
	static const FName ToggleSectionName;

	static FName GetToolSelectionRangeMenuName();

	static void AddExtension(const TSharedRef<FSimpleViewTimeline>& InTimeline);
	static void RemoveExtension();

protected:
	static TSharedRef<IDetailsView> CreateSettingsDetailsView(const TSharedRef<ToolableTimeline::FToolableTimeline>& InTimeline);
};

} // namespace UE::Sequencer::SimpleView
