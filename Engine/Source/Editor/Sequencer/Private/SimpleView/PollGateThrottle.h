// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformTime.h"

namespace UE::Sequencer::SimpleView
{

/** Simple gate used for throttling various things for performance.
 * Experimental. Currently not used. */
class FPollGateThrottle
{
public:
	static constexpr double DefaultPollRateHertz = 30.0; // Hertz

	explicit FPollGateThrottle(const double InHertz = DefaultPollRateHertz)
		: ApplyIntervalSeconds(InHertz > 0.0 ? (1.0 / InHertz) : 0.0)
	{}

	void Reset()
	{
		LastApplySeconds = 0.0;
		LastActivitySeconds = 0.0;

		bHasAppliedOnce = false;
		bIdleFlushConsumed = false;
	}

	void NotifyActivity() const
	{
		const double NowSeconds = FPlatformTime::Seconds();
		LastActivitySeconds = NowSeconds;

		// Re-arm idle flush when activity resumes
		bIdleFlushConsumed = false;
	}

	/** Returns true if the throttled stage should execute */
	bool ShouldApply() const
	{
		const double NowSeconds = FPlatformTime::Seconds();

		// First pass always allowed
		if (!bHasAppliedOnce)
		{
			bHasAppliedOnce = true;
			LastApplySeconds = NowSeconds;
			return true;
		}

		// Normal throttle
		if (ApplyIntervalSeconds <= 0.0
			|| (NowSeconds - LastApplySeconds) >= ApplyIntervalSeconds)
		{
			LastApplySeconds = NowSeconds;
			return true;
		}

		// Idle flush (one-time)
		if (!bIdleFlushConsumed
			&& LastActivitySeconds > 0.0
			&& (NowSeconds - LastActivitySeconds) >= ApplyIntervalSeconds)
		{
			LastApplySeconds = NowSeconds;
			bIdleFlushConsumed = true;
			return true;
		}

		return false;
	}

	bool HasAppliedOnce() const
	{
		return bHasAppliedOnce;
	}

private:
	/**  */
	double ApplyIntervalSeconds = 0.0;

	/** How long since the user stopped changing the value long enough to trigger a one-time idle update */
	mutable double LastActivitySeconds = 0.0;

	/**  */
	mutable double LastApplySeconds = 0.0;

	/**  */
	mutable bool bHasAppliedOnce = false;

	/**  */
	mutable bool bIdleFlushConsumed = false;
};

} // namespace UE::Sequencer::SimpleView
