// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimpleView/SequencerSimpleViewExtender.h"

namespace UE::Sequencer::SimpleView
{

TArray<ToolableTimeline::FRegisteredChannelFilter>& FSequencerSimpleViewExtender::GetRegisteredChannelFilters()
{
	static TArray<ToolableTimeline::FRegisteredChannelFilter> RegisteredFilters;
	return RegisteredFilters;
}

FSimpleMulticastDelegate& FSequencerSimpleViewExtender::OnChannelFiltersChanged()
{
	static FSimpleMulticastDelegate ChannelFiltersChangedDelegate;
	return ChannelFiltersChangedDelegate;
}

FSimpleMulticastDelegate& FSequencerSimpleViewExtender::OnAdditionalSelectedModelsChanged()
{
	static FSimpleMulticastDelegate AdditionalSelectedModelsChangedDelegate;
	return AdditionalSelectedModelsChangedDelegate;
}

FDelegateHandle FSequencerSimpleViewExtender::RegisterChannelFilter(ToolableTimeline::FChannelFilterFunction&& InFilter)
{
	ToolableTimeline::FRegisteredChannelFilter& Entry = GetRegisteredChannelFilters().AddDefaulted_GetRef();
	Entry.Handle = FDelegateHandle(FDelegateHandle::GenerateNewHandle);
	Entry.FilterFunction = MoveTemp(InFilter);

	OnChannelFiltersChanged().Broadcast();

	return Entry.Handle;
}

void FSequencerSimpleViewExtender::UnregisterChannelFilter(const FDelegateHandle& InDelegateHandle)
{
	GetRegisteredChannelFilters().RemoveAll([&InDelegateHandle]
		(const ToolableTimeline::FRegisteredChannelFilter& InFilter)
		{
			return InFilter.Handle == InDelegateHandle;
		});

	OnChannelFiltersChanged().Broadcast();
}

FDelegateHandle FSequencerSimpleViewExtender::RegisterAdditionalSelectedModels(FOnEnumerateAdditionalSimpleViewSelectedModels::FDelegate&& InDelegate)
{
	const FDelegateHandle DelegateHandle(FDelegateHandle::GenerateNewHandle);
	GetRegisteredAdditionalSelectedModels().Add(DelegateHandle, MoveTemp(InDelegate));

	OnAdditionalSelectedModelsChanged().Broadcast();

	return DelegateHandle;
}

void FSequencerSimpleViewExtender::UnregisterAdditionalSelectedModels(const FDelegateHandle& InDelegateHandle)
{
	GetRegisteredAdditionalSelectedModels().Remove(InDelegateHandle);

	OnAdditionalSelectedModelsChanged().Broadcast();
}

TMap<FDelegateHandle, FOnEnumerateAdditionalSimpleViewSelectedModels::FDelegate>& FSequencerSimpleViewExtender::GetRegisteredAdditionalSelectedModels()
{
	static TMap<FDelegateHandle, FOnEnumerateAdditionalSimpleViewSelectedModels::FDelegate> RegisteredAdditionalSelectedModels;
	return RegisteredAdditionalSelectedModels;
}

} // UE::Sequencer::SimpleView
