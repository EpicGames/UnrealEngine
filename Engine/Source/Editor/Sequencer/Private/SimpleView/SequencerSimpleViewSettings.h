// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "SequencerSimpleViewSettings.generated.h"

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, DefaultConfig)
class USequencerSimpleViewSettings : public UDeveloperSettings
{
public:
	GENERATED_BODY()

	/** Show the toolbar above the timeline */
	UPROPERTY(Config, EditAnywhere, Category = "Visibility", meta = (DisplayName = "ToolBar"))
	bool bShowToolBar = true;

	/** Show the transport controls under the timeline */
	UPROPERTY(Config, EditAnywhere, Category = "Visibility", meta = (DisplayName = "Transport Controls"))
	bool bShowTransportControls = true;

	/** Show the range controls under the timeline */
	UPROPERTY(Config, EditAnywhere, Category = "Visibility", meta = (DisplayName = "Range Controls"))
	bool bShowRangeControls = true;
};
