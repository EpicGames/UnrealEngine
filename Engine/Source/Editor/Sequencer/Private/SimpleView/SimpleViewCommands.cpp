// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimpleViewCommands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "SimpleViewEditorCommands"

UE_DEFINE_TCOMMANDS(FSimpleViewCommands)

void FSimpleViewCommands::RegisterCommands()
{
	UI_COMMAND(ToggleToolbarVisible
		, "Toggle Toolbar"
		, "Toggle the visibility of the toolbar"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Shift, EKeys::BackSpace))

	UI_COMMAND(Tool_DeleteAnchor
		, "Remove Anchor"
		, "Remove the anchor for the currently active tool"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Delete))

	UI_COMMAND(TranslateKeyLeft
		, "Translate Key(s) Left"
		, "Translate selected keys to the left by the frame offset"
		, EUserInterfaceActionType::Button
		, FInputChord())

	UI_COMMAND(TranslateKeyRight
		, "Translate Key(s) Right"
		, "Translate selected keys to the right by the frame offset"
		, EUserInterfaceActionType::Button
		, FInputChord())

	UI_COMMAND(ScaleKeyDivide
		, "Transform Key(s) Divide"
		, "Scale selected keys by dividing their time offset from the current scrub time pivot point"
		, EUserInterfaceActionType::Button
		, FInputChord())

	UI_COMMAND(ScaleKeyMultiply
		, "Transform Key(s) Multiply"
		, "Scale selected keys by multiplying their time offset from the current scrub time pivot point"
		, EUserInterfaceActionType::Button
		, FInputChord())
}

#undef LOCTEXT_NAMESPACE
