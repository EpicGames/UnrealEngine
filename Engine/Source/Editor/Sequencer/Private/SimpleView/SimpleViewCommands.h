// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FUICommandInfo;

UE_DECLARE_TCOMMANDS(class FSimpleViewCommands, SEQUENCER_API)

class FSimpleViewCommands : public TCommands<FSimpleViewCommands>
{
public:
	FSimpleViewCommands()
		: TCommands<FSimpleViewCommands>(TEXT("SimpleViewEditor"),
			NSLOCTEXT("Contexts", "SimpleViewEditor", "Simple View Editor"),
			NAME_None,
			FAppStyle::GetAppStyleSetName())
	{}

	SEQUENCER_API virtual void RegisterCommands() override;

	/** Toggle the visibility of the toolbar */
	TSharedPtr<FUICommandInfo> ToggleToolbarVisible;

	/** Delete an anchor for a tool */
	TSharedPtr<FUICommandInfo> Tool_DeleteAnchor;

	/** Translate selected keys to the left by the frame offset */
	TSharedPtr<FUICommandInfo> TranslateKeyLeft;

	/** Translate selected keys to the right by the frame offset */
	TSharedPtr<FUICommandInfo> TranslateKeyRight;

	/** Scale selected keys by dividing their time offset from the current scrub time pivot point */
	TSharedPtr<FUICommandInfo> ScaleKeyDivide;

	/** Scale selected keys by multiplying their time offset from the current scrub time pivot point */
	TSharedPtr<FUICommandInfo> ScaleKeyMultiply;
};
