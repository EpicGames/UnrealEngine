// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimpleViewTabAutosizeHelper.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/SDockingTabStack.h"
#include "Interfaces/IMainFrameModule.h"
#include "ISequencer.h"
#include "Modules/ModuleManager.h"
#include "Sequencer.h"
#include "SimpleView/SimpleViewTimeline.h"
#include "Widgets/SWindow.h"

namespace UE::Sequencer::SimpleView
{

FSimpleViewTabAutosizeHelper::FSimpleViewTabAutosizeHelper(FSimpleViewTimeline& InTimeline)
	: Timeline(InTimeline)
{
}

FSimpleViewTabAutosizeHelper::~FSimpleViewTabAutosizeHelper()
{
}

TSharedPtr<FTabManager> FSimpleViewTabAutosizeHelper::GetSequencerTabManager(const TSharedRef<ISequencer>& InSequencer)
{
	if (const TSharedPtr<IToolkitHost> ToolkitHost = InSequencer->GetToolkitHost())
	{
		return ToolkitHost->GetTabManager();
	}
	return nullptr;
}

TSharedPtr<SDockTab> FSimpleViewTabAutosizeHelper::FindSequencerTab(const TSharedRef<ISequencer>& InSequencer)
{
	const TSharedPtr<FTabManager> TabManager = GetSequencerTabManager(InSequencer);
	return TabManager.IsValid() ? TabManager->FindExistingLiveTab(ToolableTimeline::FToolableTimeline::SequencerTabId) : nullptr;
}

bool FSimpleViewTabAutosizeHelper::IsTabFloating(const TSharedRef<SDockTab>& InDockTab)
{
	if (!FSlateApplication::IsInitialized())
	{
		return false;
	}

	static const FName MainFrameModuleName = TEXT("MainFrame");
	const IMainFrameModule* const MainFrame = FModuleManager::GetModulePtr<IMainFrameModule>(MainFrameModuleName);
	if (!MainFrame)
	{
		return false;
	}

	const TSharedPtr<SWindow> MainParentWindow = MainFrame->GetParentWindow();
	const TSharedPtr<SWindow> TabParentWindow = FSlateApplication::Get().FindWidgetWindow(InDockTab);

	const bool bFloating = MainParentWindow.IsValid() && TabParentWindow.IsValid() && (MainParentWindow != TabParentWindow);

	return bFloating;
}
	
void FSimpleViewTabAutosizeHelper::SetParentDockTabStackTabWellHiddenIfOnlyTab(const TSharedRef<SDockTab>& InDockTab, const bool bInHidden)
{
	const TSharedPtr<SDockingTabStack> ParentDockTabStack = InDockTab->GetParentDockTabStack();
	if (!ParentDockTabStack.IsValid())
	{
		return;
	}

	const TSlotlessChildren<SDockTab>& Tabs = ParentDockTabStack->GetTabs();
	if (Tabs.Num() == 1 && Tabs[0] == InDockTab)
	{
		InDockTab->SetParentDockTabStackTabWellHidden(bInHidden);
	}
}

void FSimpleViewTabAutosizeHelper::RegisterAutoSizeTimer(const TSharedRef<SDockTab>& InDockTab)
{
	if (SetupWindowTimerHandle.IsValid())
	{
		InDockTab->UnRegisterActiveTimer(SetupWindowTimerHandle.ToSharedRef());
		SetupWindowTimerHandle.Reset();
	}

	const TWeakPtr<SDockTab> WeakTab = InDockTab;

	SetupWindowTimerHandle = InDockTab->RegisterActiveTimer(0.0f
		, FWidgetActiveTimerDelegate::CreateRaw(this, &FSimpleViewTabAutosizeHelper::HandleAutosizeTimer, WeakTab));
}

void FSimpleViewTabAutosizeHelper::UnregisterAutoSizeTimer(const TSharedRef<SDockTab>& InDockTab)
{
	if (!SetupWindowTimerHandle.IsValid())
	{
		return;
	}

	InDockTab->UnRegisterActiveTimer(SetupWindowTimerHandle.ToSharedRef());

	SetupWindowTimerHandle.Reset();
}

EActiveTimerReturnType FSimpleViewTabAutosizeHelper::HandleAutosizeTimer(const double InCurrentTime
	, const float InDeltaTime, const TWeakPtr<SDockTab> InWeakTab)
{
	const TSharedPtr<SDockTab> ExistingTab = InWeakTab.Pin();
	if (!ExistingTab.IsValid())
	{
		SetupWindowTimerHandle.Reset();
		return EActiveTimerReturnType::Stop;
	}

	// The tab will only have a valid parent window when it's floating
	const bool bTabFloating = IsTabFloating(ExistingTab.ToSharedRef());
	if (bTabFloating)
	{
		// Wait until the window is actually constructed/visible, otherwise sizing/locking
		// can be ignored or overwritten by later docking/window setup.
		const TSharedPtr<SWindow> Window = ExistingTab->GetParentWindow();
		if (!Window.IsValid() || !Window->GetNativeWindow().IsValid())
		{
			// Parent window not assigned yet (common on first show). Keep waiting.
			return EActiveTimerReturnType::Continue;
		}

		const bool bIsVisible = Window->IsVisible();
		const FVector2D SizeInScreen = Window->GetSizeInScreen();
		const bool bHasNonZeroSize = SizeInScreen.X > 1.f && SizeInScreen.Y > 1.f;
		if (!bIsVisible || !bHasNonZeroSize)
		{
			return EActiveTimerReturnType::Continue;
		}

		// An auto-sized tab only sets the size of the tab to the contents when the tab is first displayed.
		// Switch to a fixed layout after we use the auto-sized content to get the desired size.
		// Temporarily enable auto sizing and force a layout pass to measure the widget tree's final
		// desired size, then use this to set the size of the window based on the content
		Window->SetSizingRule(ESizingRule::Autosized);
		Window->SlatePrepass(FSlateApplicationBase::Get().GetApplicationScale());
		Window->Resize(Window->GetDesiredSize());
		Window->SetSizingRule(ESizingRule::UserSized);
	}
	else
	{
		ExistingTab->SetAutosizePolicy(EDockTabAutosizePolicy::VerticalOnly);

		ExistingTab->Invalidate(EInvalidateWidgetReason::Layout);
		ExistingTab->SlatePrepass(FSlateApplicationBase::Get().GetApplicationScale());
	}

	RegisterResetAutosizeTimer(ExistingTab.ToSharedRef(), !bTabFloating);

	SetupWindowTimerHandle.Reset();

	return EActiveTimerReturnType::Stop;
}

void FSimpleViewTabAutosizeHelper::HandleTabRelocated()
{
	const TSharedPtr<ISequencer> Sequencer = Timeline.GetSequencer();
	if (!Sequencer.IsValid() || !Sequencer->IsSimpleView())
	{
		return;
	}

	const TSharedPtr<SDockTab> ExistingTab = FindSequencerTab(Sequencer.ToSharedRef());
	if (!ExistingTab.IsValid())
	{
		return;
	}

	RegisterAutoSizeTimer(ExistingTab.ToSharedRef());
	//AutoSize(ExistingTab.ToSharedRef());
}

void FSimpleViewTabAutosizeHelper::HandleTabActivated(const TSharedRef<SDockTab> InDockTab, const ETabActivationCause InActivationCause)
{
	const TSharedPtr<ISequencer> Sequencer = Timeline.GetSequencer();
	if (!Sequencer.IsValid() || !Sequencer->IsSimpleView())
	{
		return;
	}

	//if (IsTabFloating(InDockTab))
	{
		RegisterAutoSizeTimer(InDockTab);
	}
}

void FSimpleViewTabAutosizeHelper::AutoSize(const TSharedRef<SDockTab>& InDockTab)
{
	if (IsTabFloating(InDockTab))
	{
		RegisterAutoSizeTimer(InDockTab);
	}
	else
	{
		UnregisterAutoSizeTimer(InDockTab);
		UnregisterResetAutosizeTimer(InDockTab);

		InDockTab->SetAutosizePolicy(EDockTabAutosizePolicy::None);
		InDockTab->Invalidate(EInvalidateWidgetReason::Layout);
	}
}

void FSimpleViewTabAutosizeHelper::RegisterResetAutosizeTimer(const TSharedRef<SDockTab>& InDockTab, const bool bInAutosize)
{
	UnregisterResetAutosizeTimer(InDockTab);

	const TWeakPtr<SDockTab> WeakTab = InDockTab;
	ResetAutosizeTimerHandle = InDockTab->RegisterActiveTimer(0.0f,
		FWidgetActiveTimerDelegate::CreateLambda([this, WeakTab, bInAutosize](double, float)
			{
				if (const TSharedPtr<SDockTab> Tab = WeakTab.Pin())
				{
					const EDockTabAutosizePolicy AutosizePolicy = bInAutosize
						? EDockTabAutosizePolicy::VerticalOnly : EDockTabAutosizePolicy::None;
					Tab->SetAutosizePolicy(AutosizePolicy);
					Tab->Invalidate(EInvalidateWidgetReason::Layout);

					UnregisterResetAutosizeTimer(Tab.ToSharedRef());
				}

				return EActiveTimerReturnType::Stop;
			}));
}

void FSimpleViewTabAutosizeHelper::UnregisterResetAutosizeTimer(const TSharedRef<SDockTab>& InDockTab)
{
	if (!ResetAutosizeTimerHandle.IsValid())
	{
		return;
	}

	InDockTab->UnRegisterActiveTimer(ResetAutosizeTimerHandle.ToSharedRef());

	ResetAutosizeTimerHandle.Reset();
}

} // namespace UE::Sequencer::SimpleView
