// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Docking/TabManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SWidget.h"

class ISequencer;
class SDockTab;

namespace UE::Sequencer::SimpleView
{

class FSimpleViewTimeline;

/**
 * Manages Sequencer tab autosize behavior for Sequencer Simple View.
 *
 * Goals:
 *  - Floating: resize the SWindow once to match content, then leave user-resizable
 *  - Docked: wait for stable DesiredSize, then pulse SDockTab autosize for 1 tick, then turn it off
 *  - Never leave SDockTab::SetShouldAutosize(true) persistently (prevents splitter crush)
 */
class FSimpleViewTabAutosizeHelper
{
public:
	static TSharedPtr<FTabManager> GetSequencerTabManager(const TSharedRef<ISequencer>& InSequencer);
	static TSharedPtr<SDockTab> FindSequencerTab(const TSharedRef<ISequencer>& InSequencer);

	static bool IsTabFloating(const TSharedRef<SDockTab>& InDockTab);

	static void SetParentDockTabStackTabWellHiddenIfOnlyTab(const TSharedRef<SDockTab>& InDockTab, const bool bInHidden);

	explicit FSimpleViewTabAutosizeHelper(FSimpleViewTimeline& InTimeline);
	~FSimpleViewTabAutosizeHelper();

	void RegisterAutoSizeTimer(const TSharedRef<SDockTab>& InDockTab);
	void UnregisterAutoSizeTimer(const TSharedRef<SDockTab>& InDockTab);

	void RegisterResetAutosizeTimer(const TSharedRef<SDockTab>& InDockTab, const bool bInAutosize);
	void UnregisterResetAutosizeTimer(const TSharedRef<SDockTab>& InDockTab);

private:
	EActiveTimerReturnType HandleAutosizeTimer(const double InCurrentTime
		, const float InDeltaTime, const TWeakPtr<SDockTab> InWeakTab);

	void HandleTabRelocated();
	void HandleTabActivated(const TSharedRef<SDockTab> InDockTab, const ETabActivationCause InCause);

	void AutoSize(const TSharedRef<SDockTab>& InDockTab);

	FSimpleViewTimeline& Timeline;

	TSharedPtr<FActiveTimerHandle> SetupWindowTimerHandle;
	TSharedPtr<FActiveTimerHandle> ResetAutosizeTimerHandle;

	TSharedPtr<FActiveTimerHandle> LayoutWatchTimerHandle;
	TWeakPtr<SDockTab> LayoutWatchOwnerTab;
	FVector2D LastObservedTabSize = FVector2D::ZeroVector;
	int32 StableLayoutTicks = 0;
};

} // namespace UE::Sequencer::SimpleView
