// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimpleViewTimeline.h"
#include "Framework/Docking/SDockingTabStack.h"
#include "ISequencer.h"
#include "Sequencer.h"
#include "SequencerSettings.h"
#include "SequencerSimpleViewSettings.h"
#include "Misc/KeyHelperUtils.h"
#include "MVVM/Selection/Selection.h"
#include "SimpleView/Menus/ToolableTimelineMenuExtension.h"
#include "SimpleView/SimpleViewCommands.h"
#include "ToolableTimeline/ToolableTimeSliderController.h"
#include "ToolableTimeline/Widgets/SToolableTimeline.h"
#include "Tools/Factories/SimpleViewMoveFrameKeysToolFactory.h"
#include "Tools/Factories/SimpleViewRetimeToolFactory.h"
#include "Widgets/SSequencerSimpleView.h"

namespace UE::Sequencer::SimpleView
{

FSimpleViewTimeline::FSimpleViewTimeline(const FPrivateToken& InToken)
	: FToolableTimeline(InToken)
	, TabAutosizer(*this)
{
	FSimpleViewCommands::Register();

	AddToolFactory<FSimpleViewMoveFrameKeysToolFactory>();
	AddToolFactory<FSimpleViewRetimeToolFactory>();

	ChannelFilters = FSequencerSimpleViewExtender::GetRegisteredChannelFilters();
}

void FSimpleViewTimeline::Initialize()
{
	FToolableTimeline::Initialize();

	FToolableTimelineMenuExtension::AddExtension(SharedThis(this));

	if (USequencerSimpleViewSettings* const SimpleViewSettings = GetMutableDefault<USequencerSimpleViewSettings>())
	{
		SimpleViewSettings->OnSettingChanged().AddSP(this, &FSimpleViewTimeline::HandleSimpleViewSettingsChanged);
	}

	if (const TSharedPtr<ISequencer> Sequencer = GetSequencer())
	{
		if (const TSharedPtr<FTabManager> TabManager = FSimpleViewTabAutosizeHelper::GetSequencerTabManager(Sequencer.ToSharedRef()))
		{
			TabsTemporarilySidebaredDelegate = TabManager->GetOnToggleTemporarilySidebaredTabsDelegate()
				.AddSP(this, &FSimpleViewTimeline::HandleToggleTemporarilySidebaredTabs);
		}
	}

	FSequencerSimpleViewExtender::OnChannelFiltersChanged().AddSP(this, &FSimpleViewTimeline::HandleChannelFiltersChanged);
	FSequencerSimpleViewExtender::OnAdditionalSelectedModelsChanged().AddSP(this, &FSimpleViewTimeline::HandleAdditionalSelectedModelsChanged);

	// Default to 1 frame
	const FFrameNumber DefaultDeltaTime = FFrameRate::TransformTime(FFrameTime(1)
		, TimeSliderController->GetDisplayRate()
		, TimeSliderController->GetTickResolution()).FrameNumber;
	KeyTranslateDelta = DefaultDeltaTime.Value;
	KeyScaleFactor = 1.0;
}

void FSimpleViewTimeline::Shutdown()
{
	FSequencerSimpleViewExtender::OnChannelFiltersChanged().RemoveAll(this);
	FSequencerSimpleViewExtender::OnAdditionalSelectedModelsChanged().RemoveAll(this);

	if (TabsTemporarilySidebaredDelegate.IsValid())
	{
		if (const TSharedPtr<FSequencer> Sequencer = GetSequencer())
		{
			if (const TSharedPtr<FTabManager> TabManager = FSimpleViewTabAutosizeHelper::GetSequencerTabManager(Sequencer.ToSharedRef()))
			{
				TabManager->GetOnToggleTemporarilySidebaredTabsDelegate().Remove(TabsTemporarilySidebaredDelegate);
			}
		}
	}

	if (USequencerSimpleViewSettings* const SimpleViewSettings = GetMutableDefault<USequencerSimpleViewSettings>())
	{
		SimpleViewSettings->OnSettingChanged().RemoveAll(this);
	}

	FToolableTimelineMenuExtension::RemoveExtension();

	FToolableTimeline::Shutdown();
}

void FSimpleViewTimeline::BindCommands()
{
	FToolableTimeline::BindCommands();

	if (IsInSimpleView())
	{
		BindSimpleViewCommands();
	}
}

void FSimpleViewTimeline::UnbindCommands()
{
	FToolableTimeline::UnbindCommands();

	if (IsInSimpleView())
	{
		UnbindSimpleViewCommands();
	}
}

void FSimpleViewTimeline::BindSimpleViewCommands()
{
	const FSimpleViewCommands& SimpleViewEditorCommands = FSimpleViewCommands::Get();

	CommandList->MapAction(SimpleViewEditorCommands.ToggleToolbarVisible
		, FExecuteAction::CreateSP(this, &FSimpleViewTimeline::ToggleToolbarVisible));

	CommandList->MapAction(SimpleViewEditorCommands.TranslateKeyLeft
		, FExecuteAction::CreateSP(this, &FSimpleViewTimeline::TranslateKeyLeft));
	CommandList->MapAction(SimpleViewEditorCommands.TranslateKeyRight
		, FExecuteAction::CreateSP(this, &FSimpleViewTimeline::TranslateKeyRight));

	CommandList->MapAction(SimpleViewEditorCommands.ScaleKeyMultiply
		, FExecuteAction::CreateSP(this, &FSimpleViewTimeline::ScaleKeyMultiply)
		, FCanExecuteAction::CreateSP(this, &FSimpleViewTimeline::HasKeySelection));
	CommandList->MapAction(SimpleViewEditorCommands.ScaleKeyDivide
		, FExecuteAction::CreateSP(this, &FSimpleViewTimeline::ScaleKeyDivide)
		, FCanExecuteAction::CreateSP(this, &FSimpleViewTimeline::HasKeySelection));
}

void FSimpleViewTimeline::UnbindSimpleViewCommands()
{
	const FSimpleViewCommands& SimpleViewEditorCommands = FSimpleViewCommands::Get();

	CommandList->UnmapAction(SimpleViewEditorCommands.ToggleToolbarVisible);

	CommandList->UnmapAction(SimpleViewEditorCommands.TranslateKeyLeft);
	CommandList->UnmapAction(SimpleViewEditorCommands.TranslateKeyRight);

	CommandList->UnmapAction(SimpleViewEditorCommands.ScaleKeyMultiply);
	CommandList->UnmapAction(SimpleViewEditorCommands.ScaleKeyDivide);
}

TSharedRef<SWidget> FSimpleViewTimeline::GenerateWidget()
{
	if (!SimpleViewWidget.IsValid())
	{
		SimpleViewWidget = SNew(SSequencerSimpleView, SharedThis(this));
	}
	return SimpleViewWidget.ToSharedRef();
}

void FSimpleViewTimeline::NotifyToolActivated()
{
}

void FSimpleViewTimeline::NotifyToolDeactivated()
{
}

TArray<ToolableTimeline::FRegisteredChannelFilter>& FSimpleViewTimeline::GetChannelFilters()
{
	return FSequencerSimpleViewExtender::GetRegisteredChannelFilters();
}

bool FSimpleViewTimeline::IsInSimpleView() const
{
	const USequencerSettings* const SequencerSettings = GetSequencerSettings();
	return SequencerSettings ? SequencerSettings->IsSimpleView() : false;
}

void FSimpleViewTimeline::EnableSimpleView(const bool bInEnable)
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!SequencerSettings)
	{
		return;
	}

	if (const TSharedPtr<SDockTab> ExistingTab = FSimpleViewTabAutosizeHelper::FindSequencerTab(Sequencer.ToSharedRef()))
	{
		const TSharedRef<SDockTab> ExistingTabRef = ExistingTab.ToSharedRef();

		if (bInEnable) // Setup simple Sequencer timeline
		{
			BindSimpleViewCommands();

			RequestRecacheChannels();

			FSimpleViewTabAutosizeHelper::SetParentDockTabStackTabWellHiddenIfOnlyTab(ExistingTabRef, true);

			ExistingTabRef->SetAutosizePolicy(EDockTabAutosizePolicy::VerticalOnly);
		}
		else // Setup full Sequencer
		{
			UnbindSimpleViewCommands();

			TabAutosizer.UnregisterAutoSizeTimer(ExistingTabRef);
			TabAutosizer.UnregisterResetAutosizeTimer(ExistingTabRef);

			FSimpleViewTabAutosizeHelper::SetParentDockTabStackTabWellHiddenIfOnlyTab(ExistingTab.ToSharedRef(), false);

			ExistingTabRef->SetAutosizePolicy(EDockTabAutosizePolicy::None);
		}
	}

	SequencerSettings->SetIsSimpleView(bInEnable);
}

void FSimpleViewTimeline::ToggleSimpleView()
{
	EnableSimpleView(!IsInSimpleView());
}

void FSimpleViewTimeline::HandleSimpleViewSettingsChanged(UObject* const InObject, FPropertyChangedEvent& InEvent)
{
	if (SimpleViewWidget.IsValid())
	{
		const FName PropertyName = InEvent.Property ? InEvent.Property->GetFName() : NAME_None;

		if (PropertyName == GET_MEMBER_NAME_CHECKED(USequencerSimpleViewSettings, bShowToolBar)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(USequencerSimpleViewSettings, bShowRangeControls))
		{
			SimpleViewWidget->Reconstruct();
		}
	}

	if (TimelineSettings.IsValid())
	{
		TimelineSettings->SaveConfig();
	}
}

void FSimpleViewTimeline::HandleToggleTemporarilySidebaredTabs(const bool bInSidebared, const TSet<FTabId>& InExceptions)
{
	if (!InExceptions.Contains(SequencerTabId))
	{
		return;
	}

	static bool bWasLastInSimpleView = false;

	if (bInSidebared)
	{
		bWasLastInSimpleView = IsInSimpleView();
		EnableSimpleView(true);
	}
	else if (!bWasLastInSimpleView)
	{
		EnableSimpleView(false);
	}
}

void FSimpleViewTimeline::HandleChannelFiltersChanged()
{
	ChannelFilters = FSequencerSimpleViewExtender::GetRegisteredChannelFilters();

	RequestRecacheChannels();
}

void FSimpleViewTimeline::HandleAdditionalSelectedModelsChanged()
{
	RequestRecacheChannels();
}

TSharedRef<SWidget> FSimpleViewTimeline::GenerateTimelineWidget()
{
	return FToolableTimeline::GenerateWidget();
}

void FSimpleViewTimeline::ToggleToolbarVisible()
{
	if (USequencerSimpleViewSettings* const SimpleViewSettings = GetMutableDefault<USequencerSimpleViewSettings>())
	{
		SimpleViewSettings->bShowToolBar = !SimpleViewSettings->bShowToolBar;
		SimpleViewSettings->SaveConfig();
	}
}

bool FSimpleViewTimeline::HasKeySelection() const
{
	return GetKeySelection().HasAnySelectedKeys();
}

void FSimpleViewTimeline::TranslateKeyLeft()
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid() || Sequencer->IsReadOnly())
	{
		return;
	}

	const FFrameNumber DeltaTime = FFrameTime::FromDecimal(KeyTranslateDelta).GetFrame();
	if (DeltaTime == 0)
	{
		return;
	}

	KeyHelperUtils::TransformSelectedOrRelativeKeys(*Sequencer
		, GetChannelModels(), KeySelection.GetSelectedKeys(), -DeltaTime, 1.f);
}

void FSimpleViewTimeline::TranslateKeyRight()
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid() || Sequencer->IsReadOnly())
	{
		return;
	}

	const FFrameNumber DeltaTime = FFrameTime::FromDecimal(KeyTranslateDelta).GetFrame();
	if (DeltaTime == 0)
	{
		return;
	}

	KeyHelperUtils::TransformSelectedOrRelativeKeys(*Sequencer
		, GetChannelModels(), KeySelection.GetSelectedKeys(), DeltaTime, 1.f);
}

void FSimpleViewTimeline::ScaleKeyDivide()
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid() || Sequencer->IsReadOnly())
	{
		return;
	}

	if (KeyScaleFactor != 0.f && KeyScaleFactor != 1.f)
	{
		KeyHelperUtils::TransformSelectedOrRelativeKeys(*Sequencer
			, GetChannelModels(), KeySelection.GetSelectedKeys(), 0, 1.f / KeyScaleFactor);
	}
}

void FSimpleViewTimeline::ScaleKeyMultiply()
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid() || Sequencer->IsReadOnly())
	{
		return;
	}

	if (KeyScaleFactor != 0.f && KeyScaleFactor != 1.f)
	{
		KeyHelperUtils::TransformSelectedOrRelativeKeys(*Sequencer
			, GetChannelModels(), KeySelection.GetSelectedKeys(), 0, KeyScaleFactor);
	}
}

} // namespace UE::Sequencer::SimpleView
