// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolableTimeline/ToolableTimeline.h"

namespace UE::Sequencer::SimpleView
{

class SSequencerSimpleView;

class FSimpleViewTimeline : public ToolableTimeline::FToolableTimeline
{
public:
	FSimpleViewTimeline(const FPrivateToken& InToken);
	virtual ~FSimpleViewTimeline() override {}

	bool IsInSimpleView() const;
	void EnableSimpleView(const bool bInEnable);
	void ToggleSimpleView();

	//~ Begin FToolableTimeline

	virtual void Initialize() override;
	virtual void Shutdown() override;

	virtual void BindCommands() override;
	virtual void UnbindCommands() override;

	virtual TSharedRef<SWidget> GenerateWidget() override;

	virtual void NotifyToolActivated() override;
	virtual void NotifyToolDeactivated() override;

	virtual TArray<ToolableTimeline::FRegisteredChannelFilter>& GetChannelFilters() override;

	//~ End FToolableTimeline

	TSharedRef<SWidget> GenerateTimelineWidget();

	double GetKeyTranslateDelta() const { return KeyTranslateDelta; }
	void SetKeyTranslateDelta(const double InDelta) { KeyTranslateDelta = InDelta; }

	double GetKeyScaleFactor() const { return KeyScaleFactor; }
	void SetKeyScaleFactor(const double InFactor) { KeyScaleFactor = InFactor; }

private:
	void BindSimpleViewCommands();
	void UnbindSimpleViewCommands();

	void HandleSimpleViewSettingsChanged(UObject* const InObject, FPropertyChangedEvent& InEvent);

	void HandleToggleTemporarilySidebaredTabs(const bool bInSidebared, const TSet<FTabId>& InExceptions);

	void HandleChannelFiltersChanged();
	void HandleAdditionalSelectedModelsChanged();

	void ToggleToolbarVisible();

	bool HasKeySelection() const;

	void TranslateKeyLeft();
	void TranslateKeyRight();

	void ScaleKeyDivide();
	void ScaleKeyMultiply();

	TSharedPtr<SSequencerSimpleView> SimpleViewWidget;

	/** Helper for managing the tab and auto-sizing */
	FSimpleViewTabAutosizeHelper TabAutosizer;

	/** Delegate called when the F10 key is pressed from the viewport */
	FDelegateHandle TabsTemporarilySidebaredDelegate;

	double KeyTranslateDelta = 0.0;
	double KeyScaleFactor = 1.0;
};

} // namespace UE::Sequencer::SimpleView
