// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimpleViewUtils.h"
#include "ISequencer.h"
#include "Sequencer.h"
#include "SequencerCommands.h"
#include "MVVM/Selection/Selection.h"
#include "SequencerSimpleViewSettings.h"
#include "SimpleView/SimpleViewTimeline.h"
#include "ToolableTimeline/ToolableTimeSliderController.h"

#define LOCTEXT_NAMESPACE "SimpleViewUtils"

namespace UE::Sequencer::SimpleView::Utils
{

TSharedRef<SWidget> CreateToggleButton(const TWeakPtr<FSimpleViewTimeline>& InWeakTimeline)
{
	return SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), TEXT("HoverHintOnly"))
		.IsFocusable(false)
		.VAlign(VAlign_Center)
		.ContentPadding(FMargin(6.f, 4.f))
		.ToolTipText_Lambda([InWeakTimeline]()
			{
				const FSequencerCommands& SequencerCommands = FSequencerCommands::Get();

				FText TooltipText = SequencerCommands.ToggleSimpleView->GetDescription();

				if (const TSharedPtr<FSimpleViewTimeline> Timeline = InWeakTimeline.Pin())
				{
					TooltipText = Timeline->IsInSimpleView()
						? LOCTEXT("SimpleViewToggleTooltip", "Switch to Full View")
						: LOCTEXT("FullViewToggleTooltip", "Switch to Simple View");
				}

				const FText InputText = SequencerCommands.ToggleSimpleView->GetInputText();
				if (!InputText.IsEmpty())
				{
					TooltipText = FText::Format(LOCTEXT("ToggleTooltip", "{0}\n\n{1}")
						, TooltipText, InputText);
				}

				return TooltipText;
			})
		.OnClicked_Lambda([InWeakTimeline]()
			{
				if (const TSharedPtr<FSimpleViewTimeline> Timeline = InWeakTimeline.Pin())
				{
					Timeline->ToggleSimpleView();
				}
				return FReply::Handled();
			})
		[
			SNew(SImage)
			.DesiredSizeOverride(FVector2D(20.f, 20.f))
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image_Lambda([InWeakTimeline]()
				{
					const TSharedPtr<FSimpleViewTimeline> Timeline = InWeakTimeline.Pin();
					const FName BrushName = (Timeline.IsValid() && Timeline->IsInSimpleView())
						? TEXT("Sequencer.SimpleView.ToggleToFullView")
						: TEXT("Sequencer.SimpleView.ToggleToSimpleView");
					return FAppStyle::GetBrush(BrushName);
				})
		];
}

FLinearColor BlendColors(const FLinearColor& InColorA, const FLinearColor& InColorB, const float InPercent)
{
	return FMath::Lerp(InColorA, InColorB, InPercent);
}

FLinearColor WhitenColor(const FLinearColor& InColor, const float InPercent)
{
	return BlendColors(InColor, FLinearColor::White, InPercent);
}

UToolMenus* GetToolMenusSafe()
{
	if (!FModuleManager::Get().IsModuleLoaded(TEXT("ToolMenus")))
	{
		return nullptr;
	}
	return UToolMenus::Get();
}

bool IsInSimpleView(const TWeakPtr<FSimpleViewTimeline> InWeakTimeline)
{
	if (const TSharedPtr<FSimpleViewTimeline> Timeline = InWeakTimeline.Pin())
	{
		return Timeline->IsInSimpleView();
	}

	return false;
}

EVisibility GetSimpleViewVisibility(const TWeakPtr<FSimpleViewTimeline> InWeakTimeline)
{
	return IsInSimpleView(InWeakTimeline) ? EVisibility::Visible : EVisibility::Collapsed;
}

bool IsSimpleViewToolMenuVisible(const TWeakPtr<FSimpleViewTimeline> InWeakTimeline)
{
	const TSharedPtr<FSimpleViewTimeline> Timeline = InWeakTimeline.Pin();
	if (!Timeline.IsValid() || !Timeline->IsInSimpleView())
	{
		return false;
	}

	return Timeline->GetKeySelection().HasAnySelectedKeys();
}

EVisibility GetSimpleViewToolMenuVisibility(const TWeakPtr<FSimpleViewTimeline> InWeakTimeline)
{
	return IsSimpleViewToolMenuVisible(InWeakTimeline) ? EVisibility::Visible : EVisibility::Collapsed;
}

} // namespace UE::Sequencer::SimpleView::Utils

#undef LOCTEXT_NAMESPACE
