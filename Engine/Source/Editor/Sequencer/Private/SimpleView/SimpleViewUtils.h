// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"

class SWidget;
class UToolMenus;

namespace UE::Sequencer::SimpleView
{
	class FSimpleViewTimeline;
}

namespace UE::Sequencer::SimpleView::Utils
{

/**
 * Creates a toggle button widget for switching between Simple View and Full View modes in the sequencer.
 * @param InWeakTimeline A weak pointer to an FSimpleViewTimeline instance that manages the state of the view.
 *        The toggle button interacts with this timeline to determine the current view mode and to toggle between modes.
 * @return A shared reference to the created toggle button widget.
 */
TSharedRef<SWidget> CreateToggleButton(const TWeakPtr<FSimpleViewTimeline>& InWeakTimeline);

/**
 * Blends two colors together based on a specified percentage.
 * @param InColorA The first input color to be blended.
 * @param InColorB The second input color to be blended.
 * @param InPercent A percentage value (0.0 to 1.0) that determines the influence of InColorB over InColorA.
 *        0.0 will return InColorA, while 1.0 will return InColorB.
 * @return The resulting color after blending InColorA and InColorB using the specified percentage.
 */
FLinearColor BlendColors(const FLinearColor& InColorA, const FLinearColor& InColorB, const float InPercent = .5f);

/**
 * Adjusts the given color by blending it with white, based on the specified percentage.
 * @param InColor The input color to be adjusted.
 * @param InPercent A percentage value (0.0 to 1.0) that determines the influence of white on the input color.
 *        0.0 will return the original color, while 1.0 will return pure white.
 * @return The resulting color after blending the input color with white by the given percentage.
 */
FLinearColor WhitenColor(const FLinearColor& InColor, const float InPercent = .5f);

UToolMenus* GetToolMenusSafe();

bool IsInSimpleView(const TWeakPtr<FSimpleViewTimeline> InWeakTimeline);
EVisibility GetSimpleViewVisibility(const TWeakPtr<FSimpleViewTimeline> InWeakTimeline);

bool IsSimpleViewToolMenuVisible(const TWeakPtr<FSimpleViewTimeline> InWeakTimeline);
EVisibility GetSimpleViewToolMenuVisibility(const TWeakPtr<FSimpleViewTimeline> InWeakTimeline);

} // namespace UE::Sequencer::SimpleView::Utils
