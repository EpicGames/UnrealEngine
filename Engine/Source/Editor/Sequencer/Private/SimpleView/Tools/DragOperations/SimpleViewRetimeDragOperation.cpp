// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimpleViewRetimeDragOperation.h"
#include "Algo/IsSorted.h"
#include "Async/ParallelFor.h"
#include "SimpleView/SimpleViewTimeline.h"
#include "ToolableTimeline/MouseInputData.h"

namespace UE::Sequencer::SimpleView
{

using namespace UE::Sequencer::ToolableTimeline;

FSimpleViewRetimeDragOperation::FSimpleViewRetimeDragOperation(const FMouseInputData& InMouseInput
	, const TRange<FFrameNumber>& InFrameRange
	, const TRange<double>& InViewRange
	, const EDragMode InMode
	, const TArray<FFrameTime>& InInitialAnchorTimes)
	: FKeyDragOperation(InMouseInput, InFrameRange, InViewRange)
	, CurrentDragMode(InMode)
	, AnchorInitialTimes(InInitialAnchorTimes)
{
	if (!Algo::IsSorted(InInitialAnchorTimes))
	{
		ensureMsgf(false, TEXT("Anchor times must be sorted at drag start."));
		return;
	}

	const int32 AnchorCount = AnchorInitialTimes.Num();
	if (!ensure(AnchorCount >= 2))
	{
		return;
	}

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InMouseInput.Timeline->GetTimeSliderController();

	ChannelCache.ForEachChannelKey([this, AnchorCount]
		(FChannelKeyRangeCache<FSimpleViewRetimeToolChannelKeyCache>& InChannelCache
			, const TViewModelPtr<FChannelModel>& InChannelModel
			, const int32 InKeyIndex
			, FSimpleViewRetimeToolChannelKeyCache& InKeyCache)
		{
			const FFrameTime OriginalKeyTime = InKeyCache.InitialFrameTime;
			const int32 UpperIndex = Algo::UpperBound(AnchorInitialTimes, OriginalKeyTime);
			const int32 LeftAnchorIndex = FMath::Clamp(UpperIndex - 1, 0, AnchorCount - 1);
			const int32 RightAnchorIndex = FMath::Clamp(UpperIndex, 0, AnchorCount - 1);

			const FFrameTime LeftAnchorTime = AnchorInitialTimes[LeftAnchorIndex];
			const FFrameTime RightAnchorTime = AnchorInitialTimes[RightAnchorIndex];
			const FFrameTime Range = RightAnchorTime - LeftAnchorTime;

			double NewFraction = 0.0;

			if (Range != FFrameTime(0))
			{
				const FFrameTime Offset = OriginalKeyTime - LeftAnchorTime;
				NewFraction = Offset.AsDecimal() / Range.AsDecimal();
				NewFraction = FMath::Clamp(NewFraction, 0.0, 1.0);
			}

			InKeyCache.LeftAnchorIndex = LeftAnchorIndex;
			InKeyCache.FractionBetweenAnchors = NewFraction;

			return true;
		});
}

FSimpleViewRetimeDragOperation::EDragMode FSimpleViewRetimeDragOperation::GetCurrentDragMode() const
{
	return CurrentDragMode;
}

bool FSimpleViewRetimeDragOperation::IsCurrentDragMode(const EDragMode InDragMode) const
{
	return CurrentDragMode == InDragMode;
}

const TArray<FFrameTime>& FSimpleViewRetimeDragOperation::GetAnchorStartTimes() const
{
	return AnchorInitialTimes;
}

void FSimpleViewRetimeDragOperation::RecomputeForMove(const TSharedRef<FToolableTimeline>& InTimeline
	, const TArray<FFrameTime>& InInitialAnchorTimes
	, const TArray<double, TInlineAllocator<16>>& InAnchorInfluences
	, const FFrameTime& InDeltaTickTime
	, const bool bInIgnoreAnchorInfluences
	, const bool bInSnapEnabled)
{
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InTimeline->GetTimeSliderController();
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	const FFrameRate DisplayRate = TimeSliderController->GetDisplayRate();

	const int32 ChannelCount = ChannelCache.Num();

	// Compute the new key times
	// ParallelFor has overhead; don't pay it for tiny workloads
	static constexpr int32 ParallelThreshold = 8;
	if (ChannelCount >= ParallelThreshold)
	{
		FMultiChannelKeyCache<FSimpleViewRetimeToolChannelKeyCache>& ChannelCacheRef = ChannelCache;

		ParallelFor(ChannelCount, [&InDeltaTickTime, &InInitialAnchorTimes, &ChannelCacheRef, &TickResolution, &DisplayRate, &InAnchorInfluences, bInIgnoreAnchorInfluences, bInSnapEnabled]
			(const int32 InChannelIndex)
			{
				RecomputeLastDraggedFrameTimesFromDrag(ChannelCacheRef
					, InChannelIndex, TickResolution, DisplayRate, InInitialAnchorTimes, InAnchorInfluences, InDeltaTickTime, bInIgnoreAnchorInfluences, bInSnapEnabled);
			});
	}
	else
	{
		for (int32 ChannelIndex = 0; ChannelIndex < ChannelCount; ++ChannelIndex)
		{
			RecomputeLastDraggedFrameTimesFromDrag(ChannelCache
				, ChannelIndex, TickResolution, DisplayRate, InInitialAnchorTimes, InAnchorInfluences, InDeltaTickTime, bInIgnoreAnchorInfluences, bInSnapEnabled);
		}
	}
}

void FSimpleViewRetimeDragOperation::RecomputeFromAnchorTimes(const TSharedRef<FToolableTimeline>& InTimeline
	, const TArray<FFrameTime>& InNewAnchorTimes
	, const bool bInSnapEnabled)
{
	if (!Algo::IsSorted(InNewAnchorTimes))
	{
		ensureMsgf(false, TEXT("Anchor times must be sorted before recompute."));
		return;
	}

	const int32 AnchorCount = InNewAnchorTimes.Num();
	if (AnchorCount < 2)
	{
		return;
	}

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InTimeline->GetTimeSliderController();
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	const FFrameRate DisplayRate = TimeSliderController->GetDisplayRate();

	const int32 ChannelCount = ChannelCache.Num();
	for (int32 ChannelIndex = 0; ChannelIndex < ChannelCount; ++ChannelIndex)
	{
		RecomputeLastDraggedFrameTimesFromAnchorTimes(ChannelCache
			, ChannelIndex, TickResolution, DisplayRate, InNewAnchorTimes, bInSnapEnabled);
	}
}

bool FSimpleViewRetimeDragOperation::CanRecomputeChannel(const FMultiChannelKeyCache<FSimpleViewRetimeToolChannelKeyCache>& InChannelCache
	, const FFrameRate& InTickResolution
	, const TArray<FFrameTime>& InAnchorTimes
	, const int32 InChannelIndex)
{
	if (!Algo::IsSorted(InAnchorTimes))
	{
		ensureMsgf(false, TEXT("Anchor times must be sorted before recompute."));
		return false;
	}

	if (!ensure(InChannelCache.ChannelCache.IsValidIndex(InChannelIndex)))
	{
		return false;
	}

	const FChannelKeyRangeCache<FSimpleViewRetimeToolChannelKeyCache>& ChannelData = InChannelCache.ChannelCache[InChannelIndex];
	const TViewModelPtr<FChannelModel> ChannelModel = ChannelData.WeakChannelModel.Pin();
	if (!ensure(ChannelModel.IsValid()))
	{
		return false;
	}

	const int32 KeyCount = ChannelData.KeyCache.Num();
	if (!ensure(KeyCount > 0))
	{
		return false;
	}

	const int32 AnchorCount = InAnchorTimes.Num();
	if (!ensure(AnchorCount >= 2))
	{
		return false;
	}

	if (InTickResolution.AsInterval() <= 0.0)
	{
		return false;
	}

	return true;
}

void FSimpleViewRetimeDragOperation::RecomputeLastDraggedFrameTimesFromDrag(FMultiChannelKeyCache<FSimpleViewRetimeToolChannelKeyCache>& InChannelCache
	, const int32 InChannelIndex
	, const FFrameRate& InTickResolution
	, const FFrameRate& InDisplayRate
	, const TArray<FFrameTime>& InAnchorStartTimes
	, const TArray<double, TInlineAllocator<16>>& InAnchorInfluences
	, const FFrameTime& InDeltaTickTime
	, const bool bInIgnoreAnchorInfluences
	, const bool bInSnapToFrame)
{
	if (!CanRecomputeChannel(InChannelCache, InTickResolution, InAnchorStartTimes, InChannelIndex))
	{
		return;
	}

	const int32 AnchorCount = InAnchorStartTimes.Num();
	const int32 AnchorInfluenceCount = InAnchorInfluences.Num();
	const bool bAllFullInfluenceAnchors = AnchorInfluenceCount == 0;
	if (!bAllFullInfluenceAnchors && AnchorInfluenceCount < AnchorCount)
	{
		return;
	}

	InChannelCache.ForEachChannelKey(InChannelIndex, [InAnchorInfluences, &InTickResolution, &InDisplayRate
		, InDeltaTickTime, bInIgnoreAnchorInfluences, bInSnapToFrame, AnchorCount, bAllFullInfluenceAnchors]
		(FChannelKeyRangeCache<FSimpleViewRetimeToolChannelKeyCache>& InChannelCache
			, const TViewModelPtr<FChannelModel>& InChannelModel
			, const int32 InKeyIndex
			, FSimpleViewRetimeToolChannelKeyCache& InKeyCache)
		{
			const int32 RightAnchorIndex = FMath::Min(InKeyCache.LeftAnchorIndex + 1, AnchorCount - 1);

			if (!ensureMsgf(InKeyCache.LeftAnchorIndex >= 0 && InKeyCache.LeftAnchorIndex < AnchorCount
				, TEXT("InKeyCache.LeftAnchorIndex %d out of range [0, %d)."), InKeyCache.LeftAnchorIndex, AnchorCount))
			{
				return false;
			}

			if (!ensureMsgf(RightAnchorIndex >= 0 && RightAnchorIndex < AnchorCount
				, TEXT("RightAnchorIndex %d out of range [0, %d)."), RightAnchorIndex, AnchorCount))
			{
				return false;
			}

			const double InfluenceOnKey = bAllFullInfluenceAnchors
				? 1.0
				: FMath::Lerp(InAnchorInfluences[InKeyCache.LeftAnchorIndex]
					, InAnchorInfluences[RightAnchorIndex]
					, InKeyCache.FractionBetweenAnchors);

			const FFrameTime AppliedDeltaTickTime = bInIgnoreAnchorInfluences
				? InDeltaTickTime
				: FFrameTime::FromDecimal(InDeltaTickTime.AsDecimal() * InfluenceOnKey);

			FFrameTime NewKeyTime = InKeyCache.InitialFrameTime + AppliedDeltaTickTime;

			if (bInSnapToFrame)
			{
				const FFrameTime DisplayTime = FFrameRate::TransformTime(NewKeyTime, InTickResolution, InDisplayRate);
				const FFrameNumber SnappedDisplayFrame = DisplayTime.RoundToFrame();
				NewKeyTime = FFrameRate::TransformTime(FFrameTime(SnappedDisplayFrame), InDisplayRate, InTickResolution);
			}

			InKeyCache.LastDraggedFrameTime = NewKeyTime;

			return true;
		});
}

void FSimpleViewRetimeDragOperation::RecomputeLastDraggedFrameTimesFromAnchorTimes(FMultiChannelKeyCache<FSimpleViewRetimeToolChannelKeyCache>& InChannelCache
	, const int32 InChannelIndex
	, const FFrameRate& InTickResolution
	, const FFrameRate& InDisplayRate
	, const TArray<FFrameTime>& InNewAnchorTimes
	, const bool bInSnapToFrame)
{
	if (!CanRecomputeChannel(InChannelCache, InTickResolution, InNewAnchorTimes, InChannelIndex))
	{
		return;
	}

	const int32 AnchorCount = InNewAnchorTimes.Num();

	InChannelCache.ForEachChannelKey(InChannelIndex
		, [&InNewAnchorTimes, &InTickResolution, &InDisplayRate, bInSnapToFrame, AnchorCount]
		(FChannelKeyRangeCache<FSimpleViewRetimeToolChannelKeyCache>& InChannelCache
			, const TViewModelPtr<FChannelModel>& InChannelModel
			, const int32 InKeyIndex
			, FSimpleViewRetimeToolChannelKeyCache& InKeyCache)
		{
			const int32 RightAnchorIndex = FMath::Min(InKeyCache.LeftAnchorIndex + 1, AnchorCount - 1);

			if (!ensureMsgf(InKeyCache.LeftAnchorIndex >= 0 && InKeyCache.LeftAnchorIndex < AnchorCount
				, TEXT("InKeyCache.LeftAnchorIndex %d out of range [0, %d).")
				, InKeyCache.LeftAnchorIndex, AnchorCount))
			{
				return false;
			}

			if (!ensureMsgf(RightAnchorIndex >= 0 && RightAnchorIndex < AnchorCount
				, TEXT("RightAnchorIndex %d out of range [0, %d).")
				, RightAnchorIndex, AnchorCount))
			{
				return false;
			}

			const FFrameTime LeftTime = InNewAnchorTimes[InKeyCache.LeftAnchorIndex];
			const FFrameTime RightTime = InNewAnchorTimes[RightAnchorIndex];

			const double NewKeyTimeDecimal = FMath::Lerp(LeftTime.AsDecimal(), RightTime.AsDecimal(), InKeyCache.FractionBetweenAnchors);

			FFrameTime NewKeyTime = FFrameTime::FromDecimal(NewKeyTimeDecimal);

			if (bInSnapToFrame)
			{
				const FFrameTime DisplayTime = FFrameRate::TransformTime(NewKeyTime, InTickResolution, InDisplayRate);
				const FFrameNumber SnappedDisplayFrame = DisplayTime.RoundToFrame();
				NewKeyTime = FFrameRate::TransformTime(FFrameTime(SnappedDisplayFrame), InDisplayRate, InTickResolution);
			}

			InKeyCache.LastDraggedFrameTime = NewKeyTime;

			return true;
		});
}

} // namespace UE::Sequencer::SimpleView
