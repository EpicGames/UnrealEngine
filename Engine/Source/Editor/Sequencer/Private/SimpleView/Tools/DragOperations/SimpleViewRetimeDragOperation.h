// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Range.h"
#include "MVVM/ViewModelPtr.h"
#include "ToolableTimeline/Caches/ChannelKeyRangeCache.h"
#include "ToolableTimeline/DragOperations/KeyDragOperation.h"

struct FFrameNumber;

namespace UE::Sequencer::ToolableTimeline
{
	struct FMouseInputData;
}

namespace UE::Sequencer::SimpleView
{

/** Additional cached properties for a retime tool key */
struct FSimpleViewRetimeToolChannelKeyCache : ToolableTimeline::FChannelKeyCache
{
	int32 LeftAnchorIndex = INDEX_NONE;
	double FractionBetweenAnchors = 0.0;
};

class FSimpleViewRetimeDragOperation : public ToolableTimeline::FKeyDragOperation<FSimpleViewRetimeToolChannelKeyCache>
{
public:
	/** Tool drag operation type */
	enum class EDragMode : uint8
	{
		/** Initially dragging first and second anchors */
		None,
		/** Move anchor and keys */
		MoveAnchor,
		/** Move all anchors and keys */
		MoveAllAnchors,

		/**  */
		LatticeScaleLeft,
		/**  */
		LatticeMoveAnchors,
		/**  */
		LatticeScaleRight
	};

	FSimpleViewRetimeDragOperation(const ToolableTimeline::FMouseInputData& InMouseInput
		, const TRange<FFrameNumber>& InFrameRange
		, const TRange<double>& InViewRange
		, const EDragMode InMode
		, const TArray<FFrameTime>& InInitialAnchorTimes);

	EDragMode GetCurrentDragMode() const;

	bool IsCurrentDragMode(const EDragMode InDragMode) const;

	const TArray<FFrameTime>& GetAnchorStartTimes() const;

	void RecomputeForMove(const TSharedRef<ToolableTimeline::FToolableTimeline>& InTimeline
		, const TArray<FFrameTime>& InInitialAnchorTimes
		, const TArray<double, TInlineAllocator<16>>& InAnchorInfluences
		, const FFrameTime& InDeltaTickTime
		, const bool bInIgnoreAnchorInfluences
		, const bool bInSnapEnabled);

	void RecomputeFromAnchorTimes(const TSharedRef<ToolableTimeline::FToolableTimeline>& InTimeline
		, const TArray<FFrameTime>& InAnchorStartTimes
		, const bool bInSnapEnabled);

private:
	static bool CanRecomputeChannel(const ToolableTimeline::FMultiChannelKeyCache<FSimpleViewRetimeToolChannelKeyCache>& InChannelCache
		, const FFrameRate& InTickResolution
		, const TArray<FFrameTime>& InAnchorTimes
		, const int32 InChannelIndex);

	static void RecomputeLastDraggedFrameTimesFromDrag(ToolableTimeline::FMultiChannelKeyCache<FSimpleViewRetimeToolChannelKeyCache>& InChannelCache
		, const int32 InChannelIndex
		, const FFrameRate& InTickResolution
		, const FFrameRate& InDisplayRate
		, const TArray<FFrameTime>& InAnchorStartTimes
		, const TArray<double, TInlineAllocator<16>>& InAnchorInfluences
		, const FFrameTime& InDeltaTickTime
		, const bool bInIgnoreAnchorInfluences
		, const bool bInSnapToFrame);

	static void RecomputeLastDraggedFrameTimesFromAnchorTimes(ToolableTimeline::FMultiChannelKeyCache<FSimpleViewRetimeToolChannelKeyCache>& InChannelCache
		, const int32 InChannelIndex
		, const FFrameRate& InTickResolution
		, const FFrameRate& InDisplayRate
		, const TArray<FFrameTime>& InNewAnchorTimes
		, const bool bInSnapToFrame);

protected:
	/** Active drag behavior. */
	EDragMode CurrentDragMode = EDragMode::None;

	/**  */
	TArray<FFrameTime> AnchorInitialTimes;
};

} // namespace UE::Sequencer::SimpleView
