// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimpleViewMoveFrameKeysToolFactory.h"
#include "Misc/InputState.h"
#include "SequencerCommands.h"
#include "SimpleView/Tools/SimpleViewMoveFrameKeysTool.h"
#include "ToolableTimeline/MouseInputData.h"
#include "ToolableTimeline/ToolableTimeline.h"
#include "ToolableTimeline/Tools/ToolableTimelineBaseTool.h"

namespace UE::Sequencer::SimpleView
{

const FName FSimpleViewMoveFrameKeysToolFactory::StaticToolId = TEXT("RetimeFrame");

FName FSimpleViewMoveFrameKeysToolFactory::GetIdentifier() const
{
	return StaticToolId;
}

float FSimpleViewMoveFrameKeysToolFactory::GetPriority() const
{
	return 10.f;
}

bool FSimpleViewMoveFrameKeysToolFactory::WantsToActivate(const FMouseInputData& InMouseInput, const bool bInMouseHitKeyFrame) const
{
	return bInMouseHitKeyFrame
		&& InMouseInput.PointerEvent.GetEffectingButton() == EKeys::LeftMouseButton
		&& !CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport)
		&& FSimpleViewMoveFrameKeysTool::CanMoveSelectedKeysAndMarks(InMouseInput);
}

TSharedRef<FToolableTimelineBaseTool> FSimpleViewMoveFrameKeysToolFactory::CreateTool(FToolableTimeline& InTimeline) const
{
	return MakeShared<FSimpleViewMoveFrameKeysTool>(InTimeline.AsShared());
}

} // namespace UE::Sequencer::SimpleView
