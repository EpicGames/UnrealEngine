// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimpleViewRetimeToolFactory.h"
#include "SimpleView/Tools/SimpleViewRetimeTool.h"
#include "ToolableTimeline/MouseInputData.h"
#include "ToolableTimeline/ToolableTimeline.h"
#include "ToolableTimeline/Tools/ToolableTimelineBaseTool.h"

namespace UE::Sequencer::SimpleView
{

const FName FSimpleViewRetimeToolFactory::StaticToolId = TEXT("RetimeRange");

FName FSimpleViewRetimeToolFactory::GetIdentifier() const
{
	return StaticToolId;
}

float FSimpleViewRetimeToolFactory::GetPriority() const
{
	return 20.f;
}

bool FSimpleViewRetimeToolFactory::WantsToActivate(const FMouseInputData& InMouseInput, const bool bInMouseHitKeyFrame) const
{
	return InMouseInput.PointerEvent.GetEffectingButton() == EKeys::LeftMouseButton
		&& InMouseInput.PointerEvent.IsShiftDown();
}

bool FSimpleViewRetimeToolFactory::WantsToActivateOnDoubleClick(const FMouseInputData& InMouseInput, const bool bInMouseHitKeyFrame) const
{
	return InMouseInput.PointerEvent.GetEffectingButton() == EKeys::LeftMouseButton;
}

TSharedRef<FToolableTimelineBaseTool> FSimpleViewRetimeToolFactory::CreateTool(FToolableTimeline& InTimeline) const
{
	return MakeShared<FSimpleViewRetimeTool>(InTimeline.AsShared());
}

} // namespace UE::Sequencer::SimpleView
