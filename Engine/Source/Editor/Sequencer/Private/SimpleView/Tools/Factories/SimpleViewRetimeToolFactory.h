// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolableTimeline/Tools/Factories/IToolableTimelineToolFactory.h"

namespace UE::Sequencer::ToolableTimeline
{
	class FToolableTimeline;
	class FToolableTimelineBaseTool;
	struct FMouseInputData;
}

namespace UE::Sequencer::SimpleView
{

class FSimpleViewRetimeToolFactory : public ToolableTimeline::IToolableTimelineToolFactory
{
public:
	static const FName StaticToolId;

	//~ Begin ToolableTimeline::IToolableTimelineToolFactory
	virtual FName GetIdentifier() const override;
	virtual float GetPriority() const override;
	virtual bool RequiresDragThresholdToActivate() const override { return true; }
	virtual bool WantsToActivate(const ToolableTimeline::FMouseInputData& InMouseInput, const bool bInMouseHitKeyFrame) const override;
	virtual bool WantsToActivateOnDoubleClick(const ToolableTimeline::FMouseInputData& InMouseInput, const bool bInMouseHitKeyFrame) const override;
	virtual TSharedRef<ToolableTimeline::FToolableTimelineBaseTool> CreateTool(ToolableTimeline::FToolableTimeline& InTimeline) const override;
	//~ End ToolableTimeline::IToolableTimelineToolFactory
};

} // namespace UE::Sequencer::SimpleView
