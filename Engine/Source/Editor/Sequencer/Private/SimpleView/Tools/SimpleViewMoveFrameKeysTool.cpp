// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimpleViewMoveFrameKeysTool.h"
#include "Channels/MovieSceneChannel.h"
#include "Framework/Commands/UICommandList.h"
#include "Misc/InputState.h"
#include "MovieSceneSection.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "Sequencer.h"
#include "SequencerCommands.h"
#include "SimpleView/Tools/Factories/SimpleViewMoveFrameKeysToolFactory.h"
#include "ToolableTimeline/Caches/MultiChannelKeyCache.h"
#include "ToolableTimeline/Drawing/ToolableTimelineDrawer.h"
#include "ToolableTimeline/MouseInputData.h"
#include "ToolableTimeline/ToolableTimeline.h"
#include "ToolableTimeline/ToolableTimeSliderController.h"
#include "ToolableTimeline/Widgets/SToolableTimeline.h"

#define LOCTEXT_NAMESPACE "TimelineMoveFrameKeysTool"

namespace UE::Sequencer::SimpleView
{

using namespace UE::Sequencer::ToolableTimeline;

/**
 * Simple view has its own key selection separate from full view. @See FToolableTimelineKeySelection::SyncKeySelectionToSequencer()
 * Because of that, SequencerHelpers::RemoveDuplicateKeys cannot map these moved key
 * handles back to channel models. Ideally that helper would have a channel-model based
 * overload so Sequencer full and simple views can share the same duplicate-key removal logic.
 * This helper can stay smaller because the Simple View drag cache already carries the
 * channel model for each moved key.
 */
static bool RemoveDuplicateKeysForMovedKeys(FMultiChannelKeyCache<FChannelKeyCache>& InChannelCache)
{
	TMap<TViewModelPtr<FChannelModel>, TArray<FKeyHandle>> ChannelToMovedKeyHandles;

	InChannelCache.ForEachChannelKey([&ChannelToMovedKeyHandles](FChannelKeyRangeCache<FChannelKeyCache>&
			, const TViewModelPtr<FChannelModel>& InChannelModel
			, const int32
			, const FChannelKeyCache& InKeyCache)
		{
			FMovieSceneChannel* const Channel = InChannelModel.IsValid() ? InChannelModel->GetChannel() : nullptr;
			if (!Channel || Channel->GetIndex(InKeyCache.Handle) == INDEX_NONE)
			{
				return true;
			}

			ChannelToMovedKeyHandles.FindOrAdd(InChannelModel).AddUnique(InKeyCache.Handle);
			return true;
		});

	bool bRemovedDuplicateKeys = false;

	for (const TPair<TViewModelPtr<FChannelModel>, TArray<FKeyHandle>>& ChannelToMovedKeys : ChannelToMovedKeyHandles)
	{
		if (!ChannelToMovedKeys.Key.IsValid() || ChannelToMovedKeys.Value.IsEmpty())
		{
			continue;
		}

		UMovieSceneSection* const Section = ChannelToMovedKeys.Key->GetSection();
		FMovieSceneChannel* const Channel = ChannelToMovedKeys.Key->GetChannel();
		if (!Section || Section->IsReadOnly() || !Channel)
		{
			continue;
		}

		TArray<FFrameNumber> CheckedKeyTimes;
		TArray<FKeyHandle> HandlesToRemove;

		for (const FKeyHandle MovedHandle : ChannelToMovedKeys.Value)
		{
			if (Channel->GetIndex(MovedHandle) == INDEX_NONE)
			{
				continue;
			}

			FFrameNumber KeyTime;
			Channel->GetKeyTime(MovedHandle, KeyTime);
			if (CheckedKeyTimes.Contains(KeyTime))
			{
				continue;
			}

			CheckedKeyTimes.Add(KeyTime);

			TArray<FFrameNumber> OutKeyTimes;
			TArray<FKeyHandle> OutKeyHandles;
			Channel->GetKeys(TRange<FFrameNumber>(KeyTime, KeyTime), &OutKeyTimes, &OutKeyHandles);
			if (OutKeyHandles.Num() <= 1)
			{
				continue;
			}

			FKeyHandle KeyHandleToKeep = OutKeyHandles[0];

			for (const FKeyHandle CandidateHandle : ChannelToMovedKeys.Value)
			{
				if (OutKeyHandles.Contains(CandidateHandle))
				{
					KeyHandleToKeep = CandidateHandle;
					break;
				}
			}

			for (const FKeyHandle Handle : OutKeyHandles)
			{
				if (Handle != KeyHandleToKeep)
				{
					HandlesToRemove.AddUnique(Handle);
				}
			}
		}

		if (HandlesToRemove.IsEmpty())
		{
			continue;
		}

		Section->Modify();
		Channel->DeleteKeys(HandlesToRemove);

		bRemovedDuplicateKeys = true;
	}

	return bRemovedDuplicateKeys;
}

FSimpleViewMoveFrameKeysTool::FSimpleViewMoveFrameKeysTool(const TSharedRef<FToolableTimeline>& InTimeline)
	: FToolableTimelineBaseTool(InTimeline)
{
}

FName FSimpleViewMoveFrameKeysTool::GetIdentifier() const
{
	return FSimpleViewMoveFrameKeysToolFactory::StaticToolId;
}

FText FSimpleViewMoveFrameKeysTool::GetLabel() const
{
	return LOCTEXT("MoveKeysLabel", "Move Keys");
}

FText FSimpleViewMoveFrameKeysTool::GetDescription() const
{
	return LOCTEXT("MoveKeysDescription", "Move all keys being hovered");
}

bool FSimpleViewMoveFrameKeysTool::CanMoveSelectedKeysAndMarks(const FMouseInputData& InMouseInput)
{
	bool bCanMoveSelectedKeys = !ToolableTimeline::Utils::ShouldUseControlModifierToMoveKeys()
		|| InMouseInput.PointerEvent.IsControlDown();

	if (!bCanMoveSelectedKeys)
	{
		if (const TSharedPtr<FSequencer> Sequencer = InMouseInput.Timeline->GetSequencer())
		{
			TSet<FSequencerSelectedKey> SelectedKeys = InMouseInput.Timeline->GetKeySelection().GetSelectedKeys();
			FSequencerSelectedKey::AppendKeySelection(SelectedKeys, Sequencer->GetSelection().KeySelection);

			bCanMoveSelectedKeys = !SelectedKeys.IsEmpty()
				&& !Sequencer->GetSelection().MarkedFrames.GetSelected().IsEmpty();
		}
	}

	return bCanMoveSelectedKeys;
}

void FSimpleViewMoveFrameKeysTool::BindCommands(const TSharedRef<FUICommandList>& InCommandBindings)
{
}

void FSimpleViewMoveFrameKeysTool::UnbindCommands(const TSharedRef<FUICommandList>& InCommandBindings)
{
}

void FSimpleViewMoveFrameKeysTool::NotifyToolActivated(const FMouseInputData& InMouseInput, const FFrameTime& InInputTickTime)
{
}

void FSimpleViewMoveFrameKeysTool::NotifyToolDeactivated()
{
	StopDragIfPossible(/*bInCommit*/false);
}

bool FSimpleViewMoveFrameKeysTool::WantsInputPrepass(const FMouseInputData& InMouseInput) const
{
	return CanMoveSelectedKeysAndMarks(InMouseInput)
		&& !CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport)
		&& (InMouseInput.PointerEvent.GetEffectingButton() == EKeys::LeftMouseButton
		|| InMouseInput.PointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton));
}

int32 FSimpleViewMoveFrameKeysTool::Paint(FMouseDrawInputData& MouseDrawInput) const
{
	if (!MouseDownOp.IsSet()
		|| !MouseDownOp->HasDragOp()
		|| !MouseDownOp->HasCachedKeys())
	{
		return MouseDrawInput.LayerId;
	}

	const TRange<FFrameNumber> ToolRange = GetToolRange();

	// Draw left and right movement indicators
	MouseDrawInput.LayerId = DrawFrameMovementIndicator(MouseDrawInput
		, ToolRange.GetLowerBoundValue()
		, ToolRange.GetUpperBoundValue()
		, MouseDrawInput.Timeline->GetTimelineSettings().Settings.ToolHandleColor);

	return MouseDrawInput.IncrementDrawLayer().LayerId;
}

TRange<FFrameNumber> FSimpleViewMoveFrameKeysTool::GetToolRange() const
{
	FFrameTime StartTick;
	FFrameTime EndExclusiveTick;
	if (!CalculateKeyBoundsTickSpan(StartTick, EndExclusiveTick))
	{
		return TRange<FFrameNumber>::Empty();
	}

	return TRange<FFrameNumber>(StartTick.FloorToFrame(), EndExclusiveTick.CeilToFrame());
}

TRange<FFrameTime> FSimpleViewMoveFrameKeysTool::GetDragIndicatorTickRange(const FMouseInputData& InMouseInput) const
{
	if (!IsDragging())
	{
		return TRange<FFrameTime>();
	}

	int32 DraggedKeyCount = 0;
	FFrameTime SingleDraggedKeyTickTime;

	MouseDownOp->GetChannelCache().ForEachChannelKey([&DraggedKeyCount, &SingleDraggedKeyTickTime]
		(const FChannelKeyRangeCache<FChannelKeyCache>& InChannelCache
			, const TViewModelPtr<FChannelModel>& InChannelModel
			, const int32 InKeyIndex
			, const FChannelKeyCache& InKeyCache)
		{
			++DraggedKeyCount;
			SingleDraggedKeyTickTime = InKeyCache.LastDraggedFrameTime;

			return DraggedKeyCount < 2;
		});

	if (DraggedKeyCount == 1)
	{
		return TRange<FFrameTime>(
			TRangeBound<FFrameTime>::Inclusive(SingleDraggedKeyTickTime),
			TRangeBound<FFrameTime>::Inclusive(SingleDraggedKeyTickTime)
		);
	}

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = GetTimelineChecked()->GetTimeSliderController();
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	const FFrameRate DisplayRate = TimeSliderController->GetDisplayRate();

	const FFrameTime StartTickTime = MouseDownOp->GetInitialDisplayFrameTickRange().GetLowerBoundValue();
	const FFrameTime EndTickTime = MouseDownOp->ComputeDraggedTickTime(TickResolution, DisplayRate, /*bInSnapToDisplayFrame=*/false);

	return TRange<FFrameTime>(
		TRangeBound<FFrameTime>::Inclusive(StartTickTime),
		TRangeBound<FFrameTime>::Inclusive(EndTickTime)
	);
}

bool FSimpleViewMoveFrameKeysTool::IsDragging() const
{
	return MouseDownOp.IsSet()
		&& MouseDownOp->HasDragOp()
		&& !MouseDownOp->GetAccumulatedLocalDelta().IsNearlyZero();
}

bool FSimpleViewMoveFrameKeysTool::HasHoverHit() const
{
	return GetTimelineKeySelection().HasAnyHoveredKeys();
}

FCursorReply FSimpleViewMoveFrameKeysTool::OnCursorQuery(const SWidget& OwnerWidget
	, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) const
{
	if (!MouseDownOp.IsSet()
		|| !MouseDownOp->IsDragging()
		|| !MouseDownOp->HasCachedKeys())
	{
		return FCursorReply::Unhandled();
	}

	return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
}

FReply FSimpleViewMoveFrameKeysTool::OnMouseButtonDown(SWidget& OwnerWidget
	, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	if (CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport))
	{
		return FReply::Unhandled();
	}

	const FMouseInputData MouseInput(GetTimelineChecked(), InGeometry, InPointerEvent);
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = MouseInput.Timeline->GetTimeSliderController();

	TSet<FSequencerSelectedKey> HoveredKeysAtMouse;
	TimeSliderController->GetKeysUnderMouse(MouseInput, MouseInput.PointerEvent.GetScreenSpacePosition(), HoveredKeysAtMouse);

	if (HoveredKeysAtMouse.IsEmpty())
	{
		return FReply::Unhandled();
	}

	FToolableTimelineKeySelection& KeySelection = MouseInput.Timeline->GetKeySelection();
	TSet<int32> SelectedMarkedFrames;
	TSet<FSequencerSelectedKey> SelectedKeys = KeySelection.GetSelectedKeys();
	if (const TSharedPtr<FSequencer> Sequencer = MouseInput.Timeline->GetSequencer())
	{
		SelectedMarkedFrames = Sequencer->GetSelection().MarkedFrames.GetSelected();
		FSequencerSelectedKey::AppendKeySelection(SelectedKeys, Sequencer->GetSelection().KeySelection);
	}

	const bool bClickedAlreadySelectedKey = !SelectedKeys.IsEmpty()
		&& HoveredKeysAtMouse.Intersect(SelectedKeys).Num() > 0;
	const bool bHasSelectedKeysAndMarks = !SelectedKeys.IsEmpty() && !SelectedMarkedFrames.IsEmpty();
	const bool bShouldDragCurrentSelection = bClickedAlreadySelectedKey
		&& (bHasSelectedKeysAndMarks
			|| !ToolableTimeline::Utils::ShouldUseControlModifierToMoveKeys()
			|| InPointerEvent.IsControlDown());

	TSet<FSequencerSelectedKey> KeysToDrag = HoveredKeysAtMouse;

	// If direct dragging is enabled, clicking an already selected key should drag the current selection.
	// Otherwise preserve the existing Ctrl + drag gesture for moving the full selection.
	if (bShouldDragCurrentSelection)
	{
		KeysToDrag = SelectedKeys;
	}
	else
	{
		// Preserve the normal selection behavior when clicking an unselected key
		KeySelection.SelectKeys(HoveredKeysAtMouse, InPointerEvent.IsShiftDown());
		SelectedMarkedFrames.Reset();
	}

	MouseDownOp.Emplace(MouseInput, KeysToDrag, SelectedMarkedFrames, TimeSliderController->GetViewRange());

	return FReply::Handled()
		.CaptureMouse(OwnerWidget.AsShared())
		.UseHighPrecisionMouseMovement(OwnerWidget.AsShared())
		.PreventThrottling();
}

FReply FSimpleViewMoveFrameKeysTool::OnMouseMove(SWidget& OwnerWidget
	, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	const FMouseInputData MouseInput(GetTimelineChecked(), InGeometry, InPointerEvent);

	if (MouseDownOp.IsSet())
	{
		MouseDownOp->UpdateDrag(MouseInput);

		if (!ActiveTransaction.IsValid())
		{
			OnDragStart(MouseInput);
		}

		OnDrag(MouseInput);

		return FReply::Handled()
			.CaptureMouse(OwnerWidget.AsShared())
			.UseHighPrecisionMouseMovement(OwnerWidget.AsShared())
			.PreventThrottling();
	}

	return FReply::Unhandled();
}

FReply FSimpleViewMoveFrameKeysTool::OnMouseButtonUp(SWidget& OwnerWidget
	, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (!MouseDownOp.IsSet())
	{
		return FReply::Unhandled();
	}

	const FVector2D AccumulatedDelta = MouseDownOp->GetAccumulatedLocalDelta();
	const bool bDidActuallyDrag = !FMath::IsNearlyZero(AccumulatedDelta.X) || !FMath::IsNearlyZero(AccumulatedDelta.Y);

	if (bDidActuallyDrag)
	{
		OnDragEnd();
	}
	else
	{
		MouseDownOp->ResetDrag();
		MouseDownOp.Reset();
		UpdateKeyTimeThrottle.Reset();

		RequestClose();
	}

	return FReply::Handled().ReleaseMouseCapture();
}

void FSimpleViewMoveFrameKeysTool::OnFocusLost(SWidget& OwnerWidget, const FFocusEvent& InFocusEvent)
{
	// We need to end our drag if we lose Window focus to close the transaction, otherwise alt-tabbing while dragging
	// can cause a transaction to get stuck open.
	StopDragIfPossible(/*bInCommit*/false);

	RequestClose();
}

void FSimpleViewMoveFrameKeysTool::OnDragStart(const FMouseInputData& InMouseInput)
{
	ActiveTransaction = MakeUnique<FScopedTransaction>(TEXT("MoveKeys")
		, LOCTEXT("MoveKeysTransaction", "Move Key(s)"), nullptr);
}

void FSimpleViewMoveFrameKeysTool::OnDrag(const FMouseInputData& InMouseInput)
{
	if (!ensure(MouseDownOp.IsSet()))
	{
		return;
	}

	const TSharedRef<FToolableTimeline> Timeline = GetTimelineChecked();

	const FFrameTime DeltaTickTime = MouseDownOp->DragPixelsToTickFrameTime(InMouseInput);

	MouseDownOp->GetChannelCache().RecomputeForDrag(Timeline, DeltaTickTime);
	MouseDownOp->UpdateMarkedFramesFromKeyDrag(Timeline, DeltaTickTime);

	ApplyDragResultsIfThrottleOpen(Timeline, /*bInForceApply=*/false);
}

void FSimpleViewMoveFrameKeysTool::OnDragEnd(const bool bInCommit)
{
	const TSharedRef<FToolableTimeline> Timeline = GetTimelineChecked();

	// Because we are throttling and may have missed an update, make sure the key times get updated
	if (MouseDownOp.IsSet())
	{
		if (bInCommit)
		{
			// Apply the final key times without broadcasting so duplicate-key cleanup and
			// the commit refresh produce one TrackValueChanged notification. A non-hotfix
			// cleanup should move this into a shared apply/remove path so callers do not
			// have to manually coordinate this notification order.
			UpdateKeyTimeThrottle.NotifyActivity();
			UpdateKeyTimes(Timeline, /*bInNotifyMovieSceneDataChanged=*/false);

			RemoveDuplicateKeysForMovedKeys(MouseDownOp->GetChannelCache());

			if (const TSharedPtr<FSequencer> Sequencer = Timeline->GetSequencer())
			{
				Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
			}

			MouseDownOp->SortMarkedFrames(Timeline);
		}
		else
		{
			MouseDownOp->RestoreInitialKeyAndMarkTimes(Timeline);
		}
	}

	UpdateKeyTimeThrottle.Reset();

	ActiveTransaction.Reset();

	if (MouseDownOp.IsSet())
	{
		MouseDownOp->ResetDrag();
	}
	MouseDownOp.Reset();

	RequestClose();
}

void FSimpleViewMoveFrameKeysTool::StopDragIfPossible(const bool bInCommit)
{
	if (!MouseDownOp.IsSet())
	{
		UpdateKeyTimeThrottle.Reset();
		ActiveTransaction.Reset();
		return;
	}

	if (MouseDownOp->HasDragOp())
	{
		OnDragEnd(bInCommit);
	}
	else
	{
		MouseDownOp->ResetDrag();
		MouseDownOp.Reset();
		UpdateKeyTimeThrottle.Reset();
		ActiveTransaction.Reset();
	}
}

int32 FSimpleViewMoveFrameKeysTool::DrawFrameMovementIndicator(FMouseDrawInputData& MouseDrawInput
	, const FFrameTime& InStartTick
	, const FFrameTime& InEndExclusiveTick
	, const FLinearColor& InIndicatorColor) const
{
	static const FVector2f IndicatorSize = FVector2f(10.f, 18.f);
	static constexpr double IndicatorPad = 1.f;
	static constexpr float FrameOutlineThickness = 2.f;

	if (InEndExclusiveTick <= InStartTick)
	{
		return MouseDrawInput.LayerId;
	}

	const TSharedRef<FToolableTimeline> Timeline = GetTimelineChecked();
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = Timeline->GetTimeSliderController();
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();

	FVector2f FrameAreaSize;
	FVector2f FrameAreaPosition;
	Drawing::GetFrameAreaGeometry(MouseDrawInput, 0.f, FrameAreaSize, FrameAreaPosition);

	float IndicatorOffsetY = FrameAreaPosition.Y + (FrameAreaSize.Y * .5f) - (IndicatorSize.Y * .5f);

	const double FrameStartSeconds = TickResolution.AsSeconds(InStartTick);
	const double FrameEndSeconds = TickResolution.AsSeconds(InEndExclusiveTick);

	const float FrameStartPx = MouseDrawInput.RangeToScreen.InputToLocalX(FrameStartSeconds);
	const float FrameEndPx = MouseDrawInput.RangeToScreen.InputToLocalX(FrameEndSeconds);

	const FVector2f LeftOffset(FrameStartPx - (IndicatorSize.X + IndicatorPad), IndicatorOffsetY);
	const FVector2f RightOffset(FrameEndPx + IndicatorPad, IndicatorOffsetY);

	const FPaintGeometry LeftIndicatorGeometry = MouseDrawInput.Geometry.ToPaintGeometry(
		IndicatorSize,
		FSlateLayoutTransform(LeftOffset),
		FSlateRenderTransform(FScale2f(-1.f, 1.f))
	);
	const FPaintGeometry RightIndicatorGeometry = MouseDrawInput.Geometry.ToPaintGeometry(
		IndicatorSize,
		FSlateLayoutTransform(RightOffset)
	);

	FSlateDrawElement::MakeBox(
		MouseDrawInput.DrawElements,
		MouseDrawInput.IncrementDrawLayer().LayerId,
		LeftIndicatorGeometry,
		FAppStyle::GetBrush(TEXT("TreeArrow_Collapsed")),
		MouseDrawInput.DrawEffects,
		InIndicatorColor
	);

	FSlateDrawElement::MakeBox(
		MouseDrawInput.DrawElements,
		MouseDrawInput.LayerId,
		RightIndicatorGeometry,
		FAppStyle::GetBrush(TEXT("TreeArrow_Collapsed")),
		MouseDrawInput.DrawEffects,
		InIndicatorColor
	);

	// Draw a border rectangle around the current frame
	const FVector2f RectangleStart(FrameStartPx, FrameAreaPosition.Y);
	const FVector2f RectangleSize(FrameEndPx - FrameStartPx, FrameAreaSize.Y);

	TArray<FVector2f> RectanglePoints;
	RectanglePoints.Add(RectangleStart);
	RectanglePoints.Add(RectangleStart + FVector2f(RectangleSize.X, 0.f));
	RectanglePoints.Add(RectangleStart + RectangleSize);
	RectanglePoints.Add(RectangleStart + FVector2f(0.f, RectangleSize.Y));
	RectanglePoints.Add(RectangleStart);

	FSlateDrawElement::MakeLines(
		MouseDrawInput.DrawElements,
		MouseDrawInput.IncrementDrawLayer().LayerId,
		MouseDrawInput.Geometry.ToPaintGeometry(),
		RectanglePoints,
		MouseDrawInput.DrawEffects,
		InIndicatorColor,
		true,
		FrameOutlineThickness
	);

	return MouseDrawInput.IncrementDrawLayer().LayerId;
}

void FSimpleViewMoveFrameKeysTool::UpdateKeyTimes(const TSharedRef<FToolableTimeline>& InTimeline, const bool bInNotifyMovieSceneDataChanged)
{
	if (!MouseDownOp.IsSet()
		|| !MouseDownOp->HasDragOp()
		|| !MouseDownOp->HasCachedKeys())
	{
		return;
	}

	MouseDownOp->GetChannelCache().ApplyKeyTimes(InTimeline, bInNotifyMovieSceneDataChanged);
}

void FSimpleViewMoveFrameKeysTool::ApplyDragResultsIfThrottleOpen(const TSharedRef<FToolableTimeline>& InTimeline, const bool bInForceApply)
{
	// Throttle the expensive "apply" stage: Modify + SetKeyTime + Notify + Recache
	if (InTimeline->GetTimelineSettings().Settings.bUseThrottledUpdates
		&& !bInForceApply
		&& !UpdateKeyTimeThrottle.ShouldApply())
	{
		return;
	}

	UpdateKeyTimeThrottle.NotifyActivity();

	UpdateKeyTimes(InTimeline, /*bInNotifyMovieSceneDataChanged=*/bInForceApply);
}

bool FSimpleViewMoveFrameKeysTool::CalculateKeyBoundsTickSpan(FFrameTime& OutStartTick, FFrameTime& OutEndExclusiveTick) const
{
	if (!MouseDownOp.IsSet() || !MouseDownOp->HasCachedKeys())
	{
		return false;
	}

	const TSharedRef<FToolableTimeline> Timeline = GetTimelineChecked();
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = Timeline->GetTimeSliderController();

	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	const FFrameRate DisplayRate = TimeSliderController->GetDisplayRate();

	bool bFoundKey = false;

	FFrameTime MinTickTime;
	FFrameTime MaxTickTime;

	MouseDownOp->GetChannelCache().ForEachChannelKey([this, &bFoundKey, &MinTickTime, &MaxTickTime]
		(FChannelKeyRangeCache<FChannelKeyCache>& InChannelCache,
		 const TViewModelPtr<FChannelModel>& InChannelModel,
		 const int32 InKeyIndex,
		 FChannelKeyCache& InKeyCache)
		{
			const FFrameTime KeyTickTime = MouseDownOp->HasDragOp()
				? InKeyCache.LastDraggedFrameTime : InKeyCache.InitialFrameTime;

			if (!bFoundKey)
			{
				MinTickTime = KeyTickTime;
				MaxTickTime = KeyTickTime;

				bFoundKey = true;
			}
			else
			{
				if (KeyTickTime < MinTickTime)
				{
					MinTickTime = KeyTickTime;
				}

				if (KeyTickTime > MaxTickTime)
				{
					MaxTickTime = KeyTickTime;
				}
			}

			return true;
		});

	if (!bFoundKey)
	{
		return false;
	}

	const FFrameTime OneDisplayFrameInTickResolution =
		FFrameRate::TransformTime(FFrameTime(1), DisplayRate, TickResolution);

	OutStartTick = MinTickTime;
	OutEndExclusiveTick = MaxTickTime + OneDisplayFrameInTickResolution;

	return true;
}

} // namespace UE::Sequencer::SimpleView

#undef LOCTEXT_NAMESPACE
