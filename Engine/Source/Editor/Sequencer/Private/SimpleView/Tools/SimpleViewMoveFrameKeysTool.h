// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimpleView/PollGateThrottle.h"
#include "ToolableTimeline/Caches/ChannelKeyRangeCache.h"
#include "ToolableTimeline/DragOperations/ToolableTimelineKeyAndMarkDragOperation.h"
#include "ToolableTimeline/Tools/ToolableTimelineBaseTool.h"

namespace UE::Sequencer::ToolableTimeline
{
	class FToolableTimeline;
	struct FMouseDrawInputData;
	struct FMouseInputData;
}

namespace UE::Sequencer::SimpleView
{

using namespace UE::Sequencer::ToolableTimeline;

/** Tool for moving groups of keys on a single frame. */
class FSimpleViewMoveFrameKeysTool : public FToolableTimelineBaseTool
{
public:
	FSimpleViewMoveFrameKeysTool(const TSharedRef<FToolableTimeline>& InTimeline);

	static bool CanMoveSelectedKeysAndMarks(const FMouseInputData& InMouseInput);

	//~ Begin IToolableTimelineEditTool

	virtual FName GetIdentifier() const override;
	virtual FText GetLabel() const override;
	virtual FText GetDescription() const override;

	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandBindings) override;
	virtual void UnbindCommands(const TSharedRef<FUICommandList>& InCommandBindings) override;

	virtual void NotifyToolActivated(const FMouseInputData& InMouseInput, const FFrameTime& InInputTickTime) override;
	virtual void NotifyToolDeactivated() override;

	virtual bool WantsInputPrepass(const FMouseInputData& InMouseInput) const override;

	virtual int32 Paint(FMouseDrawInputData& MouseDrawInput) const override;

	virtual TRange<FFrameNumber> GetToolRange() const override;

	virtual bool ShouldShowDragRangeIndicator() const override { return true; }

	virtual TRange<FFrameTime> GetDragIndicatorTickRange(const FMouseInputData& InMouseInput) const override;

	//~ End IToolableTimelineEditTool

	//~ Begin FToolableTimelineBaseTool
	virtual bool IsDragging() const;
	virtual bool HasHoverHit() const;
	//~ End FToolableTimelineBaseTool

	//~ Begin ISequencerInputHandler

	virtual FCursorReply OnCursorQuery(const SWidget& OwnerWidget
		, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) const override;

	virtual FReply OnMouseButtonDown(SWidget& OwnerWidget
		, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseButtonUp(SWidget& OwnerWidget
		, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseMove(SWidget& OwnerWidget
		, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;

	virtual void OnFocusLost(SWidget& OwnerWidget, const FFocusEvent& InFocusEvent) override;

	//~ End ISequencerInputHandler

protected:
	//~ Begin FToolableTimelineBaseTool
	virtual void OnDragStart(const FMouseInputData& InMouseInput) override;
	virtual void OnDrag(const FMouseInputData& InMouseInput) override;
	virtual void OnDragEnd(const bool bInCommit = true) override;
	//~ End FToolableTimelineBaseTool

	void StopDragIfPossible(const bool bInCommit = true);

	int32 DrawFrameMovementIndicator(FMouseDrawInputData& MouseDrawInput
		, const FFrameTime& InStartTick
		, const FFrameTime& InEndExclusiveTick
		, const FLinearColor& InIndicatorColor) const;

	void UpdateKeyTimes(const TSharedRef<FToolableTimeline>& InTimeline, const bool bInNotifyMovieSceneDataChanged = true);

	void ApplyDragResultsIfThrottleOpen(const TSharedRef<FToolableTimeline>& InTimeline, const bool bInForceApply = false);

	bool CalculateKeyBoundsTickSpan(FFrameTime& OutStartTick, FFrameTime& OutEndExclusiveTick) const;

	/** Set when attempting to move a drag handle. */
	mutable TOptional<FToolableTimelineKeyAndMarkDragOperation<FChannelKeyCache>> MouseDownOp;

	FPollGateThrottle UpdateKeyTimeThrottle;
};

} // namespace UE::Sequencer::SimpleView
