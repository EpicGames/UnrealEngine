// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimpleViewRetimeData.h"
#include "SimpleView/SimpleViewUtils.h"
#include "SimpleView/Tools/SimpleViewTimelineAnchor.h"
#include "ToolableTimeline/MouseInputData.h"
#include "ToolableTimeline/ToolableTimeline.h"
#include "ToolableTimeline/ToolableTimeSliderController.h"

using namespace UE::Sequencer::ToolableTimeline;

bool USimpleViewRetimeData::GetFirstAnchor(FSimpleViewTimelineAnchor& OutAnchor) const
{
	const bool bHasAnchors = RetimeAnchors.Num() > 0;
	if (bHasAnchors)
	{
		OutAnchor = RetimeAnchors[0];
	}
	return bHasAnchors;
}

void USimpleViewRetimeData::SetFirstAnchor(const FSimpleViewTimelineAnchor& InAnchor)
{
	if (RetimeAnchors.Num() > 0)
	{
		RetimeAnchors[0] = InAnchor;
	}
}

bool USimpleViewRetimeData::GetLastAnchor(FSimpleViewTimelineAnchor& OutAnchor) const
{
	const bool bHasAnchors = RetimeAnchors.Num() > 0;
	if (bHasAnchors)
	{
		OutAnchor = RetimeAnchors.Last();
	}
	return bHasAnchors;
}

void USimpleViewRetimeData::SetLastAnchor(const FSimpleViewTimelineAnchor& InAnchor)
{
	if (RetimeAnchors.Num() >= 2)
	{
		RetimeAnchors.Last() = InAnchor;
	}
}

const TArray<FSimpleViewTimelineAnchor>& USimpleViewRetimeData::GetAnchors() const
{
	return RetimeAnchors;
}

int32 USimpleViewRetimeData::GetAnchorCount() const
{
	return RetimeAnchors.Num();
}

TArray<FFrameTime> USimpleViewRetimeData::GetSortedAnchorTimes() const
{
	TArray<FFrameTime> AnchorStartTimes;
	AnchorStartTimes.Reserve(RetimeAnchors.Num());

	for (const FSimpleViewTimelineAnchor& Anchor : RetimeAnchors)
	{
		AnchorStartTimes.Add(Anchor.FrameTime);
	}

	Algo::Sort(AnchorStartTimes);

	return AnchorStartTimes;
}

void USimpleViewRetimeData::SetAnchorTimes(const TArray<FFrameTime>& InTimes)
{
	const int32 Count = FMath::Min(RetimeAnchors.Num(), InTimes.Num());
	for (int32 Index = 0; Index < Count; ++Index)
	{
		RetimeAnchors[Index].FrameTime = InTimes[Index];
	}
}

void USimpleViewRetimeData::ResetAnchors()
{
	RetimeAnchors.Reset();
}

bool USimpleViewRetimeData::AddAnchor(const FSimpleViewTimelineAnchor& InAnchor)
{
	const int32 NewIndex = RetimeAnchors.Add(InAnchor);
	return NewIndex != INDEX_NONE;
}

bool USimpleViewRetimeData::RemoveAnchor(const int32 InAnchorIndex)
{
	if (RetimeAnchors.IsValidIndex(InAnchorIndex))
	{
		RetimeAnchors.RemoveAt(InAnchorIndex);
		return true;
	}
	return false;
}

void USimpleViewRetimeData::ClearAnchorHighlights()
{
	for (FSimpleViewTimelineAnchor& Anchor : RetimeAnchors)
	{
		Anchor.bIsAnchorBarHighlighted = false;
		Anchor.bIsCloseButtonHighlighted = false;
	}
}

void USimpleViewRetimeData::UpdateAnchorHighlights(const FMouseInputData& InMouseInput)
{
	const FVector2D PointerPosition = InMouseInput.PointerEvent.GetScreenSpacePosition();

	for (FSimpleViewTimelineAnchor& Anchor : RetimeAnchors)
	{
		FGeometry AnchorBarGeometry;
		FGeometry CloseButtonGeometry;
		Anchor.GetHitGeometry(InMouseInput.Timeline, InMouseInput.Geometry, AnchorBarGeometry, CloseButtonGeometry);

		Anchor.bIsAnchorBarHighlighted = AnchorBarGeometry.IsUnderLocation(PointerPosition);
		Anchor.bIsCloseButtonHighlighted = CloseButtonGeometry.IsUnderLocation(PointerPosition);
	}
}

int32 USimpleViewRetimeData::GetIndexOfAnchorBarUnderPointer(const FMouseInputData& InMouseInput) const
{
	const FVector2D ScreenPosition = InMouseInput.PointerEvent.GetScreenSpacePosition();

	for (int32 AnchorIndex = 0; AnchorIndex < RetimeAnchors.Num(); ++AnchorIndex)
	{
		const FSimpleViewTimelineAnchor& Anchor = RetimeAnchors[AnchorIndex];

		FGeometry AnchorBarGeometry;
		FGeometry CloseButtonGeometry;
		Anchor.GetHitGeometry(InMouseInput.Timeline, InMouseInput.Geometry, AnchorBarGeometry, CloseButtonGeometry);

		if (AnchorBarGeometry.IsUnderLocation(ScreenPosition))
		{
			return AnchorIndex;
		}
	}

	return INDEX_NONE;
}

int32 USimpleViewRetimeData::GetIndexOfAnchorCloseButtonUnderPointer(const FMouseInputData& InMouseInput) const
{
	const FVector2D ScreenPosition = InMouseInput.PointerEvent.GetScreenSpacePosition();

	for (int32 AnchorIndex = 0; AnchorIndex < RetimeAnchors.Num(); ++AnchorIndex)
	{
		const FSimpleViewTimelineAnchor& Anchor = RetimeAnchors[AnchorIndex];

		FGeometry AnchorBarGeometry;
		FGeometry CloseButtonGeometry;
		Anchor.GetHitGeometry(InMouseInput.Timeline, InMouseInput.Geometry, AnchorBarGeometry, CloseButtonGeometry);

		if (CloseButtonGeometry.IsUnderLocation(ScreenPosition))
		{
			return AnchorIndex;
		}
	}

	return INDEX_NONE;
}

void USimpleViewRetimeData::ClearAnchorSelection()
{
	for (FSimpleViewTimelineAnchor& Anchor : RetimeAnchors)
	{
		Anchor.bIsSelected = false;
	}
}

TSet<int32> USimpleViewRetimeData::GetSelectedAnchorIndices() const
{
	TSet<int32> OutAnchorIndices;

	for (int32 AnchorIndex = 0; AnchorIndex < RetimeAnchors.Num(); ++AnchorIndex)
	{
		if (RetimeAnchors[AnchorIndex].bIsSelected)
		{
			OutAnchorIndices.Add(AnchorIndex);
		}
	}

	return OutAnchorIndices;
}

bool USimpleViewRetimeData::IsAnchorSelected(const int32 InAnchorIndex) const
{
	return GetSelectedAnchorIndices().Contains(InAnchorIndex);
}

bool USimpleViewRetimeData::SelectAnchorByIndex(const int32 InAnchorIndex, const bool bInAddToSelection, const bool bInRemoveFromSelection)
{
	if (RetimeAnchors.IsValidIndex(InAnchorIndex))
	{
		RetimeAnchors[InAnchorIndex].bIsSelected = !bInRemoveFromSelection || bInAddToSelection;
		return true;
	}
	return false;
}

bool USimpleViewRetimeData::TrySelectAnchorUnderPointer(const FMouseInputData& InMouseInput)
{
	const int32 HoveredAnchorIndex = GetIndexOfAnchorBarUnderPointer(InMouseInput);
	const bool bAddToSelection = InMouseInput.PointerEvent.IsShiftDown();
	const bool bRemoveFromSelection = InMouseInput.PointerEvent.IsAltDown();
	return SelectAnchorByIndex(HoveredAnchorIndex, bAddToSelection, bRemoveFromSelection);
}

bool USimpleViewRetimeData::IsPointerOnSelectedAnchorBar(const FMouseInputData& InMouseInput) const
{
	const FVector2D ScreenPosition = InMouseInput.PointerEvent.GetScreenSpacePosition();

	for (const FSimpleViewTimelineAnchor& Anchor : RetimeAnchors)
	{
		FGeometry AnchorBarGeometry;
		FGeometry CloseButtonGeometry;
		Anchor.GetHitGeometry(InMouseInput.Timeline, InMouseInput.Geometry, AnchorBarGeometry, CloseButtonGeometry);

		if (Anchor.bIsSelected && AnchorBarGeometry.IsUnderLocation(ScreenPosition))
		{
			return true;
		}
	}

	return false;
}

void USimpleViewRetimeData::GetAnchorInfluences(TArray<double, TInlineAllocator<16>>& OutAnchorInfluences) const
{
	const int32 AnchorCount = RetimeAnchors.Num();
	OutAnchorInfluences.SetNumUninitialized(AnchorCount);
	for (int32 Index = 0; Index < AnchorCount; ++Index)
	{
		OutAnchorInfluences[Index] = RetimeAnchors[Index].bIsSelected ? 1.0 : 0.0;
	}
}

void USimpleViewRetimeData::SortAnchorsByTime(const TSharedRef<FToolableTimeline>& InTimeline)
{
	Algo::SortBy(RetimeAnchors, &FSimpleViewTimelineAnchor::FrameTime); 
}

void USimpleViewRetimeData::MoveSelectedAnchorTimes(const TArray<FFrameTime>& InAnchorStartTimes
	, const FFrameTime& InDeltaTime, const bool bInAllAnchors)
{
	const int32 AnchorCount = RetimeAnchors.Num();
	const int32 StartTimeCount = InAnchorStartTimes.Num();
	if (!ensure(AnchorCount == StartTimeCount))
	{
		return;
	}

	for (int32 AnchorIndex = 0; AnchorIndex < AnchorCount; AnchorIndex++)
	{
		FSimpleViewTimelineAnchor& Anchor = RetimeAnchors[AnchorIndex];
		if (bInAllAnchors || Anchor.bIsSelected)
		{
			// Anchors don't respect snap settings because the individual keys below do, and because of the linear scaling on the keys
			// you might need a non-snapped anchor to get them to jump over as you expect.
			Anchor.FrameTime = InAnchorStartTimes[AnchorIndex] + InDeltaTime;
		}
	}
}

USimpleViewRetimeData::FAnchorPaintContext USimpleViewRetimeData::BuildAnchorPaintContext(const FMouseDrawInputData& MouseDrawInput
	, const int32 InAnchorIndex) const
{
	FAnchorPaintContext OutContext;
	
	if (!RetimeAnchors.IsValidIndex(InAnchorIndex))
	{
		return OutContext;
	}

	const int32 AnchorCount = RetimeAnchors.Num();

	OutContext.PrevAnchor = InAnchorIndex > 0 ? &RetimeAnchors[InAnchorIndex - 1] : nullptr;
	OutContext.NextAnchor = InAnchorIndex < AnchorCount - 1 ? &RetimeAnchors[InAnchorIndex + 1] : nullptr;

	RetimeAnchors[InAnchorIndex].GetPaintGeometry(
		MouseDrawInput.Timeline,
		MouseDrawInput.Geometry,
		OutContext.AnchorBarGeometry,
		OutContext.CloseButtonGeometry
	);

	return OutContext;
}

int32 USimpleViewRetimeData::PaintAnchors(FMouseDrawInputData& MouseDrawInput) const
{
	const int32 AnchorCount = RetimeAnchors.Num();

	for (int32 AnchorIndex = 0; AnchorIndex < AnchorCount; AnchorIndex++)
	{
		const FAnchorPaintContext Context = BuildAnchorPaintContext(MouseDrawInput, AnchorIndex);

		MouseDrawInput.LayerId = RetimeAnchors[AnchorIndex].DrawAnchorBars(Context.PrevAnchor, Context.NextAnchor, MouseDrawInput, Context.AnchorBarGeometry);
		MouseDrawInput.LayerId = RetimeAnchors[AnchorIndex].DrawCloseButton(MouseDrawInput, Context.CloseButtonGeometry);
	}

	return MouseDrawInput.IncrementDrawLayer().LayerId;
}

int32 USimpleViewRetimeData::PaintGradients(FMouseDrawInputData& MouseDrawInput) const
{
	const int32 AnchorCount = RetimeAnchors.Num();

	for (int32 AnchorIndex = 0; AnchorIndex < AnchorCount; AnchorIndex++)
	{
		const FAnchorPaintContext Context = BuildAnchorPaintContext(MouseDrawInput, AnchorIndex);

		MouseDrawInput.LayerId = RetimeAnchors[AnchorIndex].DrawGradients(Context.NextAnchor, MouseDrawInput, Context.AnchorBarGeometry);
	}

	return MouseDrawInput.IncrementDrawLayer().LayerId;
}
