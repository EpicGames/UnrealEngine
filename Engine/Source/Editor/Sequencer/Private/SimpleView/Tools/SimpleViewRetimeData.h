// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerTimeSliderController.h"
#include "ToolableTimeline/ToolableTimeline.h"
#include "SimpleViewRetimeData.generated.h"

struct FSimpleViewTimelineAnchor;

namespace UE::Sequencer::ToolableTimeline
{
	class FToolableTimeline;
	struct FMouseDrawInputData;
	struct FMouseInputData;
}

UCLASS()
class USimpleViewRetimeData : public UObject
{
	GENERATED_BODY()

public:
	struct FAnchorPaintContext
	{
		const FSimpleViewTimelineAnchor* PrevAnchor = nullptr;
		const FSimpleViewTimelineAnchor* NextAnchor = nullptr;

		FGeometry AnchorBarGeometry;
		FGeometry CloseButtonGeometry;
	};

	bool GetFirstAnchor(FSimpleViewTimelineAnchor& OutAnchor) const;
	void SetFirstAnchor(const FSimpleViewTimelineAnchor& InAnchor);

	bool GetLastAnchor(FSimpleViewTimelineAnchor& OutAnchor) const;
	void SetLastAnchor(const FSimpleViewTimelineAnchor& InAnchor);

	const TArray<FSimpleViewTimelineAnchor>& GetAnchors() const;
	int32 GetAnchorCount() const;

	TArray<FFrameTime> GetSortedAnchorTimes() const;
	void SetAnchorTimes(const TArray<FFrameTime>& InTimes);

	/** Resets the list of retime anchors, removing all existing anchor points */
	void ResetAnchors();
	bool AddAnchor(const FSimpleViewTimelineAnchor& InAnchor);
	bool RemoveAnchor(const int32 InAnchorIndex);

	void ClearAnchorHighlights();
	/** Updates anchor highlighting based on mouse input and dragging state. */
	void UpdateAnchorHighlights(const UE::Sequencer::ToolableTimeline::FMouseInputData& InMouseInput);

	int32 GetIndexOfAnchorBarUnderPointer(const UE::Sequencer::ToolableTimeline::FMouseInputData& InMouseInput) const;
	int32 GetIndexOfAnchorCloseButtonUnderPointer(const UE::Sequencer::ToolableTimeline::FMouseInputData& InMouseInput) const;

	void ClearAnchorSelection();
	TSet<int32> GetSelectedAnchorIndices() const;
	bool IsAnchorSelected(const int32 InAnchorIndex) const;
	bool SelectAnchorByIndex(const int32 InAnchorIndex, const bool bInAddToSelection, const bool bInRemoveFromSelection);
	bool TrySelectAnchorUnderPointer(const UE::Sequencer::ToolableTimeline::FMouseInputData& InMouseInput);

	bool IsPointerOnSelectedAnchorBar(const UE::Sequencer::ToolableTimeline::FMouseInputData& InMouseInput) const;

	void GetAnchorInfluences(TArray<double, TInlineAllocator<16>>& OutAnchorInfluences) const;

	void SortAnchorsByTime(const TSharedRef<UE::Sequencer::ToolableTimeline::FToolableTimeline>& InTimeline);

	void MoveSelectedAnchorTimes(const TArray<FFrameTime>& InAnchorStartTimes
		, const FFrameTime& InDeltaTime, const bool bInAllAnchors);

	FAnchorPaintContext BuildAnchorPaintContext(const UE::Sequencer::ToolableTimeline::FMouseDrawInputData& MouseDrawInput, const int32 InAnchorIndex) const;

	int32 PaintAnchors(UE::Sequencer::ToolableTimeline::FMouseDrawInputData& MouseDrawInput) const;
	int32 PaintGradients(UE::Sequencer::ToolableTimeline::FMouseDrawInputData& MouseDrawInput) const;

private:
	/** List of anchor points. Assumes they're in order from lowest input time to greatest. */
	UPROPERTY()
	TArray<FSimpleViewTimelineAnchor> RetimeAnchors;
};
