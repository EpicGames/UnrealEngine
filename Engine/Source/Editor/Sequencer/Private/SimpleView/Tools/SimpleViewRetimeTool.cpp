// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimpleViewRetimeTool.h"
#include "Channels/MovieSceneChannel.h"
#include "EditorFontGlyphs.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "Misc/InputState.h"
#include "MovieSceneSection.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "Sequencer.h"
#include "SequencerCommands.h"
#include "SequencerSettings.h"
#include "SimpleView/SimpleViewCommands.h"
#include "SimpleView/Tools/DragOperations/SimpleViewRetimeDragOperation.h"
#include "SimpleView/Tools/Factories/SimpleViewRetimeToolFactory.h"
#include "SimpleView/Tools/SimpleViewRetimeData.h"
#include "SimpleView/Tools/SimpleViewTimelineAnchor.h"
#include "ToolableTimeline/Drawing/FrameRangeBubbleDrawer.h"
#include "ToolableTimeline/MouseInputData.h"
#include "ToolableTimeline/ToolableTimelineSettings.h"
#include "ToolableTimeline/ToolableTimeline.h"
#include "ToolableTimeline/Tools/ToolableTimelineBaseTool.h"
#include "ToolableTimeline/Widgets/SToolableTimeline.h"

#define LOCTEXT_NAMESPACE "TimelineRetimeTool"

namespace UE::Sequencer::SimpleView
{

using namespace UE::Sequencer::ToolableTimeline;

/**
 * Simple view has its own key selection separate from full view. @See FToolableTimelineKeySelection::SyncKeySelectionToSequencer()
 * Because of that, SequencerHelpers::RemoveDuplicateKeys cannot map these moved key
 * handles back to channel models. Ideally that helper would have a channel-model based
 * overload so Sequencer full and simple views can share the same duplicate-key removal logic.
 * This helper can stay smaller because the Simple View drag cache already carries the
 * channel model for each moved key.
 */
static bool RemoveDuplicateKeysForRetimedKeys(FMultiChannelKeyCache<FSimpleViewRetimeToolChannelKeyCache>& InChannelCache)
{
	TMap<TViewModelPtr<FChannelModel>, TArray<FKeyHandle>> ChannelToMovedKeyHandles;

	InChannelCache.ForEachChannelKey([&ChannelToMovedKeyHandles](FChannelKeyRangeCache<FSimpleViewRetimeToolChannelKeyCache>&
			, const TViewModelPtr<FChannelModel>& InChannelModel
			, const int32
			, const FSimpleViewRetimeToolChannelKeyCache& InKeyCache)
		{
			FMovieSceneChannel* const Channel = InChannelModel.IsValid() ? InChannelModel->GetChannel() : nullptr;
			if (!Channel || Channel->GetIndex(InKeyCache.Handle) == INDEX_NONE)
			{
				return true;
			}

			ChannelToMovedKeyHandles.FindOrAdd(InChannelModel).AddUnique(InKeyCache.Handle);
			return true;
		});

	bool bRemovedDuplicateKeys = false;

	for (const TPair<TViewModelPtr<FChannelModel>, TArray<FKeyHandle>>& ChannelToMovedKeys : ChannelToMovedKeyHandles)
	{
		if (!ChannelToMovedKeys.Key.IsValid() || ChannelToMovedKeys.Value.IsEmpty())
		{
			continue;
		}

		UMovieSceneSection* const Section = ChannelToMovedKeys.Key->GetSection();
		FMovieSceneChannel* const Channel = ChannelToMovedKeys.Key->GetChannel();
		if (!Section || Section->IsReadOnly() || !Channel)
		{
			continue;
		}

		TArray<FFrameNumber> CheckedKeyTimes;
		TArray<FKeyHandle> HandlesToRemove;

		for (const FKeyHandle MovedHandle : ChannelToMovedKeys.Value)
		{
			if (Channel->GetIndex(MovedHandle) == INDEX_NONE)
			{
				continue;
			}

			FFrameNumber KeyTime;
			Channel->GetKeyTime(MovedHandle, KeyTime);
			if (CheckedKeyTimes.Contains(KeyTime))
			{
				continue;
			}

			CheckedKeyTimes.Add(KeyTime);

			TArray<FFrameNumber> OutKeyTimes;
			TArray<FKeyHandle> OutKeyHandles;
			Channel->GetKeys(TRange<FFrameNumber>(KeyTime, KeyTime), &OutKeyTimes, &OutKeyHandles);
			if (OutKeyHandles.Num() <= 1)
			{
				continue;
			}

			FKeyHandle KeyHandleToKeep = OutKeyHandles[0];

			for (const FKeyHandle CandidateHandle : ChannelToMovedKeys.Value)
			{
				if (OutKeyHandles.Contains(CandidateHandle))
				{
					KeyHandleToKeep = CandidateHandle;
					break;
				}
			}

			for (const FKeyHandle Handle : OutKeyHandles)
			{
				if (Handle != KeyHandleToKeep)
				{
					HandlesToRemove.AddUnique(Handle);
				}
			}
		}

		if (HandlesToRemove.IsEmpty())
		{
			continue;
		}

		Section->Modify();
		Channel->DeleteKeys(HandlesToRemove);

		bRemovedDuplicateKeys = true;
	}

	return bRemovedDuplicateKeys;
}

FSimpleViewRetimeTool::FSimpleViewRetimeTool(const TSharedRef<FToolableTimeline>& InTimeline)
	: FToolableTimelineBaseTool(InTimeline)
	, Lattice(InTimeline)
{
	FadeCurve = FadeSequence.AddCurve(/*StartTimeSeconds=*/0.0f, /*DurationSeconds=*/0.15f, ECurveEaseFunction::QuadInOut);
}

FSimpleViewRetimeTool::~FSimpleViewRetimeTool()
{
	if (RetimeData)
	{
		RetimeData->RemoveFromRoot();
	}
	RetimeData = nullptr;
}

FName FSimpleViewRetimeTool::GetIdentifier() const
{
	return FSimpleViewRetimeToolFactory::StaticToolId;
}

FText FSimpleViewRetimeTool::GetLabel() const
{
	return LOCTEXT("RetimeLabel", "Retime Keys");
}

FText FSimpleViewRetimeTool::GetDescription() const
{
	return LOCTEXT("RetimeDescription", "Move and scale key ranges");
}

void FSimpleViewRetimeTool::BindCommands(const TSharedRef<FUICommandList>& InCommandBindings)
{
	const FSimpleViewCommands& SimpleViewEditorCommands = FSimpleViewCommands::Get();

	InCommandBindings->MapAction(SimpleViewEditorCommands.Tool_DeleteAnchor
		, FExecuteAction::CreateSP(this, &FSimpleViewRetimeTool::RemoveSelectedAnchors));
}

void FSimpleViewRetimeTool::UnbindCommands(const TSharedRef<FUICommandList>& InCommandBindings)
{
	const FSimpleViewCommands& SimpleViewEditorCommands = FSimpleViewCommands::Get();

	InCommandBindings->UnmapAction(SimpleViewEditorCommands.Tool_DeleteAnchor);
}

void FSimpleViewRetimeTool::NotifyToolActivated(const FMouseInputData& InMouseInput, const FFrameTime& InInputTickTime)
{
	if (!RetimeData)
	{
		RetimeData = NewObject<USimpleViewRetimeData>();
		RetimeData->SetFlags(RF_Transactional);
		RetimeData->AddToRoot();
	}

	// If we already have a mouse-down drag op from the original press, use its initial tick time.
	// That is the true start location for the initial retime range.
	const FFrameTime InitialRangeStartTime = MouseDownOp.IsSet()
		? MouseDownOp->GetInitialTickTime()
		: InInputTickTime;

	RetimeData->ResetAnchors();
	RetimeData->AddAnchor(InitialRangeStartTime);
	RetimeData->AddAnchor(InitialRangeStartTime);

	bPendingInitialRangeDrag = true;

	StopDragIfPossible();
}

void FSimpleViewRetimeTool::NotifyToolDeactivated()
{
	StopDragIfPossible(/*bInCommit*/false);
}

bool FSimpleViewRetimeTool::WantsPriorityOverControllerHit(const FMouseInputData& InMouseInput) const
{
	return ActiveDragTarget.IsSet() && ActiveDragTarget->IsToolRangeHit();
}

bool FSimpleViewRetimeTool::ShouldShowDragRangeIndicator() const
{
	if (!MouseDownOp.IsSet()
		|| !MouseDownOp->HasDragOp()
		|| MouseDownOp->GetAccumulatedLocalDelta().IsNearlyZero())
	{
		return false;
	}

	switch (MouseDownOp->GetCurrentDragMode())
	{
	case FSimpleViewRetimeDragOperation::EDragMode::MoveAnchor:
	case FSimpleViewRetimeDragOperation::EDragMode::LatticeScaleLeft:
	case FSimpleViewRetimeDragOperation::EDragMode::LatticeScaleRight:
		return true;

	case FSimpleViewRetimeDragOperation::EDragMode::MoveAllAnchors:
		return FSlateApplication::Get().GetModifierKeys().IsControlDown();

	case FSimpleViewRetimeDragOperation::EDragMode::LatticeMoveAnchors:
	case FSimpleViewRetimeDragOperation::EDragMode::None:
	default:
		return false;
	}
}

TRange<FFrameTime> FSimpleViewRetimeTool::GetDragIndicatorTickRange(const FMouseInputData& InMouseInput) const
{
	(void)InMouseInput;

	if (!ShouldShowDragRangeIndicator())
	{
		return TRange<FFrameTime>();
	}

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = GetTimelineChecked()->GetTimeSliderController();
	const FFrameTime StartTickTime = MouseDownOp->GetInitialDisplayFrameTickRange().GetLowerBoundValue();
	const FFrameTime EndTickTime = MouseDownOp->ComputeDraggedTickTime(
		TimeSliderController->GetTickResolution(),
		TimeSliderController->GetDisplayRate(),
		/*bInSnapToDisplayFrame=*/false);

	return TRange<FFrameTime>(
		TRangeBound<FFrameTime>::Inclusive(StartTickTime),
		TRangeBound<FFrameTime>::Inclusive(EndTickTime)
	);
}

TRange<FFrameNumber> FSimpleViewRetimeTool::GetToolRange() const
{
	if (!RetimeData || RetimeData->GetAnchorCount() < 2)
	{
		return TRange<FFrameNumber>::Empty();
	}

	FSimpleViewTimelineAnchor FirstAnchor;
	const bool bHasFirstAnchor = RetimeData->GetFirstAnchor(FirstAnchor);

	FSimpleViewTimelineAnchor LastAnchor;
	const bool bHasLastAnchor = RetimeData->GetLastAnchor(LastAnchor);

	if (!bHasFirstAnchor)
	{
		return TRange<FFrameNumber>::Empty();
	}

	const FFrameTime StartTickTime = FirstAnchor.FrameTime;
	const FFrameTime EndTickTime = bHasLastAnchor ? LastAnchor.FrameTime : StartTickTime;

	FFrameNumber LowerInclusive = StartTickTime.FloorToFrame();
	FFrameNumber UpperExclusive = EndTickTime.CeilToFrame();
	if (UpperExclusive <= LowerInclusive)
	{
		UpperExclusive = LowerInclusive + 1;
	}

	return TRange<FFrameNumber>(
		TRangeBound<FFrameNumber>::Inclusive(LowerInclusive),
		TRangeBound<FFrameNumber>::Exclusive(UpperExclusive)
	);
}

int32 FSimpleViewRetimeTool::Paint(FMouseDrawInputData& MouseDrawInput) const
{
	if (!RetimeData)
	{
		return MouseDrawInput.LayerId;
	}

	const TRange<FFrameNumber> ToolRange = GetToolRange();
	if (!IsValidToolRange(ToolRange))
	{
		return MouseDrawInput.LayerId;
	}

	const TSharedRef<FToolableTimeline> Timeline = GetTimelineChecked();

	MouseDrawInput.LayerId = RetimeData->PaintGradients(MouseDrawInput);

	if (RetimeData->GetAnchorCount() > 1)
	{
		MouseDrawInput.LayerId = Lattice.Paint(MouseDrawInput.IncrementDrawLayer(), ToolRange);
	}

	MouseDrawInput.LayerId = RetimeData->PaintAnchors(MouseDrawInput.IncrementDrawLayer(10));

	const float BubbleAlpha = FadeCurve.GetLerp();

	const bool bShowFrameRangeBubbles = BubbleAlpha > KINDA_SMALL_NUMBER
		|| (HoverTarget.IsSet() && HoverTarget->IsToolRangeHit())
		|| Lattice.IsAnyButtonHovered()
		|| Lattice.IsAnyButtonPressed()
		|| IsDragging();
	if (bShowFrameRangeBubbles)
	{
		const UToolableTimelineSettings& TimelineSettings = Timeline->GetTimelineSettings();

		FDrawFrameNumberBubbleArgs DrawArgs;
		DrawArgs.LabelAlignment = TimelineSettings.Settings.LabelVerticalAlignment;
		DrawArgs.BubbleFontSize = TimelineSettings.Settings.FrameNumberBubbleFontSize;
		DrawArgs.BubblePadding = TimelineSettings.Settings.FrameNumberBubblePadding;
		DrawArgs.BubbleColor = TimelineSettings.Settings.FrameNumberBubbleColor;
		DrawArgs.BubbleTextColor = TimelineSettings.Settings.FrameNumberBubbleTextColor;
		DrawArgs.bAlignToWholeFrame = false;
		DrawArgs.bPreventLeadingZeros = true;
		DrawArgs.bTryKeepOnScreen = true;
		DrawArgs.AlphaOpacity = BubbleAlpha;

		const FFrameNumber StartFrame = ToolRange.GetLowerBoundValue();
		const FFrameNumber EndFrameExclusive = ToolRange.GetUpperBoundValue();
		const FFrameNumber EndFrameInclusive = EndFrameExclusive - 1;

		const TRange<FFrameNumber> StartToolRange(StartFrame, StartFrame);
		const TRange<FFrameNumber> EndToolRange(EndFrameInclusive, EndFrameInclusive);

		bool bShowStartBubble = false;
		bool bShowEndBubble = false;
		bool bShowCenterBubble = false;
		GetVisibleFrameBubbles(bShowStartBubble, bShowEndBubble, bShowCenterBubble);

		FFrameRangeBubbleDrawer FrameBubbleDrawer(DrawArgs);

		if (bShowStartBubble)
		{
			MouseDrawInput.LayerId = FrameBubbleDrawer.Draw(StartToolRange, MouseDrawInput.IncrementDrawLayer());
		}

		if (bShowEndBubble)
		{
			MouseDrawInput.LayerId = FrameBubbleDrawer.Draw(EndToolRange, MouseDrawInput);
		}

		if (bShowCenterBubble)
		{
			// Draw on top of start and end frame bubbles.
			MouseDrawInput.LayerId = FrameBubbleDrawer.Draw(ToolRange, MouseDrawInput.IncrementDrawLayer());
		}

		return MouseDrawInput.LayerId;
	}

	return MouseDrawInput.LayerId;
}

bool FSimpleViewRetimeTool::WantsInputPrepass(const FMouseInputData& InMouseInput) const
{
	return InMouseInput.PointerEvent.IsControlDown()
		&& (InMouseInput.PointerEvent.GetEffectingButton() == EKeys::LeftMouseButton);
}

bool FSimpleViewRetimeTool::IsDragging() const
{
	return HasDelayedDrag() && MouseDownOp->IsDragging();
}

bool FSimpleViewRetimeTool::HasHoverHit() const
{
	return HoverTarget.IsSet() && HoverTarget->IsValid();
}

bool FSimpleViewRetimeTool::CanPersistThroughChannelRecache() const
{
	return !MouseDownOp.IsSet() && !ActiveTransaction.IsValid();
}

void FSimpleViewRetimeTool::OnChannelModelsRecached()
{
	Lattice.ResetButtonState();

	HoverTarget.Reset();
	ActiveDragTarget.Reset();

	SetFrameBubbleVisibility(false);

	if (RetimeData)
	{
		RetimeData->ClearAnchorHighlights();
	}

	UpdateSelectionFromToolRange();
}

FReply FSimpleViewRetimeTool::OnMouseButtonDoubleClick(SWidget& OwnerWidget
	, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (!RetimeData)
	{
		return FReply::Unhandled();
	}

	const FMouseInputData MouseInput(GetTimelineChecked(), InGeometry, InPointerEvent);
	const FKey EffectingButton = InPointerEvent.GetEffectingButton();

	if (EffectingButton == EKeys::LeftMouseButton)
	{
		if (IsPendingInitialRangeCreation())
		{
			const TSharedRef<FToolableTimeSliderController> TimeSliderController = GetTimelineChecked()->GetTimeSliderController();
			const TRange<double> ViewRange = TimeSliderController->GetViewRange();
			const FFrameRate TickResolution = TimeSliderController->GetTickResolution();

			RetimeData->ResetAnchors();
			RetimeData->AddAnchor(TickResolution.AsFrameTime(ViewRange.GetLowerBoundValue()));
			RetimeData->AddAnchor(TickResolution.AsFrameTime(ViewRange.GetUpperBoundValue()));

			bPendingInitialRangeDrag = false;

			UpdateSelectionFromToolRange();
			SyncLatticeState();
			SetHoveringBasedOnDragging(MouseInput);

			return FReply::Handled();
		}

		if (AddAnchor(MouseInput))
		{
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply FSimpleViewRetimeTool::OnMouseButtonDown(SWidget& OwnerWidget
	, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	bPassThroughScrubActive = false;

	const bool bLeftMouseInteraction = InPointerEvent.GetEffectingButton() == EKeys::LeftMouseButton
		|| InPointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
	if (!bLeftMouseInteraction)
	{
		return FReply::Unhandled();
	}

	const FMouseInputData MouseInput(GetTimelineChecked(), InGeometry, InPointerEvent);

	// Early out if they clicked the close button on an anchor
	/*const int32 HoveredAnchorIndex = RetimeData
		? RetimeData->GetIndexOfAnchorCloseButtonUnderPointer(MouseInput)
		: INDEX_NONE;
	if (HoveredAnchorIndex != INDEX_NONE && RemoveAnchors({ HoveredAnchorIndex }))
	{
		return FReply::Handled();
	}*/

	if (IsPendingInitialRangeCreation())
	{
		return BeginInitialRangeCreation(MouseInput);
	}

	const FResolvedDragTarget NewDragTarget = ResolveDragTarget(MouseInput);

	bPassThroughScrubActive = ShouldPassThroughToScrub(MouseInput, NewDragTarget);
	if (bPassThroughScrubActive)
	{
		// Only close if the scrub STARTED outside the tool range.
		// If the scrub started inside the range, allow it to continue outside.
		if (!HitTestRange(MouseInput, GetToolRange(), /*bInTestVertical=*/false))
		{
			RequestClose();
		}

		return FReply::Unhandled();
	}

	if (ShouldStartNewRangeCreation(MouseInput, NewDragTarget))
	{
		return RestartRangeCreation(MouseInput);
	}

	if (IsDraggingOutsideToolRange(MouseInput, NewDragTarget))
	{
		RequestClose();
		return FReply::Unhandled();
	}

	if (TryBeginOrRestartDrag(MouseInput, NewDragTarget))
	{
		SyncLatticeState();

		return FReply::Handled()
			.CaptureMouse(OwnerWidget.AsShared())
			.UseHighPrecisionMouseMovement(OwnerWidget.AsShared())
			.PreventThrottling();
	}

	if (CanMouseInputTriggerClose(MouseInput))
	{
		RequestClose();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply FSimpleViewRetimeTool::OnMouseMove(SWidget& OwnerWidget
	, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	const FMouseInputData MouseInput(GetTimelineChecked(), InGeometry, InPointerEvent);

	HoverTarget = ResolveDragTarget(MouseInput);

	if (bPassThroughScrubActive
		&& !MouseDownOp.IsSet()
		&& InPointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		// Close behavior is decided on mouse-down only.
		// If scrub started inside the tool range, do not close when moving outside.
		return FReply::Unhandled();
	}

	SyncLatticeState();

	const bool bHoveringLatticeButton = Lattice.IsAnyButtonHovered() || Lattice.IsAnyButtonPressed();

	if (RetimeData)
	{
		const bool bClearHoveredAnchors = IsDragging() || bHoveringLatticeButton;
		if (bClearHoveredAnchors)
		{
			RetimeData->ClearAnchorHighlights();
		}
		else
		{
			RetimeData->UpdateAnchorHighlights(MouseInput);
		}
	}

	// Important: Do this after lattice hover/press state is up to date
	SetHoveringBasedOnDragging(MouseInput);

	if (CanBeginRangeCreation())
	{
		MouseDownOp->UpdateDrag(MouseInput);

		if (!MouseDownOp->IsDragging())
		{
			MouseDownOp->ForceDragStart();
		}

		UpdateInitialRangeCreationWhileDragging(MouseInput);

		return FReply::Handled()
			.CaptureMouse(OwnerWidget.AsShared())
			.UseHighPrecisionMouseMovement(OwnerWidget.AsShared())
			.PreventThrottling();
	}

	// Normal drag op handling
	if (MouseDownOp.IsSet())
	{
		FReply Reply = FReply::Handled()
			.CaptureMouse(OwnerWidget.AsShared())
			.UseHighPrecisionMouseMovement(OwnerWidget.AsShared())
			.PreventThrottling();

		if (MouseDownOp->IsDragging())
		{
			MouseDownOp->UpdateDrag(MouseInput);
			OnDrag(MouseInput);
		}
		else if (!MouseDownOp->IsCurrentDragMode(FSimpleViewRetimeDragOperation::EDragMode::None)
			&& MouseDownOp->AttemptDragStart(MouseInput))
		{
			OnDragStart(MouseInput);
			MouseDownOp->UpdateDrag(MouseInput);
			OnDrag(MouseInput);
		}

		return Reply;
	}

	return FReply::Unhandled();
}

FReply FSimpleViewRetimeTool::OnMouseButtonUp(SWidget& OwnerWidget
	, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	bPassThroughScrubActive = false;

	const FMouseInputData MouseInput(GetTimelineChecked(), InGeometry, InPointerEvent);

	if (!MouseDownOp.IsSet())
	{
		SyncLatticeState();
		SetHoveringBasedOnDragging(MouseInput);

		return FReply::Unhandled();
	}

	const TArray<FFrameTime>& AnchorStartTimes = MouseDownOp->GetAnchorStartTimes();

	const bool bWasInitialRangeMode = MouseDownOp->IsCurrentDragMode(FSimpleViewRetimeDragOperation::EDragMode::None)
		&& AnchorStartTimes.Num() >= 2;

	const FFrameTime Start0 = bWasInitialRangeMode ? AnchorStartTimes[0] : FFrameTime();
	const FFrameTime Start1 = bWasInitialRangeMode ? AnchorStartTimes[1] : FFrameTime();

	const FVector2D AccumulatedDelta = MouseDownOp->GetAccumulatedLocalDelta();
	const bool bDidActuallyDrag = !FMath::IsNearlyZero(AccumulatedDelta.X) || !FMath::IsNearlyZero(AccumulatedDelta.Y);
	if (bDidActuallyDrag)
	{
		OnDragEnd(/*bInCommit=*/true);
	}
	else
	{
		StopDragIfPossible();
	}

	bool bRangeChanged = false;

	if (bWasInitialRangeMode && RetimeData && RetimeData->GetAnchorCount() >= 2)
	{
		const TArray<FSimpleViewTimelineAnchor>& Anchors = RetimeData->GetAnchors();
		bRangeChanged = Anchors[0].FrameTime != Start0 || Anchors[1].FrameTime != Start1;
	}

	if (bWasInitialRangeMode && bRangeChanged)
	{
		bPendingInitialRangeDrag = false;
	}

	SyncLatticeState();
	SetHoveringBasedOnDragging(MouseInput);

	return FReply::Handled().ReleaseMouseCapture();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FSimpleViewRetimeTool::OnMouseLeave(SWidget& OwnerWidget, const FPointerEvent& InPointerEvent)
{
	HoverTarget.Reset();

	// Do not clear visual hover/pressed state while dragging or while a mouse down operation exists.
	// We still want the frame bubbles and lattice pressed state to remain stable outside the widget.
	if (MouseDownOp.IsSet() || IsDragging() || Lattice.IsAnyButtonPressed())
	{
		return;
	}

	SetFrameBubbleVisibility(false);

	if (RetimeData && RetimeData->GetAnchorCount() > 1)
	{
		RetimeData->ClearAnchorHighlights();
	}

	Lattice.ResetButtonState();
}

FReply FSimpleViewRetimeTool::OnKeyDown(SWidget& OwnerWidget
	, const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		StopDragIfPossible(/*bCommit=*/false);
		return FReply::Handled();
	}

	bScrubKeyDown = CurveEditor::InputState::IsCommandPressed(FSequencerCommands::Get().ScrubTimeViewport, InKeyEvent);

	return FReply::Unhandled();
}

FReply FSimpleViewRetimeTool::OnKeyUp(SWidget& OwnerWidget
	, const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	bScrubKeyDown = false;

	return FReply::Unhandled();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FCursorReply FSimpleViewRetimeTool::OnCursorQuery(const SWidget& OwnerWidget
	, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) const
{
	const FMouseInputData MouseInput(GetTimelineChecked(), InGeometry, InPointerEvent);

	if (ActiveDragTarget.IsSet())
	{
		switch (ActiveDragTarget->HitType)
		{
		default:
		case EResolvedMouseHit::None:
			return FCursorReply::Cursor(EMouseCursor::Default);

		case EResolvedMouseHit::ToolRange:
			return WantsInputPrepass(MouseInput)
				? FCursorReply::Cursor(EMouseCursor::CardinalCross)
				: FCursorReply::Cursor(EMouseCursor::Default);

		case EResolvedMouseHit::AnchorBar:
		case EResolvedMouseHit::SelectedAnchorBar:
			return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);

		case EResolvedMouseHit::LatticeCenterMove:
			return FCursorReply::Cursor(EMouseCursor::CardinalCross);

		case EResolvedMouseHit::LatticeLeftScale:
		case EResolvedMouseHit::LatticeRightScale:
			return FCursorReply::Cursor(EMouseCursor::Default);
		}
	}

	return FCursorReply::Unhandled();
}

void FSimpleViewRetimeTool::OnFocusLost(SWidget& OwnerWidget, const FFocusEvent& InFocusEvent)
{
	HoverTarget.Reset();

	// We need to end our drag if we lose Window focus to close the transaction, otherwise alt-tabbing while dragging
	// can cause a transaction to get stuck open.
	StopDragIfPossible(/*bInCommit*/false);

	Lattice.ResetButtonState();
}

void FSimpleViewRetimeTool::OnDragStart(const FMouseInputData& InMouseInput)
{
	if (!MouseDownOp.IsSet())
	{
		return;
	}

	ActiveTransaction = MakeUnique<FScopedTransaction>(TEXT("RetimeKeys")
		, LOCTEXT("RetimeKeysTransaction", "Retime Key(s)"), nullptr);

	// Store retime data so the handles reset to the correct location upon undo/redo
	RetimeData->Modify();
}

void FSimpleViewRetimeTool::OnDrag(const FMouseInputData& InMouseInput)
{
	if (!ensure(MouseDownOp.IsSet()))
	{
		return;
	}

	const TArray<FFrameTime>& AnchorStartTimes = MouseDownOp->GetAnchorStartTimes();
	const int32 AnchorCount = AnchorStartTimes.Num();
	if (AnchorCount <= 0 || AnchorCount != RetimeData->GetAnchorCount())
	{
		return;
	}

	const TSharedRef<FToolableTimeline> Timeline = GetTimelineChecked();
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = Timeline->GetTimeSliderController();
	const bool bSnapEnabled = TimeSliderController->IsSnapEnabled();

	switch (MouseDownOp->GetCurrentDragMode())
	{
	case FSimpleViewRetimeDragOperation::EDragMode::None:
		break;

	case FSimpleViewRetimeDragOperation::EDragMode::MoveAnchor:
		// Anchors are sorted once on mouse up to avoid unnecessary operations since it has no effect during drag
		OnDrag_MoveAnchors(InMouseInput, /*bInAllAnchors=*/false);
		if (!InMouseInput.PointerEvent.GetModifierKeys().IsShiftDown())
		{
			OnDrag_RetimeKeys(InMouseInput, /*bInAllKeys=*/false, bSnapEnabled);
		}
		break;

	case FSimpleViewRetimeDragOperation::EDragMode::MoveAllAnchors:
		OnDrag_MoveAnchors(InMouseInput, /*bInAllAnchors=*/true);
		if (!InMouseInput.PointerEvent.GetModifierKeys().IsShiftDown())
		{
			OnDrag_RetimeKeys(InMouseInput, /*bInAllKeys=*/true, bSnapEnabled);
		}
		break;

	case FSimpleViewRetimeDragOperation::EDragMode::LatticeMoveAnchors:
		OnDrag_LatticeMoveAnchors(InMouseInput, bSnapEnabled);
		break;

	case FSimpleViewRetimeDragOperation::EDragMode::LatticeScaleLeft:
		OnDrag_LatticeScaleLeft(InMouseInput, bSnapEnabled);
		break;

	case FSimpleViewRetimeDragOperation::EDragMode::LatticeScaleRight:
		OnDrag_LatticeScaleRight(InMouseInput, bSnapEnabled);
		break;

	default:
		checkNoEntry();
		break;
	}

	ApplyDragResultsIfThrottleOpen(Timeline);
}

void FSimpleViewRetimeTool::OnDrag_RetimeKeys(const FMouseInputData& InMouseInput
	, const bool bInIgnoreAnchorInfluences
	, const bool bInSnapToFrame)
{
	if (!HasDraggedCacheKeys())
	{
		return;
	}

	const TArray<FFrameTime>& AnchorStartTimes = MouseDownOp->GetAnchorStartTimes();
	const FFrameTime DeltaTickFrameTime = MouseDownOp->DragPixelsToTickFrameTime(InMouseInput);

	TArray<double, TInlineAllocator<16>> AnchorInfluences;
	RetimeData->GetAnchorInfluences(AnchorInfluences);

	MouseDownOp->RecomputeForMove(InMouseInput.Timeline
		, AnchorStartTimes, AnchorInfluences, DeltaTickFrameTime, bInIgnoreAnchorInfluences, bInSnapToFrame);
}

void FSimpleViewRetimeTool::OnDrag_MoveAnchors(const FMouseInputData& InMouseInput, const bool bInAllAnchors)
{
	if (!HasPreDragData())
	{
		return;
	}

	const TArray<FFrameTime>& AnchorStartTimes = MouseDownOp->GetAnchorStartTimes();
	const FFrameTime DeltaTickFrameTime = MouseDownOp->DragPixelsToTickFrameTime(InMouseInput);

	RetimeData->MoveSelectedAnchorTimes(AnchorStartTimes, DeltaTickFrameTime, bInAllAnchors);

	// Anchors are sorted during drag because key positions depend on their current order
	RetimeData->SortAnchorsByTime(InMouseInput.Timeline);
}

void FSimpleViewRetimeTool::OnDrag_LatticeScaleLeft(const FMouseInputData& InMouseInput, const bool bInSnapToFrame)
{
	if (!HasPreDragData())
	{
		return;
	}

	const TArray<FFrameTime>& AnchorStartTimes = MouseDownOp->GetAnchorStartTimes();
	const FFrameTime DeltaTickFrameTime = MouseDownOp->DragPixelsToTickFrameTime(InMouseInput);

	TArray<FFrameTime> NewAnchorTimes;
	if (!BuildScaledAnchorTimes(InMouseInput.Timeline, AnchorStartTimes, /*bLockRightSide=*/true, DeltaTickFrameTime, NewAnchorTimes))
	{
		return;
	}

	ApplyAnchorTimes(InMouseInput.Timeline, NewAnchorTimes);
	OnDrag_RetimeKeysFromAnchorTimes(InMouseInput, AnchorStartTimes, NewAnchorTimes, bInSnapToFrame);
}

void FSimpleViewRetimeTool::OnDrag_LatticeScaleRight(const FMouseInputData& InMouseInput, const bool bInSnapToFrame)
{
	if (!HasPreDragData())
	{
		return;
	}

	const TArray<FFrameTime>& AnchorStartTimes = MouseDownOp->GetAnchorStartTimes();
	const FFrameTime DeltaTickFrameTime = MouseDownOp->DragPixelsToTickFrameTime(InMouseInput);

	TArray<FFrameTime> NewAnchorTimes;
	if (!BuildScaledAnchorTimes(InMouseInput.Timeline, AnchorStartTimes, /*bLockRightSide=*/false, DeltaTickFrameTime, NewAnchorTimes))
	{
		return;
	}

	ApplyAnchorTimes(InMouseInput.Timeline, NewAnchorTimes);
	OnDrag_RetimeKeysFromAnchorTimes(InMouseInput, AnchorStartTimes, NewAnchorTimes, bInSnapToFrame);
}

void FSimpleViewRetimeTool::OnDrag_LatticeMoveAnchors(const FMouseInputData& InMouseInput, const bool bInSnapToFrame)
{
	if (!HasPreDragData())
	{
		return;
	}

	const TArray<FFrameTime>& AnchorStartTimes = MouseDownOp->GetAnchorStartTimes();
	const FFrameTime DeltaTickFrameTime = MouseDownOp->DragPixelsToTickFrameTime(InMouseInput);

	TArray<FFrameTime> NewAnchorTimes;
	if (!BuildMovedAnchorTimes(AnchorStartTimes, DeltaTickFrameTime, NewAnchorTimes))
	{
		return;
	}

	ApplyAnchorTimes(InMouseInput.Timeline, NewAnchorTimes);
	OnDrag_RetimeKeys(InMouseInput, /*bInIgnoreAnchorInfluences=*/true, bInSnapToFrame);
}

void FSimpleViewRetimeTool::OnDrag_RetimeKeysFromAnchorTimes(const FMouseInputData& InMouseInput
	, const TArray<FFrameTime>& InOriginalAnchorTimes
	, const TArray<FFrameTime>& InNewAnchorTimes
	, const bool bInSnapToFrame)
{
	if (!HasPreDragData())
	{
		return;
	}

	const int32 OriginalAnchorCount = InOriginalAnchorTimes.Num();
	const int32 NewAnchorCount = InNewAnchorTimes.Num();

	if (!ensure(OriginalAnchorCount == NewAnchorCount))
	{
		return;
	}

	if (!ensure(OriginalAnchorCount >= 2))
	{
		return;
	}

	if (!ensure(RetimeData->GetAnchorCount() == NewAnchorCount))
	{
		return;
	}

	MouseDownOp->RecomputeFromAnchorTimes(InMouseInput.Timeline, InNewAnchorTimes, bInSnapToFrame);
}

void FSimpleViewRetimeTool::OnDragEnd(const bool bInCommit)
{
	if (MouseDownOp.IsSet())
	{
		const TSharedRef<FToolableTimeline> Timeline = GetTimelineChecked();

		// Because we are throttling and may have missed an update, make sure the key times get updated
		if (bInCommit)
		{
			// Apply the final key times without broadcasting so duplicate-key cleanup and
			// the commit refresh produce one TrackValueChanged notification. A non-hotfix
			// cleanup should move this into a shared apply/remove path so callers do not
			// have to manually coordinate this notification order.
			UpdateKeyTimeThrottle.NotifyActivity();
			UpdateKeyTimes(Timeline, /*bInNotifyMovieSceneDataChanged=*/false);

			RemoveDuplicateKeysForRetimedKeys(MouseDownOp->GetChannelCache());

			if (const TSharedPtr<FSequencer> Sequencer = Timeline->GetSequencer())
			{
				Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
			}
		}
		else
		{
			RestoreInitialDragState(Timeline);
		}

		UpdateSelectionFromToolRange();
	}

	ResetDragOperation();
}

void FSimpleViewRetimeTool::StopDragIfPossible(const bool bInCommit)
{
	if (!MouseDownOp.IsSet() || !MouseDownOp->HasDragOp())
	{
		return;
	}

	const FVector2D AccumulatedDelta = MouseDownOp->GetAccumulatedLocalDelta();
	const bool bDidActuallyDrag = !FMath::IsNearlyZero(AccumulatedDelta.X) || !FMath::IsNearlyZero(AccumulatedDelta.Y);
	if (MouseDownOp->IsDragging() || bDidActuallyDrag)
	{
		OnDragEnd(bInCommit);
		return;
	}

	ResetDragOperation();
}

bool FSimpleViewRetimeTool::HitTestRange(const FMouseInputData& InMouseInput
	, const TRange<FFrameNumber>& InRange, const bool bInTestVertical) const
{
	if (!IsValidToolRange(InRange))
	{
		return false;
	}

	static constexpr double DragToleranceSlateUnits = 0.0;//2.0;
	static constexpr double MouseTolerance = 0.0;//2.0;
	constexpr double TotalTolerance = DragToleranceSlateUnits + MouseTolerance;

	const FVector2D LocalGeometrySize = InMouseInput.Geometry.GetLocalSize();
	const FVector2D PointerPosition = InMouseInput.PointerEvent.GetScreenSpacePosition();
	const FVector2D HitTestPixel = InMouseInput.Geometry.AbsoluteToLocal(PointerPosition);

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InMouseInput.Timeline->GetTimeSliderController();
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();

	auto HitTestLowerBounds = [&InMouseInput, &InRange, &HitTestPixel, TotalTolerance, &TickResolution]() -> bool
	{
		const double RangeStartSeconds = TickResolution.AsSeconds(InRange.GetLowerBoundValue());
		const float RangeStartPixel = InMouseInput.RangeToScreen.InputToLocalX(RangeStartSeconds);
		return HitTestPixel.X >= (RangeStartPixel - TotalTolerance);
	};

	auto HitTestUpperBounds = [&InMouseInput, &InRange, &HitTestPixel, TotalTolerance, &TickResolution]() -> bool
	{
		const double RangeEndSeconds = TickResolution.AsSeconds(InRange.GetUpperBoundValue());
		const float RangeEndPixel = InMouseInput.RangeToScreen.InputToLocalX(RangeEndSeconds);
		return HitTestPixel.X <= (RangeEndPixel + TotalTolerance);
	};

	const bool bPassLowerBoundsX = InRange.HasLowerBound() ? HitTestLowerBounds() : true;
	const bool bPassUpperBoundsX = InRange.HasUpperBound() ? HitTestUpperBounds() : true;

	if (!bInTestVertical)
	{
		return bPassLowerBoundsX && bPassUpperBoundsX;
	}

	const bool bPassLowerBoundsY = HitTestPixel.Y >= 0.f;
	const bool bPassUpperBoundsY = HitTestPixel.Y <= LocalGeometrySize.Y;

	return bPassLowerBoundsX && bPassUpperBoundsX
		&& bPassLowerBoundsY && bPassUpperBoundsY;
}

void FSimpleViewRetimeTool::UpdateKeyTimes(const TSharedRef<FToolableTimeline>& InTimeline, const bool bInNotifyMovieSceneDataChanged)
{
	if (!HasDraggedCacheKeys())
	{
		return;
	}

	MouseDownOp->GetChannelCache().ApplyKeyTimes(InTimeline, bInNotifyMovieSceneDataChanged);
}

void FSimpleViewRetimeTool::CreateMouseDragOperation(const FMouseInputData& InMouseInput, const FResolvedDragTarget& InTarget)
{
	if (!RetimeData)
	{
		return;
	}

	const bool bAllowBegin = InTarget.IsValid() || InTarget.bAllowNoneDragMode;
	if (!bAllowBegin)
	{
		return;
	}

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InMouseInput.Timeline->GetTimeSliderController();

	MouseDownOp.Emplace(
		InMouseInput,
		GetToolRange(),
		TimeSliderController->GetViewRange(),
		InTarget.DragMode,
		RetimeData->GetSortedAnchorTimes());

	ActiveDragTarget = InTarget;
}

void FSimpleViewRetimeTool::ResetDragOperation()
{
	UpdateKeyTimeThrottle.Reset();
	UpdateKeyTimeThrottle.NotifyActivity();

	ActiveTransaction.Reset();

	bPassThroughScrubActive = false;

	Lattice.ResetButtonState();

	MouseDownOp.Reset();

	HoverTarget.Reset();
	ActiveDragTarget.Reset();
}

bool FSimpleViewRetimeTool::HasMouseDownOp() const
{
	return MouseDownOp.IsSet();
}

bool FSimpleViewRetimeTool::HasDelayedDrag() const
{
	return HasMouseDownOp() && MouseDownOp->HasDragOp();
}

bool FSimpleViewRetimeTool::HasPreDragData() const
{
	return RetimeData && HasDelayedDrag();
}

bool FSimpleViewRetimeTool::HasDraggedCacheKeys() const
{
	return HasPreDragData()
		&& MouseDownOp->HasCachedKeys();
}

FSimpleViewRetimeTool::FResolvedDragTarget FSimpleViewRetimeTool::ResolveDragTarget(const FMouseInputData& InMouseInput) const
{
	FResolvedDragTarget Result;

	if (!RetimeData)
	{
		return Result;
	}

	const TRange<FFrameNumber> ToolRange = GetToolRange();

	Result.bWantsAddToSelection = InMouseInput.PointerEvent.IsShiftDown();
	Result.bWantsRemoveFromSelection = InMouseInput.PointerEvent.IsAltDown();

	// 1) Lattice buttons
	if (RetimeData->GetAnchorCount() > 1)
	{
		switch (Lattice.HitTestHandle(InMouseInput, ToolRange))
		{
		case FSimpleViewRetimeToolLattice::ELatticeButtonType::LeftScale:
			Result.HitType = EResolvedMouseHit::LatticeLeftScale;
			Result.DragMode = FSimpleViewRetimeDragOperation::EDragMode::LatticeScaleLeft;
			return Result;

		case FSimpleViewRetimeToolLattice::ELatticeButtonType::CenterMove:
			Result.HitType = EResolvedMouseHit::LatticeCenterMove;
			Result.DragMode = FSimpleViewRetimeDragOperation::EDragMode::LatticeMoveAnchors;
			return Result;

		case FSimpleViewRetimeToolLattice::ELatticeButtonType::RightScale:
			Result.HitType = EResolvedMouseHit::LatticeRightScale;
			Result.DragMode = FSimpleViewRetimeDragOperation::EDragMode::LatticeScaleRight;
			return Result;

		case FSimpleViewRetimeToolLattice::ELatticeButtonType::None:
		default:
			break;
		}
	}

	// 2) Anchor bars
	const int32 HoveredAnchorIndex = RetimeData->GetIndexOfAnchorBarUnderPointer(InMouseInput);
	if (HoveredAnchorIndex != INDEX_NONE)
	{
		Result.AnchorIndex = HoveredAnchorIndex;
		Result.DragMode = FSimpleViewRetimeDragOperation::EDragMode::MoveAnchor;
		Result.HitType = RetimeData->IsAnchorSelected(HoveredAnchorIndex)
			? EResolvedMouseHit::SelectedAnchorBar : EResolvedMouseHit::AnchorBar;
		return Result;
	}

	// 3) Tool range
	const bool bHitToolRange = HitTestRange(InMouseInput, ToolRange, /*bInHitTestVertical=*/true);
	if (bHitToolRange)
	{
		if (ShouldIgnoreToolRangeDragInput(InMouseInput))
		{
			return Result;
		}

		Result.HitType = EResolvedMouseHit::ToolRange;
		Result.DragMode = FSimpleViewRetimeDragOperation::EDragMode::MoveAllAnchors;

		return Result;
	}

	return Result;
}

void FSimpleViewRetimeTool::ApplySelectionForResolvedTarget(const FMouseInputData& InMouseInput, const FResolvedDragTarget& InTarget)
{
	if (!RetimeData || !InTarget.IsAnchorTarget())
	{
		return;
	}

	if (!InTarget.IsSelectedAnchorTarget()
		&& !InTarget.bWantsAddToSelection
		&& !InTarget.bWantsRemoveFromSelection)
	{
		RetimeData->ClearAnchorSelection();
	}

	RetimeData->SelectAnchorByIndex(InTarget.AnchorIndex, InTarget.bWantsAddToSelection, InTarget.bWantsRemoveFromSelection);
}

bool FSimpleViewRetimeTool::IsPendingInitialRangeCreation() const
{
	return bPendingInitialRangeDrag
		&& !MouseDownOp.IsSet()
		&& RetimeData
		&& RetimeData->GetAnchorCount() == 2;
}

FReply FSimpleViewRetimeTool::BeginInitialRangeCreation(const FMouseInputData& InMouseInput)
{
	bPendingInitialRangeDrag = false;

	FResolvedDragTarget InitialTarget;
	InitialTarget.DragMode = FSimpleViewRetimeDragOperation::EDragMode::None;
	InitialTarget.bAllowNoneDragMode = true;

	CreateMouseDragOperation(InMouseInput, InitialTarget);

	if (!MouseDownOp.IsSet() || !RetimeData || RetimeData->GetAnchorCount() < 2)
	{
		ResetDragOperation();

		return FReply::Unhandled();
	}

	// Make absolutely sure the initial anchors are seeded from the drag op's original mouse-down time.
	const FFrameTime InitialRangeStartTime = MouseDownOp->GetInitialTickTime();
	RetimeData->SetFirstAnchor(InitialRangeStartTime);
	RetimeData->SetLastAnchor(InitialRangeStartTime);

	MouseDownOp->ForceDragStart();

	SyncLatticeState();

	return FReply::Handled()
		.CaptureMouse(InMouseInput.OwnerWidget)
		.UseHighPrecisionMouseMovement(InMouseInput.OwnerWidget)
		.PreventThrottling();
}

void FSimpleViewRetimeTool::UpdateInitialRangeCreationWhileDragging(const FMouseInputData& InMouseInput)
{
	if (!RetimeData || !HasMouseDownOp())
	{
		return;
	}

	const TArray<FFrameTime>& AnchorStartTimes = MouseDownOp->GetAnchorStartTimes();
	if (AnchorStartTimes.Num() < 2)
	{
		return;
	}

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InMouseInput.Timeline->GetTimeSliderController();
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	const FFrameRate DisplayRate = TimeSliderController->GetDisplayRate();

	// Fixed side is the original start anchor for this initial-range creation
	const FFrameTime FixedTickTime = AnchorStartTimes[0];

	const FFrameTime DraggedTickTime = MouseDownOp->GetAccumulatedTickTime(InMouseInput, /*bInSnapEnabled=*/false);

	const FFrameNumber FixedDisplayFrame =
		FFrameRate::TransformTime(FixedTickTime, TickResolution, DisplayRate).FloorToFrame();

	const FFrameNumber DraggedDisplayFrame =
		FFrameRate::TransformTime(DraggedTickTime, TickResolution, DisplayRate).FloorToFrame();

	const FFrameNumber StartDisplayFrame = FMath::Min(FixedDisplayFrame, DraggedDisplayFrame);
	const FFrameNumber EndExclusiveDisplayFrame = FMath::Max(FixedDisplayFrame, DraggedDisplayFrame) + 1;

	const FFrameTime StartTick =
		FFrameRate::TransformTime(FFrameTime(StartDisplayFrame), DisplayRate, TickResolution);

	const FFrameTime EndExclusiveTick =
		FFrameRate::TransformTime(FFrameTime(EndExclusiveDisplayFrame), DisplayRate, TickResolution);

	RetimeData->SetFirstAnchor(StartTick);
	RetimeData->SetLastAnchor(EndExclusiveTick);
}

bool FSimpleViewRetimeTool::ShouldPassThroughToScrub(const FMouseInputData& InMouseInput, const FResolvedDragTarget& InTarget) const
{
	if (!RetimeData
		|| MouseDownOp.IsSet()
		|| InTarget.IsLatticeTarget()
		|| InTarget.IsAnchorTarget()
		|| !HitTestRange(InMouseInput, GetToolRange(), /*bInTestVertical=*/false)
		|| WantsInputPrepass(InMouseInput)
		|| ToolFactoryWantsToActivate(InMouseInput))
	{
		return false;
	}

	return CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport);
}

bool FSimpleViewRetimeTool::ShouldStartNewRangeCreation(const FMouseInputData& InMouseInput, const FResolvedDragTarget& InTarget) const
{
	if (!RetimeData || !ToolFactoryWantsToActivate(InMouseInput))
	{
		return false;
	}

	// Allow ctrl-drag to restart range creation when clicking empty space or the existing tool range,
	// but keep direct anchor/lattice interaction working
	if (InTarget.IsAnchorTarget() || InTarget.IsLatticeTarget())
	{
		return false;
	}

	return true;
}

FReply FSimpleViewRetimeTool::RestartRangeCreation(const FMouseInputData& MouseInput)
{
	if (!RetimeData)
	{
		return FReply::Unhandled();
	}

	StopDragIfPossible();

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = MouseInput.Timeline->GetTimeSliderController();

	const FFrameTime ClickTime = TimeSliderController->ComputeMouseFrameTime(MouseInput, /*bInCheckSnapping=*/true);

	RetimeData->ResetAnchors();
	RetimeData->AddAnchor(ClickTime);
	RetimeData->AddAnchor(ClickTime);

	FResolvedDragTarget InitialTarget;
	InitialTarget.DragMode = FSimpleViewRetimeDragOperation::EDragMode::None;
	InitialTarget.bAllowNoneDragMode = true;

	CreateMouseDragOperation(MouseInput, InitialTarget);

	if (!MouseDownOp.IsSet())
	{
		ResetDragOperation();
		return FReply::Unhandled();
	}

	MouseDownOp->ForceDragStart();

	return FReply::Handled()
		.CaptureMouse(MouseInput.OwnerWidget)
		.UseHighPrecisionMouseMovement(MouseInput.OwnerWidget)
		.PreventThrottling();
}

bool FSimpleViewRetimeTool::IsDraggingOutsideToolRange(const FMouseInputData& InMouseInput, const FResolvedDragTarget& InTarget) const
{
	return RetimeData
		&& RetimeData->GetAnchorCount() >= 2
		&& !InTarget.IsValid()
		&& !HitTestRange(InMouseInput, GetToolRange(), /*bInTestVertical=*/false);
}

bool FSimpleViewRetimeTool::TryBeginOrRestartDrag(const FMouseInputData& InMouseInput, const FResolvedDragTarget& InTarget)
{
	if (!InTarget.IsValid())
	{
		return false;
	}

	if (ActiveDragTarget.IsSet() && ActiveDragTarget->HasSameDragIdentity(InTarget))
	{
		return false;
	}

	if (MouseDownOp.IsSet())
	{
		StopDragIfPossible();
	}

	ApplySelectionForResolvedTarget(InMouseInput, InTarget);
	CreateMouseDragOperation(InMouseInput, InTarget);

	return MouseDownOp.IsSet();
}

bool FSimpleViewRetimeTool::CanMouseInputTriggerClose(const FMouseInputData& InMouseInput) const
{
	if (!RetimeData
		|| RetimeData->GetAnchorCount() <= 0
		|| InMouseInput.PointerEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		return false;
	}

	const TRange<FFrameNumber> ToolRange = GetToolRange();

	// Clicking any lattice handle should count as interacting with the tool,
	// even if the left/right buttons sit outside the main time range.
	if (RetimeData->GetAnchorCount() > 1
		&& Lattice.HitTestHandle(InMouseInput, ToolRange) != FSimpleViewRetimeToolLattice::ELatticeButtonType::None)
	{
		return false;
	}

	return ActiveDragTarget.IsSet() && !ActiveDragTarget->IsToolRangeHit();
}

bool FSimpleViewRetimeTool::CanBeginRangeCreation() const
{
	return HasMouseDownOp()
		&& MouseDownOp->IsCurrentDragMode(FSimpleViewRetimeDragOperation::EDragMode::None)
		&& MouseDownOp->GetAnchorStartTimes().Num() >= 2;
}

bool FSimpleViewRetimeTool::AddAnchor(const FMouseInputData& InMouseInput)
{
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InMouseInput.Timeline->GetTimeSliderController();

	const FFrameTime MouseFrameTime = TimeSliderController->ComputeMouseFrameTime(InMouseInput, /*bInCheckSnapping=*/true);

	FScopedTransaction Transaction(LOCTEXT("AddAnchorTransaction", "Add Retiming Anchor"));

	RetimeData->Modify();

	if (!ensure(RetimeData->AddAnchor(MouseFrameTime)))
	{
		Transaction.Cancel();
		return false;
	}

	// Sort to ensure they're always sorted from least to greatest as drawing and between segments depends on this behavior
	RetimeData->SortAnchorsByTime(InMouseInput.Timeline);

	return true;
}

bool FSimpleViewRetimeTool::RemoveAnchors(const TSet<int32>& InAnchorIndices, const bool bInCloseToolIfNoAnchors)
{
	if (!RetimeData || InAnchorIndices.IsEmpty())
	{
		return false;
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveAnchorsTransaction", "Remove Retiming Anchor(s)"));

	RetimeData->Modify();

	for (const int32 Index : InAnchorIndices)
	{
		RetimeData->RemoveAnchor(Index);
	}

	if (bInCloseToolIfNoAnchors && RetimeData->GetAnchorCount() == 0)
	{
		RequestClose();
	}

	return true;
}

void FSimpleViewRetimeTool::SetHoveringBasedOnDragging(const FMouseInputData& InMouseInput)
{
	const bool bIsDragging = IsDragging();
	if (bIsDragging)
	{
		// Lock it on while dragging so alpha can't oscillate
		SetFrameBubbleVisibility(true);

		return;
	}

	const bool bShowHovering = HoverTarget.IsSet() && ShouldShowFrameBubbleForHoverTarget();
	SetFrameBubbleVisibility(bShowHovering);
}

bool FSimpleViewRetimeTool::BuildScaledAnchorTimes(const TSharedRef<FToolableTimeline>& InTimeline
	, const TArray<FFrameTime>& InOriginalAnchorTimes
	, const bool bLockRightSide
	, const FFrameTime& InDragDeltaTime
	, TArray<FFrameTime>& OutNewAnchorTimes) const
{
	OutNewAnchorTimes.Reset();

	const int32 AnchorCount = InOriginalAnchorTimes.Num();
	if (AnchorCount < 2)
	{
		return false;
	}

	const FFrameTime OriginalStart = InOriginalAnchorTimes[0];
	const FFrameTime OriginalEnd = InOriginalAnchorTimes.Last();

	const FFrameTime OriginalSpan = OriginalEnd - OriginalStart;
	if (OriginalSpan <= FFrameTime(0))
	{
		return false;
	}

	// Minimum span of 1 tick
	const FFrameTime MinSpan(1);

	FFrameTime NewStart = OriginalStart;
	FFrameTime NewEnd = OriginalEnd;

	if (bLockRightSide)
	{
		const FFrameTime CandidateStart = OriginalStart + InDragDeltaTime;
		const FFrameTime MaxStart = OriginalEnd - MinSpan;
		NewStart = (CandidateStart < MaxStart) ? CandidateStart : MaxStart;
	}
	else
	{
		const FFrameTime CandidateEnd = OriginalEnd + InDragDeltaTime;
		const FFrameTime MinEnd = OriginalStart + MinSpan;
		NewEnd = (CandidateEnd > MinEnd) ? CandidateEnd : MinEnd;
	}

	OutNewAnchorTimes.SetNumUninitialized(AnchorCount);

	const double OriginalStartDecimal = OriginalStart.AsDecimal();
	const double OriginalSpanDecimal = OriginalSpan.AsDecimal();
	const double NewStartDecimal = NewStart.AsDecimal();
	const double NewEndDecimal = NewEnd.AsDecimal();

	for (int32 Index = 0; Index < AnchorCount; ++Index)
	{
		const double Alpha = (InOriginalAnchorTimes[Index].AsDecimal() - OriginalStartDecimal) / OriginalSpanDecimal;
		const double NewTimeDecimal = FMath::Lerp(NewStartDecimal, NewEndDecimal, Alpha);
		OutNewAnchorTimes[Index] = FFrameTime::FromDecimal(NewTimeDecimal);
	}

	return true;
}

bool FSimpleViewRetimeTool::BuildMovedAnchorTimes(const TArray<FFrameTime>& InOriginalAnchorTimes
	, const FFrameTime& InDragDeltaTime, TArray<FFrameTime>& OutNewAnchorTimes) const
{
	OutNewAnchorTimes.Reset();

	if (InOriginalAnchorTimes.IsEmpty())
	{
		return false;
	}

	OutNewAnchorTimes.SetNumUninitialized(InOriginalAnchorTimes.Num());

	for (int32 Index = 0; Index < InOriginalAnchorTimes.Num(); ++Index)
	{
		OutNewAnchorTimes[Index] = InOriginalAnchorTimes[Index] + InDragDeltaTime;
	}

	return true;
}

void FSimpleViewRetimeTool::ApplyAnchorTimes(const TSharedRef<FToolableTimeline>& InTimeline
	, const TArray<FFrameTime>& InNewAnchorTimes)
{
	if (!RetimeData)
	{
		return;
	}

	// Intentionally do not sort here: lattice retime depends on anchor index stability
	RetimeData->SetAnchorTimes(InNewAnchorTimes);
}

void FSimpleViewRetimeTool::ApplyDragResultsIfThrottleOpen(const TSharedRef<FToolableTimeline>& InTimeline, const bool bInForceApply)
{
	const bool bUseThrottledUpdates = InTimeline->GetTimelineSettings().Settings.bUseThrottledUpdates;
	const bool bShouldApplyNow = !bUseThrottledUpdates || bInForceApply || UpdateKeyTimeThrottle.ShouldApply();

	// Throttle the expensive "apply" stage: Modify + SetKeyTime + Notify + Recache
	if (!bShouldApplyNow)
	{
		return;
	}

	UpdateKeyTimeThrottle.NotifyActivity();

	UpdateKeyTimes(InTimeline, /*bInNotifyMovieSceneDataChanged=*/bInForceApply);
}

void FSimpleViewRetimeTool::RestoreInitialDragState(const TSharedRef<FToolableTimeline>& InTimeline)
{
	if (!MouseDownOp.IsSet())
	{
		return;
	}

	if (RetimeData)
	{
		const TArray<FFrameTime>& AnchorStartTimes = MouseDownOp->GetAnchorStartTimes();
		if (AnchorStartTimes.Num() == RetimeData->GetAnchorCount())
		{
			ApplyAnchorTimes(InTimeline, AnchorStartTimes);
		}
	}

	MouseDownOp->RestoreInitialKeyTimes();

	UpdateKeyTimes(InTimeline);
}

void FSimpleViewRetimeTool::RemoveSelectedAnchors()
{
	if (!RetimeData)
	{
		return;
	}

	const TSet<int32> SelectedAnchorIndices = RetimeData->GetSelectedAnchorIndices();
	if (SelectedAnchorIndices.IsEmpty())
	{
		return;
	}

	RemoveAnchors(SelectedAnchorIndices);
}

bool FSimpleViewRetimeTool::ShouldIgnoreToolRangeDragInput(const FMouseInputData& InMouseInput) const
{
	return InMouseInput.PointerEvent.IsShiftDown() ||
		(InMouseInput.PointerEvent.IsAltDown() && !InMouseInput.PointerEvent.IsControlDown());
}

void FSimpleViewRetimeTool::SyncLatticeState()
{
	if (!RetimeData || RetimeData->GetAnchorCount() <= 1)
	{
		Lattice.ResetButtonState();
		return;
	}

	if (MouseDownOp.IsSet())
	{
		// Real lattice drag target always wins
		if (ActiveDragTarget.IsSet() && ActiveDragTarget->IsLatticeTarget())
		{
			Lattice.SetPressedButton(ToLatticeButtonType(ActiveDragTarget.GetValue()));

			return;
		}

		// Show left or right pressed depending on drag direction for initial range creation
		if (MouseDownOp->IsCurrentDragMode(FSimpleViewRetimeDragOperation::EDragMode::None)
			&& MouseDownOp->GetAnchorStartTimes().Num() >= 2)
		{
			const double DragDeltaX = MouseDownOp->GetAccumulatedLocalDelta().X;
			if (!FMath::IsNearlyZero(DragDeltaX))
			{
				const FSimpleViewRetimeToolLattice::ELatticeButtonType ButtonToPress = DragDeltaX < 0.0
					? FSimpleViewRetimeToolLattice::ELatticeButtonType::LeftScale
					: FSimpleViewRetimeToolLattice::ELatticeButtonType::RightScale;
				Lattice.SetPressedButton(ButtonToPress);

				return;
			}

			// No meaningful drag yet, so fall through to hover state
		}
	}

	const FSimpleViewRetimeToolLattice::ELatticeButtonType HoveredButton = HoverTarget.IsSet()
		? ToLatticeButtonType(HoverTarget.GetValue()) : FSimpleViewRetimeToolLattice::ELatticeButtonType::None;

	if (HoveredButton != FSimpleViewRetimeToolLattice::ELatticeButtonType::None)
	{
		Lattice.SetHoveredButton(HoveredButton);

		return;
	}

	Lattice.ResetButtonState();
}

FSimpleViewRetimeToolLattice::ELatticeButtonType FSimpleViewRetimeTool::ToLatticeButtonType(const FResolvedDragTarget& InTarget)
{
	switch (InTarget.HitType)
	{
	case EResolvedMouseHit::LatticeLeftScale:
		return FSimpleViewRetimeToolLattice::ELatticeButtonType::LeftScale;

	case EResolvedMouseHit::LatticeCenterMove:
		return FSimpleViewRetimeToolLattice::ELatticeButtonType::CenterMove;

	case EResolvedMouseHit::LatticeRightScale:
		return FSimpleViewRetimeToolLattice::ELatticeButtonType::RightScale;

	case EResolvedMouseHit::None:
	case EResolvedMouseHit::AnchorBar:
	case EResolvedMouseHit::SelectedAnchorBar:
	case EResolvedMouseHit::ToolRange:
	default:
		return FSimpleViewRetimeToolLattice::ELatticeButtonType::None;
	}
}

bool FSimpleViewRetimeTool::ToolFactoryWantsToActivate(const FMouseInputData& InMouseInput) const
{
	if (const TSharedPtr<FSimpleViewRetimeToolFactory> ToolFactory = InMouseInput.Timeline->FindToolFactory<FSimpleViewRetimeToolFactory>())
	{
		return ToolFactory->WantsToActivate(InMouseInput, false);
	}
	return false;
}

void FSimpleViewRetimeTool::PlayFadeInOutAnimation(const bool bInReverse)
{
	if (bInReverse)
	{
		FadeSequence.Reverse();
	}
	else
	{
		const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin();
		const TSharedPtr<SToolableTimeline> ToolableWidget = Timeline.IsValid() ? Timeline->GetWidget() : nullptr;
		if (ToolableWidget.IsValid())
		{
			FadeSequence.Play(ToolableWidget.ToSharedRef());
		}
	}
}

void FSimpleViewRetimeTool::SetFrameBubbleVisibility(const bool bInVisible)
{
	if (bFrameBubblesVisible == bInVisible)
	{
		return;
	}

	bFrameBubblesVisible = bInVisible;
	PlayFadeInOutAnimation(!bInVisible);
}

bool FSimpleViewRetimeTool::ShouldShowFrameBubbleForHoverTarget() const
{
	if (!HoverTarget.IsSet())
	{
		return false;
	}

	if (HoverTarget->IsLatticeTarget())
	{
		return true;
	}

	if (!HoverTarget->IsAnchorTarget() || !RetimeData)
	{
		return false;
	}

	const int32 AnchorCount = RetimeData->GetAnchorCount();
	if (AnchorCount <= 0)
	{
		return false;
	}

	return HoverTarget->AnchorIndex == 0 || HoverTarget->AnchorIndex == AnchorCount - 1;
}

void FSimpleViewRetimeTool::GetVisibleFrameBubbles(bool& bOutShowStartBubble, bool& bOutShowEndBubble, bool& bOutShowCenterBubble) const
{
	bOutShowStartBubble = false;
	bOutShowEndBubble = false;
	bOutShowCenterBubble = false;

	if (IsDragging())
	{
		bOutShowCenterBubble = true;
		return;
	}

	const FSimpleViewRetimeToolLattice::ELatticeButtonType PressedButton = Lattice.GetPressedButton();
	if (PressedButton != FSimpleViewRetimeToolLattice::ELatticeButtonType::None)
	{
		switch (PressedButton)
		{
		case FSimpleViewRetimeToolLattice::ELatticeButtonType::LeftScale:
			bOutShowStartBubble = true;
			return;

		case FSimpleViewRetimeToolLattice::ELatticeButtonType::CenterMove:
			bOutShowCenterBubble = true;
			return;

		case FSimpleViewRetimeToolLattice::ELatticeButtonType::RightScale:
			bOutShowEndBubble = true;
			return;

		case FSimpleViewRetimeToolLattice::ELatticeButtonType::None:
		default:
			break;
		}
	}

	const FSimpleViewRetimeToolLattice::ELatticeButtonType HoveredButton = Lattice.GetHoveredButton();
	if (HoveredButton != FSimpleViewRetimeToolLattice::ELatticeButtonType::None)
	{
		switch (HoveredButton)
		{
		case FSimpleViewRetimeToolLattice::ELatticeButtonType::LeftScale:
			bOutShowStartBubble = true;
			return;

		case FSimpleViewRetimeToolLattice::ELatticeButtonType::CenterMove:
			bOutShowCenterBubble = true;
			return;

		case FSimpleViewRetimeToolLattice::ELatticeButtonType::RightScale:
			bOutShowEndBubble = true;
			return;

		case FSimpleViewRetimeToolLattice::ELatticeButtonType::None:
		default:
			break;
		}
	}

	if (HoverTarget.IsSet())
	{
		switch (HoverTarget->HitType)
		{
		case EResolvedMouseHit::LatticeLeftScale:
			bOutShowStartBubble = true;
			return;

		case EResolvedMouseHit::LatticeCenterMove:
			bOutShowCenterBubble = true;
			return;

		case EResolvedMouseHit::LatticeRightScale:
			bOutShowEndBubble = true;
			return;

		case EResolvedMouseHit::AnchorBar:
		case EResolvedMouseHit::SelectedAnchorBar:
			if (RetimeData && HoverTarget->AnchorIndex == 0)
			{
				bOutShowStartBubble = true;
				return;
			}

			if (RetimeData && HoverTarget->AnchorIndex == RetimeData->GetAnchorCount() - 1)
			{
				bOutShowEndBubble = true;
			}
			return;

		case EResolvedMouseHit::ToolRange:
		case EResolvedMouseHit::None:
		default:
			break;
		}
	}

	if (bFrameBubblesVisible)
	{
		bOutShowCenterBubble = true;
	}
}

void FSimpleViewRetimeTool::UpdateSelectionFromToolRange()
{
	if (!RetimeData)
	{
		return;
	}

	const TRange<FFrameNumber> ToolRange = GetToolRange();
	if (!IsValidToolRange(ToolRange))
	{
		return;
	}

	const TSharedRef<FToolableTimeline> Timeline = GetTimelineChecked();
	const TSet<FSequencerSelectedKey> RangeKeys = GetToolRangeKeys();

	FToolableTimelineKeySelection& KeySelection = Timeline->GetKeySelection();
	KeySelection.SetSelectedAndHoveredKeys(RangeKeys);
}

} // namespace UE::Sequencer::SimpleView

#undef LOCTEXT_NAMESPACE
