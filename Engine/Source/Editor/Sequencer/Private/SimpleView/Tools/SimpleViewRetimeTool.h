// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "SimpleView/PollGateThrottle.h"
#include "SimpleView/Tools/DragOperations/SimpleViewRetimeDragOperation.h"
#include "SimpleView/Tools/SimpleViewRetimeToolLattice.h"
#include "ToolableTimeline/Tools/ToolableTimelineBaseTool.h"

class USimpleViewRetimeData;

namespace UE::Sequencer::ToolableTimeline
{
	class FToolableTimeline;
	struct FMouseInputData;
}

namespace UE::Sequencer::SimpleView::RetimeTool
{
	constexpr double AnchorWidth = 3.0;
	constexpr double AnchorWidthHalf = AnchorWidth * 0.5;

	constexpr double CloseButtonSize = 14.0;
	constexpr double CloseButtonSizeHalf = CloseButtonSize * 0.5;
	constexpr double CloseButtonPadding = 1.0;

	constexpr double ExtraHitPadding = 4.0;

	constexpr double HighlightOpacity = 0.75;
}

namespace UE::Sequencer::SimpleView
{

using namespace UE::Sequencer::ToolableTimeline;

/** Tool for retiming groups of keys in a range. */
class FSimpleViewRetimeTool : public FToolableTimelineBaseTool
{
public:
	FSimpleViewRetimeTool(const TSharedRef<FToolableTimeline>& InTimeline);
	virtual ~FSimpleViewRetimeTool() override;

	//~ Begin IToolableTimelineEditTool

	virtual FName GetIdentifier() const override;
	virtual FText GetLabel() const override;
	virtual FText GetDescription() const override;

	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandBindings) override;
	virtual void UnbindCommands(const TSharedRef<FUICommandList>& InCommandBindings) override;

	virtual void NotifyToolActivated(const FMouseInputData& InMouseInput, const FFrameTime& InInputTickTime) override;
	virtual void NotifyToolDeactivated() override;

	virtual TRange<FFrameNumber> GetToolRange() const override;

	virtual bool WantsDeferredDragThresholdActivation() const override { return true; }

	virtual bool WantsInputPrepass(const FMouseInputData& InMouseInput) const override;

	virtual int32 Paint(FMouseDrawInputData& MouseDrawInput) const override;

	virtual bool WantsPriorityOverControllerHit(const FMouseInputData& InMouseInput) const override;

	virtual bool ShouldShowDragRangeIndicator() const override;

	virtual bool ShouldDeactivateOnOutlinerSelectionChanged() const override { return true; }

	virtual TRange<FFrameTime> GetDragIndicatorTickRange(const FMouseInputData& InMouseInput) const override;

	//~ End IToolableTimelineEditTool

	//~ Begin FToolableTimelineBaseTool
	virtual bool IsDragging() const override;
	virtual bool HasHoverHit() const override;
	virtual bool CanPersistThroughChannelRecache() const override;
	virtual void OnChannelModelsRecached() override;
	//~ End FToolableTimelineBaseTool

	//~ Begin ISequencerInputHandler

	virtual FCursorReply OnCursorQuery(const SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) const override;

	virtual FReply OnMouseButtonDoubleClick(SWidget& OwnerWidget
		, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseButtonDown(SWidget& OwnerWidget
		, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseMove(SWidget& OwnerWidget
		, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseButtonUp(SWidget& OwnerWidget
		, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual void OnMouseLeave(SWidget& OwnerWidget, const FPointerEvent& InPointerEvent) override;

	virtual FReply OnKeyDown(SWidget& OwnerWidget, const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyUp(SWidget& OwnerWidget, const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void OnFocusLost(SWidget& OwnerWidget, const FFocusEvent& InFocusEvent) override;

	//~ End ISequencerInputHandler
	
protected:
	//~ Begin FToolableTimelineBaseTool
	virtual void OnDragStart(const FMouseInputData& InMouseInput) override;
	virtual void OnDrag(const FMouseInputData& InMouseInput) override;
	virtual void OnDragEnd(const bool bInCommit = true) override;
	//~ End FToolableTimelineBaseTool

private:
	/** Represents what the mouse hit after collision resolving */
	enum EResolvedMouseHit
	{
		None = 0,

		ToolRange,
		AnchorBar,
		SelectedAnchorBar,

		LatticeCenterMove,
		LatticeLeftScale,
		LatticeRightScale
	};

	struct FResolvedDragTarget
	{
		bool operator==(const FResolvedDragTarget& InOther) const
		{
			return HitType == InOther.HitType
				&& DragMode == InOther.DragMode
				&& AnchorIndex == InOther.AnchorIndex
				&& bWantsAddToSelection == InOther.bWantsAddToSelection
				&& bWantsRemoveFromSelection == InOther.bWantsRemoveFromSelection
				&& bAllowNoneDragMode == InOther.bAllowNoneDragMode;
		}

		bool IsValid() const { return HitType != None; }

		bool IsToolRangeHit() const { return HitType == ToolRange; }

		bool IsAnchorTarget() const { return HitType == AnchorBar || HitType == SelectedAnchorBar; }

		bool IsSelectedAnchorTarget() const { return HitType == SelectedAnchorBar; }

		bool IsLatticeTarget() const
		{
			return HitType == LatticeLeftScale
				|| HitType == LatticeCenterMove
				|| HitType == LatticeRightScale;
		}

		bool HasSameDragIdentity(const FResolvedDragTarget& InOther) const
		{
			return HitType == InOther.HitType
				&& DragMode == InOther.DragMode
				&& AnchorIndex == InOther.AnchorIndex;
		}

		EResolvedMouseHit HitType = EResolvedMouseHit::None;

		FSimpleViewRetimeDragOperation::EDragMode DragMode = FSimpleViewRetimeDragOperation::EDragMode::None;

		int32 AnchorIndex = INDEX_NONE;

		bool bWantsAddToSelection = false;
		bool bWantsRemoveFromSelection = false;

		/** Determines if the drag operation is allowed in the scenario where no specific drag target is resolved.
		 * If true, the user can initiate a drag even without a valid drag target. */
		bool bAllowNoneDragMode = false;
	};

	void CreateMouseDragOperation(const FMouseInputData& InMouseInput, const FResolvedDragTarget& InTarget);

	void OnDrag_RetimeKeys(const FMouseInputData& InMouseInput
		, const bool bInIgnoreAnchorInfluences
		, const bool bInSnapToFrame);
	void OnDrag_MoveAnchors(const FMouseInputData& InMouseInput, const bool bInAllAnchors);

	void OnDrag_LatticeScaleLeft(const FMouseInputData& InMouseInput, const bool bInSnapToFrame);
	void OnDrag_LatticeScaleRight(const FMouseInputData& InMouseInput, const bool bInSnapToFrame);
	void OnDrag_LatticeMoveAnchors(const FMouseInputData& InMouseInput, const bool bInAllAnchors);

	void OnDrag_RetimeKeysFromAnchorTimes(const FMouseInputData& InMouseInput
		, const TArray<FFrameTime>& InOriginalAnchorTimes
		, const TArray<FFrameTime>& InNewAnchorTimes
		, const bool bInSnapToFrame);

	void StopDragIfPossible(const bool bInCommit = true);

	bool HitTestRange(const FMouseInputData& InMouseInput
		, const TRange<FFrameNumber>& InRange, const bool bInTestVertical) const;

	void UpdateKeyTimes(const TSharedRef<FToolableTimeline>& InTimeline, const bool bInNotifyMovieSceneDataChanged = true);

	void ResetDragOperation();
	bool HasMouseDownOp() const;
	bool HasDelayedDrag() const;
	bool HasPreDragData() const;
	bool HasDraggedCacheKeys() const;

	/**  */
	FResolvedDragTarget ResolveDragTarget(const FMouseInputData& InMouseInput) const;
	void ApplySelectionForResolvedTarget(const FMouseInputData& InMouseInput, const FResolvedDragTarget& InTarget);

	bool IsPendingInitialRangeCreation() const;
	FReply BeginInitialRangeCreation(const FMouseInputData& InMouseInput);
	void UpdateInitialRangeCreationWhileDragging(const FMouseInputData& InMouseInput);

	bool ShouldPassThroughToScrub(const FMouseInputData& InMouseInput, const FResolvedDragTarget& InTarget) const;

	bool ShouldStartNewRangeCreation(const FMouseInputData& InMouseInput, const FResolvedDragTarget& InTarget) const;
	FReply RestartRangeCreation(const FMouseInputData& MouseInput);

	bool IsDraggingOutsideToolRange(const FMouseInputData& InMouseInput, const FResolvedDragTarget& InTarget) const;

	bool TryBeginOrRestartDrag(const FMouseInputData& InMouseInput, const FResolvedDragTarget& InTarget);

	bool CanMouseInputTriggerClose(const FMouseInputData& InMouseInput) const;

	bool CanBeginRangeCreation() const;

	bool AddAnchor(const FMouseInputData& InMouseInput);
	bool RemoveAnchors(const TSet<int32>& InAnchorIndices, const bool bInCloseToolIfNoAnchors = true);

	void SetHoveringBasedOnDragging(const FMouseInputData& InMouseInput);

	bool BuildScaledAnchorTimes(const TSharedRef<FToolableTimeline>& InTimeline
		, const TArray<FFrameTime>& InOriginalAnchorTimes
		, const bool bInLockRightSide
		, const FFrameTime& InDragDeltaTime
		, TArray<FFrameTime>& OutNewAnchorTimes) const;

	bool BuildMovedAnchorTimes(const TArray<FFrameTime>& InOriginalAnchorTimes
		, const FFrameTime& InDragDeltaTime, TArray<FFrameTime>& OutNewAnchorTimes) const;

	void ApplyAnchorTimes(const TSharedRef<FToolableTimeline>& InTimeline
		, const TArray<FFrameTime>& InNewAnchorTimes);

	void ApplyDragResultsIfThrottleOpen(const TSharedRef<FToolableTimeline>& InTimeline, const bool bInForceApply = false);
	void RestoreInitialDragState(const TSharedRef<FToolableTimeline>& InTimeline);

	void RemoveSelectedAnchors();

	bool ShouldIgnoreToolRangeDragInput(const FMouseInputData& InMouseInput) const;

	/** Updates the lattice state based on the mouse input and current drag operation. */
	void SyncLatticeState();

	/**  */
	static FSimpleViewRetimeToolLattice::ELatticeButtonType ToLatticeButtonType(const FResolvedDragTarget& InTarget);

	/**
	 * Determines whether the tool factory wants to activate based on the provided mouse input.
	 *
	 * @param InMouseInput Mouse input data for the event.
	 * @return True if the tool factory wants to activate, otherwise false.
	 */
	bool ToolFactoryWantsToActivate(const FMouseInputData& InMouseInput) const;

	/**  */
	void PlayFadeInOutAnimation(const bool bInReverse);

	/**  */
	void SetFrameBubbleVisibility(const bool bInVisible);

	/** Returns whether the current hover target should show a frame bubble. */
	bool ShouldShowFrameBubbleForHoverTarget() const;

	/** Resolves which frame bubbles should be visible for the current hover/drag state. */
	void GetVisibleFrameBubbles(bool& bOutShowStartBubble, bool& bOutShowEndBubble, bool& bOutShowCenterBubble) const;

	/**  */
	void UpdateSelectionFromToolRange();

	/** The retiming data for this tool. This is a UObject for Undo/Redo purposes. */
	TObjectPtr<USimpleViewRetimeData> RetimeData;

	/** Set when attempting to move a drag handle. */
	TOptional<FSimpleViewRetimeDragOperation> MouseDownOp;

	/**  */
	bool bPendingInitialRangeDrag = false;

	/** Hit targets for hovering and drag. */
	TOptional<FResolvedDragTarget> HoverTarget;
	TOptional<FResolvedDragTarget> PendingDragTarget;
	TOptional<FResolvedDragTarget> ActiveDragTarget;

	/** Indicates whether a key event related to scrubbing time is currently active.
	 * Set to true when a scrub time key event is detected during key press and reset to false on key release. */
	bool bScrubKeyDown = false;

	/** Indicates whether the tool allows scrubbing input to pass through to other components.
	 * This variable is set to true when the tool determines scrubbing interactions should be passed through,
	 * based on the mouse input and active drag targets. It is reset to false when scrubbing pass-through is not active. */
	bool bPassThroughScrubActive = false;

	/** Indicates whether frame bubbles are currently visible. */
	bool bFrameBubblesVisible = false;

	/** Represents a lattice structure used within the retime tool to manage and manipulate
	 * visual elements like buttons and states during interactions. */
	FSimpleViewRetimeToolLattice Lattice;

	/** Fade animation properties. */
	FCurveSequence FadeSequence;
	FCurveHandle FadeCurve;

	/** Throttle mechanism to control the frequency of key time updates in the retiming tool. */
	FPollGateThrottle UpdateKeyTimeThrottle;
};

} // namespace UE::Sequencer::SimpleView
