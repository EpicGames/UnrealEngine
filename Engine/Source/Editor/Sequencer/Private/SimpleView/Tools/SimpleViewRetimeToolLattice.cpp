// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimpleViewRetimeToolLattice.h"
#include "SimpleViewTimelineAnchor.h"
#include "ToolableTimeline/MouseInputData.h"
#include "ToolableTimeline/ToolableTimeline.h"
#include "ToolableTimeline/ToolableTimelineUtils.h"
#include "ToolableTimeline/Tools/ToolableTimelineBaseTool.h"

namespace UE::Sequencer::SimpleView
{

using namespace UE::Sequencer::ToolableTimeline;

FSimpleViewRetimeToolLatticeButton::FSimpleViewRetimeToolLatticeButton(const FSimpleViewRetimeToolLatticeButtonArgs& InArgs)
{
	SetArgs(InArgs);
}

void FSimpleViewRetimeToolLatticeButton::SetArgs(const FSimpleViewRetimeToolLatticeButtonArgs& InArgs)
{
	Args = InArgs;
}

FVector2f FSimpleViewRetimeToolLatticeButton::GetSize() const
{
	return FVector2f(
		FMath::Max(Args.ForegroundSize.X, Args.BackgroundSize.X),
		FMath::Max(Args.ForegroundSize.Y, Args.BackgroundSize.Y)
	);
}

FGeometry FSimpleViewRetimeToolLatticeButton::GetGeometry(const FMouseInputData& InMouseInput) const
{
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InMouseInput.Timeline->GetTimeSliderController();
	const UToolableTimelineSettings& TimelineSettings = InMouseInput.Timeline->GetTimelineSettings();

	const float TickSize = TimeSliderController->GetMajorTickDrawSize();
	const FVector2f LocalSize = InMouseInput.Geometry.GetLocalSize();
	const float FrameSize = LocalSize.Y - TickSize;
	const FVector2f ButtonSize = GetSize();

	float LocalPy = 0.f;

	switch (TimelineSettings.Settings.LabelVerticalAlignment)
	{
	default:
	case VAlign_Top:
		LocalPy = TickSize + ((FrameSize - ButtonSize.Y) * 0.5f);
		break;
	case VAlign_Bottom:
		LocalPy = (FrameSize - ButtonSize.Y) * 0.5f;
		break;
	case VAlign_Center:
	case VAlign_Fill:
		LocalPy = (LocalSize.Y - ButtonSize.Y) * 0.5f;
		break;
	}

	return InMouseInput.Geometry.MakeChild(
		ButtonSize,
		FSlateLayoutTransform(FVector2f(LocalPositionX, LocalPy))
	);
}

bool FSimpleViewRetimeToolLatticeButton::HitTest(const FMouseInputData& InMouseInput) const
{
	if (!bVisible)
	{
		return false;
	}

	const FGeometry ButtonGeometry = GetGeometry(InMouseInput);
	return ButtonGeometry.IsUnderLocation(InMouseInput.PointerEvent.GetScreenSpacePosition());
}

bool FSimpleViewRetimeToolLatticeButton::IsPressed() const
{
	return bVisible && bPressed;
}

bool FSimpleViewRetimeToolLatticeButton::TryPress()
{
	if (!bVisible)
	{
		return false;
	}

	if (bHovered)
	{
		bPressed = true;
	}

	return bPressed;
}

void FSimpleViewRetimeToolLatticeButton::ReleasePress()
{
	bPressed = false;
}

void FSimpleViewRetimeToolLatticeButton::Internal_Press()
{
	if (bVisible)
	{
		bPressed = true;
	}
}

void FSimpleViewRetimeToolLatticeButton::Internal_HideShow(const bool bInVisible)
{
	bVisible = bInVisible;

	if (!bVisible)
	{
		bHovered = false;
		bPressed = false;
	}
}

void FSimpleViewRetimeToolLatticeButton::Paint(FMouseDrawInputData& MouseDrawInput) const
{
	if (!bVisible || (!Args.ForegroundBrush && !Args.BackgroundBrush))
	{
		return;
	}

	const FGeometry ButtonGeometry = GetGeometry(MouseDrawInput);
	const bool bHoveredOrPressed = bHovered || bPressed;

	const FVector2f VisualSize(
		FMath::Max(Args.BackgroundSize.X, Args.ForegroundSize.X),
		FMath::Max(Args.BackgroundSize.Y, Args.ForegroundSize.Y)
	);

	if (Args.BackgroundBrush)
	{
		const FVector2f BackgroundOffset = (VisualSize - Args.BackgroundSize) * .5f;

		FSlateDrawElement::MakeBox(
			MouseDrawInput.DrawElements,
			MouseDrawInput.IncrementDrawLayer().LayerId,
			ButtonGeometry.ToPaintGeometry(
				Args.BackgroundSize,
				FSlateLayoutTransform(BackgroundOffset),
				Args.BackgroundRenderTransform
			),
			Args.BackgroundBrush,
			MouseDrawInput.DrawEffects,
			bHoveredOrPressed ? Args.ForegroundColor : Args.BackgroundColor
		);
	}

	if (Args.ForegroundBrush)
	{
		const FVector2f ForegroundOffset = (VisualSize - Args.ForegroundSize) * .5f;

		FSlateDrawElement::MakeBox(
			MouseDrawInput.DrawElements,
			MouseDrawInput.IncrementDrawLayer().LayerId,
			ButtonGeometry.ToPaintGeometry(
				Args.ForegroundSize,
				FSlateLayoutTransform(ForegroundOffset),
				Args.ForegroundRenderTransform
			),
			Args.ForegroundBrush,
			MouseDrawInput.DrawEffects,
			bHoveredOrPressed ? Args.BackgroundColor : Args.ForegroundColor
		);
	}

	MouseDrawInput.IncrementDrawLayer();
}

FSimpleViewRetimeToolLattice::FSimpleViewRetimeToolLattice(const TSharedRef<FToolableTimeline>& InTimeline)
	: Timeline(InTimeline)
{
	static const auto LeftFrontBrush = FAppStyle::GetBrush(TEXT("Sequencer.SimpleView.DragHandleLeft"));
	static const auto RightFrontBrush = FAppStyle::GetBrush(TEXT("Sequencer.SimpleView.DragHandleRight"));

	const FToolableTimelineInstanceSettings& TimelineSettings = Timeline->GetTimelineSettings().Settings;

	FSimpleViewRetimeToolLatticeButtonArgs ButtonArgs;
	/*ButtonArgs.ForegroundColor = FStyleColors::Foreground.GetSpecifiedColor();
	ButtonArgs.BackgroundColor = TimelineSettings.ToolHandleColor;
	ButtonArgs.HoverForegroundColor = TimelineSettings.ToolHandleColor;
	ButtonArgs.HoverBackgroundColor = FLinearColor::White;*/
	ButtonArgs.ForegroundColor = TimelineSettings.ToolHandleColor;
	ButtonArgs.BackgroundColor = FStyleColors::Foreground.GetSpecifiedColor();
	ButtonArgs.HoverForegroundColor = FLinearColor::White;
	ButtonArgs.HoverBackgroundColor = TimelineSettings.ToolHandleColor;
	ButtonArgs.ForegroundSize = FVector2f(LeftRightButtonSize, LeftRightButtonSize);
	ButtonArgs.BackgroundSize = ButtonArgs.ForegroundSize;

	// Left button
	ButtonArgs.ForegroundBrush = LeftFrontBrush;
	LeftButton.SetArgs(ButtonArgs);

	// Right button
	ButtonArgs.ForegroundBrush = RightFrontBrush;
	RightButton.SetArgs(ButtonArgs);

	// Common button args for center left button and center right button
	ButtonArgs.ForegroundSize = FVector2f(CenterButtonSize, CenterButtonSize);
	ButtonArgs.BackgroundSize = ButtonArgs.ForegroundSize;

	// Center left button
	ButtonArgs.ForegroundBrush = LeftFrontBrush;
	CenterLeftButton.SetArgs(ButtonArgs);

	// Center right button
	ButtonArgs.ForegroundBrush = RightFrontBrush;
	CenterRightButton.SetArgs(ButtonArgs);
}

FSimpleViewRetimeToolLattice::ELatticeButtonType FSimpleViewRetimeToolLattice::HitTestHandle(const FMouseInputData& InMouseInput
	, const TRange<FFrameNumber>& InRange) const
{
	NotifyRangeChanged(InMouseInput, InRange);

	if (LeftButton.HitTest(InMouseInput))
	{
		return ELatticeButtonType::LeftScale;
	}
	if (CenterLeftButton.HitTest(InMouseInput) || CenterRightButton.HitTest(InMouseInput))
	{
		return ELatticeButtonType::CenterMove;
	}
	if (RightButton.HitTest(InMouseInput))
	{
		return ELatticeButtonType::RightScale;
	}

	return ELatticeButtonType::None;
}

void FSimpleViewRetimeToolLattice::UpdateCenterButtonMouseMove(const FMouseInputData& InMouseInput)
{
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InMouseInput.Timeline->GetTimeSliderController();
	const TSharedPtr<FToolableTimelineBaseTool> ActiveTool = TimeSliderController->GetActiveTool();
	if (!ActiveTool.IsValid())
	{
		return;
	}

	const TRange<FFrameNumber> ToolRange = ActiveTool->GetToolRange();
	const TRange<FFrameNumber> NormalizedRange = ToolableTimeline::Utils::NormalizeRange(ToolRange);
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();

	const double LowerBoundSeconds = TickResolution.AsSeconds(NormalizedRange.GetLowerBoundValue());
	const double UpperBoundSeconds = TickResolution.AsSeconds(NormalizedRange.GetUpperBoundValue());

	const float LeftPixel = InMouseInput.RangeToScreen.InputToLocalX(LowerBoundSeconds);
	const float RightPixel = InMouseInput.RangeToScreen.InputToLocalX(UpperBoundSeconds);
	const float RangePixelWidth = RightPixel - LeftPixel;

	constexpr float CenterButtonWidth = CenterButtonSize * 2.f;
	const bool bShowCenterButtons = RangePixelWidth >= CenterButtonWidth;

	// CenterLeft and CenterRight buttons should act as a single button
	CenterLeftButton.Internal_HideShow(bShowCenterButtons);
	CenterRightButton.Internal_HideShow(bShowCenterButtons);
}

void FSimpleViewRetimeToolLattice::ResetHighlights()
{
	LeftButton.bHovered = false;
	CenterLeftButton.bHovered = false;
	CenterRightButton.bHovered = false;
	RightButton.bHovered = false;

	HoveredButton = ELatticeButtonType::None;
}

void FSimpleViewRetimeToolLattice::NotifyRangeChanged(const FMouseInputData& InMouseInput, const TRange<FFrameNumber>& InRange) const
{
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InMouseInput.Timeline->GetTimeSliderController();
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();

	const TRange<FFrameNumber> NormalizedRange = ToolableTimeline::Utils::NormalizeRange(InRange);

	const FFrameNumber LowerBoundValue = NormalizedRange.GetLowerBoundValue();
	const FFrameNumber UpperBoundValue = NormalizedRange.GetUpperBoundValue();

	const FFrameNumber FrameSpan = UpperBoundValue - LowerBoundValue;
	const FFrameNumber CenterFrame = LowerBoundValue + (FrameSpan * 0.5);

	const double LeftButtonStartSeconds = TickResolution.AsSeconds(LowerBoundValue);
	const double RightButtonStartSeconds = TickResolution.AsSeconds(UpperBoundValue);
	const double CenterButtonStartSeconds = TickResolution.AsSeconds(CenterFrame);

	const float LeftButtonPixel = InMouseInput.RangeToScreen.InputToLocalX(LeftButtonStartSeconds);
	const float RightButtonPixel = InMouseInput.RangeToScreen.InputToLocalX(RightButtonStartSeconds);
	const float CenterButtonPixel = InMouseInput.RangeToScreen.InputToLocalX(CenterButtonStartSeconds);

	static constexpr float StartAndEndPadding = 0.f;//2.f;

	// Update button positions
	LeftButton.LocalPositionX = LeftButtonPixel - LeftRightButtonSize - StartAndEndPadding;
	RightButton.LocalPositionX = RightButtonPixel + StartAndEndPadding;

	CenterRightButton.LocalPositionX = CenterButtonPixel;
	CenterLeftButton.LocalPositionX = CenterRightButton.LocalPositionX - CenterButtonSize;

	// Update center button visibility
	constexpr float CenterButtonsThresholdPx = CenterButtonSize * 2.f;

	const float RangePixelWidth = RightButtonPixel - LeftButtonPixel;
	const bool bShowCenterButtons = RangePixelWidth >= CenterButtonsThresholdPx;

	CenterLeftButton.Internal_HideShow(bShowCenterButtons);
	CenterRightButton.Internal_HideShow(bShowCenterButtons);
}

int32 FSimpleViewRetimeToolLattice::Paint(FMouseDrawInputData InMouseDrawInput, const TRange<FFrameNumber>& InRange) const
{
	NotifyRangeChanged(InMouseDrawInput, InRange);

	LeftButton.Paint(InMouseDrawInput);
	CenterLeftButton.Paint(InMouseDrawInput);
	CenterRightButton.Paint(InMouseDrawInput);
	RightButton.Paint(InMouseDrawInput);

	return InMouseDrawInput.IncrementDrawLayer().LayerId;
}

void FSimpleViewRetimeToolLattice::LockPressedState()
{
	LeftButton.bHovered = LeftButton.IsPressed();
	CenterLeftButton.bHovered = CenterLeftButton.IsPressed();
	CenterRightButton.bHovered = CenterRightButton.IsPressed();
	RightButton.bHovered = RightButton.IsPressed();

	if (LeftButton.IsPressed())
	{
		HoveredButton = ELatticeButtonType::LeftScale;
	}
	else if (CenterLeftButton.IsPressed() || CenterRightButton.IsPressed())
	{
		HoveredButton = ELatticeButtonType::CenterMove;
		CenterLeftButton.bHovered = true;
		CenterRightButton.bHovered = true;
	}
	else if (RightButton.IsPressed())
	{
		HoveredButton = ELatticeButtonType::RightScale;
	}
	else
	{
		HoveredButton = ELatticeButtonType::None;
	}
}

void FSimpleViewRetimeToolLattice::ResetButtonState()
{
	LeftButton.ReleasePress();
	CenterLeftButton.ReleasePress();
	CenterRightButton.ReleasePress();
	RightButton.ReleasePress();

	PressedButton = ELatticeButtonType::None;

	ResetHighlights();
}

FSimpleViewRetimeToolLattice::ELatticeButtonType FSimpleViewRetimeToolLattice::GetHoveredButton() const
{
	return HoveredButton;
}

bool FSimpleViewRetimeToolLattice::IsAnyButtonHovered() const
{
	return HoveredButton != ELatticeButtonType::None;
}

void FSimpleViewRetimeToolLattice::SetHoveredButton(const ELatticeButtonType InButtonType)
{
	if (IsAnyButtonPressed())
	{
		LockPressedState();
		return;
	}

	ResetHighlights();

	HoveredButton = InButtonType;

	switch (HoveredButton)
	{
	case ELatticeButtonType::LeftScale:
		LeftButton.bHovered = true;
		break;

	case ELatticeButtonType::CenterMove:
		CenterLeftButton.bHovered = true;
		CenterRightButton.bHovered = true;
		break;

	case ELatticeButtonType::RightScale:
		RightButton.bHovered = true;
		break;

	case ELatticeButtonType::None:
	default:
		break;
	}
}

FSimpleViewRetimeToolLattice::ELatticeButtonType FSimpleViewRetimeToolLattice::GetPressedButton() const
{
	return PressedButton;
}

bool FSimpleViewRetimeToolLattice::IsAnyButtonPressed() const
{
	return PressedButton != ELatticeButtonType::None;
}

void FSimpleViewRetimeToolLattice::SetPressedButton(const ELatticeButtonType InButton)
{
	ResetButtonState();

	PressedButton = InButton;

	switch (PressedButton)
	{
	case ELatticeButtonType::LeftScale:
		LeftButton.Internal_Press();
		break;

	case ELatticeButtonType::CenterMove:
		CenterLeftButton.Internal_Press();
		CenterRightButton.Internal_Press();
		break;

	case ELatticeButtonType::RightScale:
		RightButton.Internal_Press();
		break;

	case ELatticeButtonType::None:
	default:
		break;
	}

	LockPressedState();
}

} // namespace UE::Sequencer::SimpleView
