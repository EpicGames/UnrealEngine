// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerTimeSliderController.h"
#include "ToolableTimeline/ToolableTimeSliderController.h"

namespace UE::Sequencer::ToolableTimeline
{
	struct FMouseDrawInputData;
}

namespace UE::Sequencer::SimpleView
{

using namespace UE::Sequencer::ToolableTimeline;

struct FSimpleViewRetimeToolLatticeButtonArgs
{
	const FSlateBrush* ForegroundBrush = nullptr;
	const FSlateBrush* BackgroundBrush = nullptr;

	FSlateRenderTransform ForegroundRenderTransform = FSlateRenderTransform();
	FSlateRenderTransform BackgroundRenderTransform = FSlateRenderTransform();

	FVector2f ForegroundSize = FVector2f(16.f, 16.f);
	FVector2f BackgroundSize = FVector2f(16.f, 16.f);

	FLinearColor ForegroundColor = FStyleColors::Foreground.GetSpecifiedColor();
	FLinearColor BackgroundColor = FStyleColors::Select.GetSpecifiedColor();

	FLinearColor HoverForegroundColor = BackgroundColor;
	FLinearColor HoverBackgroundColor = FLinearColor::White;
};

class FSimpleViewRetimeToolLatticeButton
{
public:
	FSimpleViewRetimeToolLatticeButton() = default;
	explicit FSimpleViewRetimeToolLatticeButton(const FSimpleViewRetimeToolLatticeButtonArgs& InArgs);

	void SetArgs(const FSimpleViewRetimeToolLatticeButtonArgs& InArgs);

	FVector2f GetSize() const;

	bool IsPressed() const;
	bool TryPress();
	void ReleasePress();

	void Paint(FMouseDrawInputData& MouseDrawInput) const;

protected:
	friend class FSimpleViewRetimeToolLattice;
	
	FGeometry GetGeometry(const FMouseInputData& InMouseInput) const;

	bool HitTest(const FMouseInputData& InMouseInput) const;

	void Internal_Press();

	void Internal_HideShow(const bool bInVisible);

	FSimpleViewRetimeToolLatticeButtonArgs Args;

	float LocalPositionX = 0.f;

	bool bVisible = true;
	bool bHovered = false;
	bool bPressed = false;
};

class FSimpleViewRetimeToolLattice
{
public:
	enum class ELatticeButtonType : uint8
	{
		None,
		LeftScale,
		CenterMove,
		RightScale
	};

	FSimpleViewRetimeToolLattice(const TSharedRef<FToolableTimeline>& InTimeline);
	virtual ~FSimpleViewRetimeToolLattice() = default;

	int32 Paint(FMouseDrawInputData InMouseDrawInput, const TRange<FFrameNumber>& InRange) const;

	ELatticeButtonType HitTestHandle(const FMouseInputData& InMouseInput
		, const TRange<FFrameNumber>& InRange) const;

	ELatticeButtonType GetHoveredButton() const;
	bool IsAnyButtonHovered() const;
	void SetHoveredButton(const ELatticeButtonType InButtonType);

	ELatticeButtonType GetPressedButton() const;
	bool IsAnyButtonPressed() const;
	void SetPressedButton(const ELatticeButtonType InButton);

	void ResetButtonState();

private:
	static constexpr float CenterButtonSize = 12.f;
	static constexpr float LeftRightButtonSize = 12.f;

	void NotifyRangeChanged(const FMouseInputData& InMouseInput, const TRange<FFrameNumber>& InRange) const;

	void UpdateCenterButtonMouseMove(const FMouseInputData& InMouseInput);

	void ResetHighlights();

	void LockPressedState();

	const TSharedRef<FToolableTimeline>& Timeline;

	mutable FSimpleViewRetimeToolLatticeButton LeftButton;
	mutable FSimpleViewRetimeToolLatticeButton CenterLeftButton;
	mutable FSimpleViewRetimeToolLatticeButton CenterRightButton;
	mutable FSimpleViewRetimeToolLatticeButton RightButton;

	ELatticeButtonType HoveredButton = ELatticeButtonType::None;
	ELatticeButtonType PressedButton = ELatticeButtonType::None;
};

} // namespace UE::Sequencer::SimpleView
