// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimpleView/Tools/SimpleViewTimelineAnchor.h"
#include "CurveEditorScreenSpace.h"
#include "EditorFontGlyphs.h"
#include "Sequencer.h"
#include "SimpleView/SimpleViewUtils.h"
#include "SimpleView/Tools/SimpleViewRetimeTool.h"
#include "ToolableTimeline/MouseInputData.h"
#include "ToolableTimeline/ToolableTimelineSettings.h"
#include "ToolableTimeline/ToolableTimeline.h"
#include "ToolableTimeline/ToolableTimeSliderController.h"

using namespace UE::Sequencer::ToolableTimeline;

FSimpleViewTimelineAnchor::FSimpleViewTimelineAnchor(const FFrameTime& InFrameTime)
	: FrameTime(InFrameTime)
{
}

void FSimpleViewTimelineAnchor::GetPaintGeometry(const TSharedRef<FToolableTimeline>& InTimeline
	, const FGeometry& InGeometry
	, FGeometry& OutAnchorBarGeometry
	, FGeometry& OutCloseButtonGeometry) const
{
	Internal_GetGeometry(InTimeline, InGeometry, 0., OutAnchorBarGeometry, OutCloseButtonGeometry);
}

void FSimpleViewTimelineAnchor::GetHitGeometry(const TSharedRef<FToolableTimeline>& InTimeline
	, const FGeometry& InGeometry
	, FGeometry& OutAnchorBarGeometry
	, FGeometry& OutCloseButtonGeometry) const
{
	Internal_GetGeometry(InTimeline, InGeometry, UE::Sequencer::SimpleView::RetimeTool::ExtraHitPadding, OutAnchorBarGeometry, OutCloseButtonGeometry);
}

void FSimpleViewTimelineAnchor::Internal_GetGeometry(const TSharedRef<FToolableTimeline>& InTimeline
	, const FGeometry& InGeometry
	, const double InExtraHorizontalPadding
	, FGeometry& OutAnchorBarGeometry
	, FGeometry& OutCloseButtonGeometry) const
{
	using namespace UE::Sequencer::SimpleView;
	using namespace UE::Sequencer::SimpleView::RetimeTool;

	const UToolableTimelineSettings& TimelineSettings = InTimeline->GetTimelineSettings();
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InTimeline->GetTimeSliderController();

	const FVector2d GeometrySize = InGeometry.GetLocalSize();
	const double TickSize = TimeSliderController->GetMajorTickDrawSize();

	const FSequencerTimeSliderController::FScrubRangeToScreen RangeToScreen(TimeSliderController->GetViewRange(), GeometrySize);

	//const double BarOffsetPx = RangeToScreen.InputToLocalX(ValueInSeconds);
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	const double BarOffsetPx = RangeToScreen.InputToLocalX(TickResolution.AsSeconds(FrameTime));

	//constexpr double ButtonSizeWithPadding = CloseButtonSize + CloseButtonPadding;

	double AnchorPadding;
	double AnchorBarOffsetPy;
	//double CloseButtonOffsetPy;

	switch (TimelineSettings.Settings.LabelVerticalAlignment)
	{
	default:
	case VAlign_Fill:
	case VAlign_Center:
		AnchorPadding = 2.f;
		AnchorBarOffsetPy = (bIsAnchorBarHighlighted/* || bIsCloseButtonHighlighted*/)
			? (TickSize + AnchorPadding) : AnchorPadding;
		//CloseButtonOffsetPy = 0.0;
		break;
	case VAlign_Top:
		AnchorPadding = 0.f;
		AnchorBarOffsetPy = TickSize + AnchorPadding;
		//CloseButtonOffsetPy = 0.0;
		break;
	case VAlign_Bottom:
		AnchorPadding = 0.f;
		AnchorBarOffsetPy = AnchorPadding;
		//CloseButtonOffsetPy = GeometrySize.Y - CloseButtonSize;
		break;
	}

	// Anchor Bar
	const double ActualAnchorWidth = AnchorWidth + InExtraHorizontalPadding;
	const double AnchorBarHeight = GeometrySize.Y - AnchorBarOffsetPy;
	//const double AnchorBarHeight = ((bIsAnchorBarHighlighted/* || bIsCloseButtonHighlighted*/) && !TimeSliderController->IsDragging())
	//	? (GeometrySize.Y/* - ButtonSizeWithPadding*/) : (GeometrySize.Y - AnchorBarOffsetPy);
	const FVector2d AnchorBarSize = FVector2d(ActualAnchorWidth, AnchorBarHeight);

	const double AnchorBarOffsetPx = BarOffsetPx - (AnchorBarSize.X * 0.5);
	const FVector2d AnchorBarOffset = FVector2d(AnchorBarOffsetPx, AnchorBarOffsetPy);

	OutAnchorBarGeometry = InGeometry.MakeChild(AnchorBarSize, FSlateLayoutTransform(AnchorBarOffset));

	// Close Button
	/*const FVector2d ButtonSize = FVector2d(CloseButtonSize, CloseButtonSize);
	// Because we are drawing the close button as a text font glyph, adjust a bit by adding 1 for better visual
	const FVector2d ButtonOffset = FVector2d(BarOffsetPx - CloseButtonSizeHalf + 1.0, CloseButtonOffsetPy);

	OutCloseButtonGeometry = InGeometry.MakeChild(ButtonSize, FSlateLayoutTransform(ButtonOffset));*/
}

int32 FSimpleViewTimelineAnchor::DrawAnchor(const FSimpleViewTimelineAnchor* const InOptionalPrevAnchor
	, const FSimpleViewTimelineAnchor* const InOptionalNextAnchor
	, FMouseDrawInputData& MouseDrawInput) const
{
	FGeometry AnchorBarGeometry;
	FGeometry CloseButtonGeometry;
	GetPaintGeometry(MouseDrawInput.Timeline, MouseDrawInput.Geometry, AnchorBarGeometry, CloseButtonGeometry);

	DrawGradients(InOptionalNextAnchor, MouseDrawInput, AnchorBarGeometry);
	DrawAnchorBars(InOptionalPrevAnchor, InOptionalNextAnchor, MouseDrawInput, AnchorBarGeometry);
	DrawCloseButton(MouseDrawInput, CloseButtonGeometry);

	return MouseDrawInput.IncrementDrawLayer().LayerId;
}

int32 FSimpleViewTimelineAnchor::DrawGradients(const FSimpleViewTimelineAnchor* const InOptionalNextAnchor
	, FMouseDrawInputData& MouseDrawInput
	, const FGeometry& AnchorBarGeometry) const
{
	if (!InOptionalNextAnchor)
	{
		return MouseDrawInput.LayerId;
	}

	const TSharedRef<FToolableTimeline> Timeline = MouseDrawInput.Timeline;
	const UToolableTimelineSettings& TimelineSettings = Timeline->GetTimelineSettings();

	// Now we attempt to draw a 'background' between the anchors. This background is gradated based on the selection state
	// of the anchors - it'll fade off to gray where it doesn't affect keys. This doesn't apply to the last handle to be drawn,
	// we always try to draw from n to n+1.
	const FLinearColor SelectedGradientColor = TimelineSettings.Settings.ToolHandleColor.CopyWithNewOpacity(.5f);
	const FLinearColor UnselectedGradientColor = TimelineSettings.Settings.ToolRangeColor;//.CopyWithNewOpacity(.25f);

	FGeometry NextAnchorBarGeometry;
	FGeometry NextCloseButtonGeometry;
	InOptionalNextAnchor->GetPaintGeometry(Timeline, MouseDrawInput.Geometry, NextAnchorBarGeometry, NextCloseButtonGeometry);

	// The width is measured from the top left corners. We subtract the width of one anchor (and then offset by that much) to not overlap the anchors.
	const FVector2D NextAnchorBarLocalPosition =
		MouseDrawInput.Geometry.AbsoluteToLocal(NextAnchorBarGeometry.LocalToAbsolute(FVector2D::ZeroVector));
	const FVector2D ThisAnchorBarLocalPosition =
		MouseDrawInput.Geometry.AbsoluteToLocal(AnchorBarGeometry.LocalToAbsolute(FVector2D::ZeroVector));

	// Build a left->right span so width never goes negative
	const double LeftX = FMath::Min(ThisAnchorBarLocalPosition.X, NextAnchorBarLocalPosition.X)
		+ UE::Sequencer::SimpleView::RetimeTool::AnchorWidthHalf;
	const double RightX = FMath::Max(ThisAnchorBarLocalPosition.X, NextAnchorBarLocalPosition.X)
		+ UE::Sequencer::SimpleView::RetimeTool::AnchorWidthHalf;

	const double GradientWidth = FMath::Max(0.0, RightX - LeftX);
	if (GradientWidth <= 0.0)
	{
		return MouseDrawInput.LayerId;
	}

	const FVector2D GradientSize = FVector2D(GradientWidth, NextAnchorBarGeometry.GetLocalSize().Y);
	const FVector2D GradientOffset = FVector2D(LeftX, ThisAnchorBarLocalPosition.Y);

	const FLinearColor ThisColor = bIsSelected ? SelectedGradientColor : UnselectedGradientColor;
	const FLinearColor NextColor = InOptionalNextAnchor->bIsSelected ? SelectedGradientColor : UnselectedGradientColor;

	const bool bForwardOnScreen = (ThisAnchorBarLocalPosition.X <= NextAnchorBarLocalPosition.X);
	const FLinearColor StartColor = bForwardOnScreen ? ThisColor : NextColor;
	const FLinearColor EndColor = bForwardOnScreen ? NextColor : ThisColor;

	const FGeometry GradientGeometry = MouseDrawInput.Geometry.MakeChild(GradientSize, FSlateLayoutTransform(GradientOffset));

	TArray<FSlateGradientStop> GradientStops;
	GradientStops.Add(FSlateGradientStop(FVector2D::ZeroVector, StartColor));
	GradientStops.Add(FSlateGradientStop(GradientSize, EndColor));

	FSlateDrawElement::MakeGradient(
		MouseDrawInput.DrawElements,
		MouseDrawInput.LayerId,
		GradientGeometry.ToPaintGeometry(),
		GradientStops,
		Orient_Vertical,
		MouseDrawInput.DrawEffects
	);

	return MouseDrawInput.LayerId;
}

int32 FSimpleViewTimelineAnchor::DrawAnchorBars(const FSimpleViewTimelineAnchor* const InOptionalPrevAnchor
	, const FSimpleViewTimelineAnchor* const InOptionalNextAnchor
	, FMouseDrawInputData& MouseDrawInput
	, const FGeometry& InAnchorBarGeometry) const
{
	const TSharedRef<FToolableTimeline> Timeline = MouseDrawInput.Timeline;
	const UToolableTimelineSettings& TimelineSettings = Timeline->GetTimelineSettings();

	const bool bStartAnchor = InOptionalPrevAnchor == nullptr;
	const bool bEndAnchor = InOptionalNextAnchor == nullptr;

	// A selected anchor is solid while a non-selected one is partially transparent
	const FLinearColor SelectedAnchorColor = bIsSelected
		? TimelineSettings.Settings.ToolHandleColor.CopyWithNewOpacity(1.f)
		: TimelineSettings.Settings.ToolHandleColor.CopyWithNewOpacity(.75f);
	const FLinearColor AnchorBarColor = bIsAnchorBarHighlighted
		? UE::Sequencer::SimpleView::Utils::WhitenColor(SelectedAnchorColor) : SelectedAnchorColor;

	// Draw corner direction markers
	if (bStartAnchor || bEndAnchor)
	{
		static constexpr float CornerBrushWidth = 6.f;

		//const FVector2f AnchorBarPos = InAnchorBarGeometry.GetLocalPositionAtCoordinates(FVector2D::ZeroVector);
		const FVector2f AnchorBarPos =
			MouseDrawInput.Geometry.AbsoluteToLocal(InAnchorBarGeometry.LocalToAbsolute(FVector2D::ZeroVector));
		const FVector2f AnchorBarSize = InAnchorBarGeometry.GetLocalSize();

		const float AnchorBarLeftX = AnchorBarPos.X;
		const float AnchorBarRightX = AnchorBarPos.X + AnchorBarSize.X;

		const FVector2f BarSize = FVector2f(CornerBrushWidth, AnchorBarSize.Y);
		const FVector2f BarPosition = bStartAnchor
			? FVector2f(AnchorBarLeftX, AnchorBarPos.Y)
			: FVector2f(AnchorBarRightX - BarSize.X, AnchorBarPos.Y);

		// Bottom
		FSlateDrawElement::MakeBox(
			MouseDrawInput.DrawElements,
			MouseDrawInput.IncrementDrawLayer().LayerId,
			MouseDrawInput.Geometry.ToPaintGeometry(BarSize, FSlateLayoutTransform(BarPosition)),
			bStartAnchor
				? FAppStyle::GetBrush(TEXT("Sequencer.Timeline.PlayRange_Bottom_L"))
				: FAppStyle::GetBrush(TEXT("Sequencer.Timeline.PlayRange_Bottom_R")),
			MouseDrawInput.DrawEffects,
			AnchorBarColor
		);

		// Top
		FSlateDrawElement::MakeBox(
			MouseDrawInput.DrawElements,
			MouseDrawInput.LayerId,
			MouseDrawInput.Geometry.ToPaintGeometry(BarSize, FSlateLayoutTransform(BarPosition)),
			bStartAnchor
				? FAppStyle::GetBrush(TEXT("Sequencer.Timeline.PlayRange_Top_L"))
				: FAppStyle::GetBrush(TEXT("Sequencer.Timeline.PlayRange_Top_R")),
			MouseDrawInput.DrawEffects,
			AnchorBarColor
		);
	}

	FSlateDrawElement::MakeBox(
		MouseDrawInput.DrawElements,
		MouseDrawInput.IncrementDrawLayer().LayerId,
		InAnchorBarGeometry.ToPaintGeometry(),
		FAppStyle::GetBrush(TEXT("WhiteBrush")),
		MouseDrawInput.DrawEffects,
		AnchorBarColor
	);

	return MouseDrawInput.LayerId;
}

int32 FSimpleViewTimelineAnchor::DrawCloseButton(FMouseDrawInputData& MouseDrawInput
	, const FGeometry& InButtonGeometry) const
{
	const TSharedRef<FToolableTimeline> Timeline = MouseDrawInput.Timeline;
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = Timeline->GetTimeSliderController();

	if (TimeSliderController->IsDragging()
		|| !bIsAnchorBarHighlighted
		|| !bIsCloseButtonHighlighted)
	{
		return MouseDrawInput.LayerId;
	}

	static const FSlateFontInfo FontInfo = FAppStyle::Get().GetFontStyle(TEXT("FontAwesome.9"));

	const UToolableTimelineSettings& TimelineSettings = Timeline->GetTimelineSettings();
	const FLinearColor SelectedAnchorColor = bIsSelected
		? TimelineSettings.Settings.ToolHandleColor.CopyWithNewOpacity(1.f)
		: TimelineSettings.Settings.ToolHandleColor.CopyWithNewOpacity(.75f);
	const FLinearColor CloseButtonColor = bIsCloseButtonHighlighted
		? UE::Sequencer::SimpleView::Utils::WhitenColor(SelectedAnchorColor)
		: SelectedAnchorColor;

	FSlateDrawElement::MakeText(
		MouseDrawInput.DrawElements,
		MouseDrawInput.LayerId,
		InButtonGeometry.ToPaintGeometry(),
		FEditorFontGlyphs::Times_Circle,
		FontInfo,
		MouseDrawInput.DrawEffects,
		CloseButtonColor
	);

	return MouseDrawInput.IncrementDrawLayer().LayerId;
}
