// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerTimeSliderController.h"
#include "Templates/SharedPointer.h"
#include "SimpleViewTimelineAnchor.generated.h"

struct FGeometry;

namespace UE::Sequencer::ToolableTimeline
{
	class FToolableTimeline;
	struct FMouseDrawInputData;
}

USTRUCT()
struct FSimpleViewTimelineAnchor
{
	GENERATED_BODY()

	FSimpleViewTimelineAnchor() = default;
	FSimpleViewTimelineAnchor(const FFrameTime& InFrameTime);

	bool operator==(const FSimpleViewTimelineAnchor& InOther) const
	{
		return FrameTime == InOther.FrameTime;
	}

	void GetPaintGeometry(const TSharedRef<UE::Sequencer::ToolableTimeline::FToolableTimeline>& InTimeline
		, const FGeometry& InGeometry
		, FGeometry& OutAnchorBarGeometry
		, FGeometry& OutCloseButtonGeometry) const;

	/** Calculate the geometry for this anchor based on the supplied geometry. */
	void GetHitGeometry(const TSharedRef<UE::Sequencer::ToolableTimeline::FToolableTimeline>& InTimeline
		, const FGeometry& InGeometry
		, FGeometry& OutAnchorBarGeometry
		, FGeometry& OutCloseButtonGeometry) const;

	int32 DrawAnchor(const FSimpleViewTimelineAnchor* const InOptionalPrevAnchor
		, const FSimpleViewTimelineAnchor* const InOptionalNextAnchor
		, UE::Sequencer::ToolableTimeline::FMouseDrawInputData& MouseDrawInput) const;

	int32 DrawGradients(const FSimpleViewTimelineAnchor* InOptionalNextAnchor
		, UE::Sequencer::ToolableTimeline::FMouseDrawInputData& MouseDrawInput
		, const FGeometry& AnchorBarGeometry) const;

	int32 DrawAnchorBars(const FSimpleViewTimelineAnchor* InOptionalPrevAnchor
		, const FSimpleViewTimelineAnchor* InOptionalNextAnchor
		, UE::Sequencer::ToolableTimeline::FMouseDrawInputData& MouseDrawInput
		, const FGeometry& InAnchorBarGeometry) const;

	int32 DrawCloseButton(UE::Sequencer::ToolableTimeline::FMouseDrawInputData& MouseDrawInput
		, const FGeometry& InButtonGeometry) const;

	/** The time on the timeline that this anchor is anchored at. */
	UPROPERTY()
	FFrameTime FrameTime;

	/** Is this anchor currently selected? */
	UPROPERTY()
	bool bIsSelected = false;

	/** Is this anchor bar section currently highlighted? An anchor can be both selected and highlighted. */
	bool bIsAnchorBarHighlighted = false;

	/** Is the close button highlighted? */
	bool bIsCloseButtonHighlighted = false;

private:
	void Internal_GetGeometry(const TSharedRef<UE::Sequencer::ToolableTimeline::FToolableTimeline>& InTimeline
		, const FGeometry& InGeometry
		, const double InExtraHorizontalPadding
		, FGeometry& OutAnchorBarGeometry
		, FGeometry& OutCloseButtonGeometry) const;
};
