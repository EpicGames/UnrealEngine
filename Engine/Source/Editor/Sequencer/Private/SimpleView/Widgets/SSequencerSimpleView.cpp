// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerSimpleView.h"
#include "CurveEditor.h"
#include "ISequencer.h"
#include "ISequencerWidgetsModule.h"
#include "Misc/TransportControlsHelper.h"
#include "MVVM/CurveEditorExtension.h"
#include "MVVM/Views/SSequencerTrackAreaView.h"
#include "SequencerCommands.h"
#include "SimpleView/SequencerSimpleViewSettings.h"
#include "SimpleView/SimpleViewTimeline.h"
#include "SPopoutTabInlineContent.h"
#include "SSequencer.h"
#include "ToolableTimeline/ToolableTimeSliderController.h"
#include "ToolableTimeline/Widgets/SToolableTimeline.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SCompoundWidget.h"

#define LOCTEXT_NAMESPACE "SSequencerSimpleView"

namespace UE::Sequencer::SimpleView
{

void SSequencerSimpleView::Construct(const FArguments& InArgs, const TSharedRef<FSimpleViewTimeline>& InTimeline)
{
	Timeline = InTimeline;

	const TSharedRef<ToolableTimeline::FToolableTimeSliderController> TimeRangeController = Timeline->GetTimeSliderController();

	ISequencerWidgetsModule& SequencerWidgets = FModuleManager::Get().LoadModuleChecked<ISequencerWidgetsModule>(TEXT("SequencerWidgets"));

	TimeRangeWidget = SequencerWidgets.CreateTimeRange(Timeline->GetTimeRangeArgs()
		, SequencerWidgets.CreateTimeRangeSlider(TimeRangeController));

	Reconstruct();
}

void SSequencerSimpleView::Reconstruct()
{
	if (!Timeline.IsValid())
	{
		return;
	}
	
	const USequencerSimpleViewSettings* const SimpleViewSettings = GetDefault<USequencerSimpleViewSettings>();
	if (!SimpleViewSettings)
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = Timeline->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedRef<ISequencer> SequencerRef = Sequencer.ToSharedRef();

	const bool bTransportControlsVisible = IsPopoutTransportControlsVisible(SequencerRef, SimpleViewSettings->bShowTransportControls);

	// If the time range controls are hidden, put them on instead of showing a bunch of empty space
	const bool bTransportControlsNextToTimeline = bTransportControlsVisible
		&& !SimpleViewSettings->bShowRangeControls;

	const TSharedRef<SVerticalBox> MainVerticalBox = SNew(SVerticalBox);

	if (bTransportControlsNextToTimeline)
	{
		MainVerticalBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(2.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(0.f, 0.f, 2.f, 0.f)
				[
					ConstructTransportControls(SequencerRef)
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					ConstructTimelineArea()
				]
			];
	}
	else
	{
		MainVerticalBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(2.f)
			[
				ConstructTimelineArea()
			];
	}

	if (SimpleViewSettings->bShowRangeControls)
	{
		MainVerticalBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(2.f, 0.f, 2.f, 2.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Fill)
				[
					ConstructTransportControls(SequencerRef)
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush(TEXT("ToolPanel.GroupBorder")))
					.BorderBackgroundColor(FLinearColor(.5f, .5f, .5f, 1.f))
					.Clipping(EWidgetClipping::ClipToBoundsAlways)
					.Padding(0.f, 1.f)
					[
						TimeRangeWidget.ToSharedRef()
					]
				]
			];
	}

	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		MainVerticalBox
	];

	// Give sequencer a chance to create the outliner view widget
	if (GEditor)
	{
		const TWeakPtr<SSequencerSimpleView> ThisWeak = SharedThis(this);
		GEditor->GetTimerManager()->SetTimerForNextTick([ThisWeak]()
			{
				if (const TSharedPtr<SSequencerSimpleView> This = ThisWeak.Pin())
				{
					if (const TSharedPtr<ISequencer> Sequencer = This->Timeline->GetSequencer())
					{
						This->PinnedTrackAreaWidget->SetOutliner(Sequencer->GetOutlinerViewWidget());
					}
				}
			});
	}

	Timeline->RequestRecacheChannels();
}

TSharedRef<SBox> SSequencerSimpleView::ConstructTimelineArea()
{
	const TSharedRef<ToolableTimeline::FToolableTimeSliderController> TimeSliderController = Timeline->GetTimeSliderController();
	const TSharedRef<FTrackAreaViewModel> TrackAreaViewModel = Timeline->GetTrackAreaViewModel();

	SAssignNew(PinnedTrackAreaWidget, SSequencerTrackAreaView, TrackAreaViewModel, TimeSliderController);
	PinnedTrackAreaWidget->SetShowPinned(true);
	PinnedTrackAreaWidget->SetIsPinned(true);

	//SAssignNew(TrackAreaWidget, SSequencerTrackAreaView, TrackAreaViewModel, TimeSliderController);

	return SNew(SBox)
		.MinDesiredWidth(200.f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(0.f, 0.f, 0.f, 2.f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush(TEXT("ToolPanel.GroupBorder")))
					.BorderBackgroundColor(FLinearColor(.5f, .5f, .5f, 1.f))
					.Clipping(EWidgetClipping::ClipToBoundsAlways)
					.Padding(0)
					[
						Timeline->GenerateTimelineWidget()
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBox)
					.Clipping(EWidgetClipping::ClipToBounds)
					[
						PinnedTrackAreaWidget.ToSharedRef()
					]
				]
			]
		];
}

TSharedRef<SBox> SSequencerSimpleView::ConstructTransportControls(const TSharedRef<ISequencer>& InSequencer)
{
	return SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(0.f, 0.f, 2.f, 0.f)
		.Visibility_Lambda([this]()
			{
				const TSharedPtr<ISequencer> Sequencer = Timeline->GetSequencer();
				if (!Sequencer.IsValid())
				{
					return EVisibility::Collapsed;
				}
				const USequencerSimpleViewSettings* const SimpleViewSettings = GetDefault<USequencerSimpleViewSettings>();
				const bool bShowTransportControls = SimpleViewSettings ? SimpleViewSettings->bShowTransportControls : true;
				return IsPopoutTransportControlsVisible(Sequencer.ToSharedRef(), bShowTransportControls)
					? EVisibility::Visible : EVisibility::Collapsed;
			})
		[
			MakePopoutTransportControls(InSequencer, Timeline->GetTabManager())
		];
}

FReply SSequencerSimpleView::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) 
{
	if (InKeyEvent.GetKey() == EKeys::Escape && !Timeline->GetActiveTool().IsValid())
	{
		if (Timeline->GetKeySelection().ClearSelectedKeys())
		{
			return FReply::Handled();
		}
	}

	if (Timeline->GetCommandList()->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	// If CanDeleteSelectedKeys() returns false for the local GenericCommands.Delete binding,
	// do not fall through to Sequencer's global GenericCommands.Delete binding.
	if (InKeyEvent.GetKey() == EKeys::Delete)
	{
		return FReply::Handled();
	}

	const TSharedPtr<ISequencer> Sequencer = Timeline->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return FReply::Handled();
	}

	if (const TSharedPtr<FUICommandList> CommandBindings = Sequencer->GetCommandBindings())
	{
		if (CommandBindings->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}
	}

	if (FCurveEditorExtension* const CurveEditorExtension = Sequencer->GetViewModel()->CastDynamic<FCurveEditorExtension>())
	{
		if (const TSharedPtr<FCurveEditor> CurveEditor = CurveEditorExtension->GetCurveEditor())
		{
			if (const TSharedPtr<FUICommandList> CurveEditorCommands = CurveEditor->GetCommands())
			{
				if (CurveEditorCommands->ProcessCommandBindings(InKeyEvent))
				{
					return FReply::Handled();
				}
			}
		}
	}

	return SCompoundWidget::OnKeyDown(InGeometry, InKeyEvent);
}

FReply SSequencerSimpleView::OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.GetEffectingButton() != EKeys::RightMouseButton)
	{
		return FReply::Unhandled();
	}

	return FReply::Handled();
}

TSharedPtr<SSequencer> SSequencerSimpleView::GetSequencerWidget() const
{
	if (const TSharedPtr<FSequencer> Sequencer = Timeline->GetSequencer())
	{
		return Sequencer->GetUnderlyingSequencerWidget();
	}
	return nullptr;
}

} // namespace UE::Sequencer::SimpleView

#undef LOCTEXT_NAMESPACE
