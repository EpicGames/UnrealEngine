// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITimeSlider.h"
#include "MVVM/ViewModelPtr.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API SEQUENCER_API

class ISequencer;
class SBox;
class SSequencer;

namespace UE::Sequencer
{
	class SSequencerTrackAreaView;
}

namespace UE::Sequencer::SimpleView
{

class FSimpleViewTimeline;

/**
 * Toolable timeline widget that manages layout for the timeline, transport controls, and time range slider.
 */
class SSequencerSimpleView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSequencerSimpleView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FSimpleViewTimeline>& InTimeline);

	void Reconstruct();

	//~ Begin SWidget
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	//~ End SWidget

protected:
	TSharedRef<SBox> ConstructTimelineArea();
	TSharedRef<SBox> ConstructTransportControls(const TSharedRef<ISequencer>& InSequencer);

	TSharedPtr<SSequencer> GetSequencerWidget() const;

	/** Owner timeline instance */
	TSharedPtr<FSimpleViewTimeline> Timeline;

	/** Time range slider widget */
	TSharedPtr<ITimeSlider> TimeRangeWidget;

	/** Section area widget for pinned tracks */
	TSharedPtr<SSequencerTrackAreaView> PinnedTrackAreaWidget;

	/** Section area widget for tracks */
	//TSharedPtr<SSequencerTrackAreaView> TrackAreaWidget;
};

} // namespace UE::Sequencer::SimpleView

#undef UE_API
