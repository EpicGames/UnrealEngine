// Copyright Epic Games, Inc. All Rights Reserved.

#include "SValueStepper.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SValueStepper"

namespace UE::Sequencer::SimpleView
{

void SValueStepper::Construct(const FArguments& InArgs)
{
	IsVisibleAttribute = InArgs._IsVisible;

	ValueCommittedDelegate = InArgs._OnValueCommitted;
	LeftButtonClickDelegate = InArgs._OnLeftButtonClick;
	RightButtonClickDelegate = InArgs._OnRightButtonClick;

	static constexpr float WidgetSpacing = 2.f;

	ChildSlot
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	.Padding(0.f)
	[
		SNew(SHorizontalBox)
		.IsEnabled(InArgs._IsEnabled)
		.Visibility_Lambda([this]()
			{
				return IsVisibleAttribute.Get(false) ? EVisibility::Visible : EVisibility::Collapsed;
			})
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0.f, 0.f, WidgetSpacing, 0.f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), TEXT("HoverHintOnly"))
			.VAlign(VAlign_Center)
			.ContentPadding(0.f)
			.ToolTipText(InArgs._LeftButtonTooltipText)
			.OnClicked(this, &SValueStepper::HandleLeftButtonClick)
			[
				SNew(SBox)
				.Padding(InArgs._ButtonContentPadding)
				[
					SNew(SImage)
					.DesiredSizeOverride(InArgs._ButtonSize)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(InArgs._LeftButtonIcon.GetSmallIcon())
				]
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SSpinBox<double>)
			.TypeInterface(InArgs._NumericTypeInterface)
			.MinDesiredWidth(InArgs._MinDesiredSpinBoxWidth)
			.MinFractionalDigits(InArgs._MinFractionalDigits)
			.ToolTipText(InArgs._ValueTooltipText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Value(InArgs._InitialValue)
			.OnValueCommitted(this, &SValueStepper::HandleValueCommitted)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(WidgetSpacing, 0.f, 0.f, 0.f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), TEXT("HoverHintOnly"))
			.VAlign(VAlign_Center)
			.ContentPadding(0.f)
			.ToolTipText(InArgs._RightButtonTooltipText)
			.OnClicked(this, &SValueStepper::HandleRightButtonClick)
			[
				SNew(SBox)
				.Padding(InArgs._ButtonContentPadding)
				[
					SNew(SImage)
					.DesiredSizeOverride(InArgs._ButtonSize)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(InArgs._RightButtonIcon.GetSmallIcon())
				]
			]
		]
	];
}

void SValueStepper::HandleValueCommitted(const double InValue, const ETextCommit::Type InCommitType)
{
	ValueCommittedDelegate.ExecuteIfBound(InValue);
}

FReply SValueStepper::HandleLeftButtonClick()
{
	LeftButtonClickDelegate.ExecuteIfBound();
	return FReply::Handled();
}

FReply SValueStepper::HandleRightButtonClick()
{
	RightButtonClickDelegate.ExecuteIfBound();
	return FReply::Handled();
}

} // namespace UE::Sequencer::SimpleView

#undef LOCTEXT_NAMESPACE
