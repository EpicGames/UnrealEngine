// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::Sequencer::SimpleView
{

class SValueStepper : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnValueCommitted, double)

	SLATE_BEGIN_ARGS(SValueStepper)
		: _ButtonSize(FVector2D(22.f, 22.f))
		, _ButtonContentPadding(4.f, 2.f)
		, _MinDesiredSpinBoxWidth(28.f)
	{}
		SLATE_ARGUMENT(FSlateIcon, LeftButtonIcon)
		SLATE_ATTRIBUTE(FText, LeftButtonTooltipText)
		SLATE_EVENT(FSimpleDelegate, OnLeftButtonClick)

		SLATE_ARGUMENT(FSlateIcon, RightButtonIcon)
		SLATE_ATTRIBUTE(FText, RightButtonTooltipText)
		SLATE_EVENT(FSimpleDelegate, OnRightButtonClick)

		SLATE_ARGUMENT(FVector2D, ButtonSize)
		SLATE_ARGUMENT(FMargin, ButtonContentPadding)

		SLATE_ATTRIBUTE(bool, IsEnabled)
		SLATE_ATTRIBUTE(bool, IsVisible)

		SLATE_ATTRIBUTE(FText, ValueTooltipText)
		SLATE_ARGUMENT(double, InitialValue)
		SLATE_EVENT(FOnValueCommitted, OnValueCommitted)

		SLATE_ATTRIBUTE(FText, ToolTipText)

		SLATE_ATTRIBUTE(float, MinDesiredSpinBoxWidth)
		SLATE_ARGUMENT(int32, MinFractionalDigits)

		SLATE_ARGUMENT(TSharedPtr<INumericTypeInterface<double>>, NumericTypeInterface)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:
	void HandleValueCommitted(const double InValue, const ETextCommit::Type InCommitType);
	FReply HandleLeftButtonClick();
	FReply HandleRightButtonClick();

	TAttribute<bool> IsVisibleAttribute;

	FOnValueCommitted ValueCommittedDelegate;
	FSimpleDelegate LeftButtonClickDelegate;
	FSimpleDelegate RightButtonClickDelegate;
};

} // namespace UE::Sequencer::SimpleView
