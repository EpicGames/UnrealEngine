// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshComponentSchema.h"
#include "UniversalObjectLocators/AnimInstanceLocatorFragment.h"

#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"

#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MovieSceneBindingReferences.h"

#include "ISequencer.h"
#include "ISequencerModule.h"
#include "ScopedTransaction.h"

#include "Textures/SlateIcon.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"

#define LOCTEXT_NAMESPACE "FSkeletalMeshComponentSchema"

namespace UE::Sequencer
{

UObject* FSkeletalMeshComponentSchema::GetParentObject(UObject* Object) const
{
	if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(Object))
	{
		return AnimInstance->GetOwningComponent();
	}
	if (USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(Object))
	{
		return Component->GetOwner();
	}
	return nullptr;
}

FObjectSchemaRelevancy FSkeletalMeshComponentSchema::GetRelevancy(const UObject* InObject) const
{
	if (InObject->IsA<USkeletalMeshComponent>())
	{
		return USkeletalMeshComponent::StaticClass();
	}

	return FObjectSchemaRelevancy();
}

TSharedPtr<FExtender> FSkeletalMeshComponentSchema::ExtendObjectBindingMenu(TSharedRef<FUICommandList> CommandList, TWeakPtr<ISequencer> WeakSequencer, TArrayView<UObject* const> ContextSensitiveObjects) const
{
	UMovieSceneSequence* Sequence = WeakSequencer.Pin()->GetFocusedMovieSceneSequence();
	if (!Sequence->GetBindingReferences())
	{
		// Not supported here
		return nullptr;
	}

	TArray<USkeletalMeshComponent*> Components;
	for (UObject* Object : ContextSensitiveObjects)
	{
		if (USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(Object))
		{
			Components.Add(Component);
		}
	}

	if (Components.Num() > 0)
	{
		TSharedRef<FExtender> AddTrackMenuExtender = MakeShared<FExtender>();
		AddTrackMenuExtender->AddMenuExtension(
			SequencerMenuExtensionPoints::AddTrackMenu_PropertiesSection,
			EExtensionHook::Before,
			CommandList,
			FMenuExtensionDelegate::CreateRaw(this, &FSkeletalMeshComponentSchema::HandleTrackMenuExtensionAddTrack, WeakSequencer, Components));
		return AddTrackMenuExtender;
	}

	return nullptr;
}

void FSkeletalMeshComponentSchema::HandleTrackMenuExtensionAddTrack(FMenuBuilder& AddTrackMenuBuilder, TWeakPtr<ISequencer> WeakSequencer, TArray<USkeletalMeshComponent*> Components) const
{
	UAnimInstance* AnyAnimInstance = nullptr;
	UClass* AnyPostProcessAnimBP = nullptr;

	for (USkeletalMeshComponent* Component : Components)
	{
		if (UAnimInstance* AnimInstance = Component->GetAnimInstance())
		{
			AnyAnimInstance = AnimInstance;
		}
		if (USkeletalMesh* SkelMesh = Component->GetSkeletalMeshAsset())
		{
			TSubclassOf<UAnimInstance> PostProcessAnimBlueprint = SkelMesh->GetPostProcessAnimBlueprint();
			if (UClass* PPAnimInstanceClass = PostProcessAnimBlueprint.Get())
			{
				AnyPostProcessAnimBP = PPAnimInstanceClass;
			}
		}
	}

	FText AnimInstanceLabel = LOCTEXT("AnimInstanceLabel", "Anim Instance");
	FText DetailedAnimInstanceText = AnyAnimInstance
		? FText::FromName(AnyAnimInstance->GetClass()->ClassGeneratedBy ? AnyAnimInstance->GetClass()->ClassGeneratedBy->GetFName() : AnyAnimInstance->GetClass()->GetFName())
		: AnimInstanceLabel;

	AddTrackMenuBuilder.BeginSection("Anim Instance", AnimInstanceLabel);
	{
		AddTrackMenuBuilder.AddMenuEntry(
			DetailedAnimInstanceText,
			LOCTEXT("AnimInstanceToolTip", "Add this skeletal mesh component's animation instance."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FSkeletalMeshComponentSchema::BindAnimationInstance, WeakSequencer, Components, EAnimInstanceLocatorFragmentType::AnimInstance)
			)
		);

		if (AnyPostProcessAnimBP)
		{
			FText DetailedPostProcessInstanceText = FText::Format(LOCTEXT("PostProcessInstanceLabelFormat", "{0} [Post Process]"),
				FText::FromName(AnyPostProcessAnimBP->ClassGeneratedBy ? AnyPostProcessAnimBP->ClassGeneratedBy->GetFName() : AnyPostProcessAnimBP->GetFName()));

			AddTrackMenuBuilder.AddMenuEntry(
				DetailedPostProcessInstanceText,
				LOCTEXT("PostProcessInstanceToolTip", "Add this skeletal mesh component's post process instance."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &FSkeletalMeshComponentSchema::BindAnimationInstance, WeakSequencer, Components, EAnimInstanceLocatorFragmentType::PostProcessAnimInstance)
				)
			);
		}
	}
	AddTrackMenuBuilder.EndSection();
}

void FSkeletalMeshComponentSchema::BindAnimationInstance(TWeakPtr<ISequencer> WeakSequencer, TArray<USkeletalMeshComponent*> Components, EAnimInstanceLocatorFragmentType Type) const
{
	using namespace UE::UniversalObjectLocator;

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("BindAnimInstance", "Add Anim Instance to Sequencer"));

	UMovieSceneSequence*          Sequence          = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return;
	}

	UMovieScene*                  MovieScene        = Sequence->GetMovieScene();
	FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences();

	if (!MovieScene || !BindingReferences)
	{
		return;
	}

	FUniversalObjectLocator AnimInstanceLocator;
	AnimInstanceLocator.AddFragment<FAnimInstanceLocatorFragment>(Type);

	TMap<FGuid, USkeletalMeshComponent*> AllComponentGuids;

	static constexpr bool bCreate = true;
	for (USkeletalMeshComponent* Component : Components)
	{
		FGuid ComponentID = Sequencer->GetHandleToObject(Component, bCreate);
		if (ComponentID.IsValid())
		{
			AllComponentGuids.Add(ComponentID, Component);
		}
	}

	// Remove any that already have an anim instance binding that matches our locator
	// Currently components are not supported as spawnables so we don't need to check those
	const int32 Num = MovieScene->GetPossessableCount();
	for (int32 Index = 0; Index < Num; ++Index)
	{
		const FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);

		FGuid ParentID = Possessable.GetParent();
		if (ParentID.IsValid() && AllComponentGuids.Contains(ParentID))
		{
			for (const FMovieSceneBindingReference& Reference : BindingReferences->GetReferences(Possessable.GetGuid()))
			{
				if (Reference.Locator == AnimInstanceLocator)
				{
					AllComponentGuids.Remove(ParentID);
					break;
				}
			}
		}
		if (AllComponentGuids.Num() == 0)
		{
			return;
		}
	}

	// Make a new possessable for this binding
	Sequence->Modify();
	MovieScene->Modify();

	for (const TPair<FGuid, USkeletalMeshComponent*>& Pair : AllComponentGuids)
	{
		UClass* AnimClass = nullptr;
		FString NameString;

		if (Type == EAnimInstanceLocatorFragmentType::AnimInstance)
		{
			AnimClass = Pair.Value->AnimClass.Get();
			if (AnimClass)
			{
				NameString = AnimClass->ClassGeneratedBy ? AnimClass->ClassGeneratedBy->GetName() : AnimClass->GetName();
			}
			else
			{
				NameString = LOCTEXT("AnimInstanceLabel", "Anim Instance").ToString();
			}
		}
		else
		{
			if (USkeletalMesh* SkeletalMesh = Pair.Value->GetSkeletalMeshAsset())
			{
				AnimClass = SkeletalMesh->GetPostProcessAnimBlueprint().Get();
			}

			if (AnimClass)
			{
				NameString = FText::Format(LOCTEXT("PostProcessAnimInstanceFormat", "{0} (Post Process)"),
					FText::FromName(AnimClass->ClassGeneratedBy ? AnimClass->ClassGeneratedBy->GetFName() : AnimClass->GetFName())
				).ToString();
			}
			else
			{
				NameString = LOCTEXT("PostProcessAnimInstanceLabel", "Post Process Anim Instance").ToString();
			}
		}

		FGuid PossessableGuid = MovieScene->AddPossessable(NameString, AnimClass ? AnimClass : UAnimInstance::StaticClass());
		BindingReferences->AddBinding(PossessableGuid, CopyTemp(AnimInstanceLocator));

		FMovieScenePossessable* NewPossessable = MovieScene->FindPossessable(PossessableGuid);
		if (ensure(NewPossessable))
		{
			NewPossessable->SetParent(Pair.Key, MovieScene);
		}
	}
}


} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE