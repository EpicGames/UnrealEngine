// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ISequencerObjectSchema.h"

class ISequencer;
class FMenuBuilder;
class USkeletalMeshComponent;

enum class EAnimInstanceLocatorFragmentType;

namespace UE::Sequencer
{

class FSkeletalMeshComponentSchema : public IObjectSchema
{
public:

	UObject* GetParentObject(UObject* Object) const override;
	FObjectSchemaRelevancy GetRelevancy(const UObject* InObject) const override;
	TSharedPtr<FExtender> ExtendObjectBindingMenu(TSharedRef<FUICommandList> CommandList, TWeakPtr<ISequencer> WeakSequencer, TArrayView<UObject* const> ContextSensitiveObjects) const override;

private:

	void HandleTrackMenuExtensionAddTrack(FMenuBuilder& AddTrackMenuBuilder, TWeakPtr<ISequencer> WeakSequencer, TArray<USkeletalMeshComponent*> Components) const;
	void BindAnimationInstance(TWeakPtr<ISequencer> WeakSequencer, TArray<USkeletalMeshComponent*> Components, EAnimInstanceLocatorFragmentType Type) const;
};

} // namespace UE::Sequencer