// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannel.h"
#include "IKeyArea.h"
#include "Math/Range.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "Sequencer.h"
#include "ToolableTimeline/DragOperations/ToolableTimelineDragOperation.h"

struct FFrameNumber;
struct FKeyHandle;

namespace UE::Sequencer
{
	class FChannelModel;
}

namespace UE::Sequencer::ToolableTimeline
{

/** Cache for a single channel key */
struct FChannelKeyCache
{
	FKeyHandle Handle;

	FFrameTime InitialFrameTime;

	FFrameTime LastDraggedFrameTime;

	FFrameTime LastAppliedFrameTime;
};

/**  */
struct FChannelCacheSectionMinMax
{
	FFrameTime Min;
	FFrameTime Max;
	bool bInitialized = false;
};

/** Key range cache for a single channel */
template <typename TKeyCacheType>
struct FChannelKeyRangeCache
{
	explicit FChannelKeyRangeCache() = delete;

	explicit FChannelKeyRangeCache(const TViewModelPtr<FChannelModel>& InChannelModel
		, const TRange<FFrameNumber>& InFrameRange = TRange<FFrameNumber>::All())
		: WeakChannelModel(InChannelModel)
	{
		if (!InChannelModel.IsValid())
		{
			return;
		}

		const TSharedPtr<IKeyArea> KeyArea = InChannelModel->GetKeyArea();
		if (!KeyArea.IsValid())
		{
			return;
		}

		TArray<FKeyHandle> KeyHandles;
		KeyArea->GetKeyHandles(KeyHandles, InFrameRange);
		if (KeyHandles.IsEmpty())
		{
			return;
		}

		TArray<FFrameNumber> KeyTimes;
		KeyTimes.SetNumUninitialized(KeyHandles.Num());
		KeyArea->GetKeyTimes(KeyHandles, KeyTimes);

		KeyCache.Reserve(KeyHandles.Num());

		for (int32 KeyIndex = 0; KeyIndex < KeyHandles.Num(); ++KeyIndex)
		{
			TKeyCacheType& CachedKey = KeyCache.AddDefaulted_GetRef();
			CachedKey.Handle = KeyHandles[KeyIndex];
			CachedKey.InitialFrameTime = FFrameTime(KeyTimes[KeyIndex]);
			CachedKey.LastDraggedFrameTime = CachedKey.InitialFrameTime;
			CachedKey.LastAppliedFrameTime = CachedKey.InitialFrameTime;
		}
	}

	explicit FChannelKeyRangeCache(const TViewModelPtr<FChannelModel>& InChannelModel
		, const TArray<FKeyHandle>& InKeyHandles)
		: WeakChannelModel(InChannelModel)
	{
		if (!InChannelModel.IsValid() || InKeyHandles.IsEmpty())
		{
			return;
		}

		const TSharedPtr<IKeyArea> KeyArea = InChannelModel->GetKeyArea();
		if (!KeyArea.IsValid())
		{
			return;
		}

		TSet<FKeyHandle> UniqueKeyHandles;
		UniqueKeyHandles.Reserve(InKeyHandles.Num());

		TArray<FKeyHandle> KeyHandleArray;
		KeyHandleArray.Reserve(InKeyHandles.Num());

		for (const FKeyHandle KeyHandle : InKeyHandles)
		{
			if (!UniqueKeyHandles.Contains(KeyHandle))
			{
				UniqueKeyHandles.Add(KeyHandle);
				KeyHandleArray.Add(KeyHandle);
			}
		}

		if (UniqueKeyHandles.IsEmpty())
		{
			return;
		}

		TArray<FFrameNumber> KeyTimes;
		KeyTimes.SetNumUninitialized(UniqueKeyHandles.Num());
		KeyArea->GetKeyTimes(KeyHandleArray, KeyTimes);

		KeyCache.Reserve(UniqueKeyHandles.Num());

		for (int32 KeyIndex = 0; KeyIndex < KeyHandleArray.Num(); ++KeyIndex)
		{
			TKeyCacheType& CachedKey = KeyCache.AddDefaulted_GetRef();
			CachedKey.Handle = KeyHandleArray[KeyIndex];
			CachedKey.InitialFrameTime = FFrameTime(KeyTimes[KeyIndex]);
			CachedKey.LastDraggedFrameTime = CachedKey.InitialFrameTime;
			CachedKey.LastAppliedFrameTime = CachedKey.InitialFrameTime;
		}
	}

	void ForEachKey(TFunctionRef<bool(FChannelKeyRangeCache<TKeyCacheType>&
		, const TViewModelPtr<FChannelModel>&, const int32, TKeyCacheType&)> InFunction)
	{
		const TViewModelPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
		if (!ChannelModel.IsValid())
		{
			return;
		}

		// For each key on the channel, compute new time
		for (int32 KeyIndex = 0; KeyIndex < KeyCache.Num(); ++KeyIndex)
		{
			if (!InFunction(*this, ChannelModel, KeyIndex, KeyCache[KeyIndex]))
			{
				break;
			}
		}
	}

	/** Channel model of the key handles */
	TWeakViewModelPtr<FChannelModel> WeakChannelModel;

	TArray<TKeyCacheType> KeyCache;
};

} // namespace UE::Sequencer::ToolableTimeline
