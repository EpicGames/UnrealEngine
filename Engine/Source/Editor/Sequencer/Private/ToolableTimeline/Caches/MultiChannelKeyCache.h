// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/ParallelFor.h"
#include "ChannelKeyRangeCache.h"
#include "CurveEditor.h"
#include "IKeyArea.h"
#include "ISequencer.h"
#include "SCurveEditorView.h"
#include "Math/Range.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "Sequencer.h"
#include "ToolableTimeline/Caches/ToolableTimelineChannelCache.h"
#include "ToolableTimeline/ToolableTimeline.h"
#include "ToolableTimeline/ToolableTimelineUtils.h"
#include "ToolableTimeline/ToolableTimeSliderController.h"

class UMovieSceneSection;
struct FFrameNumber;
struct FFrameRate;
struct FKeyHandle;
struct FMovieSceneChannel;

namespace UE::Sequencer
{
	class FChannelModel;
}

namespace UE::Sequencer::ToolableTimeline
{

/** Key range cache for multiple channels */
template <typename TKeyCacheType>
class FMultiChannelKeyCache : public TSharedFromThis<FMultiChannelKeyCache<TKeyCacheType>>
{
public:
	static_assert(TIsDerivedFrom<TKeyCacheType, FChannelKeyCache>::IsDerived,
		"TKeyCacheType must derive from FChannelKeyCache");

	explicit FMultiChannelKeyCache() = delete;

	/** Constructor to cache all keys in a specific tick frame range */
	explicit FMultiChannelKeyCache(const TSharedRef<FToolableTimeline>& InTimeline
		, const TRange<FFrameNumber>& InFrameRange = TRange<FFrameNumber>::All())
	{
		for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : InTimeline->GetChannelModels())
		{
			const TViewModelPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
			if (!ChannelModel.IsValid()/* || !ChannelModel->IsAnimated()*/)
			{
				continue;
			}

			FChannelKeyRangeCache<TKeyCacheType>& NewChannelCache = ChannelCache.Emplace_GetRef(ChannelModel, InFrameRange);
			if (NewChannelCache.KeyCache.IsEmpty())
			{
				ChannelCache.Pop(EAllowShrinking::No);
			}
		}
	}

	/** Constructor to cache all keys in a specific tick frame range */
	explicit FMultiChannelKeyCache(const TSet<FSequencerSelectedKey>& InKeys)
	{
		TMap<TWeakViewModelPtr<FChannelModel>, TArray<FKeyHandle>> KeyHandlesByChannel;

		for (const FSequencerSelectedKey& SelectedKey : InKeys)
		{
			if (const TViewModelPtr<FChannelModel> ChannelModel = SelectedKey.WeakChannel.Pin())
			{
				KeyHandlesByChannel.FindOrAdd(ChannelModel).Add(SelectedKey.KeyHandle);
			}
		}

		for (const TPair<TWeakViewModelPtr<FChannelModel>, TArray<FKeyHandle>>& Pair : KeyHandlesByChannel)
		{
			const TViewModelPtr<FChannelModel> ChannelModel = Pair.Key.Pin();
			if (!ChannelModel.IsValid())
			{
				continue;
			}

			FChannelKeyRangeCache<TKeyCacheType>& NewChannelCache = ChannelCache.Emplace_GetRef(ChannelModel, Pair.Value);
			if (NewChannelCache.KeyCache.IsEmpty())
			{
				ChannelCache.Pop(EAllowShrinking::No);
			}
		}
	}

	virtual ~FMultiChannelKeyCache() = default;

	void Reset()
	{
		ChannelCache.Reset();

		WeakModifiedObjectsThisDrag.Reset();
		WeakModifiedSectionExtentsThisDrag.Reset();
	}

	bool IsEmpty() const
	{
		return ChannelCache.IsEmpty();
	}

	int32 Num() const
	{
		return ChannelCache.Num();
	}

	/** @return All channel key handles that are a part of this cache */
	TSet<FKeyHandle> GetAllKeyHandles() const
	{
		TSet<FKeyHandle> OutKeyHandles;

		for (const FChannelKeyRangeCache<TKeyCacheType>& Cache : ChannelCache)
		{
			for (const TKeyCacheType& KeyCache : Cache.KeyCache)
			{
				OutKeyHandles.Add(KeyCache.Handle);
			}
		}

		return OutKeyHandles;
	}

	/** Applies a function to each channel key range cache.
	 * Return true in the function to continue processing. Return false to stop processing early. */
	void ForEachChannelCache(TFunctionRef<bool(FChannelKeyRangeCache<TKeyCacheType>&
		, const TViewModelPtr<FChannelModel>&)> InFunction)
	{
		for (int32 ChannelIndex = 0; ChannelIndex < ChannelCache.Num(); ChannelIndex++)
		{
			FChannelKeyRangeCache<TKeyCacheType>& Cache = ChannelCache[ChannelIndex];
			if (!ensure(Cache.ChannelModel.IsValid()))
			{
				continue;
			}

			if (!InFunction(Cache, Cache.ChannelModel))
			{
				break;
			}
		}
	}

	void ForEachChannelKey(TFunctionRef<bool(FChannelKeyRangeCache<TKeyCacheType>&
		, const TViewModelPtr<FChannelModel>&, const int32, TKeyCacheType&)> InFunction)
	{
		for (int32 ChannelIndex = 0; ChannelIndex < ChannelCache.Num(); ChannelIndex++)
		{
			bool bContinue = true;

			ForEachChannelKey(ChannelIndex, [&InFunction, &bContinue](FChannelKeyRangeCache<TKeyCacheType>& InCache
				, const TViewModelPtr<FChannelModel>& InChannelModel
				, const int32 InKeyIndex
				, TKeyCacheType& InKeyCache)
			{
				if (!InFunction(InCache, InChannelModel, InKeyIndex, InKeyCache))
				{
					bContinue = false;
					return false;
				}
				return true;
			});
			
			if (!bContinue)
			{
				return;
			}
		}
	}

	void ForEachChannelKey(const int32 InChannelIndex
		, TFunctionRef<bool(FChannelKeyRangeCache<TKeyCacheType>&
			, const TViewModelPtr<FChannelModel>&, const int32, TKeyCacheType&)> InFunction)
	{
		if (ensure(ChannelCache.IsValidIndex(InChannelIndex)))
		{
			ChannelCache[InChannelIndex].ForEachKey(InFunction);
		}
	}

	void RecomputeForDrag(const TSharedRef<FToolableTimeline>& InTimeline
		, const FFrameTime& InDeltaTickTime)
	{
		const TSharedRef<FToolableTimeSliderController> TimeSliderController = InTimeline->GetTimeSliderController();
		const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
		const FFrameRate DisplayRate = TimeSliderController->GetDisplayRate();

		const bool bSnapEnabled = TimeSliderController->IsSnapEnabled();
		const int32 ChannelCount = ChannelCache.Num();

		// Compute the new key times
		// ParallelFor has overhead; don't pay it for tiny workloads
		if (ChannelCount >= ParallelThreshold)
		{
			TArray<FChannelKeyRangeCache<TKeyCacheType>>& ChannelCacheRef = ChannelCache;

			ParallelFor(ChannelCount, [&ChannelCacheRef, &TickResolution, &DisplayRate, InDeltaTickTime, bSnapEnabled]
				(const int32 InChannelIndex)
				{
					if (ensure(ChannelCacheRef.IsValidIndex(InChannelIndex)))
					{
						RecomputeLastDraggedFrameTimesFromDrag(ChannelCacheRef[InChannelIndex]
							, TickResolution, DisplayRate, InDeltaTickTime, bSnapEnabled);
					}
				});
		}
		else
		{
			for (int32 ChannelIndex = 0; ChannelIndex < ChannelCount; ++ChannelIndex)
			{
				RecomputeLastDraggedFrameTimesFromDrag(ChannelCache[ChannelIndex]
					, TickResolution, DisplayRate, InDeltaTickTime, bSnapEnabled);
			}
		}
	}

	void RestoreInitialFrameTimes()
	{
		const int32 ChannelCount = ChannelCache.Num();

		if (ChannelCount >= ParallelThreshold)
		{
			TArray<FChannelKeyRangeCache<TKeyCacheType>>& ChannelCacheRef = ChannelCache;

			ParallelFor(ChannelCount, [&ChannelCacheRef](const int32 InChannelIndex)
			{
				if (!ensure(ChannelCacheRef.IsValidIndex(InChannelIndex)))
				{
					return;
				}

				FChannelKeyRangeCache<TKeyCacheType>& Cache = ChannelCacheRef[InChannelIndex];
				for (TKeyCacheType& KeyCache : Cache.KeyCache)
				{
					KeyCache.LastDraggedFrameTime = KeyCache.InitialFrameTime;
					KeyCache.LastAppliedFrameTime = KeyCache.InitialFrameTime;
				}
			});
		}
		else
		{
			for (FChannelKeyRangeCache<TKeyCacheType>& Cache : ChannelCache)
			{
				for (TKeyCacheType& KeyCache : Cache.KeyCache)
				{
					KeyCache.LastDraggedFrameTime = KeyCache.InitialFrameTime;
					KeyCache.LastAppliedFrameTime = KeyCache.InitialFrameTime;
				}
			}
		}
	}

	void ApplyKeyTimes(const TSharedRef<FToolableTimeline>& InTimeline, const bool bInNotifyMovieSceneDataChanged = true)
	{
		TArray<FPreparedChannelApply> PreparedChannels;
		PrepareKeyTimeChanges(InTimeline, PreparedChannels);

		bool bAnyChanged = false;

		for (FPreparedChannelApply& Prepared : PreparedChannels)
		{
			if (!Prepared.bAnyChanged || !Prepared.KeyArea.IsValid())
			{
				continue;
			}

			UObject* const OwningObject = Prepared.OwningObject.Get();
			UMovieSceneSection* const Section = Prepared.Section.Get();
			if (!Section || Section->IsReadOnly())
			{
				continue;
			}

			MarkOwningObjectModifiedOnce(OwningObject);

			if (Prepared.Extents.bInitialized)
			{
				const TRange<FFrameNumber> SectionRange = Section->GetRange();
				const FFrameNumber MinFrame = Prepared.Extents.Min.FloorToFrame();
				const FFrameNumber MaxFrame = Prepared.Extents.Max.CeilToFrame();

				if (!SectionRange.Contains(MinFrame))
				{
					Section->ExpandToFrame(MinFrame);
				}
				if (!SectionRange.Contains(MaxFrame))
				{
					Section->ExpandToFrame(MaxFrame);
				}
			}

			Prepared.AppliedFrames.SetNumUninitialized(Prepared.KeyHandles.Num());
			Prepared.KeyArea->SetKeyTimes(Prepared.KeyHandles, Prepared.RequestedFrames);
			Prepared.KeyArea->GetKeyTimes(Prepared.KeyHandles, Prepared.AppliedFrames);

			for (int32 KeyIndex = 0; KeyIndex < Prepared.KeyChanges.Num(); ++KeyIndex)
			{
				FPreparedKeyTimeChange& KeyChange = Prepared.KeyChanges[KeyIndex];
				if (ensure(KeyChange.KeyCache) && ensure(Prepared.AppliedFrames.IsValidIndex(KeyIndex)))
				{
					// Track the key's actual post-apply frame rather than assuming the requested
					// frame always stuck. This keeps future drag ticks from being skipped forever
					// when SetKeyTime is rejected, clamped, or otherwise lands somewhere else.
					KeyChange.KeyCache->LastAppliedFrameTime = FFrameTime(Prepared.AppliedFrames[KeyIndex]);
				}
			}
			bAnyChanged = true;
		}

		if (bAnyChanged)
		{
			for (const TWeakObjectPtr<> WeakObject : WeakModifiedObjectsThisDrag)
			{
				if (UMovieSceneSignedObject* const SignedObject = Cast<UMovieSceneSignedObject>(WeakObject.Get()))
				{
					SignedObject->MarkAsChanged();
				}
			}

			if (const TSharedPtr<ISequencer> Sequencer = InTimeline->GetSequencer())
			{
				if (bInNotifyMovieSceneDataChanged)
				{
					Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
				}
				else
				{
					InTimeline->GetTimeSliderController()->InvalidateKeyRendererCache();
					Sequencer->RequestEvaluate();
				}
			}
		}
	}

	TArray<FChannelKeyRangeCache<TKeyCacheType>> ChannelCache;

protected:
	struct FPreparedKeyTimeChange
	{
		FKeyHandle Handle;
		FFrameTime NewTime;
		FFrameNumber RequestedFrame;
		TKeyCacheType* KeyCache = nullptr;
	};

	struct FPreparedChannelApply
	{
		int32 ChannelIndex = INDEX_NONE;

		TSharedPtr<IKeyArea> KeyArea;
		TWeakObjectPtr<UMovieSceneSection> Section;
		TWeakObjectPtr<UObject> OwningObject;

		TArray<FPreparedKeyTimeChange> KeyChanges;
		TArray<FKeyHandle> KeyHandles;
		TArray<FFrameNumber> RequestedFrames;
		TArray<FFrameNumber> AppliedFrames;

		FChannelCacheSectionMinMax Extents;
		bool bAnyChanged = false;
	};

	/**
	 * Recompute frame numbers for keys during a drag operation.
	 * Called from ParallelFor and must be thread-safe. Reads from PreDragCurveData (immutable during drag)
	 * and writes only to the channel at InChannelIndex, ensuring no race conditions between parallel iterations.
	 */
	static void RecomputeLastDraggedFrameTimesFromDrag(FChannelKeyRangeCache<TKeyCacheType>& InChannelCache
		, const FFrameRate& InTickResolution
		, const FFrameRate& InDisplayRate
		, const FFrameTime& InDeltaTickTime
		, const bool bInSnapToFrame)
	{
		InChannelCache.ForEachKey([&InTickResolution, &InDisplayRate, bInSnapToFrame, &InDeltaTickTime]
			(FChannelKeyRangeCache<TKeyCacheType>& InKeyRangeCache
				, const TViewModelPtr<FChannelModel>& InChannelModel
				, const int32 InKeyIndex
				, TKeyCacheType& InKeyCache)
			{
				FFrameTime NewKeyTime = InKeyCache.InitialFrameTime + InDeltaTickTime;

				if (bInSnapToFrame)
				{
					const FFrameTime DisplayTime = FFrameRate::TransformTime(NewKeyTime, InTickResolution, InDisplayRate);
					const FFrameNumber SnappedDisplayFrame = DisplayTime.RoundToFrame();
					NewKeyTime = FFrameRate::TransformTime(FFrameTime(SnappedDisplayFrame), InDisplayRate, InTickResolution);
				}

				InKeyCache.LastDraggedFrameTime = NewKeyTime;

				return true;
			});
	}

	void PrepareKeyTimeChanges(const TSharedRef<FToolableTimeline>& InTimeline
		, TArray<FPreparedChannelApply>& OutPreparedChannels)
	{
		const int32 ChannelCount = ChannelCache.Num();

		OutPreparedChannels.Reset();
		OutPreparedChannels.SetNum(ChannelCount);

		auto PrepareSingleChannel = [this, &OutPreparedChannels](const int32 InChannelIndex)
		{
			if (!ensure(ChannelCache.IsValidIndex(InChannelIndex)))
			{
				return;
			}

			FChannelKeyRangeCache<TKeyCacheType>& InChannelCache = ChannelCache[InChannelIndex];

			const TViewModelPtr<FChannelModel> InChannelModel = InChannelCache.WeakChannelModel.Pin();
			if (!InChannelModel.IsValid())
			{
				return;
			}

			const TSharedPtr<IKeyArea> KeyArea = InChannelModel->GetKeyArea();
			if (!KeyArea.IsValid())
			{
				return;
			}

			UMovieSceneSection* const Section = InChannelModel->GetSection();
			if (!Section || Section->IsReadOnly())
			{
				return;
			}

			UObject* const OwningObject = InChannelModel->GetOwningObject();

			FPreparedChannelApply Prepared;
			Prepared.ChannelIndex = InChannelIndex;
			Prepared.KeyArea = KeyArea;
			Prepared.Section = Section;
			Prepared.OwningObject = OwningObject;
			Prepared.KeyChanges.Reserve(InChannelCache.KeyCache.Num());
			Prepared.KeyHandles.Reserve(InChannelCache.KeyCache.Num());
			Prepared.RequestedFrames.Reserve(InChannelCache.KeyCache.Num());

			for (TKeyCacheType& InKeyCache : InChannelCache.KeyCache)
			{
				// IMPORTANT:
				// RecomputeForDrag() already produced LastDraggedFrameTime.
				// Do not resnap here.
				const FFrameTime NewKeyTime = InKeyCache.LastDraggedFrameTime;

				// Skip if this key would apply to the same rounded frame as last time.
				if (AreFrameTimesEquivalentForApply(InKeyCache.LastAppliedFrameTime, NewKeyTime))
				{
					continue;
				}

				FPreparedKeyTimeChange& NewChange = Prepared.KeyChanges.AddDefaulted_GetRef();
				NewChange.Handle = InKeyCache.Handle;
				NewChange.NewTime = NewKeyTime;
				NewChange.RequestedFrame = NewKeyTime.RoundToFrame();
				NewChange.KeyCache = &InKeyCache;

				Prepared.KeyHandles.Add(NewChange.Handle);
				Prepared.RequestedFrames.Add(NewChange.RequestedFrame);

				if (!Prepared.Extents.bInitialized)
				{
					Prepared.Extents.Min = NewKeyTime;
					Prepared.Extents.Max = NewKeyTime;
					Prepared.Extents.bInitialized = true;
				}
				else
				{
					Prepared.Extents.Min = FMath::Min(Prepared.Extents.Min, NewKeyTime);
					Prepared.Extents.Max = FMath::Max(Prepared.Extents.Max, NewKeyTime);
				}

				Prepared.bAnyChanged = true;
			}

			OutPreparedChannels[InChannelIndex] = MoveTemp(Prepared);
		};

		if (ChannelCount >= ParallelThreshold)
		{
			ParallelFor(ChannelCount, [&PrepareSingleChannel](const int32 InChannelIndex)
			{
				PrepareSingleChannel(InChannelIndex);
			});
		}
		else
		{
			for (int32 ChannelIndex = 0; ChannelIndex < ChannelCount; ++ChannelIndex)
			{
				PrepareSingleChannel(ChannelIndex);
			}
		}
	}

	void ExpandModifiedSectionsToAppliedKeyTimes()
	{
		for (const TPair<TWeakObjectPtr<UMovieSceneSection>, FChannelCacheSectionMinMax>& Pair : WeakModifiedSectionExtentsThisDrag)
		{
			UMovieSceneSection* const Section = Pair.Key.Get();
			const FChannelCacheSectionMinMax& Extents = Pair.Value;
			if (!Section || !Extents.bInitialized)
			{
				continue;
			}

			const TRange<FFrameNumber> SectionRange = Section->GetRange();
			const FFrameNumber MinFrame = Extents.Min.FloorToFrame();
			const FFrameNumber MaxFrame = Extents.Max.CeilToFrame();

			if (!SectionRange.Contains(MinFrame))
			{
				Section->ExpandToFrame(MinFrame);
			}
			if (!SectionRange.Contains(MaxFrame))
			{
				Section->ExpandToFrame(MaxFrame);
			}
		}
	}

	static bool AreFrameTimesEquivalentForApply(const FFrameTime& InA, const FFrameTime& InB)
	{
		return InA.RoundToFrame() == InB.RoundToFrame();
	}

	void MarkOwningObjectModifiedOnce(UObject* const InOwningObject)
	{
		if (!InOwningObject || WeakModifiedObjectsThisDrag.Contains(InOwningObject))
		{
			return;
		}

		InOwningObject->SetFlags(RF_Transactional);
		InOwningObject->Modify();

		if (UMovieSceneSignedObject* const SignedObject = Cast<UMovieSceneSignedObject>(InOwningObject))
		{
			SignedObject->MarkAsChanged();
			SignedObject->BroadcastChanged();
		}

		WeakModifiedObjectsThisDrag.Add(InOwningObject);
	}

	void MergeSectionExtents(UMovieSceneSection* const InSection
		, const FChannelCacheSectionMinMax& InExtents)
	{
		if (!InSection || !InExtents.bInitialized)
		{
			return;
		}

		FChannelCacheSectionMinMax& ExistingExtents = WeakModifiedSectionExtentsThisDrag.FindOrAdd(InSection);
		if (!ExistingExtents.bInitialized)
		{
			ExistingExtents = InExtents;
		}
		else
		{
			ExistingExtents.Min = FMath::Min(ExistingExtents.Min, InExtents.Min);
			ExistingExtents.Max = FMath::Max(ExistingExtents.Max, InExtents.Max);
		}
	}

	/** Save which sections have been Modified() during this drag so we don't spam transactions */
	TSet<TWeakObjectPtr<>> WeakModifiedObjectsThisDrag;
	/** Save the extents of sections that have been modified during this drag */
	TMap<TWeakObjectPtr<UMovieSceneSection>, FChannelCacheSectionMinMax> WeakModifiedSectionExtentsThisDrag;

private:
	static constexpr int32 ParallelThreshold = 16;
};

} // namespace UE::Sequencer::ToolableTimeline
