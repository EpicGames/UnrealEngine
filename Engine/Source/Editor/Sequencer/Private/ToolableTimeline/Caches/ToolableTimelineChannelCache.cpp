// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimelineChannelCache.h"
#include "ISequencerTrackEditor.h"
#include "MVVM/Extensions/ITopLevelChannelHolderExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "Sequencer.h"
#include "SimpleView/SequencerSimpleViewExtender.h"
#include "ToolableTimeline/ToolableTimeline.h"

namespace UE::Sequencer::ToolableTimeline
{

static void AddChannelModelIfKeyable(const TViewModelPtr<FChannelModel>& InChannelModel
	, TSet<TViewModelPtr<FChannelModel>>& InOutVisitedChannelModels
	, TSet<TWeakViewModelPtr<FChannelModel>>& OutWeakChannelModels)
{
	if (InChannelModel.IsValid()
		&& InChannelModel->GetKeyArea().IsValid()
		&& !InOutVisitedChannelModels.Contains(InChannelModel))
	{
		InOutVisitedChannelModels.Add(InChannelModel);
		OutWeakChannelModels.Add(InChannelModel);
	}
}

static void GatherChannelsFromViewModel(const TViewModelPtr<FViewModel>& InViewModel
	, TSet<TViewModelPtr<FChannelModel>>& InOutVisitedChannelModels
	, TSet<TWeakViewModelPtr<FChannelModel>>& OutWeakChannelModels)
{
	if (!InViewModel.IsValid())
	{
		return;
	}

	if (FChannelModel* const ChannelModel = InViewModel->CastThis<FChannelModel>())
	{
		AddChannelModelIfKeyable(InViewModel->CastThisShared<FChannelModel>(), InOutVisitedChannelModels, OutWeakChannelModels);
	}

	if (FChannelGroupModel* const ChannelGroup = InViewModel->CastThis<FChannelGroupModel>())
	{
		for (const TWeakViewModelPtr<FChannelModel>& WeakChannel : ChannelGroup->GetChannels())
		{
			AddChannelModelIfKeyable(WeakChannel.Pin(), InOutVisitedChannelModels, OutWeakChannelModels);
		}
	}

	if (ITopLevelChannelHolderExtension* const TopLevelChannelHolder = InViewModel->CastThis<ITopLevelChannelHolderExtension>())
	{
		for (const TViewModelPtr<FChannelGroupModel>& TopLevelChannel : TopLevelChannelHolder->GetTopLevelChannels().IterateSubList<FChannelGroupModel>())
		{
			for (const TWeakViewModelPtr<FChannelModel>& WeakChannel : TopLevelChannel->GetChannels())
			{
				AddChannelModelIfKeyable(WeakChannel.Pin(), InOutVisitedChannelModels, OutWeakChannelModels);
			}
		}
	}

	for (FViewModelListIterator ChildIt = InViewModel->GetChildren(EViewModelListType::Outliner); ChildIt; ++ChildIt)
	{
		if (const TViewModelPtr<IOutlinerExtension> ChildOutlinerItem = ChildIt->CastThisShared<IOutlinerExtension>())
		{
			GatherChannelsFromViewModel(ChildOutlinerItem, InOutVisitedChannelModels, OutWeakChannelModels);
		}
	}
}

static void GatherChannelsFromPinnedOutlinerItem(const TViewModelPtr<FViewModel>& InViewModel
	, TSet<TViewModelPtr<FChannelModel>>& InOutVisitedChannelModels
	, TSet<TWeakViewModelPtr<FChannelModel>>& OutWeakChannelModels)
{
	if (!InViewModel.IsValid())
	{
		return;
	}

	if (const IPinnableExtension* const PinnableExtension = InViewModel->CastThis<IPinnableExtension>())
	{
		if (PinnableExtension && PinnableExtension->IsPinned())
		{
			GatherChannelsFromViewModel(InViewModel, InOutVisitedChannelModels, OutWeakChannelModels);
			return;
		}
	}

	for (FViewModelListIterator ChildIt = InViewModel->GetChildren(EViewModelListType::Outliner); ChildIt; ++ChildIt)
	{
		GatherChannelsFromPinnedOutlinerItem(*ChildIt, InOutVisitedChannelModels, OutWeakChannelModels);
	}
}

static TSet<TWeakViewModelPtr<FChannelModel>> GetAllChannels(const ISequencer& InSequencer)
{
	TSet<TWeakViewModelPtr<FChannelModel>> OutWeakChannelModels;
	TSet<TViewModelPtr<FChannelModel>> VisitedChannelModels;

	const TSharedPtr<FSequencerEditorViewModel> RootViewModel = InSequencer.GetViewModel();
	if (!RootViewModel.IsValid())
	{
		return OutWeakChannelModels;
	}

	const TViewModelPtr<FSequenceModel> RootSequenceModel = RootViewModel->GetRootSequenceModel();
	if (!RootSequenceModel.IsValid())
	{
		return OutWeakChannelModels;
	}

	GatherChannelsFromViewModel(RootSequenceModel, VisitedChannelModels, OutWeakChannelModels);

	return OutWeakChannelModels;
}

static TSet<TWeakViewModelPtr<FChannelModel>> GetChannelsFromSelectedOutlinerItems(const ISequencer& InSequencer)
{
	TSet<TWeakViewModelPtr<FChannelModel>> OutWeakChannelModels;
	TSet<TViewModelPtr<FChannelModel>> VisitedChannelModels;

	const TSharedPtr<FSequencerEditorViewModel> RootViewModel = InSequencer.GetViewModel();
	if (!RootViewModel.IsValid())
	{
		return OutWeakChannelModels;
	}

	const TSharedPtr<FSequencerSelection> Selection = RootViewModel->GetSelection();
	if (!Selection.IsValid())
	{
		return OutWeakChannelModels;
	}

	for (const TViewModelPtr<IOutlinerExtension>& SelectedOutlinerItem : Selection->Outliner)
	{
		GatherChannelsFromViewModel(SelectedOutlinerItem, VisitedChannelModels, OutWeakChannelModels);
	}

	return OutWeakChannelModels;
}

static TSet<TWeakViewModelPtr<FChannelModel>> GetChannelsFromPinnedOutlinerItems(const ISequencer& InSequencer)
{
	TSet<TWeakViewModelPtr<FChannelModel>> OutWeakChannelModels;
	TSet<TViewModelPtr<FChannelModel>> VisitedChannelModels;

	const TSharedPtr<FSequencerEditorViewModel> RootViewModel = InSequencer.GetViewModel();
	if (!RootViewModel.IsValid())
	{
		return OutWeakChannelModels;
	}

	const TViewModelPtr<FSequenceModel> RootSequenceModel = RootViewModel->GetRootSequenceModel();
	if (!RootSequenceModel.IsValid())
	{
		return OutWeakChannelModels;
	}

	GatherChannelsFromPinnedOutlinerItem(RootSequenceModel, VisitedChannelModels, OutWeakChannelModels);

	return OutWeakChannelModels;
}

FToolableTimelineChannelCache::FToolableTimelineChannelCache(const FToolableTimeline& InTimeline)
{
	const TSharedPtr<FSequencer> Sequencer = InTimeline.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const FSequencer& SequencerRef = *Sequencer;

	switch (InTimeline.GetTimelineSettings().Settings.KeyDisplay)
	{
	default:
	case ETimelineKeyDisplay::SelectedAndPinned:
		WeakChannelModels.Append(GetChannelsFromSelectedOutlinerItems(SequencerRef));
		GatherChannelsFromAdditionalSelectedViewModels(SequencerRef, WeakChannelModels);
		WeakChannelModels.Append(GetChannelsFromPinnedOutlinerItems(SequencerRef));
		break;

	case ETimelineKeyDisplay::Selected:
		WeakChannelModels.Append(GetChannelsFromSelectedOutlinerItems(SequencerRef));
		GatherChannelsFromAdditionalSelectedViewModels(SequencerRef, WeakChannelModels);
		break;

	case ETimelineKeyDisplay::All:
		WeakChannelModels.Append(GetAllChannels(SequencerRef));
		break;
	}

	ApplyExtendedFilters(InTimeline);
}

const TSet<TWeakViewModelPtr<FChannelModel>>& FToolableTimelineChannelCache::GetChannelModels() const
{
	return WeakChannelModels;
}

void FToolableTimelineChannelCache::ApplyExtendedFilters(const FToolableTimeline& InTimeline)
{
	const TSharedPtr<FSequencer> Sequencer = InTimeline.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	if (InTimeline.GetChannelFilters().IsEmpty())
	{
		return;
	}

	for (auto It = WeakChannelModels.CreateIterator(); It; ++It)
	{
		const TViewModelPtr<FChannelModel> ChannelModel = It->Pin();
		if (!ChannelModel.IsValid())
		{
			It.RemoveCurrent();
			continue;
		}

		if (!InTimeline.PassesChannelFilters(*ChannelModel))
		{
			It.RemoveCurrent();
		}
	}
}

void FToolableTimelineChannelCache::GatherChannelsFromAdditionalSelectedViewModels(const ISequencer& InSequencer
	, TSet<TWeakViewModelPtr<FChannelModel>>& OutWeakChannelModels)
{
	TSet<TViewModelPtr<FChannelModel>> VisitedChannelModels;
	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : OutWeakChannelModels)
	{
		const TViewModelPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
		if (ChannelModel.IsValid())
		{
			VisitedChannelModels.Add(ChannelModel);
		}
	}

	for (const TPair<FDelegateHandle, FOnEnumerateAdditionalSimpleViewSelectedModels::FDelegate>& AdditionalModels
		: SimpleView::FSequencerSimpleViewExtender::GetRegisteredAdditionalSelectedModels())
	{
		AdditionalModels.Value.ExecuteIfBound(InSequencer, [&VisitedChannelModels, &OutWeakChannelModels](const FViewModelPtr& InViewModel)
			{
				GatherChannelsFromViewModel(InViewModel, VisitedChannelModels, OutWeakChannelModels);
				return true;
			});
	}
}

} // namespace UE::Sequencer::ToolableTimeline
