// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelPtr.h"
#include "ToolableTimelineChannelCache.generated.h"

class ISequencer;

UENUM()
enum class ETimelineKeyDisplay : uint8
{
	/** Show keys for selected and pinned objects on the timeline */
	SelectedAndPinned,
	/** Show keys for selected objects on the timeline */
	Selected,
	/** Show all keys for all objects on the timeline */
	All
};

namespace UE::Sequencer
{
	class FChannelModel;
}

namespace UE::Sequencer::ToolableTimeline
{

class FToolableTimeline;

class FToolableTimelineChannelCache
{
public:
	FToolableTimelineChannelCache(const FToolableTimeline& InTimeline);

	const TSet<TWeakViewModelPtr<FChannelModel>>& GetChannelModels() const;

private:
	void ApplyExtendedFilters(const FToolableTimeline& InTimeline);

	void GatherChannelsFromAdditionalSelectedViewModels(const ISequencer& InSequencer
		, TSet<TWeakViewModelPtr<FChannelModel>>& OutWeakChannelModels);

	/** Cached channel models being processed by the timeline */
	TSet<TWeakViewModelPtr<FChannelModel>> WeakChannelModels;
};

} // namespace UE::Sequencer::ToolableTimeline
