// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimelineKeyViewCacheState.h"
#include "IKeyArea.h"
#include "MovieScene.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/Views/KeyRenderer.h"
#include "Sequencer.h"

namespace UE::Sequencer::ToolableTimeline
{

FToolableTimelineKeyViewCacheState::FToolableTimelineKeyViewCacheState()
	: ValidPlayRangeMin(TNumericLimits<int32>::Lowest())
	, ValidPlayRangeMax(TNumericLimits<int32>::Max())
	, VisibleRange(0, 1)
	, KeySizePx(12., 12.)
{
}

FToolableTimelineKeyViewCacheState::FToolableTimelineKeyViewCacheState(const TRange<FFrameTime>& InVisibleRange
	, FSequencer& InSequencer)
{
	// Gather keys for a region larger than the view range to ensure we draw keys that are only just offscreen.
	// Compute visible range taking into account a half-frame offset for keys, plus half a key width for keys that are partially offscreen
	const TRange<FFrameNumber> ValidKeyRange = InSequencer.GetSubSequenceRange().Get(InSequencer.GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange());

	ValidPlayRangeMin = MovieScene::DiscreteInclusiveLower(ValidKeyRange);
	ValidPlayRangeMax = MovieScene::DiscreteExclusiveUpper(ValidKeyRange);
	VisibleRange = InVisibleRange;
	KeySizePx = FVector2D(12, 12);
	SelectionSerial = InSequencer.GetViewModel()->GetSelection()->GetSerialNumber();
	SelectionPreviewHash = InSequencer.GetSelectionPreview().GetSelectionHash();
	bShowKeyBars = InSequencer.GetSequencerSettings()->GetShowKeyBars();
	bIsChannelHovered = false;
}

EViewDependentCacheFlags FToolableTimelineKeyViewCacheState::CompareTo(const FToolableTimelineKeyViewCacheState& InOther) const
{
	EViewDependentCacheFlags Flags = EViewDependentCacheFlags::None;

	if (InOther.NonLinearTransform.Pin() != NonLinearTransform.Pin())
	{
		Flags |= EViewDependentCacheFlags::ViewChanged;
	}

	if (ValidPlayRangeMin != InOther.ValidPlayRangeMin || ValidPlayRangeMax != InOther.ValidPlayRangeMax)
	{
		// The valid key ranges for the data has changed
		Flags |= EViewDependentCacheFlags::KeyStateChanged;
	}

	if (SelectionSerial != InOther.SelectionSerial || SelectionPreviewHash != InOther.SelectionPreviewHash)
	{
		// Selection states have changed
		Flags |= EViewDependentCacheFlags::KeyStateChanged;
	}

	if (VisibleRange != InOther.VisibleRange)
	{
		Flags |= EViewDependentCacheFlags::ViewChanged;

		const double RangeSize = VisibleRange.Size<FFrameTime>().AsDecimal();
		const double OtherRangeSize = InOther.VisibleRange.Size<FFrameTime>().AsDecimal();

		if (!FMath::IsNearlyEqual(RangeSize, OtherRangeSize, RangeSize * 0.001))
		{
			Flags |= EViewDependentCacheFlags::ViewZoomed;
		}
	}

	if (bShowKeyBars != InOther.bShowKeyBars)
	{
		Flags |= EViewDependentCacheFlags::DataChanged;
	}

	if (KeySizePx != InOther.KeySizePx)
	{
		Flags |= EViewDependentCacheFlags::ViewChanged | EViewDependentCacheFlags::ViewZoomed;
	}

	return Flags;
}

} // namespace UE::Sequencer::ToolableTimeline
