// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Range.h"
#include "Math/Vector2D.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "MVVM/ViewModelPtr.h"

class FSequencer;

namespace UE::Sequencer
{
	enum class EViewDependentCacheFlags : uint8;
	struct INonLinearTimeTransform;
	struct ITrackAreaHotspot;
}

namespace UE::Sequencer::ToolableTimeline
{

/** Cache to store screen-space rendered key states */
struct FToolableTimelineKeyViewCacheState
{
	/** Default constructor for SWidget construction - not to be used under other circumstances */
	FToolableTimelineKeyViewCacheState();

	/** Real constructor for use when rendering */
	FToolableTimelineKeyViewCacheState(const TRange<FFrameTime>& InVisibleRange
		, FSequencer& InSequencer);

	/** Compare this cache to another, returning what (if anything) has changed */
	EViewDependentCacheFlags CompareTo(const FToolableTimelineKeyViewCacheState& InOther) const;

	/** Weak ptr to the last used non-linear transform for rendering these keys */
	TWeakPtr<INonLinearTimeTransform> NonLinearTransform;

	/** The min/max tick value relating to the FMovieSceneSubSequenceData::ValidPlayRange bounds, or the current playback range */
	FFrameNumber ValidPlayRangeMin, ValidPlayRangeMax;

	/** The current view range */
	TRange<FFrameTime> VisibleRange;

	/** The size of the keys to draw */
	FVector2D KeySizePx;

	/** The value of FSequencerSelection::GetSerialNumber when this cache was created */
	uint32 SelectionSerial = 0;

	/** The value of FSequencerSelectionPreview::GetSelectionHash when this cache was created */
	uint32 SelectionPreviewHash = 0;

	/** Whether to show key bars */
	bool bShowKeyBars = false;

	/** Whether this channel is hovered */
	bool bIsChannelHovered = false;
};

} // namespace UE::Sequencer::ToolableTimeline
