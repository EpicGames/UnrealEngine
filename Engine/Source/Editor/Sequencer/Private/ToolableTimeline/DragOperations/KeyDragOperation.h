// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolableTimeline/Caches/ChannelKeyRangeCache.h"
#include "ToolableTimeline/Caches/MultiChannelKeyCache.h"
#include "ToolableTimeline/DragOperations/ToolableTimelineDragOperation.h"
#include "ToolableTimeline/MouseInputData.h"

namespace UE::Sequencer::ToolableTimeline
{

/** Drag operation that caches keys */
template <typename TKeyCacheType>
class FKeyDragOperation : public FToolableTimelineDragOperation
{
public:
	FKeyDragOperation(const FMouseInputData& InMouseInput
		, const TRange<FFrameNumber>& InFrameRange, const TRange<double>& InViewRange)
		: FToolableTimelineDragOperation(InMouseInput, InViewRange)
		, ChannelCache(InMouseInput.Timeline, InFrameRange)
	{}
	FKeyDragOperation(const FMouseInputData& InMouseInput
		, const TSet<FSequencerSelectedKey>& InKeys, const TRange<double>& InViewRange)
		: FToolableTimelineDragOperation(InMouseInput, InViewRange)
		, ChannelCache(InKeys)
	{}

	FMultiChannelKeyCache<TKeyCacheType>& GetChannelCache()
	{
		return ChannelCache;
	}
	const FMultiChannelKeyCache<TKeyCacheType>& GetChannelCacheConst() const
	{
		return ChannelCache;
	}

	bool HasCachedKeys() const
	{
		for (const FChannelKeyRangeCache<TKeyCacheType>& Cache : ChannelCache.ChannelCache)
		{
			if (!Cache.KeyCache.IsEmpty())
			{
				return true;
			}
		}
		return false;
	}

	void RestoreInitialKeyTimes()
	{
		ChannelCache.RestoreInitialFrameTimes();
	}

protected:
	/** Allocated when the drag starts and freed when the drag is finished.
	 * Used to do calculations based off of the original timing of the data. */
	FMultiChannelKeyCache<TKeyCacheType> ChannelCache;
};

} // namespace UE::Sequencer::ToolableTimeline
