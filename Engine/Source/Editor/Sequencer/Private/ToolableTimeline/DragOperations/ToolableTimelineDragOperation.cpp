// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimelineDragOperation.h"
#include "ToolableTimeline/MouseInputData.h"

namespace UE::Sequencer::ToolableTimeline
{

FToolableTimelineDragOperation::FToolableTimelineDragOperation(const FMouseInputData& InMouseInput
	, const TRange<double>& InViewRange, const EAccumulationMode InAccumulationMode)
	: AccumulationMode(InAccumulationMode)
	, DelayedDrag(FDelayedDrag(
		InMouseInput.LocalPosition,
		InMouseInput.PointerEvent.GetEffectingButton()))
	, InitialViewRange(InViewRange)
	, InitialLocalWidth(FMath::Max(1.0, static_cast<double>(InMouseInput.Geometry.GetLocalSize().X)))
	, InitialLocalPointerPosition(InMouseInput.LocalPosition)
	, InitialScreenSpacePosition(InMouseInput.PointerEvent.GetScreenSpacePosition())
	, LastLocalPointerPosition(InitialLocalPointerPosition)
	, LastScreenSpacePosition(InitialScreenSpacePosition)
	, AccumulatedLocalDelta(FVector2d::ZeroVector)
	, AccumulatedScreenDelta(FVector2d::ZeroVector)
{
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InMouseInput.Timeline->GetTimeSliderController();
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	const FFrameRate DisplayRate = TimeSliderController->GetDisplayRate();

	InitialTickTime = TimeSliderController->ComputeMouseFrameTime(InMouseInput, /*bInCheckSnapping=*/false);

	const FFrameNumber InitialDisplayFrame =
		FFrameRate::TransformTime(InitialTickTime, TickResolution, DisplayRate).FloorToFrame();

	const FFrameTime StartTick = FFrameRate::TransformTime(FFrameTime(InitialDisplayFrame), DisplayRate, TickResolution);
	const FFrameTime EndExclusiveTick = FFrameRate::TransformTime(FFrameTime(InitialDisplayFrame + 1), DisplayRate, TickResolution);

	InitialDisplayFrameTickRange = TRange<FFrameTime>(
		TRangeBound<FFrameTime>::Inclusive(StartTick),
		TRangeBound<FFrameTime>::Exclusive(EndExclusiveTick)
	);
}

FToolableTimelineDragOperation::EAccumulationMode FToolableTimelineDragOperation::GetAccumulationMode() const
{
	return AccumulationMode;
}

void FToolableTimelineDragOperation::SetAccumulationMode(const FMouseInputData& InMouseInput, const EAccumulationMode InAccumulationMode)
{
	if (AccumulationMode == InAccumulationMode)
	{
		return;
	}

	AccumulationMode = InAccumulationMode;

	// Rebase so switching modes does not create a jump
	LastScreenSpacePosition = InMouseInput.PointerEvent.GetScreenSpacePosition();
	LastLocalPointerPosition = InMouseInput.Geometry.AbsoluteToLocal(LastScreenSpacePosition);

	AccumulatedScreenDelta = LastScreenSpacePosition - InitialScreenSpacePosition;
	AccumulatedLocalDelta = LastLocalPointerPosition - InitialLocalPointerPosition;
}

bool FToolableTimelineDragOperation::AttemptDragStart(const FMouseInputData& InMouseInput)
{
	return DelayedDrag.IsSet() && DelayedDrag->AttemptDragStart(InMouseInput.PointerEvent);
}

void FToolableTimelineDragOperation::ForceDragStart()
{
	if (DelayedDrag.IsSet())
	{
		DelayedDrag->ForceDragStart();
	}
}

void FToolableTimelineDragOperation::UpdateDrag(const FMouseInputData& InMouseInput)
{
	AccumulateDrag(InMouseInput);
}

void FToolableTimelineDragOperation::ResetDrag()
{
	DelayedDrag.Reset();
	ResetAccumulatedDelta();
}

bool FToolableTimelineDragOperation::HasDragOp() const
{
	return DelayedDrag.IsSet();
}

bool FToolableTimelineDragOperation::IsDragging() const
{
	return DelayedDrag.IsSet() && DelayedDrag->IsDragging();
}

void FToolableTimelineDragOperation::ResetAccumulatedDelta()
{
	LastLocalPointerPosition = InitialLocalPointerPosition;
	LastScreenSpacePosition = InitialScreenSpacePosition;
	AccumulatedLocalDelta = FVector2d::ZeroVector;
	AccumulatedScreenDelta = FVector2d::ZeroVector;
}

void FToolableTimelineDragOperation::AccumulateDrag(const FMouseInputData& InMouseInput)
{
	const FVector2D PreviousAccumulatedLocalDelta = AccumulatedLocalDelta;
	const FVector2D PreviousAccumulatedScreenDelta = AccumulatedScreenDelta;
	const EAccumulationMode PreviousAccumulationMode = AccumulationMode;
	const FVector2D ReportedScreenSpacePosition = InMouseInput.PointerEvent.GetScreenSpacePosition();
	const FVector2D CursorDelta = InMouseInput.PointerEvent.GetCursorDelta();

	// Under high-precision mouse capture the reported screen-space position can stall while
	// cursor delta continues to advance. Prefer the real pointer position when it is moving,
	// but fall back to delta accumulation when the reported position stops updating.
	FVector2D CurrentScreenSpacePosition = ReportedScreenSpacePosition;
	bool bUsedCursorDeltaFallback = false;
	if (!CursorDelta.IsNearlyZero()
		&& (ReportedScreenSpacePosition - LastScreenSpacePosition).IsNearlyZero())
	{
		CurrentScreenSpacePosition = LastScreenSpacePosition + CursorDelta;
		bUsedCursorDeltaFallback = true;
	}

	const FVector2D CurrentLocalFromScreen = InMouseInput.Geometry.AbsoluteToLocal(CurrentScreenSpacePosition);
	const double LocalWidth = InMouseInput.Geometry.GetLocalSize().X;
	const bool bInsideHorizontalBounds = CurrentLocalFromScreen.X >= 0.0 && CurrentLocalFromScreen.X <= LocalWidth;
	const EAccumulationMode NewAccumulationMode = bInsideHorizontalBounds ? EAccumulationMode::Absolute : EAccumulationMode::Delta;

	if (AccumulationMode != NewAccumulationMode)
	{
		AccumulationMode = NewAccumulationMode;
	}

	if (AccumulationMode == EAccumulationMode::Delta)
	{
		const FVector2D PreviousScreenSpacePosition = LastScreenSpacePosition;
		AccumulatedScreenDelta += (CurrentScreenSpacePosition - PreviousScreenSpacePosition);
		LastScreenSpacePosition = CurrentScreenSpacePosition;

		const FVector2D PreviousLocalFromScreen = InMouseInput.Geometry.AbsoluteToLocal(PreviousScreenSpacePosition);
		AccumulatedLocalDelta += (CurrentLocalFromScreen - PreviousLocalFromScreen);
		LastLocalPointerPosition = InitialLocalPointerPosition + AccumulatedLocalDelta;
	}
	else if (AccumulationMode == EAccumulationMode::Absolute)
	{
		LastScreenSpacePosition = CurrentScreenSpacePosition;
		LastLocalPointerPosition = CurrentLocalFromScreen;

		AccumulatedScreenDelta = LastScreenSpacePosition - InitialScreenSpacePosition;
		AccumulatedLocalDelta = LastLocalPointerPosition - InitialLocalPointerPosition;
	}

	const FVector2D AccumulatedLocalAdvance = AccumulatedLocalDelta - PreviousAccumulatedLocalDelta;
	const FVector2D AccumulatedScreenAdvance = AccumulatedScreenDelta - PreviousAccumulatedScreenDelta;
}

const FFrameTime& FToolableTimelineDragOperation::GetInitialTickTime() const
{
	return InitialTickTime;
}

FFrameTime FToolableTimelineDragOperation::GetAccumulatedDelta(const FMouseInputData& InMouseInput, const bool bInSnapEnabled) const
{
	const FFrameTime AccumulatedTickTime = GetAccumulatedTickTime(InMouseInput, bInSnapEnabled);
	return AccumulatedTickTime - InitialTickTime;
}

FFrameTime FToolableTimelineDragOperation::GetAccumulatedTickTime(const FMouseInputData& InMouseInput, const bool bInSnapEnabled) const
{
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InMouseInput.Timeline->GetTimeSliderController();
	return ComputeDraggedTickTime(TimeSliderController->GetTickResolution()
		, TimeSliderController->GetDisplayRate()
		, bInSnapEnabled);
}

FFrameNumber FToolableTimelineDragOperation::GetInitialDisplayFrame(const FFrameRate& InTickResolution, const FFrameRate& InDisplayRate) const
{
	return FFrameRate::TransformTime(InitialTickTime, InTickResolution, InDisplayRate).FloorToFrame();
}

const TRange<FFrameTime>& FToolableTimelineDragOperation::GetInitialDisplayFrameTickRange() const
{
	return InitialDisplayFrameTickRange;
}

const FVector2d& FToolableTimelineDragOperation::GetInitialLocalPointerPosition() const
{
	return InitialLocalPointerPosition;
}

const FVector2d& FToolableTimelineDragOperation::GetInitialScreenSpacePosition() const
{
	return InitialScreenSpacePosition;
}

const FVector2d& FToolableTimelineDragOperation::GetLastLocalPointerPosition() const
{
	return LastLocalPointerPosition;
}
	
const FVector2d& FToolableTimelineDragOperation::GetLastScreenSpacePosition() const
{
	return LastScreenSpacePosition;
}

const FVector2d& FToolableTimelineDragOperation::GetAccumulatedLocalDelta() const
{
	return AccumulatedLocalDelta;
}

const FVector2d& FToolableTimelineDragOperation::GetAccumulatedScreenDelta() const
{
	return AccumulatedScreenDelta;
}

FVector2d FToolableTimelineDragOperation::GetVirtualLocalPosition() const
{
	return InitialLocalPointerPosition + AccumulatedLocalDelta;
}

double FToolableTimelineDragOperation::GetInitialLocalWidth() const
{
	return InitialLocalWidth;
}

const TRange<double>& FToolableTimelineDragOperation::GetInitialViewRange() const
{
	return InitialViewRange;
}

double FToolableTimelineDragOperation::GetInitialViewSizeSeconds() const
{
	return InitialViewRange.GetUpperBoundValue() - InitialViewRange.GetLowerBoundValue();
}

double FToolableTimelineDragOperation::GetInitialSecondsPerLocalPixel() const
{
	return GetInitialViewSizeSeconds() / InitialLocalWidth;
}

FFrameTime FToolableTimelineDragOperation::ComputeDraggedTickTime(const FFrameRate& InTickResolution
	, const FFrameRate& InDisplayRate, const bool bInSnapToDisplayFrame) const
{
	const double SecondsPerLocalPixel = GetInitialSecondsPerLocalPixel();
	const double DeltaSeconds = AccumulatedLocalDelta.X * SecondsPerLocalPixel;

	FFrameTime TickTime = InitialTickTime + InTickResolution.AsFrameTime(DeltaSeconds);

	if (bInSnapToDisplayFrame)
	{
		const FFrameTime DisplayTime = FFrameRate::TransformTime(TickTime, InTickResolution, InDisplayRate);
		const FFrameNumber SnappedDisplayFrame = DisplayTime.RoundToFrame();
		TickTime = FFrameRate::TransformTime(FFrameTime(SnappedDisplayFrame), InDisplayRate, InTickResolution);
	}

	return TickTime;
}

double FToolableTimelineDragOperation::ComputePanSecondsFromVirtualPointerOverflow() const
{
	const double VirtualLocalX = GetVirtualLocalPosition().X;

	double OverflowPx = 0.0;

	if (VirtualLocalX < 0.0)
	{
		OverflowPx = VirtualLocalX;
	}
	else if (VirtualLocalX > InitialLocalWidth)
	{
		OverflowPx = VirtualLocalX - InitialLocalWidth;
	}

	return OverflowPx * GetInitialSecondsPerLocalPixel();
}

FVector2d FToolableTimelineDragOperation::GetScrubScreenSpacePosition(const FGeometry& InGeometry) const
{
	const FVector2d CurrentAbsoluteScreen = LastScreenSpacePosition;

	const FVector2d CurrentAbsoluteLocal = InGeometry.AbsoluteToLocal(CurrentAbsoluteScreen);
	const double LocalWidth = InGeometry.GetLocalSize().X;

	const bool bInsideHorizontalBounds = CurrentAbsoluteLocal.X >= 0.0 && CurrentAbsoluteLocal.X <= LocalWidth;

	FVector2d Result = CurrentAbsoluteScreen;

	if (!bInsideHorizontalBounds)
	{
		// Preserve infinite drag horizontally using accumulated delta
		Result.X = InitialScreenSpacePosition.X + AccumulatedScreenDelta.X;
	}

	// Always preserve true screen Y, even if above/below widget
	Result.Y = CurrentAbsoluteScreen.Y;

	return Result;
}

bool FToolableTimelineDragOperation::IsMouseInsideWidget(const FMouseInputData& InMouseInput) const
{
	const FVector2D LocalMouse = InMouseInput.Geometry.AbsoluteToLocal(InMouseInput.PointerEvent.GetScreenSpacePosition());
	const FVector2D LocalSize = InMouseInput.Geometry.GetLocalSize();
	return LocalMouse.X >= 0.0 && LocalMouse.X <= LocalSize.X
		&& LocalMouse.Y >= 0.0 && LocalMouse.Y <= LocalSize.Y;
}

bool FToolableTimelineDragOperation::IsMouseInsideHorizontalBounds(const FMouseInputData& InMouseInput) const
{
	const FVector2D LocalMouse = InMouseInput.Geometry.AbsoluteToLocal(InMouseInput.PointerEvent.GetScreenSpacePosition());
	const double LocalWidth = InMouseInput.Geometry.GetLocalSize().X;
	return LocalMouse.X >= 0.0 && LocalMouse.X <= LocalWidth;
}

double FToolableTimelineDragOperation::DragPixelsToInputSeconds(const FMouseInputData& InMouseInput) const
{
	const double PixelsPerInput = InMouseInput.CalculatePixelsPerFrame();
	if (PixelsPerInput <= SMALL_NUMBER)
	{
		return 0.0;
	}

	const double AccumulatedLocalDeltaX = GetAccumulatedLocalDelta().X;
	return AccumulatedLocalDeltaX / PixelsPerInput;
}

FFrameTime FToolableTimelineDragOperation::DragPixelsToTickFrameTime(const FMouseInputData& InMouseInput) const
{
	const double PixelsPerDisplayFrame = InMouseInput.CalculatePixelsPerFrame();
	if (PixelsPerDisplayFrame <= SMALL_NUMBER)
	{
		return FFrameTime();
	}

	const double AccumulatedLocalDeltaX = GetAccumulatedLocalDelta().X;
	const double DeltaDisplayFrames = AccumulatedLocalDeltaX / PixelsPerDisplayFrame;

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InMouseInput.Timeline->GetTimeSliderController();
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	const FFrameRate DisplayRate = TimeSliderController->GetDisplayRate();

	return FFrameRate::TransformTime(FFrameTime::FromDecimal(DeltaDisplayFrames), DisplayRate, TickResolution);
}

FFrameTime FToolableTimelineDragOperation::SnapTime(const FFrameTime& InTime
	, const FFrameRate& InTickResolution, const FFrameRate& InDisplayRate, const bool bInSnapEnabled)
{
	if (!bInSnapEnabled)
	{
		return InTime;
	}

	const FFrameTime DisplayTime = FFrameRate::TransformTime(InTime, InTickResolution, InDisplayRate);

	return FFrameRate::TransformTime(
		FFrameTime(DisplayTime.RoundToFrame()), InDisplayRate, InTickResolution);
}

} // UE::Sequencer::ToolableTimeline
