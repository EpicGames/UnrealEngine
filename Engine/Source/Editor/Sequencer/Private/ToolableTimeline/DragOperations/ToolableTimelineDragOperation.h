// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/DelayedDrag.h"
#include "Math/MathFwd.h"
#include "Misc/FrameTime.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"

namespace UE::Sequencer::ToolableTimeline
{

struct FMouseInputData;

/**
 * Represents a drag operation in a timeline, providing tools for managing and processing
 * interactions such as dragging, accumulating deltas, and adjusting view ranges.
 * This class facilitates user input handling during interactive operations on timeline widgets.
 */
class FToolableTimelineDragOperation : public TSharedFromThis<FToolableTimelineDragOperation>
{
public:
	static FFrameTime SnapTime(const FFrameTime& InTime
		, const FFrameRate& InTickResolution, const FFrameRate& InDisplayRate, const bool bInSnapEnabled);

	enum class EAccumulationMode : uint8
	{
		Absolute,
		Delta
	};

	explicit FToolableTimelineDragOperation(const FMouseInputData& InMouseInput
		, const TRange<double>& InViewRange, const EAccumulationMode InAccumulationMode = EAccumulationMode::Absolute);

	FToolableTimelineDragOperation(const FToolableTimelineDragOperation&) = delete;
	FToolableTimelineDragOperation(FToolableTimelineDragOperation&&) = delete;
	virtual ~FToolableTimelineDragOperation() {}

	FToolableTimelineDragOperation& operator=(const FToolableTimelineDragOperation&) = delete;
	FToolableTimelineDragOperation& operator=(FToolableTimelineDragOperation&&) = delete;

	virtual void UpdateDrag(const FMouseInputData& InMouseInput);
	virtual void ResetDrag();

	/**
	 * Retrieves the accumulation mode used in the drag operation.
	 * The accumulation mode determines how deltas from drag operations are accumulated
	 * and processed during the interaction with the timeline.
	 * @return The current accumulation mode for the drag operation.
	 */
	EAccumulationMode GetAccumulationMode() const;
	void SetAccumulationMode(const FMouseInputData& InMouseInput, const EAccumulationMode InAccumulationMode);

	bool AttemptDragStart(const FMouseInputData& InMouseInput);
	void ForceDragStart();

	bool HasDragOp() const;

	bool IsDragging() const;

	void ResetAccumulatedDelta();
	void AccumulateDrag(const FMouseInputData& InMouseInput);

	const FFrameTime& GetInitialTickTime() const;

	FFrameTime GetAccumulatedDelta(const FMouseInputData& InMouseInput, const bool bInSnapEnabled) const;
	FFrameTime GetAccumulatedTickTime(const FMouseInputData& InMouseInput, const bool bInSnapEnabled) const;

	FFrameNumber GetInitialDisplayFrame(const FFrameRate& InTickResolution, const FFrameRate& InDisplayRate) const;
	const TRange<FFrameTime>& GetInitialDisplayFrameTickRange() const;

	const FVector2d& GetInitialLocalPointerPosition() const;
	const FVector2d& GetInitialScreenSpacePosition() const;

	const FVector2d& GetLastLocalPointerPosition() const;
	const FVector2d& GetLastScreenSpacePosition() const;

	const FVector2d& GetAccumulatedLocalDelta() const;
	const FVector2d& GetAccumulatedScreenDelta() const;

	FVector2d GetVirtualLocalPosition() const;

	double GetInitialLocalWidth() const;
	const TRange<double>& GetInitialViewRange() const;
	double GetInitialViewSizeSeconds() const;
	double GetInitialSecondsPerLocalPixel() const;

	/**
	 * Computes the tick time based on the drag operation's accumulated delta and the provided frame rates.
	 * The method can optionally snap the resulting tick time to the nearest display frame when enabled.
	 * @param InTickResolution The frame rate at which the timeline's ticks are resolved.
	 * @param InDisplayRate The frame rate used for display purposes on the timeline.
	 * @param bInSnapToDisplayFrame A flag indicating whether to snap the computed tick time to the nearest display frame.
	 * @return The computed tick time adjusted by the drag operation.
	 */
	FFrameTime ComputeDraggedTickTime(const FFrameRate& InTickResolution
		, const FFrameRate& InDisplayRate, bool bInSnapToDisplayFrame) const;

	/**
	 * Computes the pan amount in seconds, derived from the virtual pointer's overflow beyond the boundaries
	 * of the local view during a drag operation.
	 * The result indicates how far the pointer has moved beyond the defined timeline boundaries,
	 * where positive values represent overflow to the right and negative values represent overflow to the left.
	 * @return The computed pan amount in seconds resulting from the virtual pointer's overflow.
	 */
	double ComputePanSecondsFromVirtualPointerOverflow() const;

	/**
	 * Determines the current screen space position of the scrub handle during a drag operation.
	 * This method takes into account the widget's geometry and accumulated deltas,
	 * ensuring proper handling even when the pointer moves beyond the widget's bounds.
	 * @param InGeometry The geometry of the widget currently being interacted with. Provides
	 *                   information such as the size and position of the widget in screen space.
	 * @return The screen space position of the scrub handle, adjusted for
	 *         both local and screen boundaries.
	 */
	FVector2d GetScrubScreenSpacePosition(const FGeometry& InGeometry) const;

	double DragPixelsToInputSeconds(const FMouseInputData& InMouseInput) const;
	FFrameTime DragPixelsToTickFrameTime(const FMouseInputData& InMouseInput) const;

private:
	bool IsMouseInsideWidget(const FMouseInputData& InMouseInput) const;
	bool IsMouseInsideHorizontalBounds(const FMouseInputData& InMouseInput) const;

	/**
	 * Specifies the mode of accumulation for deltas during drag operations.
	 * This setting determines whether deltas are accumulated in an absolute mode,
	 * where the total delta is tracked, or in a relative mode, where only the
	 * immediate delta for each interaction is considered.
	 */
	EAccumulationMode AccumulationMode = EAccumulationMode::Absolute;

	/**
	 * Represents an optional delayed drag operation within the timeline interaction system.
	 * This variable encapsulates additional state or configuration that tracks a deferred drag action,
	 * enabling the handling of user input and related events during extended or postponed interactions.
	 */
	TOptional<FDelayedDrag> DelayedDrag;

	/**
	 * Stores the initial tick time at the start of a drag operation.
	 * This value represents the position on the timeline where the interaction began,
	 * expressed in frame time units. It serves as a reference point for computing
	 * deltas during the drag operation, allowing for adjustments and updates to the
	 * timeline's state based on user interactions.
	 */
	FFrameTime InitialTickTime;

	/**  */
	TRange<FFrameTime> InitialDisplayFrameTickRange;

	/**
	 * Represents the initial view range of the timeline at the start of a drag operation.
	 * This range defines the bounds of the visible timeline section when the interaction begins.
	 * It is used as a baseline for calculating deltas and updating the timeline's state during the drag operation.
	 */
	TRange<double> InitialViewRange;

	/**
	 * Represents the initial width of the local view in screen space at the start of a drag operation.
	 * This value is used as a baseline for calculating pointer movements and interactions during
	 * timeline adjustments. It ensures consistent handling of user input based on the initial geometry
	 * of the timeline widget.
	 */
	double InitialLocalWidth = 0.0;

	FVector2d InitialLocalPointerPosition = FVector2d::ZeroVector;
	FVector2d InitialScreenSpacePosition = FVector2d::ZeroVector;

	FVector2d LastLocalPointerPosition = FVector2d::ZeroVector;
	FVector2d LastScreenSpacePosition = FVector2d::ZeroVector;

	FVector2d AccumulatedLocalDelta = FVector2d::ZeroVector;
	FVector2d AccumulatedScreenDelta = FVector2d::ZeroVector;
};

} // namespace UE::Sequencer::ToolableTimeline
