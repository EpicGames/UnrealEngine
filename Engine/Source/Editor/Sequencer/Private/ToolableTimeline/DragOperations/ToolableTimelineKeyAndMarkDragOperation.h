// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Selection/Selection.h"
#include "Sequencer.h"
#include "SequencerSelectedKey.h"
#include "ToolableTimeline/DragOperations/KeyDragOperation.h"
#include "ToolableTimeline/DragOperations/ToolableTimelineMarkDragOperation.h"
#include "ToolableTimeline/MouseInputData.h"
#include "ToolableTimeline/ToolableTimeline.h"

namespace UE::Sequencer::ToolableTimeline
{

template <typename TKeyCacheType>
class FToolableTimelineKeyAndMarkDragOperation
{
public:
	FToolableTimelineKeyAndMarkDragOperation(const FMouseInputData& InMouseInput
		, const TSet<FSequencerSelectedKey>& InKeys, const TSet<int32>& InMarkedFrames, const TRange<double>& InViewRange)
	{
		DragSource = EDragSource::Key;

		KeyDragOperation.Emplace(InMouseInput, InKeys, InViewRange);
		MarkDragOperation.Emplace(InMouseInput, InViewRange, InMarkedFrames);
	}

	FToolableTimelineKeyAndMarkDragOperation(const FMouseInputData& InMouseInput
		, const TRange<double>& InViewRange, const int32 InHitMarkIndex)
	{
		DragSource = EDragSource::Mark;

		const TSet<FSequencerSelectedKey> SelectedKeys = GetSelectedKeys(InMouseInput);
		if (!SelectedKeys.IsEmpty())
		{
			KeyDragOperation.Emplace(InMouseInput, SelectedKeys, InViewRange);
		}

		MarkDragOperation.Emplace(InMouseInput, InViewRange, InHitMarkIndex);
	}

	void UpdateDrag(const FMouseInputData& InMouseInput)
	{
		if (DragSource == EDragSource::Mark && MarkDragOperation.IsSet())
		{
			MarkDragOperation->UpdateDrag(InMouseInput);
		}
		else if (DragSource == EDragSource::Key && KeyDragOperation.IsSet())
		{
			KeyDragOperation->UpdateDrag(InMouseInput);
		}
	}

	void UpdateMarkDragAndKeys(const FMouseInputData& InMouseInput)
	{
		if (!MarkDragOperation.IsSet())
		{
			return;
		}

		MarkDragOperation->UpdateDrag(InMouseInput);

		const FFrameTime TotalKeyDelta = MarkDragOperation->GetTotalDraggedMarkDelta();
		if (TotalKeyDelta == 0 || !KeyDragOperation.IsSet())
		{
			return;
		}

		KeyDragOperation->GetChannelCache().RecomputeForDrag(InMouseInput.Timeline, TotalKeyDelta);
		KeyDragOperation->GetChannelCache().ApplyKeyTimes(InMouseInput.Timeline, /*bInNotifyMovieSceneDataChanged=*/false);
	}

	void UpdateMarkedFramesFromKeyDrag(const TSharedRef<FToolableTimeline>& InTimeline, const FFrameTime& InTotalDeltaTime)
	{
		if (MarkDragOperation.IsSet())
		{
			MarkDragOperation->ApplyDelta(InTimeline, InTotalDeltaTime);
		}
	}

	void RestoreInitialKeyAndMarkTimes(const TSharedRef<FToolableTimeline>& InTimeline)
	{
		if (KeyDragOperation.IsSet())
		{
			KeyDragOperation->RestoreInitialKeyTimes();
		}

		if (MarkDragOperation.IsSet())
		{
			MarkDragOperation->ApplyDelta(InTimeline, FFrameTime(0));
		}
	}

	void SortMarkedFrames(const TSharedRef<FToolableTimeline>& InTimeline)
	{
		if (MarkDragOperation.IsSet())
		{
			MarkDragOperation->SortMarkedFrames(InTimeline);
		}
	}

	void ResetDrag()
	{
		if (KeyDragOperation.IsSet())
		{
			KeyDragOperation->ResetDrag();
		}
		if (MarkDragOperation.IsSet())
		{
			MarkDragOperation->ResetDrag();
		}
	}

	bool HasDragOp() const
	{
		const FToolableTimelineDragOperation* const DragOperation = GetDragOperation();
		return DragOperation ? DragOperation->HasDragOp() : false;
	}

	bool IsDragging() const
	{
		const FToolableTimelineDragOperation* const DragOperation = GetDragOperation();
		return DragOperation ? DragOperation->IsDragging() : false;
	}

	bool HasCachedKeys() const
	{
		return KeyDragOperation.IsSet() && KeyDragOperation->HasCachedKeys();
	}

	FVector2d GetAccumulatedLocalDelta() const
	{
		const FToolableTimelineDragOperation* const DragOperation = GetDragOperation();
		return DragOperation ? DragOperation->GetAccumulatedLocalDelta() : FVector2d::ZeroVector;
	}

	FFrameTime DragPixelsToTickFrameTime(const FMouseInputData& InMouseInput) const
	{
		const FToolableTimelineDragOperation* const DragOperation = GetDragOperation();
		return DragOperation ? DragOperation->DragPixelsToTickFrameTime(InMouseInput) : FFrameTime(0);
	}

	TRange<FFrameTime> GetInitialDisplayFrameTickRange() const
	{
		const FToolableTimelineDragOperation* const DragOperation = GetDragOperation();
		return DragOperation ? DragOperation->GetInitialDisplayFrameTickRange() : TRange<FFrameTime>();
	}

	FFrameTime ComputeDraggedTickTime(const FFrameRate& InTickResolution
		, const FFrameRate& InDisplayRate, const bool bInSnapToDisplayFrame) const
	{
		const FToolableTimelineDragOperation* const DragOperation = GetDragOperation();
		return DragOperation
			? DragOperation->ComputeDraggedTickTime(InTickResolution, InDisplayRate, bInSnapToDisplayFrame)
			: FFrameTime(0);
	}

	FMultiChannelKeyCache<TKeyCacheType>& GetChannelCache()
	{
		check(KeyDragOperation.IsSet());
		return KeyDragOperation->GetChannelCache();
	}

private:
	enum class EDragSource : uint8
	{
		Key,
		Mark
	};

	static TSet<FSequencerSelectedKey> GetSelectedKeys(const FMouseInputData& InMouseInput)
	{
		TSet<FSequencerSelectedKey> SelectedKeys = InMouseInput.Timeline->GetKeySelection().GetSelectedKeys();

		if (const TSharedPtr<FSequencer> Sequencer = InMouseInput.Timeline->GetSequencer())
		{
			FSequencerSelectedKey::AppendKeySelection(SelectedKeys, Sequencer->GetSelection().KeySelection);
		}

		return SelectedKeys;
	}

	const FToolableTimelineDragOperation* GetDragOperation() const
	{
		if (DragSource == EDragSource::Mark && MarkDragOperation.IsSet())
		{
			return &MarkDragOperation.GetValue();
		}
		if (DragSource == EDragSource::Key && KeyDragOperation.IsSet())
		{
			return &KeyDragOperation.GetValue();
		}
		return nullptr;
	}

	EDragSource DragSource = EDragSource::Key;

	TOptional<FKeyDragOperation<TKeyCacheType>> KeyDragOperation;
	TOptional<FToolableTimelineMarkDragOperation> MarkDragOperation;
};

} // namespace UE::Sequencer::ToolableTimeline
