// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimelineMarkDragOperation.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MVVM/Selection/Selection.h"
#include "Sequencer.h"
#include "ToolableTimeline/MouseInputData.h"

namespace UE::Sequencer::ToolableTimeline
{

FToolableTimelineMarkDragOperation::FToolableTimelineMarkDragOperation(const FMouseInputData& InMouseInput
	, const TRange<double>& InViewRange, const TSet<int32>& InMarkedFrames)
	: FToolableTimelineDragOperation(InMouseInput, InViewRange)
	, TotalDraggedMarkDelta(FFrameTime(0))
	, DraggedMarkIndex(INDEX_NONE)
{
	CacheMarks(InMouseInput, InMarkedFrames);
}

FToolableTimelineMarkDragOperation::FToolableTimelineMarkDragOperation(const FMouseInputData& InMouseInput
	, const TRange<double>& InViewRange, const int32 InHitMarkIndex)
	: FToolableTimelineDragOperation(InMouseInput, InViewRange)
	, TotalDraggedMarkDelta(FFrameTime(0))
	, DraggedMarkIndex(InHitMarkIndex)
{
	CacheMarks(InMouseInput, InHitMarkIndex, /*bInFallbackToHitMark=*/true);
}

void FToolableTimelineMarkDragOperation::UpdateDrag(const FMouseInputData& InMouseInput)
{
	FToolableTimelineDragOperation::UpdateDrag(InMouseInput);

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InMouseInput.Timeline->GetTimeSliderController();
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	const FFrameRate DisplayRate = TimeSliderController->GetDisplayRate();
	const bool bSnapEnabled = TimeSliderController->IsSnapEnabled();

	const FFrameTime NewTotalDraggedMarkDelta = Recompute(InMouseInput, TickResolution, DisplayRate, bSnapEnabled);
	TotalDraggedMarkDelta = NewTotalDraggedMarkDelta;

	for (const TPair<int32, FFrameNumber>& Pair : CurrentTimes)
	{
		TimeSliderController->SetMarkAtFrame(Pair.Key, Pair.Value);
	}
}

void FToolableTimelineMarkDragOperation::ResetDrag()
{
	InitialTimes.Empty();
	CurrentTimes.Empty();

	TotalDraggedMarkDelta = FFrameTime(0);
}

FFrameTime FToolableTimelineMarkDragOperation::GetTotalDraggedMarkDelta() const
{
	return TotalDraggedMarkDelta;
}

void FToolableTimelineMarkDragOperation::ApplyDelta(const TSharedRef<FToolableTimeline>& InTimeline
	, const FFrameTime& InTotalDeltaTime)
{
	TotalDraggedMarkDelta = InTotalDeltaTime;

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InTimeline->GetTimeSliderController();
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	const FFrameRate DisplayRate = TimeSliderController->GetDisplayRate();
	const bool bSnapEnabled = TimeSliderController->IsSnapEnabled();

	for (const TPair<int32, FFrameNumber>& Pair : InitialTimes)
	{
		const FFrameTime NewTickTime = SnapTime(FFrameTime(Pair.Value) + TotalDraggedMarkDelta
			, TickResolution, DisplayRate, bSnapEnabled);
		const FFrameNumber NewMarkTime = NewTickTime.FrameNumber;
		CurrentTimes.FindOrAdd(Pair.Key) = NewMarkTime;
		TimeSliderController->SetMarkAtFrame(Pair.Key, NewMarkTime);
	}
}

void FToolableTimelineMarkDragOperation::SortMarkedFrames(const TSharedRef<FToolableTimeline>& InTimeline)
{
	const TSharedPtr<FSequencer> Sequencer = InTimeline->GetSequencer();
	const UMovieSceneSequence* const FocusedMovieSequence = Sequencer.IsValid() ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	UMovieScene* const FocusedMovieScene = FocusedMovieSequence ? FocusedMovieSequence->GetMovieScene() : nullptr;
	if (FocusedMovieScene)
	{
		FocusedMovieScene->SortMarkedFrames();
	}
}

void FToolableTimelineMarkDragOperation::CacheMarks(const FMouseInputData& InMouseInput
	, const int32 InHitMarkIndex, const bool bInFallbackToHitMark)
{
	const TSharedPtr<FSequencer> Sequencer = InMouseInput.Timeline->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencerEditorViewModel> ViewModel = Sequencer->GetViewModel();
	if (!ViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencerSelection> SequencerSelection = ViewModel->GetSelection();
	if (!SequencerSelection.IsValid())
	{
		return;
	}

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InMouseInput.Timeline->GetTimeSliderController();
	const TArray<FMovieSceneMarkedFrame>& MarkedFrames = TimeSliderController->GetTimeSliderArgs().MarkedFrames.Get();
	const int32 MarkCount = MarkedFrames.Num();

	InitialTimes.Empty(MarkCount);
	CurrentTimes.Empty(MarkCount);

	bool bAddedAnySelected = false;

	for (const int32 MarkIndex : SequencerSelection->MarkedFrames.GetSelected())
	{
		if (MarkedFrames.IsValidIndex(MarkIndex))
		{
			InitialTimes.Add(MarkIndex, MarkedFrames[MarkIndex].FrameNumber);
			CurrentTimes.Add(MarkIndex, MarkedFrames[MarkIndex].FrameNumber);

			bAddedAnySelected = true;
		}
	}

	if (bInFallbackToHitMark && !bAddedAnySelected && MarkedFrames.IsValidIndex(InHitMarkIndex))
	{
		InitialTimes.Add(InHitMarkIndex, MarkedFrames[InHitMarkIndex].FrameNumber);
		CurrentTimes.Add(InHitMarkIndex, MarkedFrames[InHitMarkIndex].FrameNumber);
	}
}

void FToolableTimelineMarkDragOperation::CacheMarks(const FMouseInputData& InMouseInput, const TSet<int32>& InMarkedFrames)
{
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InMouseInput.Timeline->GetTimeSliderController();
	const TArray<FMovieSceneMarkedFrame>& MarkedFrames = TimeSliderController->GetTimeSliderArgs().MarkedFrames.Get();

	InitialTimes.Empty(InMarkedFrames.Num());
	CurrentTimes.Empty(InMarkedFrames.Num());

	for (const int32 MarkIndex : InMarkedFrames)
	{
		if (MarkedFrames.IsValidIndex(MarkIndex))
		{
			InitialTimes.Add(MarkIndex, MarkedFrames[MarkIndex].FrameNumber);
			CurrentTimes.Add(MarkIndex, MarkedFrames[MarkIndex].FrameNumber);
		}
	}
}

FFrameTime FToolableTimelineMarkDragOperation::Recompute(const FMouseInputData& InMouseInput
	, const FFrameRate& InTickResolution, const FFrameRate& InDisplayRate, const bool bInSnapEnabled)
{
	const FFrameTime TotalDeltaTickTime = GetAccumulatedDelta(InMouseInput, bInSnapEnabled);
	FFrameTime ComputedDraggedMarkDelta = TotalDeltaTickTime;

	for (TPair<int32, FFrameNumber>& Pair : CurrentTimes)
	{
		const int32 MarkIndex = Pair.Key;
		const FFrameNumber* const StartFramePtr = InitialTimes.Find(MarkIndex);
		if (!StartFramePtr)
		{
			continue;
		}

		FFrameTime NewTickTime = FFrameTime(*StartFramePtr) + TotalDeltaTickTime;
		NewTickTime = SnapTime(NewTickTime, InTickResolution, InDisplayRate, bInSnapEnabled);

		Pair.Value = NewTickTime.FrameNumber;
		if (MarkIndex == DraggedMarkIndex)
		{
			ComputedDraggedMarkDelta = Pair.Value - *StartFramePtr;
		}
	}

	return ComputedDraggedMarkDelta;
}

} // UE::Sequencer::ToolableTimeline
