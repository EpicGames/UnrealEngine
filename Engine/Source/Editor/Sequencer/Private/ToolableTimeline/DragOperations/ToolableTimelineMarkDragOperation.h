// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolableTimelineDragOperation.h"

namespace UE::Sequencer::ToolableTimeline
{

class FToolableTimeline;
struct FMouseInputData;

class FToolableTimelineMarkDragOperation : public FToolableTimelineDragOperation
{
public:
	FToolableTimelineMarkDragOperation(const FMouseInputData& InMouseInput
		, const TRange<double>& InViewRange, const TSet<int32>& InMarkedFrames);

	FToolableTimelineMarkDragOperation(const FMouseInputData& InMouseInput
		, const TRange<double>& InViewRange, const int32 InHitMarkIndex);

	//~ Begin FToolableTimelineDragOperation
	virtual void UpdateDrag(const FMouseInputData& InMouseInput) override;
	virtual void ResetDrag() override;
	//~ End FToolableTimelineDragOperation

	FFrameTime GetTotalDraggedMarkDelta() const;

	void ApplyDelta(const TSharedRef<FToolableTimeline>& InTimeline, const FFrameTime& InTotalDeltaTime);

	void SortMarkedFrames(const TSharedRef<FToolableTimeline>& InTimeline);

private:
	void CacheMarks(const FMouseInputData& InMouseInput, const int32 InHitMarkIndex, const bool bInFallbackToHitMark);
	void CacheMarks(const FMouseInputData& InMouseInput, const TSet<int32>& InMarkedFrames);

	FFrameTime Recompute(const FMouseInputData& InMouseInput
		, const FFrameRate& InTickResolution, const FFrameRate& InDisplayRate, const bool bInSnapEnabled);

	/** Map of the indices of the marks being edited and their initial frame numbers when pressed */
	TMap<int32, FFrameNumber> InitialTimes;
	TMap<int32, FFrameNumber> CurrentTimes;

	FFrameTime TotalDraggedMarkDelta;
	int32 DraggedMarkIndex = INDEX_NONE;
};

} // namespace UE::Sequencer::ToolableTimeline
