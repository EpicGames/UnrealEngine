// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimelineScrubDragOperation.h"
#include "ToolableTimeline/MouseInputData.h"

namespace UE::Sequencer::ToolableTimeline
{

FToolableTimelineScrubDragOperation::FToolableTimelineScrubDragOperation(const FMouseInputData& InMouseInput
	, const TRange<double>& InViewRange)
	: FToolableTimelineDragOperation(InMouseInput, InViewRange)
{
}

void FToolableTimelineScrubDragOperation::UpdateDrag(const FMouseInputData& InMouseInput)
{
	FToolableTimelineDragOperation::UpdateDrag(InMouseInput);
}

void FToolableTimelineScrubDragOperation::ResetDrag()
{
}

} // UE::Sequencer::ToolableTimeline
