// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolableTimelineDragOperation.h"

namespace UE::Sequencer::ToolableTimeline
{

struct FMouseInputData;

/**  */
class FToolableTimelineScrubDragOperation : public FToolableTimelineDragOperation
{
public:
	FToolableTimelineScrubDragOperation(const FMouseInputData& InMouseInput
		, const TRange<double>& InViewRange);

	//~ Begin FToolableTimelineDragOperation
	virtual void UpdateDrag(const FMouseInputData& InMouseInput) override;
	virtual void ResetDrag() override;
	//~ End FToolableTimelineDragOperation
};

} // namespace UE::Sequencer::ToolableTimeline
