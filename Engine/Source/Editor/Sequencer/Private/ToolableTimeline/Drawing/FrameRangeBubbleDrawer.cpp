// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameRangeBubbleDrawer.h"
#include "Fonts/FontMeasure.h"
#include "ISequencer.h"
#include "Rendering/DrawElementTypes.h"
#include "Sequencer.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "ToolableTimeline/MouseInputData.h"
#include "ToolableTimeline/ToolableTimelineUtils.h"

namespace UE::Sequencer::ToolableTimeline
{

FFrameRangeBubbleDrawer::FFrameRangeBubbleDrawer(const FDrawFrameNumberBubbleArgs& InDrawArgs)
	: DrawArgs(InDrawArgs)
{
}

void FFrameRangeBubbleDrawer::CacheDrawData(const TRange<FFrameNumber>& InRange
	, FMouseDrawInputData& MouseDrawInput
	, const FText* const InOptionalText)
{
	const TSharedPtr<FSequencer> Sequencer = MouseDrawInput.Timeline->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	NumericTypeInterface = Sequencer->GetNumericTypeInterface();
	TickResolution = Sequencer->GetFocusedTickResolution();
	DisplayRate = Sequencer->GetFocusedDisplayRate();

	FrameDifference = GetSignedFrameDifference(InRange);

	const TRange<FFrameNumber> NormalizedRange = ToolableTimeline::Utils::NormalizeRange(InRange);
	StartFrameInclusive = NormalizedRange.GetLowerBoundValue();
	EndFrameExclusive = NormalizedRange.GetUpperBoundValue();

	FFrameNumber EndLabelFrame = EndFrameExclusive - 1;
	if (EndLabelFrame <= StartFrameInclusive)
	{
		EndLabelFrame = StartFrameInclusive;
	}

	bSingleFrameRange = (EndLabelFrame == StartFrameInclusive);

	if (InOptionalText)
	{
		FrameString = InOptionalText->ToString();
	}
	else
	{
		if (DrawArgs.bShowFrameOffsetInsteadOfRange)
		{
			if (FrameDifference.Value == 0)
			{
				FrameString = TEXT("0");
			}
			else
			{
				const bool bNegative = FrameDifference.Value < 0;
				const TCHAR Sign = bNegative ? TEXT('-') : TEXT('+');
				const FFrameNumber AbsoluteTickDelta(FMath::Abs(FrameDifference.Value));

				const FFrameTime DisplayTimeDifference = FFrameRate::TransformTime(FFrameTime(AbsoluteTickDelta), TickResolution, DisplayRate);
				const FFrameNumber DisplayFrameDifference = DisplayTimeDifference.RoundToFrame();

				const FFrameTime TickTimeDifference = FFrameRate::TransformTime(FFrameTime(DisplayFrameDifference), DisplayRate, TickResolution);
				const FFrameNumber TickFrameDifference = TickTimeDifference.RoundToFrame();

				FrameString = FString::Printf(TEXT("%c %s"), Sign, *Utils::TickFrameToString(*Sequencer
					, TickFrameDifference, DrawArgs.bPreventLeadingZeros, !bNegative));
			}
		}
		else
		{
			FrameString = bSingleFrameRange
				? *Utils::TickFrameToString(*Sequencer, StartFrameInclusive, DrawArgs.bPreventLeadingZeros, false)
				: FString::Printf(TEXT("%s - %s")
					, *Utils::TickFrameToString(*Sequencer, StartFrameInclusive, DrawArgs.bPreventLeadingZeros, false)
					, *Utils::TickFrameToString(*Sequencer, EndLabelFrame, DrawArgs.bPreventLeadingZeros, false));
		}
	}

	StartInput = TickResolution.AsSeconds(StartFrameInclusive);
	EndInput = TickResolution.AsSeconds(EndFrameExclusive);

	if (DrawArgs.bAlignToWholeFrame)
	{
		const FFrameTime StartDisplayTime = FFrameRate::TransformTime(FFrameTime(StartFrameInclusive), TickResolution, DisplayRate);
		const FFrameTime EndDisplayTime = FFrameRate::TransformTime(FFrameTime(EndFrameExclusive), TickResolution, DisplayRate);

		const FFrameNumber AlignedStartDisplayFrame = StartDisplayTime.FloorToFrame();
		const FFrameNumber AlignedEndDisplayFrameExclusive = EndDisplayTime.CeilToFrame();

		const FFrameTime TickFrameStart = FFrameRate::TransformTime(FFrameTime(AlignedStartDisplayFrame), DisplayRate, TickResolution);
		const FFrameTime TickFrameEnd = FFrameRate::TransformTime(FFrameTime(AlignedEndDisplayFrameExclusive), DisplayRate, TickResolution);

		RangeStartX = MouseDrawInput.RangeToScreen.InputToLocalX(TickResolution.AsSeconds(TickFrameStart));
		RangeEndX = MouseDrawInput.RangeToScreen.InputToLocalX(TickResolution.AsSeconds(TickFrameEnd));
		AnchorX = (RangeStartX + RangeEndX) * .5f;
	}
	else if (bSingleFrameRange)
	{
		const double AnchorInput = TickResolution.AsSeconds(StartFrameInclusive);

		RangeStartX = MouseDrawInput.RangeToScreen.InputToLocalX(AnchorInput);
		RangeEndX = RangeStartX;
		AnchorX = RangeStartX;
	}
	else
	{
		const double PlacementStartInput = TickResolution.AsSeconds(StartFrameInclusive);
		const double PlacementEndInput = TickResolution.AsSeconds(EndFrameExclusive);

		RangeStartX = MouseDrawInput.RangeToScreen.InputToLocalX(PlacementStartInput);
		RangeEndX = MouseDrawInput.RangeToScreen.InputToLocalX(PlacementEndInput);
		AnchorX = (RangeStartX + RangeEndX) * .5f;
	}

	bShowOffscreenLeft = (RangeEndX < 0.f);  
	bShowOffscreenRight = (RangeStartX > MouseDrawInput.Geometry.Size.X);

	OffscreenIconBrush = GetOffscreenIconBrush();

	TextSize = CalculateTextSize();

	// Base icon size on TextSize.Y
	IconSize = FVector2D(TextSize.Y, TextSize.Y);

	// Compute extra size BEFORE bubble size/position so clamping uses the final BubbleSize
	ExtraBubbleSize = (DrawArgs.bDrawOffscreenGlyph && OffscreenIconBrush)
		? FVector2D(IconSize.X + IconGap, 0.f) : FVector2D::ZeroVector;

	CalculateBubbleSizeAndPosition(MouseDrawInput, ExtraBubbleSize, BubbleSize, BubblePosition);

	// Base text position on BubblePosition
	TextPosition = BubblePosition + DrawArgs.BubblePadding.GetTopLeft();

	// Icon/Text placement inside bubble
	IconPosition = FVector2D::ZeroVector;

	if (DrawArgs.bDrawOffscreenGlyph && OffscreenIconBrush)
	{
		const FVector2D BubbleInnerTopLeft = BubblePosition + DrawArgs.BubblePadding.GetTopLeft();

		if (bShowOffscreenLeft)
		{
			TextPosition = BubbleInnerTopLeft + ExtraBubbleSize;
			IconPosition = BubbleInnerTopLeft;
		}
		else if (bShowOffscreenRight)
		{
			TextPosition = BubbleInnerTopLeft;
			IconPosition = BubbleInnerTopLeft + FVector2D(TextSize.X + IconGap, 0.f);
		}
		else
		{
			TextPosition = BubbleInnerTopLeft;
			IconPosition = BubbleInnerTopLeft + FVector2D(TextSize.X + IconGap, 0.f);
		}
	}
}

FFrameNumber FFrameRangeBubbleDrawer::GetSignedFrameDifference(const TRange<FFrameNumber>& InRange) const
{
	const FFrameNumber InputStart = InRange.GetLowerBound().IsOpen() ? 0 : InRange.GetLowerBoundValue();
	const FFrameNumber InputEnd = InRange.GetUpperBound().IsOpen() ? InputStart : InRange.GetUpperBoundValue();

	if (!DrawArgs.bShowFrameOffsetInsteadOfRange)
	{
		return InputEnd - InputStart;
	}

	// Interpret InRange as a directional whole-frame range:
	//  right/same drag: [StartFrame, EndFrame+1)
	//  left drag:       [StartFrame+1, EndFrame)
	const FFrameNumber StartDisplayBoundary =
		FFrameRate::TransformTime(FFrameTime(InputStart), TickResolution, DisplayRate).FloorToFrame();
	const FFrameNumber EndDisplayBoundary =
		FFrameRate::TransformTime(FFrameTime(InputEnd), TickResolution, DisplayRate).FloorToFrame();

	const int32 DisplayFrameDelta = (InputEnd >= InputStart)
		? EndDisplayBoundary.Value - StartDisplayBoundary.Value - 1  // [StartFrame, EndFrame+1)
		: EndDisplayBoundary.Value - StartDisplayBoundary.Value + 1; // [StartFrame+1, EndFrame)
	const FFrameNumber AbsDisplayDelta(FMath::Abs(DisplayFrameDelta));
	const FFrameTime TickDeltaTime = FFrameRate::TransformTime(FFrameTime(AbsDisplayDelta), DisplayRate, TickResolution);
	const bool bNegative = DisplayFrameDelta < 0;

	FFrameNumber TickDelta = TickDeltaTime.RoundToFrame();

	if (bNegative)
	{
		TickDelta = -TickDelta;
	}

	return TickDelta;
}

FVector2D FFrameRangeBubbleDrawer::CalculateTextSize() const
{
	const FSlateFontInfo TickFrameFont = FCoreStyle::GetDefaultFontStyle(TEXT("Bold"), DrawArgs.BubbleFontSize);
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	return FontMeasureService->Measure(FrameString, TickFrameFont);
}

void FFrameRangeBubbleDrawer::CalculateBubbleSizeAndPosition(FMouseDrawInputData& MouseDrawInput
	, const FVector2D& InExtraBubbleSize, FVector2D& OutBubbleSize, FVector2D& OutBubblePosition) const
{
	const FVector2D PaddingSize = DrawArgs.BubblePadding.GetDesiredSize();
	OutBubbleSize = TextSize + PaddingSize + InExtraBubbleSize;

	OutBubblePosition = FVector2D::ZeroVector;
	OutBubblePosition.X = AnchorX - (OutBubbleSize.X * .5f);

	switch (DrawArgs.LabelAlignment)
	{
	default:
	case VAlign_Fill:
	case VAlign_Top:
	case VAlign_Center:
		OutBubblePosition.Y = 0.f;
		break;
	case VAlign_Bottom:
		OutBubblePosition.Y = MouseDrawInput.Geometry.Size.Y - OutBubbleSize.Y;
		break;
	}

	if (DrawArgs.bTryKeepOnScreen)
	{
		const float MaxX = FMath::Max(0.f, MouseDrawInput.Geometry.Size.X - OutBubbleSize.X);
		OutBubblePosition.X = FMath::Clamp(OutBubblePosition.X, 0.f, MaxX);
	}
}
	
const FSlateBrush* FFrameRangeBubbleDrawer::GetOffscreenIconBrush() const
{
	if (DrawArgs.bDrawOffscreenGlyph)
	{
		if (bShowOffscreenLeft)
		{
			return FAppStyle::GetBrush(TEXT("Icons.ChevronLeft"));
		}
		if (bShowOffscreenRight)
		{
			return FAppStyle::GetBrush(TEXT("Icons.ChevronRight"));
		}
	}
	return nullptr;
}

int32 FFrameRangeBubbleDrawer::Draw(const TRange<FFrameNumber>& InRange, FMouseDrawInputData& MouseDrawInput)
{
	const FSlateRoundedBoxBrush BubbleBrush = FSlateRoundedBoxBrush(DrawArgs.BubbleColor, 5.f, DrawArgs.BubbleBorderColor, 1.f);

	CacheDrawData(InRange, MouseDrawInput);

	FSlateDrawElement::MakeBox(
		MouseDrawInput.DrawElements,
		MouseDrawInput.LayerId,
		MouseDrawInput.Geometry.ToPaintGeometry(BubbleSize, FSlateLayoutTransform(BubblePosition)),
		&BubbleBrush,
		MouseDrawInput.DrawEffects,
		DrawArgs.BubbleColor.CopyWithNewOpacity(DrawArgs.BubbleColor.A * DrawArgs.AlphaOpacity)
	);

	if (DrawArgs.bDrawOffscreenGlyph && OffscreenIconBrush)
	{
		FSlateDrawElement::MakeBox(
			MouseDrawInput.DrawElements,
			MouseDrawInput.LayerId,
			MouseDrawInput.Geometry.ToPaintGeometry(IconSize, FSlateLayoutTransform(IconPosition)),
			OffscreenIconBrush,
			MouseDrawInput.DrawEffects,
			DrawArgs.BubbleTextColor.CopyWithNewOpacity(DrawArgs.BubbleTextColor.A * DrawArgs.AlphaOpacity)
		);
	}

	if (!FrameString.IsEmpty())
	{
		FSlateDrawElement::MakeText(
			MouseDrawInput.DrawElements,
			MouseDrawInput.IncrementDrawLayer().LayerId,
			MouseDrawInput.Geometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextPosition)), 
			FrameString, 
			FCoreStyle::GetDefaultFontStyle(TEXT("Bold"), DrawArgs.BubbleFontSize),
			MouseDrawInput.DrawEffects,
			DrawArgs.BubbleTextColor.CopyWithNewOpacity(DrawArgs.BubbleTextColor.A * DrawArgs.AlphaOpacity)
		);
	}

	return MouseDrawInput.IncrementDrawLayer().LayerId;
}

} // UE::Sequencer::ToolableTimeline
