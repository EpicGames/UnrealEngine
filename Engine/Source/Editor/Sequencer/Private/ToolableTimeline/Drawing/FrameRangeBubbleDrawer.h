// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerTimeSliderController.h"
#include "Templates/SharedPointer.h"

namespace UE::Sequencer::ToolableTimeline
{

struct FMouseDrawInputData;

struct FDrawFrameNumberBubbleArgs
{
	EVerticalAlignment LabelAlignment = VAlign_Fill;

	float BubbleFontSize = 9.f;
	FMargin BubblePadding = FMargin(3.f, 2.f);

	FLinearColor BubbleColor = FLinearColor::White;
	FLinearColor BubbleTextColor = FLinearColor::Black;
	FLinearColor BubbleBorderColor = FLinearColor::Black;

	bool bAlignToWholeFrame = true;
	bool bPreventLeadingZeros = true;
	bool bTryKeepOnScreen = true;
	bool bDrawOffscreenGlyph = true;
	bool bShowFrameOffsetInsteadOfRange = false;

	float AlphaOpacity = 1.f;
};

/** Helper class for drawing frame range bubbles */
class FFrameRangeBubbleDrawer
{
public:
	FFrameRangeBubbleDrawer(const FDrawFrameNumberBubbleArgs& InDrawArgs);

	int32 Draw(const TRange<FFrameNumber>& InRange, FMouseDrawInputData& MouseDrawInput);

protected:
	static constexpr float IconGap = 4.f;

	/** Helper for caching and managing draw data. */
	void CacheDrawData(const TRange<FFrameNumber>& InRange
		, FMouseDrawInputData& MouseDrawInput
		, const FText* const InOptionalText = nullptr);

	/**
	 * Calculates the signed frame difference between the start and end of the given frame range.
	 *
	 * @param InRange The input frame range to compute the signed frame difference for.
	 * @return The signed difference between the end (exclusive) and the start (inclusive) of the given range.
	 */
	FFrameNumber GetSignedFrameDifference(const TRange<FFrameNumber>& InRange) const;

	FVector2D CalculateTextSize() const;

	void CalculateBubbleSizeAndPosition(FMouseDrawInputData& MouseDrawInput
		, const FVector2D& InExtraBubbleSize, FVector2D& OutBubbleSize, FVector2D& OutBubblePosition) const;

	const FSlateBrush* GetOffscreenIconBrush() const;

	/** Variables cached in constructor and unchanged by this class */
	FDrawFrameNumberBubbleArgs DrawArgs;
	TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface;
	FFrameRate TickResolution;
	FFrameRate DisplayRate;

	/** Variables cached with CacheDrawData that are based on the frame range */
	const FSlateBrush* OffscreenIconBrush = nullptr;

	FFrameNumber StartFrameInclusive;
	FFrameNumber EndFrameExclusive;

	FFrameNumber FrameDifference;
	bool bSingleFrameRange = false;

	FString FrameString;

	bool bShowOffscreenLeft = false;
	bool bShowOffscreenRight = false;

	double StartInput = 0.0;
	double EndInput = 0.0;

	double RangeStartX = 0.0;
	double RangeEndX = 0.0;

	double AnchorX = 0.0;
	
	FVector2D BubbleSize = FVector2D::ZeroVector;
	FVector2D BubblePosition = FVector2D::ZeroVector;

	FVector2D TextSize = FVector2D::ZeroVector;
	FVector2D TextPosition = FVector2D::ZeroVector;

	FVector2D IconSize = FVector2D::ZeroVector;
	FVector2D IconPosition = FVector2D::ZeroVector;

	/** Additional size caused by the icon and its gap */
	FVector2D ExtraBubbleSize = FVector2D::ZeroVector;
};

} // namespace UE::Sequencer::ToolableTimeline
