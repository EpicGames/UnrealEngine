// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimelineDrawer.h"
#include "Fonts/FontMeasure.h"
#include "MVVM/Extensions/IClockExtension.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/Views/SOutlinerView.h"
#include "SSequencer.h"
#include "SSequencerSection.h"
#include "ToolableTimeline/Caches/ToolableTimelineKeyViewCacheState.h"
#include "ToolableTimeline/Drawing/FrameRangeBubbleDrawer.h"
#include "ToolableTimeline/MouseInputData.h"
#include "ToolableTimeline/ToolableTimelineUtils.h"
#include "ToolableTimeline/Tools/ToolableTimelineBaseTool.h"
#include "ToolableTimeline/Tools/ToolableTimelineKeyRenderer.h"
#include "TrackEditors/TimeWarpTrackEditor.h"

namespace UE::Sequencer::ToolableTimeline::Drawing
{

void MirrorTransformY(FPaintGeometry& InPaintGeometry, const FGeometry& InGeometry, const bool bInMirrorLabels)
{
	if (bInMirrorLabels)
	{
		const FSlateRenderTransform FlipTransform(
			FScale2f(1.f, -1.f),
			FVector2f(0.f, InGeometry.GetLocalSize().Y)
		);

		InPaintGeometry.SetRenderTransform(
			Concatenate(FlipTransform, InPaintGeometry.GetAccumulatedRenderTransform())
		);
	}
}

void GetFrameAreaGeometry(const FMouseDrawInputData& InMouseDrawInput, const FMargin& InPadding, FVector2f& OutSize, FVector2f& OutPosition)
{
	const UToolableTimelineSettings& TimelineSettings = InMouseDrawInput.Timeline->GetTimelineSettings();

	const FVector2f LocalSize = InMouseDrawInput.Geometry.GetLocalSize();
	const float TickSize = CalculateTickSize(InMouseDrawInput);
	const float FrameHeight = LocalSize.Y - TickSize;

	switch (TimelineSettings.Settings.LabelVerticalAlignment)
	{
	default:
	case VAlign_Top:
		OutSize = FVector2f(
			LocalSize.X - InPadding.Left - InPadding.Right,
			FrameHeight - InPadding.Top - InPadding.Bottom
		);
		OutPosition = FVector2f(
			InPadding.Left,
			TickSize + InPadding.Top
		);
		break;

	case VAlign_Bottom:
		OutSize = FVector2f(
			LocalSize.X - InPadding.Left - InPadding.Right,
			FrameHeight - InPadding.Top - InPadding.Bottom
		);
		OutPosition = FVector2f(
			InPadding.Left,
			InPadding.Top
		);
		break;

	case VAlign_Fill:
	case VAlign_Center:
		OutSize = FVector2f(
			LocalSize.X - InPadding.Left - InPadding.Right,
			LocalSize.Y - InPadding.Top - InPadding.Bottom
		);
		OutPosition = FVector2f(
			InPadding.Left,
			InPadding.Top
		);
		break;
	}
}

float CalculateTickSize(const FMouseDrawInputData& InMouseDrawInput)
{
	const UToolableTimelineSettings& TimelineSettings = InMouseDrawInput.Timeline->GetTimelineSettings();

	if (TimelineSettings.Settings.LabelVerticalAlignment == VAlign_Fill)
	{
		return InMouseDrawInput.Geometry.GetLocalSize().Y;
	}

	const float FontSize = TimelineSettings.Settings.MajorTickFontSize;
	const FSlateFontInfo TickFrameFont = FCoreStyle::GetDefaultFontStyle(TEXT("Regular"), FontSize);
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const float TextDrawHeight = FontMeasureService->Measure(TEXT("0"), TickFrameFont).Y;
	const float OutTickSize = TextDrawHeight + (Constants::FrameLabelOffset.Y * 2.f);

	return OutTickSize;
}

void GetMajorLinePy(const FMouseDrawInputData& InMouseDrawInput, const float InTickSize, float& OutMajorLinePy, float& OutHorizontalLineStartY)
{
	const UToolableTimelineSettings& TimelineSettings = InMouseDrawInput.Timeline->GetTimelineSettings();

	OutMajorLinePy = 0.f;
	OutHorizontalLineStartY = InTickSize;

	switch (TimelineSettings.Settings.LabelVerticalAlignment)
	{
	default:
	case VAlign_Fill:
		OutMajorLinePy = 0.f;
		break;
	case VAlign_Top:
		OutMajorLinePy = 0.f;
		break;
	case VAlign_Center:
		OutMajorLinePy = (InMouseDrawInput.Geometry.GetLocalSize().Y - InTickSize) * .5f;
		OutHorizontalLineStartY = InMouseDrawInput.Geometry.GetLocalSize().Y * .5f;
		break;
	case VAlign_Bottom:
		OutMajorLinePy = InMouseDrawInput.Geometry.GetLocalSize().Y - InTickSize;
		OutHorizontalLineStartY = OutMajorLinePy;
		break;
	}
}

int32 DrawBackground(FMouseDrawInputData& MouseDrawInput)
{
	static const FSlateBrush* const WhiteBrush = FAppStyle::GetBrush(TEXT("WhiteBrush"));

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = MouseDrawInput.Timeline->GetTimeSliderController();
	const UToolableTimelineSettings& TimelineSettings = MouseDrawInput.Timeline->GetTimelineSettings();

	const double MajorTickSize = TimeSliderController->GetMajorTickDrawSize();
	const FVector2f LocalSize = MouseDrawInput.Geometry.GetLocalSize();
	const float FrameHeight = LocalSize.Y - MajorTickSize;

	// Draw label area background
	const FVector2f LabelAreaSize = LocalSize;
	const FVector2f LabelAreaPosition = FVector2f(0.f, 0.f);

	FSlateDrawElement::MakeBox(
		MouseDrawInput.DrawElements,
		MouseDrawInput.LayerId,
		MouseDrawInput.Geometry.ToPaintGeometry(LabelAreaSize, FSlateLayoutTransform(LabelAreaPosition)),
		WhiteBrush,
		MouseDrawInput.DrawEffects,
		TimelineSettings.Settings.LabelBackgroundColor
	);

	if (TimelineSettings.Settings.LabelVerticalAlignment == VAlign_Fill
		|| TimelineSettings.Settings.LabelVerticalAlignment == VAlign_Center)
	{
		return MouseDrawInput.IncrementDrawLayer().LayerId;
	}

	const float PixelsPerFrame = MouseDrawInput.CalculatePixelsPerFrame();

	// Draw key/frame area background
	if (PixelsPerFrame >= Constants::MinPixelsPerFrameBackgroundLine)
	{
		FVector2f KeyAreaSize;
		FVector2f KeyAreaPosition;
		GetFrameAreaGeometry(MouseDrawInput, FMargin(Constants::FrameAreaPadding), KeyAreaSize, KeyAreaPosition);

		const FFrameRate DisplayRate = TimeSliderController->GetDisplayRate();
		const FAnimatedRange ViewRange = TimeSliderController->GetViewRange();
		const int32 GridColorBlockSize = FMath::Max(1, TimelineSettings.Settings.GridColorBlockSize);

		const double ViewStartSeconds = ViewRange.GetLowerBoundValue();
		const double ViewEndSeconds = ViewRange.GetUpperBoundValue();

		const int32 StartFrame = FMath::FloorToInt(ViewStartSeconds * DisplayRate.AsDecimal());
		const int32 EndFrame = FMath::CeilToInt(ViewEndSeconds * DisplayRate.AsDecimal());
		const int32 FrameWidth = EndFrame - StartFrame;

		if (TimelineSettings.Settings.bUseGridColorBlocks
			&& PixelsPerFrame > 1.0f
			&& FrameWidth < 6000)
		{
			const int32 CurrentBlockStartFrame =
				FMath::FloorToInt(static_cast<double>(StartFrame) / GridColorBlockSize) * GridColorBlockSize;

			for (int32 CurrentFrame = CurrentBlockStartFrame; CurrentFrame < EndFrame; CurrentFrame += GridColorBlockSize)
			{
				const int32 NextBlockFrame = CurrentFrame + GridColorBlockSize;

				const double BlockStartTime = CurrentFrame / DisplayRate.AsDecimal();
				const double BlockEndTime = NextBlockFrame / DisplayRate.AsDecimal();

				const float FrameStartPx = MouseDrawInput.RangeToScreen.InputToLocalX(BlockStartTime);
				const float FrameEndPx = MouseDrawInput.RangeToScreen.InputToLocalX(BlockEndTime);
				const float BlockWidthPx = FrameEndPx - FrameStartPx;

				if (BlockWidthPx <= 0.f)
				{
					continue;
				}

				FVector2f BlockSize;
				FVector2f BlockPosition;

				switch (TimelineSettings.Settings.LabelVerticalAlignment)
				{
				default:
				case VAlign_Top:
					BlockSize = FVector2f(BlockWidthPx, FrameHeight - Constants::FrameAreaDoublePadding);
					BlockPosition = FVector2f(FrameStartPx, MajorTickSize + Constants::FrameAreaPadding);
					break;

				case VAlign_Bottom:
					BlockSize = FVector2f(BlockWidthPx, FrameHeight - Constants::FrameAreaDoublePadding);
					BlockPosition = FVector2f(FrameStartPx, Constants::FrameAreaPadding);
					break;

				case VAlign_Fill:
				case VAlign_Center:
					BlockSize = FVector2f(BlockWidthPx, LocalSize.Y - Constants::FrameAreaDoublePadding);
					BlockPosition = FVector2f(FrameStartPx, Constants::FrameAreaPadding);
					break;
				}

				const int32 BlockIndex = CurrentFrame / GridColorBlockSize;
				const bool bUseAlternateColor = (BlockIndex % 2) != 0;
				const FLinearColor FrameBlockColor = bUseAlternateColor
					? TimelineSettings.Settings.GridBlockAlternateColor
					: TimelineSettings.Settings.GridBlockColor;

				FSlateDrawElement::MakeBox(
					MouseDrawInput.DrawElements,
					MouseDrawInput.LayerId,
					MouseDrawInput.Geometry.ToPaintGeometry(BlockSize, FSlateLayoutTransform(BlockPosition)),
					WhiteBrush,
					MouseDrawInput.DrawEffects,
					FrameBlockColor
				);
			}
		}
		else
		{
			FSlateDrawElement::MakeBox(
				MouseDrawInput.DrawElements,
				MouseDrawInput.LayerId,
				MouseDrawInput.Geometry.ToPaintGeometry(KeyAreaSize, FSlateLayoutTransform(KeyAreaPosition)),
				WhiteBrush,
				MouseDrawInput.DrawEffects,
				TimelineSettings.Settings.FrameBackgroundColor
			);
		}
	}

	if (PixelsPerFrame < Constants::MinPixelsPerFrameBackgroundLine)
	{
		return MouseDrawInput.IncrementDrawLayer().LayerId;
	}

	// Draw vertical lines between frames
	const FFrameRate DisplayRate = TimeSliderController->GetDisplayRate();
	const FAnimatedRange ViewRange = TimeSliderController->GetViewRange();

	const double ViewStartSeconds = ViewRange.GetLowerBoundValue();
	const double ViewEndSeconds = ViewRange.GetUpperBoundValue();

	const int32 StartFrame = FMath::FloorToInt(ViewStartSeconds * DisplayRate.AsDecimal());
	const int32 EndFrame = FMath::CeilToInt(ViewEndSeconds * DisplayRate.AsDecimal());

	for (int32 CurrentFrame = StartFrame; CurrentFrame <= EndFrame; ++CurrentFrame)
	{
		const double FrameTime = CurrentFrame / DisplayRate.AsDecimal();
		const float FrameLinePx = MouseDrawInput.RangeToScreen.InputToLocalX(FrameTime);

		FVector2f LabelAreaLineStart;
		FVector2f LabelAreaLineEnd;
		FVector2f FrameAreaLineStart;
		FVector2f FrameAreaLineEnd;

		switch (TimelineSettings.Settings.LabelVerticalAlignment)
		{
		default:
		case VAlign_Top:
			LabelAreaLineStart = FVector2f(FrameLinePx, 0);
			LabelAreaLineEnd = FVector2f(FrameLinePx, MajorTickSize);
			FrameAreaLineStart = FVector2f(FrameLinePx, MajorTickSize);
			FrameAreaLineEnd = FVector2f(FrameLinePx, LocalSize.Y);
			break;

		case VAlign_Bottom:
			LabelAreaLineStart = FVector2f(FrameLinePx, FrameHeight);
			LabelAreaLineEnd = FVector2f(FrameLinePx, FrameHeight + MajorTickSize);
			FrameAreaLineStart = FVector2f(FrameLinePx, 0.f);
			FrameAreaLineEnd = FVector2f(FrameLinePx, FrameHeight);
			break;

		case VAlign_Fill:
		case VAlign_Center:
			LabelAreaLineStart = FVector2f(FrameLinePx, 0.f);
			LabelAreaLineEnd = FVector2f(FrameLinePx, LocalSize.Y);
			FrameAreaLineStart = FVector2f(FrameLinePx, 0.f);
			FrameAreaLineEnd = FVector2f(FrameLinePx, LocalSize.Y);
			break;
		}

		TArray<FVector2f> LinePoints;
		LinePoints.SetNumUninitialized(2);
		LinePoints[0] = FrameAreaLineStart;
		LinePoints[1] = FrameAreaLineEnd;

		FSlateDrawElement::MakeLines(
			MouseDrawInput.DrawElements,
			MouseDrawInput.LayerId,
			MouseDrawInput.Geometry.ToPaintGeometry(),
			LinePoints,
			MouseDrawInput.DrawEffects,
			TimelineSettings.Settings.LabelBackgroundColor,
			/*bAntialias=*/true,
			/*Thickness=*/2.f
		);

		static constexpr bool bDrawLabelAreaVerticalLines = false;
		if (bDrawLabelAreaVerticalLines)
		{
			LinePoints[0] = LabelAreaLineStart;
			LinePoints[1] = LabelAreaLineEnd;

			FSlateDrawElement::MakeLines(
				MouseDrawInput.DrawElements,
				MouseDrawInput.LayerId,
				MouseDrawInput.Geometry.ToPaintGeometry(),
				LinePoints,
				MouseDrawInput.DrawEffects,
				TimelineSettings.Settings.FrameBackgroundColor,
				/*bAntialias=*/true,
				/*Thickness=*/2.f
			);
		}
	}

	return MouseDrawInput.IncrementDrawLayer().LayerId;
}

int32 DrawTicks(FMouseDrawInputData& MouseDrawInput, FSequencerTimeSliderController::FDrawTickArgs& InArgs)
{
	const TSharedPtr<FSequencer> Sequencer = MouseDrawInput.Timeline->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return MouseDrawInput.LayerId;
	}

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = MouseDrawInput.Timeline->GetTimeSliderController();
	const UToolableTimelineSettings& TimelineSettings = MouseDrawInput.Timeline->GetTimelineSettings();

	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	const FFrameRate DisplayRate = TimeSliderController->GetDisplayRate();

	const FAnimatedRange ViewRange = TimeSliderController->GetViewRange();
	const FPaintGeometry PaintGeometry = MouseDrawInput.Geometry.ToPaintGeometry();

	const FFrameTime ScrubPosition = TimeSliderController->GetScrubPosition();
	const float FlooredScrubPx = MouseDrawInput.RangeToScreen.InputToLocalX(ConvertFrameTime(ScrubPosition
		, TickResolution, DisplayRate).FloorToFrame() / DisplayRate);
	const float TickSize = CalculateTickSize(MouseDrawInput);

	float MajorLinePy = 0.f;
	float HorizontalLineStartY = 0.f;
	GetMajorLinePy(MouseDrawInput, TickSize, MajorLinePy, HorizontalLineStartY);

	double MajorGridStep = 0.f;
	int32 MinorDivisions = 0;
	if (!TimeSliderController->GetGridMetrics(MouseDrawInput.Geometry, MajorGridStep, MinorDivisions))
	{
		return MouseDrawInput.LayerId;
	}
	if (InArgs.bOnlyDrawMajorTicks)
	{
		MinorDivisions = 0;
	}

	if (const TViewModelPtr<IClockExtension> Clock = Sequencer->GetViewModel()->GetRootSequenceModel().ImplicitCast())
	{
		Clock->DrawTicks(Sequencer, MouseDrawInput.DrawElements, ViewRange, MouseDrawInput.RangeToScreen, InArgs);
	}

	const float PixelsPerFrame = MouseDrawInput.CalculatePixelsPerFrame();
	const bool bNoPerFrameDrawing = PixelsPerFrame < Constants::MinPixelsPerFrameBackgroundLine;

	// Draw major tick horizontal line
	if (TimelineSettings.Settings.LabelVerticalAlignment == VAlign_Center
		|| bNoPerFrameDrawing)
	{
		FLinearColor Color = InArgs.TickColor;
		Color.A = .2f;

		TArray<FVector2D> HorizontalLinePoints;
		HorizontalLinePoints.SetNumUninitialized(2);
		HorizontalLinePoints[0] = FVector2D(0.f, HorizontalLineStartY);
		HorizontalLinePoints[1] = FVector2D(MouseDrawInput.Geometry.GetLocalSize().X, HorizontalLineStartY);

		FSlateDrawElement::MakeLines(MouseDrawInput.DrawElements, MouseDrawInput.IncrementDrawLayer().LayerId, PaintGeometry,
			HorizontalLinePoints, InArgs.DrawEffects, Color, /*bAntialias=*/true);
	}

	// Draw vertical tick marks and text
	const int32 TickLayerId = MouseDrawInput.LayerId;
	const double FirstMajorLine = FMath::FloorToDouble(ViewRange.GetLowerBoundValue() / MajorGridStep) * MajorGridStep;
	const double LastMajorLine = FMath::CeilToDouble(ViewRange.GetUpperBoundValue() / MajorGridStep) * MajorGridStep;

	for (double CurrentMajorLine = FirstMajorLine; CurrentMajorLine < LastMajorLine; CurrentMajorLine += MajorGridStep)
	{
		const int32 FrameNumber = (CurrentMajorLine * TickResolution).RoundToFrame().Value;
		const float MajorLinePx = MouseDrawInput.RangeToScreen.InputToLocalX(CurrentMajorLine);

		// Draw minor tick vertical lines
		for (int32 Step = 1; Step < MinorDivisions; ++Step)
		{
			// Compute the size of each tick mark. If we are halfway between visible values, display a slightly larger tick mark.
			const float MinorTickStep = (Step * MajorGridStep) / MinorDivisions;
			const float MinorLinePx = MouseDrawInput.RangeToScreen.InputToLocalX(CurrentMajorLine + MinorTickStep);

			float MinorTickHeight = ((MinorDivisions % 2 == 0) && (Step % (MinorDivisions / 2)) == 0) ? 6.f : 2.f;
			float MinorLinePy = MajorLinePy + TickSize;
			switch (TimelineSettings.Settings.LabelVerticalAlignment)
			{
			default:
			case VAlign_Fill:
				MinorTickHeight = MouseDrawInput.Geometry.GetLocalSize().Y;
				MinorLinePy = 0.f;
				break;
			case VAlign_Top:
				MinorLinePy = TickSize - MinorTickHeight;
				break;
			case VAlign_Center:
				MinorLinePy = (MouseDrawInput.Geometry.GetLocalSize().Y * .5f) - (MinorTickHeight * .5f);
				break;
			case VAlign_Bottom:
				MinorLinePy = MouseDrawInput.Geometry.GetLocalSize().Y - TickSize;
				break;
			}

			TArray<FVector2D> VerticalLinePoints;
			VerticalLinePoints.SetNumUninitialized(2);
			VerticalLinePoints[0] = FVector2D(MinorLinePx, MinorLinePy);
			VerticalLinePoints[1] = FVector2D(MinorLinePx, MinorLinePy + MinorTickHeight);

			// Make minor tick color opaque since the tick line is the same size as minor lines with fill alignment
			FLinearColor MinorTickColor = TimelineSettings.Settings.LabelVerticalAlignment != VAlign_Fill
				? InArgs.TickColor : InArgs.TickColor.CopyWithNewOpacity(.2f);

			FSlateDrawElement::MakeLines(MouseDrawInput.DrawElements, TickLayerId, PaintGeometry,
				VerticalLinePoints, MouseDrawInput.DrawEffects, MinorTickColor, /*bAntialias=*/true);
		}

		// Draw major tick vertical line
		{
			TArray<FVector2D> VerticalLinePoints;
			VerticalLinePoints.SetNumUninitialized(2);
			VerticalLinePoints[0] = FVector2D(MajorLinePx, MajorLinePy);
			VerticalLinePoints[1] = FVector2D(MajorLinePx, MajorLinePy + TickSize);

			FSlateDrawElement::MakeLines(MouseDrawInput.DrawElements, TickLayerId, PaintGeometry,
				VerticalLinePoints, MouseDrawInput.DrawEffects, InArgs.TickColor, /*bAntialias=*/true);
		}

		// Draw tick frame number text
		if (!InArgs.bOnlyDrawMajorTicks
			&& (TimeSliderController->IsScrubHandleHidden() || !FMath::IsNearlyEqual(MajorLinePx, FlooredScrubPx, 3.f)))
		{
			const FSlateFontInfo TickFrameFont = FCoreStyle::GetDefaultFontStyle(TEXT("Regular"), TimelineSettings.Settings.MajorTickFontSize);
			const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

			const FString FrameString = Utils::TickFrameToString(*Sequencer, FrameNumber, true, false);
			const FVector2D TextDrawSize = FontMeasureService->Measure(FrameString, TickFrameFont);

			FVector2D TextOffset = FVector2D(MajorLinePx, 0);

			switch (TimelineSettings.Settings.LabelVerticalAlignment)
			{
			default:
			case VAlign_Fill:
			case VAlign_Top:
			case VAlign_Center:
				TextOffset.Y = Constants::FrameLabelOffset.Y;
				break;
			case VAlign_Bottom:
				TextOffset.Y = MouseDrawInput.Geometry.GetLocalSize().Y - TextDrawSize.Y - Constants::FrameLabelOffset.Y;
				break;
			}

			const FVector2D TextPosition = TextOffset + Constants::FrameLabelOffset;
			const FVector2D BoxPad(4.0, 2.0);
			const FVector2D BoxSize = TextDrawSize + (BoxPad * 2.0);
			const FVector2D BoxPosition = TextPosition - BoxPad;

			// Needs be high enough to be above keys, 100 is not high enough
			//const int32 TextDrawLayerId = TickLayerId + 300;
			MouseDrawInput.DrawLayer(TickLayerId);

			// Draw background box under the text to help see frame numbers when frame block keys are displayed in "Fill" orientation
			if (TimelineSettings.Settings.LabelVerticalAlignment == VAlign_Fill
				|| TimelineSettings.Settings.LabelVerticalAlignment == VAlign_Center)
			{
				FSlateDrawElement::MakeBox(
					MouseDrawInput.DrawElements,
					MouseDrawInput.IncrementDrawLayer().LayerId,
					MouseDrawInput.Geometry.ToPaintGeometry(FVector2f(BoxSize), FSlateLayoutTransform(FVector2f(BoxPosition))),
					FAppStyle::GetBrush(TEXT("WhiteBrush")),
					MouseDrawInput.DrawEffects,
					FStyleColors::Background.GetSpecifiedColor().CopyWithNewOpacity(.5f)
				);
			}

			FSlateDrawElement::MakeText(
				MouseDrawInput.DrawElements,
				MouseDrawInput.IncrementDrawLayer().LayerId,
				MouseDrawInput.Geometry.ToPaintGeometry(TextDrawSize,FSlateLayoutTransform(TextPosition)), 
				FrameString, 
				TickFrameFont, 
				MouseDrawInput.DrawEffects, 
				InArgs.TickTextColor
			);
		}
	}

	return MouseDrawInput.LayerId;
}

int32 DrawKeys(FMouseDrawInputData& MouseDrawInput
	, const FKeyRenderer& InKeyRenderer
	, TOptional<FToolableTimelineKeyViewCacheState>& InViewCache
	, EViewDependentCacheFlags& InOutInvalidationFlags)
{
	const TSharedPtr<SToolableTimeline> TimelineWidget = MouseDrawInput.Timeline->GetWidget();
	if (!TimelineWidget.IsValid())
	{
		return MouseDrawInput.LayerId;
	}

	const TSharedPtr<FSequencer> Sequencer = MouseDrawInput.Timeline->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return MouseDrawInput.LayerId;
	}

	const TSharedPtr<FTimeToPixelSpace> RelativeTimeToPixel = TimelineWidget->GetTimeToPixel();
	if (!RelativeTimeToPixel.IsValid())
	{
		return MouseDrawInput.LayerId;
	}

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = MouseDrawInput.Timeline->GetTimeSliderController();
	const UToolableTimelineSettings& TimelineSettings = MouseDrawInput.Timeline->GetTimelineSettings();

	// Paint keys
	FKeyRendererPaintArgs PaintArgs;
	PaintArgs.DrawElements = &MouseDrawInput.DrawElements;
	PaintArgs.DrawEffects = MouseDrawInput.DrawEffects;
	PaintArgs.KeyThrobValue = SSequencerSection::GetKeySelectionThrobValue();
	PaintArgs.KeyStyle = static_cast<EKeyRendererStyle>(TimelineSettings.Settings.KeyDrawStyle);
	PaintArgs.KeyFillColor = TimelineSettings.Settings.KeyColor;
	PaintArgs.HoveredKeyFillColor = TimelineSettings.Settings.HoveredKeyColor;
	PaintArgs.KeySelectionFillColor = TimelineSettings.Settings.KeySelectionColor;
	PaintArgs.PartialKeyFillColor = TimelineSettings.Settings.PartialKeyColor;
	PaintArgs.PartialKeySelectionFillColor = TimelineSettings.Settings.PartialKeySelectionColor;
	PaintArgs.bBlendHoveredKeyColorWithBaseKeyColor = true;

	const FVector2D LocalSize = MouseDrawInput.Geometry.GetLocalSize();
	const FVector2D TopLeft = MouseDrawInput.Geometry.AbsoluteToLocal(MouseDrawInput.CullingRect.GetTopLeft());
	const FVector2D BottomRight = MouseDrawInput.Geometry.AbsoluteToLocal(MouseDrawInput.CullingRect.GetBottomRight());

	const TRange<FFrameTime> VisibleRange = TRange<FFrameTime>(
		RelativeTimeToPixel->PixelToFrame(TopLeft.X),
		RelativeTimeToPixel->PixelToFrame(BottomRight.X)
	);

	FToolableTimelineKeyViewCacheState NewCachedState(VisibleRange, *Sequencer.Get());
	NewCachedState.NonLinearTransform = RelativeTimeToPixel->NonLinearTransform;

	FKeyBatchParameters Params(*RelativeTimeToPixel);
	Params.VisibleRange = VisibleRange;
	Params.ValidPlayRangeMin = NewCachedState.ValidPlayRangeMin;
	Params.ValidPlayRangeMax = NewCachedState.ValidPlayRangeMax;
	Params.bShowKeyBars = false;
	Params.bShowCurve = false;
	Params.bCollapseChildren = true;
	Params.bUpdateViewIndependentData = false;

	FToolableTimelineKeyRenderer DefaultKeyRendererInterface(MouseDrawInput.Timeline->GetKeySelection());
	Params.ClientInterface = &DefaultKeyRendererInterface;

	const double TickSize = TimeSliderController->GetMajorTickDrawSize();
	const double KeyHeight = LocalSize.Y - TickSize;

	if (PaintArgs.KeyStyle == EKeyRendererStyle::Line)
	{
		switch (TimelineSettings.Settings.LabelVerticalAlignment)
		{
		case VAlign_Fill:
			Params.KeySizePx = FVector2D(0.0, LocalSize.Y);
			PaintArgs.KeyOffset = FVector2D(0.0, 0.0);
			break;
		default:
		case VAlign_Top:
			Params.KeySizePx = FVector2D(0.0, KeyHeight);
			PaintArgs.KeyOffset = FVector2D(0.0, LocalSize.Y - Params.KeySizePx.Y);
			break;
		case VAlign_Center:
			Params.KeySizePx = FVector2D(0.0, TickSize);
			PaintArgs.KeyOffset = FVector2D(0.0, KeyHeight * 0.5);
			break;
		case VAlign_Bottom:
			Params.KeySizePx = FVector2D(0.0, KeyHeight);
			PaintArgs.KeyOffset = FVector2D(0.0, 0.0);
			break;
		}

		NewCachedState.KeySizePx = Params.KeySizePx;
	}
	else if (PaintArgs.KeyStyle == EKeyRendererStyle::FrameBlock)
	{
		static constexpr double BlockPadding = 1.0;
		static constexpr double BlockPadding2x = BlockPadding * 2.0;

		const FFrameRate DisplayRate = TimeSliderController->GetDisplayRate();
		const TRange<double> ViewRange = TimeSliderController->GetViewRange();

		const double SecondsPerFrame = 1.0 / DisplayRate.AsDecimal();
		const double ViewSizeSeconds = ViewRange.GetUpperBoundValue() - ViewRange.GetLowerBoundValue();

		const double LocalWidthPx = FMath::Max(1.0, LocalSize.X);
		const double PixelsPerSecond = ViewSizeSeconds > 0.0 ? (LocalWidthPx / ViewSizeSeconds) : 1.0;

		const double BlockWidthPx =  FMath::RoundToDouble(SecondsPerFrame * PixelsPerSecond);
		const double FinalBlockWidthPx = FMath::Max(BlockWidthPx - BlockPadding2x, 1.0);

		switch (TimelineSettings.Settings.LabelVerticalAlignment)
		{
		default:
		case VAlign_Fill:
		case VAlign_Center:
			Params.KeySizePx = FVector2D(FinalBlockWidthPx, LocalSize.Y - BlockPadding2x);
			PaintArgs.KeyOffset = FVector2D(BlockPadding, BlockPadding);
			break;
		case VAlign_Top:
			Params.KeySizePx = FVector2D(FinalBlockWidthPx, KeyHeight - BlockPadding2x);
			PaintArgs.KeyOffset = FVector2D(BlockPadding, TickSize + BlockPadding);
			break;
		case VAlign_Bottom:
			Params.KeySizePx = FVector2D(FinalBlockWidthPx, KeyHeight - BlockPadding2x);
			PaintArgs.KeyOffset = FVector2D(BlockPadding, BlockPadding);
			break;
		}

		NewCachedState.KeySizePx = Params.KeySizePx;
	}
	else
	{
		Params.KeySizePx = SequencerSectionConstants::KeySize;
		NewCachedState.KeySizePx = Params.KeySizePx;

		switch (TimelineSettings.Settings.LabelVerticalAlignment)
		{
		case VAlign_Fill:
			break;
		default:
		case VAlign_Top:
			PaintArgs.KeyOffset = FVector2D(0.0, TickSize * 0.5);
			break;
		case VAlign_Center:
			break;
		case VAlign_Bottom:
			PaintArgs.KeyOffset = FVector2D(0.0, TickSize * -0.5);
			break;
		}
	}

	Params.CacheState = InViewCache.IsSet()
		? NewCachedState.CompareTo(InViewCache.GetValue())
		: EViewDependentCacheFlags::All;

	if (InOutInvalidationFlags != EViewDependentCacheFlags::None)
	{
		Params.CacheState |= InOutInvalidationFlags;
		InOutInvalidationFlags = EViewDependentCacheFlags::None;
	}

	InKeyRenderer.Update(Params, MouseDrawInput.Geometry);
	MouseDrawInput.LayerId = InKeyRenderer.Draw(Params
		, MouseDrawInput.Geometry, MouseDrawInput.CullingRect, PaintArgs, MouseDrawInput.LayerId);

	InViewCache = NewCachedState;

	return MouseDrawInput.IncrementDrawLayer().LayerId;
}

int32 DrawScrubHandle(FMouseDrawInputData& MouseDrawInput)
{
	const TSharedPtr<FSequencer> Sequencer = MouseDrawInput.Timeline->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return MouseDrawInput.LayerId;
	}

	const USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!SequencerSettings)
	{
		return MouseDrawInput.LayerId;
	}

	const TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = Sequencer->GetViewModel();
	if (!SequencerViewModel.IsValid())
	{
		return MouseDrawInput.LayerId;
	}

	const TViewModelPtr<FSequenceModel> RootSequenceModel = SequencerViewModel->GetRootSequenceModel();
	if (!RootSequenceModel.IsValid())
	{
		return MouseDrawInput.LayerId;
	}

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = MouseDrawInput.Timeline->GetTimeSliderController();
	const UToolableTimelineSettings& TimelineSettings = MouseDrawInput.Timeline->GetTimelineSettings();
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	const FTimeSliderArgs& TimeSliderArgs = TimeSliderController->GetTimeSliderArgs();

	const FQualifiedFrameTime ScrubPosition = FQualifiedFrameTime(TimeSliderArgs.ScrubPosition.Get(), TickResolution);
	const FFrameNumber ScrubFrame = ScrubPosition.Time.GetFrame();
	const FSequencerTimeSliderController::FScrubberMetrics ScrubMetrics
		= TimeSliderController->GetScrubPixelMetrics(ScrubPosition, MouseDrawInput.RangeToScreen);

	const float HandleStart = ScrubMetrics.HandleRangePx.GetLowerBoundValue();
	const float HandleEnd = ScrubMetrics.HandleRangePx.GetUpperBoundValue();

	const FTimeWarpTrackExtension* const TimeWarpExtension = RootSequenceModel->GetSharedData()->CastDynamic<FTimeWarpTrackExtension>();
	const FTimeWarpTrackModel* const ActiveTimeWarpTrack = TimeWarpExtension
		? TimeWarpExtension->GetActiveTimeWarpTrack() : nullptr;

	const bool bIsEvaluating = TimeSliderController->IsEvaluating();
	const bool bIsWarped = SequencerSettings->GetTimeWarpDisplayMode() == ESequencerTimeWarpDisplay::WarpedTime
		&& ActiveTimeWarpTrack != nullptr;
	const bool bIsNonEvaluatingOrWarped = !bIsEvaluating || bIsWarped;

	// Determine scrub frame overlay color
	FLinearColor ScrubFrameOverlayColor = TimelineSettings.Settings.ScrubFrameOverlayColor;
	if (!bIsEvaluating)
	{
		ScrubFrameOverlayColor = TimelineSettings.Settings.NonEvaluatingScrubColor;
	}
	else if (bIsWarped)
	{
		ScrubFrameOverlayColor = TimelineSettings.Settings.TimeWarpedScrubColor;
	}

	// Determine scrub color
	FLinearColor ScrubColorToDraw = TimelineSettings.Settings.ScrubColor;
	if (!bIsEvaluating)
	{
		ScrubColorToDraw = TimelineSettings.Settings.NonEvaluatingScrubColor;
	}
	else if (bIsWarped)
	{
		ScrubColorToDraw = TimelineSettings.Settings.TimeWarpedScrubColor;
	}

	// Determine scrub text color
	const TArray<FMovieSceneSequenceID> ParentChain = TimeSliderArgs.ScrubPositionParentChain.Get();
	const bool bValidScrubPosition = TimeSliderArgs.ScrubPositionParent.Get() != MovieSceneSequenceID::Invalid
		&& ParentChain.Contains(TimeSliderArgs.ScrubPositionParent.Get())
		&& TimeSliderArgs.ScrubPositionParent.Get() != Sequencer->GetFocusedTemplateID()
		&& TimeSliderArgs.ScrubPositionText.IsSet();

	FLinearColor ScrubTextColorToDraw = TimelineSettings.Settings.ScrubTextColor;
	if (!bIsEvaluating)
	{
		ScrubTextColorToDraw = TimelineSettings.Settings.NonEvaluatingScrubTextColor;
	}
	else if (bIsWarped)
	{
		ScrubTextColorToDraw = TimelineSettings.Settings.TimeWarpedScrubTextColor;
	}

	// Draw the scrub handle overlay
	const FSlateBrush* const Brush = ScrubMetrics.Style == ESequencerScrubberStyle::Vanilla
		? FAppStyle::GetBrush(TEXT("Sequencer.Timeline.ScrubHandle"))
		: FAppStyle::GetBrush(TEXT("Sequencer.Timeline.FrameBlockScrubHandle"));

	FPaintGeometry ScrubHandleGeometry = MouseDrawInput.Geometry.ToPaintGeometry(
		FVector2f(HandleEnd - HandleStart, MouseDrawInput.Geometry.GetLocalSize().Y),
		FSlateLayoutTransform(FVector2f(HandleStart, 0.f))
	);
	MirrorTransformY(ScrubHandleGeometry, MouseDrawInput.Geometry, MouseDrawInput.bMirrorLabels);
	
	FSlateDrawElement::MakeBox(
		MouseDrawInput.DrawElements,
		MouseDrawInput.LayerId,
		ScrubHandleGeometry,
		Brush,
		MouseDrawInput.DrawEffects,
		ScrubFrameOverlayColor.CopyWithNewOpacity(TimelineSettings.Settings.ScrubFrameOverlayColor.A)
	);

	// Draw top and bottom horizontal line accents
	const float HandleWidth = HandleEnd - HandleStart;
	const float ScrubHandleHeight = MouseDrawInput.Geometry.GetLocalSize().Y;

	FSlateDrawElement::MakeBox(
		MouseDrawInput.DrawElements,
		MouseDrawInput.LayerId,
		MouseDrawInput.Geometry.ToPaintGeometry(
			FVector2f(HandleWidth, 2.f),
			FSlateLayoutTransform(FVector2f(HandleStart, 0.f))
		),
		FAppStyle::GetBrush(TEXT("WhiteBrush")),
		MouseDrawInput.DrawEffects,
		ScrubColorToDraw
	);

	FSlateDrawElement::MakeBox(
		MouseDrawInput.DrawElements,
		MouseDrawInput.LayerId,
		MouseDrawInput.Geometry.ToPaintGeometry(
			FVector2f(HandleWidth, 2.f),
			FSlateLayoutTransform(FVector2f(HandleStart, ScrubHandleHeight - 2.f))
		),
		FAppStyle::GetBrush(TEXT("WhiteBrush")),
		MouseDrawInput.DrawEffects,
		ScrubColorToDraw
	);

	// Draw vertical sub frame line if we are not snapping to frame
	if (!SequencerSettings->GetForceWholeFrames())
	{
		const float ScrubPixelX = MouseDrawInput.RangeToScreen.InputToLocalX(ScrubPosition.AsSeconds());

		FSlateDrawElement::MakeBox(
			MouseDrawInput.DrawElements,
			MouseDrawInput.LayerId,
			MouseDrawInput.Geometry.ToPaintGeometry(
				FVector2f(2.f, ScrubHandleHeight),
				FSlateLayoutTransform(FVector2f(ScrubPixelX - 1.f, 0.f))
			),
			FAppStyle::GetBrush(TEXT("WhiteBrush")),
			MouseDrawInput.DrawEffects,
			ScrubColorToDraw
		);
	}

	const FName FontStyleName = TimelineSettings.Settings.ScrubTextBold ? TEXT("Bold") : TEXT("Regular");
	const FSlateFontInfo TickFrameFont = FCoreStyle::GetDefaultFontStyle(FontStyleName, TimelineSettings.Settings.ScrubFontSize);

	auto MeasureText = [&TickFrameFont](const FStringView& InText) -> FVector2D
	{
		const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		return FontMeasureService->Measure(InText, TickFrameFont);
	};

	auto GetLocalFrameString = [&Sequencer, &ScrubFrame]() -> FString
	{
		return Utils::TickFrameToString(*Sequencer, ScrubFrame.Value, /*bInRemoveLeadingZeros=*/true, false);
	};

	auto GetParentFrameString = [&Sequencer, &TimeSliderArgs, bValidScrubPosition]() -> FString
	{
		if (!bValidScrubPosition)
		{
			return FString();
		}
		const FString TickFrameString = Utils::CleanTickFrameString(*Sequencer
			, TimeSliderArgs.ScrubPositionText.Get(), /*bInRemoveLeadingZeros=*/true, false);
		return FString::Printf(TEXT("%s :"), *TickFrameString);
	};

	auto DrawFrameText = [&MouseDrawInput, &TickFrameFont, &ScrubTextColorToDraw](
		const FVector2D& InTextOffset,
		int32 InLayerId,
		const FString& InParentFrameString,
		const FVector2D& InParentTextSize,
		const FVector2D& InSpaceTextSize,
		const FString& InLocalFrameString,
		const FVector2D& InLocalTextSize)
	{
		FVector2D TextOffset = InTextOffset;

		if (!InParentFrameString.IsEmpty())
		{
			FSlateDrawElement::MakeText(
				MouseDrawInput.DrawElements,
				InLayerId,
				MouseDrawInput.Geometry.ToPaintGeometry(InParentTextSize, FSlateLayoutTransform(TextOffset)),
				InParentFrameString,
				TickFrameFont,
				MouseDrawInput.DrawEffects,
				FLinearColor::Yellow
			);

			TextOffset.X += InParentTextSize.X + InSpaceTextSize.X;
		}

		FSlateDrawElement::MakeText(
			MouseDrawInput.DrawElements,
			InLayerId,
			MouseDrawInput.Geometry.ToPaintGeometry(InLocalTextSize, FSlateLayoutTransform(TextOffset)),
			InLocalFrameString,
			TickFrameFont,
			MouseDrawInput.DrawEffects,
			ScrubTextColorToDraw
		);
	};

	const FString LocalFrameString = GetLocalFrameString();
	const FString ParentFrameString = GetParentFrameString();
	const FVector2D LocalTextSize = MeasureText(LocalFrameString);
	const FVector2D ParentTextSize = ParentFrameString.IsEmpty() ? FVector2D::ZeroVector : MeasureText(ParentFrameString);
	const FVector2D SpaceTextSize = ParentFrameString.IsEmpty() ? FVector2D::ZeroVector : MeasureText(TEXT(" "));
	const FVector2D TextSize = ParentFrameString.IsEmpty()
		? LocalTextSize
		: FVector2D(ParentTextSize.X + SpaceTextSize.X + LocalTextSize.X, FMath::Max(ParentTextSize.Y, LocalTextSize.Y));

	// Draw current scrub time and handle
	switch (TimelineSettings.Settings.ScrubHeadStyle)
	{
	default:
	case EToolableTimelineScrubHeadStyle::NewCenteredBlock:
		{
			// Center the text on the scrub handle
			const float HandleCenter = (HandleStart + HandleEnd) * 0.5f;
			const float TickSize = CalculateTickSize(MouseDrawInput);
			FVector2D NormalTextOffset = FVector2D::ZeroVector;
			
			NormalTextOffset.X = HandleCenter - (TextSize.X * 0.5f);

			switch (TimelineSettings.Settings.LabelVerticalAlignment)
			{
			default:
			case VAlign_Fill:
			case VAlign_Center:
				NormalTextOffset.Y = MouseDrawInput.Geometry.GetLocalSize().Y - TextSize.Y - Constants::ScrubTextPad.Y;
				break;
			case VAlign_Top:
				NormalTextOffset.Y = Constants::FrameLabelOffset.Y;
				break;
			case VAlign_Bottom:
				NormalTextOffset.Y = MouseDrawInput.Geometry.GetLocalSize().Y - TickSize;
				break;
			}

			// Draw white box behind text
			const FVector2D BoxPadding(4.0, 2.0);
			const FVector2D BoxSize = TextSize + (BoxPadding * 2.0);
			const FVector2D BoxPosition = NormalTextOffset - BoxPadding;

			FSlateDrawElement::MakeBox(
				MouseDrawInput.DrawElements,
				MouseDrawInput.IncrementDrawLayer().LayerId,
				MouseDrawInput.Geometry.ToPaintGeometry(BoxSize, FSlateLayoutTransform(BoxPosition)),
				FAppStyle::GetBrush(TEXT("WhiteBrush")),
				MouseDrawInput.DrawEffects,
				ScrubColorToDraw
			);

			DrawFrameText(NormalTextOffset, MouseDrawInput.IncrementDrawLayer().LayerId, ParentFrameString, ParentTextSize, SpaceTextSize, LocalFrameString, LocalTextSize);
		}
		break;

	case EToolableTimelineScrubHeadStyle::FrameBubble:
		{
			const TRange<FFrameNumber> ScrubRange = TRange<FFrameNumber>{ ScrubFrame, ScrubFrame };

			FDrawFrameNumberBubbleArgs DrawArgs;
			DrawArgs.LabelAlignment = TimelineSettings.Settings.LabelVerticalAlignment;
			DrawArgs.BubbleFontSize = TimelineSettings.Settings.ScrubFontSize;
			DrawArgs.BubblePadding = FMargin(5.f, 2.f);
			DrawArgs.BubbleColor = ScrubColorToDraw;
			DrawArgs.BubbleTextColor = ScrubTextColorToDraw;
			DrawArgs.bPreventLeadingZeros = true;
			DrawArgs.bTryKeepOnScreen = true;
			DrawArgs.bAlignToWholeFrame = TimelineSettings.Settings.bDrawWholeFrameScrubber;

			FFrameRangeBubbleDrawer FrameBubbleDrawer(DrawArgs);
			MouseDrawInput.LayerId = FrameBubbleDrawer.Draw(ScrubRange, MouseDrawInput.IncrementDrawLayer());
		}
		break;

	case EToolableTimelineScrubHeadStyle::OldTextOnly:
		{
			// Flip the text position if getting near the end of the view range
			const double DistanceFromRight = MouseDrawInput.Geometry.GetLocalSize().X - HandleEnd;
			const double TextSizePlusPad = (TextSize.X + 14.0) - Constants::ScrubTextPad.X;
			const bool bDrawLeftInsteadOfRight = DistanceFromRight < TextSizePlusPad;

			FVector2D NormalTextOffset = FVector2D::ZeroVector;

			NormalTextOffset.X = bDrawLeftInsteadOfRight
				? HandleStart - TextSize.X - Constants::ScrubTextPad.X
				: HandleEnd + Constants::ScrubTextPad.X;

			switch (TimelineSettings.Settings.LabelVerticalAlignment)
			{
			default:
			case VAlign_Fill:
			case VAlign_Top:
			case VAlign_Center:
				NormalTextOffset.Y = MouseDrawInput.Geometry.GetLocalSize().Y - TextSize.Y - Constants::ScrubTextPad.Y;
				break;
			case VAlign_Bottom:
				NormalTextOffset.Y = Constants::ScrubTextPad.Y;
				break;
			}

			DrawFrameText(NormalTextOffset, MouseDrawInput.IncrementDrawLayer().LayerId, ParentFrameString, ParentTextSize, SpaceTextSize, LocalFrameString, LocalTextSize);
		}
		break;
	}

	return MouseDrawInput.LayerId;
}

int32 DrawDragRangeIndicator(FMouseDrawInputData& MouseDrawInput)
{
	const TSharedPtr<FSequencer> Sequencer = MouseDrawInput.Timeline->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return MouseDrawInput.LayerId;
	}

	const USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!SequencerSettings)
	{
		return MouseDrawInput.LayerId;
	}

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = MouseDrawInput.Timeline->GetTimeSliderController();
	const TSharedPtr<FToolableTimelineBaseTool> ActiveTool = TimeSliderController->GetActiveTool();

	bool bDrawToolIndicator = false;
	const bool bDrawControllerIndicator = TimeSliderController->IsDragging() && FSlateApplication::Get().GetModifierKeys().IsAltDown();
	bool bDrawIndicator = false;

	if (ActiveTool.IsValid())
	{
		bDrawToolIndicator = ActiveTool.IsValid()
			&& ActiveTool->IsDragging()
			&& ActiveTool->ShouldShowDragRangeIndicator();
		bDrawIndicator = bDrawToolIndicator || bDrawControllerIndicator;
	}
	else
	{
		bDrawIndicator = bDrawControllerIndicator;
	}

	if (!bDrawIndicator)
	{
		return MouseDrawInput.LayerId;
	}

	const UToolableTimelineSettings& TimelineSettings = MouseDrawInput.Timeline->GetTimelineSettings();
	const FMouseInputData MouseInput = MouseDrawInput.ToMouseInputData();

	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	const FFrameRate DisplayRate = TimeSliderController->GetDisplayRate();

	// Get the indicator start and end tick times based on mouse input
	FFrameTime StartTickTime;
	FFrameTime EndTickTime;

	const TRange<FFrameTime> DragIndicatorTickRange = ActiveTool.IsValid()
		? ActiveTool->GetDragIndicatorTickRange(MouseInput) : TRange<FFrameTime>();
	if (bDrawToolIndicator
		&& DragIndicatorTickRange.GetLowerBound().IsClosed()
		&& DragIndicatorTickRange.GetUpperBound().IsClosed())
	{
		StartTickTime = DragIndicatorTickRange.GetLowerBoundValue();
		EndTickTime = DragIndicatorTickRange.GetUpperBoundValue();
	}
	else
	{
		const TOptional<FToolableTimelineScrubDragOperation>& ScrubDragOperation = TimeSliderController->GetScrubDragOperation();

		StartTickTime = ScrubDragOperation.IsSet()
			? ScrubDragOperation->GetInitialTickTime()
			: TimeSliderController->ComputeMouseFrameTime(MouseInput, /*bInCheckSnapping=*/false);

		EndTickTime = TimeSliderController->ComputeMouseFrameTime(MouseInput, /*bInCheckSnapping=*/false);
	}

	const FFrameNumber StartDisplayFrame = FFrameRate::TransformTime(StartTickTime, TickResolution, DisplayRate).FloorToFrame();
	const FFrameNumber EndDisplayFrame = FFrameRate::TransformTime(EndTickTime, TickResolution, DisplayRate).FloorToFrame();

	const FFrameNumber MinDisplayFrame = FMath::Min(StartDisplayFrame, EndDisplayFrame);
	const FFrameNumber MaxDisplayFrame = FMath::Max(StartDisplayFrame, EndDisplayFrame);
	const FFrameNumber MaxDisplayFrameExclusive = MaxDisplayFrame + 1;

	const FFrameTime MinTickTime = FFrameRate::TransformTime(FFrameTime(MinDisplayFrame), DisplayRate, TickResolution);
	const FFrameTime MaxExclusiveTickTime = FFrameRate::TransformTime(FFrameTime(MaxDisplayFrameExclusive), DisplayRate, TickResolution);

	const double IndicatorStartPx = MouseDrawInput.RangeToScreen.InputToLocalX(MinTickTime / TickResolution);
	const double IndicatorEndPx = MouseDrawInput.RangeToScreen.InputToLocalX(MaxExclusiveTickTime / TickResolution);

	// Draw drag start and end vertical lines
	auto DrawVerticalLine = [&TimelineSettings, &MouseDrawInput](const double& InLocationX)
	{
		TArray<FVector2f> LinePoints;
		LinePoints.SetNumUninitialized(2);
		LinePoints[0] = FVector2f(InLocationX, 0.f);
		LinePoints[1] = FVector2f(InLocationX, MouseDrawInput.Geometry.GetLocalSize().Y);

		FSlateDrawElement::MakeDashedLines(
			MouseDrawInput.DrawElements,
			MouseDrawInput.LayerId,
			MouseDrawInput.Geometry.ToPaintGeometry(),
			MoveTemp(LinePoints),
			MouseDrawInput.DrawEffects,
			TimelineSettings.Settings.DragRangeIndicatorLineColor,
			1.f,
			5.f
		);
	};

	DrawVerticalLine(IndicatorStartPx);
	DrawVerticalLine(IndicatorEndPx);

	// Draw drag span horizontal lines
	auto DrawHorizontalLine = [&TimelineSettings, &MouseDrawInput, &IndicatorStartPx, &IndicatorEndPx](const double& InLocationY)
	{
		TArray<FVector2f> LinePoints;
		LinePoints.SetNumUninitialized(2);
		LinePoints[0] = FVector2f(IndicatorStartPx, InLocationY);
		LinePoints[1] = FVector2f(IndicatorEndPx, InLocationY);

		FSlateDrawElement::MakeDashedLines(
			MouseDrawInput.DrawElements,
			MouseDrawInput.LayerId,
			MouseDrawInput.Geometry.ToPaintGeometry(),
			MoveTemp(LinePoints),
			MouseDrawInput.DrawEffects,
			TimelineSettings.Settings.DragRangeIndicatorLineColor,
			1.f,
			10.f
		);
	};

	const float GeometryHeight = MouseDrawInput.Geometry.GetLocalSize().Y;
	if (GeometryHeight > 0.f)
	{
		DrawHorizontalLine(1.f);
		DrawHorizontalLine(GeometryHeight);
	}

	// Draw frame range number
	FDrawFrameNumberBubbleArgs DrawArgs;
	DrawArgs.bTryKeepOnScreen = false;
	DrawArgs.bShowFrameOffsetInsteadOfRange = true;
	DrawArgs.BubbleTextColor = FStyleColors::Black.GetSpecifiedColor();
	DrawArgs.BubbleColor = FLinearColor(0.4f, 0.4f, 0.4f, 1.0f);
	DrawArgs.BubbleFontSize = 6.5f;
	DrawArgs.BubbleBorderColor = FLinearColor::Black;
	DrawArgs.bPreventLeadingZeros = true;
	DrawArgs.bAlignToWholeFrame = SequencerSettings->GetForceWholeFrames();

	switch (TimelineSettings.Settings.LabelVerticalAlignment)
	{
	default:
	case VAlign_Fill:
	case VAlign_Top:
	case VAlign_Center:
		DrawArgs.LabelAlignment = VAlign_Bottom;
		break;
	case VAlign_Bottom:
		DrawArgs.LabelAlignment = VAlign_Top;
		break;
	}

	FFrameNumber StartBubbleTick;
	FFrameNumber EndBubbleTick;

	if (EndDisplayFrame >= StartDisplayFrame)
	{
		StartBubbleTick = FFrameRate::TransformTime(FFrameTime(StartDisplayFrame), DisplayRate, TickResolution).FloorToFrame();
		EndBubbleTick = FFrameRate::TransformTime(FFrameTime(EndDisplayFrame + 1), DisplayRate, TickResolution).CeilToFrame();
	}
	else
	{
		StartBubbleTick = FFrameRate::TransformTime(FFrameTime(StartDisplayFrame + 1), DisplayRate, TickResolution).CeilToFrame();
		EndBubbleTick = FFrameRate::TransformTime(FFrameTime(EndDisplayFrame), DisplayRate, TickResolution).FloorToFrame();
	}

	const TRange<FFrameNumber> BubbleScrubRange(StartBubbleTick, EndBubbleTick);

	FFrameRangeBubbleDrawer FrameBubbleDrawer(DrawArgs);
	FrameBubbleDrawer.Draw(BubbleScrubRange, MouseDrawInput);

	return MouseDrawInput.IncrementDrawLayer().LayerId;
}

int32 DrawAreaViewScrubPosition(FMouseDrawInputData& MouseDrawInput, const bool bInDisplayScrubPosition)
{
	const TSharedPtr<FSequencer> Sequencer = MouseDrawInput.Timeline->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return MouseDrawInput.LayerId;
	}

	const USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!SequencerSettings)
	{
		return MouseDrawInput.LayerId;
	}

	static const FSlateBrush* const ScrubFillBrush = FAppStyle::GetBrush(TEXT("Sequencer.Timeline.ScrubFill"));

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = MouseDrawInput.Timeline->GetTimeSliderController();
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	const UToolableTimelineSettings& TimelineSettings = MouseDrawInput.Timeline->GetTimelineSettings();
	const FTimeSliderArgs& TimeSliderArgs = TimeSliderController->GetTimeSliderArgs();

	const FTimeWarpTrackExtension* const TimeWarpExtension = Sequencer->GetViewModel()->GetRootSequenceModel()->GetSharedData()->CastDynamic<FTimeWarpTrackExtension>();
	const FTimeWarpTrackModel* const ActiveTimeWarpTrack = TimeWarpExtension ? TimeWarpExtension->GetActiveTimeWarpTrack() : nullptr;

	const FLinearColor TimeWarpColor = FStyleColors::AccentOrange.GetSpecifiedColor();

	// If we have no active timewarp track, just draw the single scrub position
	if (!ActiveTimeWarpTrack
		|| SequencerSettings->GetTimeWarpDisplayMode() != ESequencerTimeWarpDisplay::Both)
	{
		const bool bIsWarped = (SequencerSettings->GetTimeWarpDisplayMode() == ESequencerTimeWarpDisplay::WarpedTime
			&& ActiveTimeWarpTrack != nullptr);
		const FLinearColor ScrubColor = bIsWarped ? TimeWarpColor : TimelineSettings.Settings.ScrubFrameOverlayColor;

		const FQualifiedFrameTime ScrubPosition = FQualifiedFrameTime(TimeSliderArgs.ScrubPosition.Get(), TickResolution);
		const FSequencerTimeSliderController::FScrubberMetrics ScrubMetrics
			= TimeSliderController->GetScrubPixelMetrics(ScrubPosition, MouseDrawInput.RangeToScreen);

		if (ScrubMetrics.bDrawExtents)
		{
			FSlateDrawElement::MakeBox(
				MouseDrawInput.DrawElements,
				MouseDrawInput.IncrementDrawLayer().LayerId,
				MouseDrawInput.Geometry.ToPaintGeometry(
					FVector2f(ScrubMetrics.FrameExtentsPx.Size<float>(), MouseDrawInput.Geometry.GetLocalSize().Y),
					FSlateLayoutTransform(FVector2f(ScrubMetrics.FrameExtentsPx.GetLowerBoundValue(), 0.f))
				),
				ScrubFillBrush,
				MouseDrawInput.DrawEffects,
				ScrubColor.CopyWithNewOpacity(0.5f)
			);
		}

		if (bInDisplayScrubPosition)
		{
			const float ScrubPixelX = MouseDrawInput.RangeToScreen.InputToLocalX(ScrubPosition.AsSeconds());

			TArray<FVector2f> LinePoints;
			LinePoints.SetNumUninitialized(2);
			LinePoints[0] = FVector2f(ScrubPixelX, 0.f);
			LinePoints[1] = FVector2f(ScrubPixelX, MouseDrawInput.Geometry.GetLocalSize().Y);

			// Draw a white line for the unwarped scrub position
			FSlateDrawElement::MakeLines(
				MouseDrawInput.DrawElements,
				MouseDrawInput.IncrementDrawLayer().LayerId,
				MouseDrawInput.Geometry.ToPaintGeometry(),
				LinePoints,
				MouseDrawInput.DrawEffects,
				ScrubColor,
				false
			);
		}
	}
	else
	{
		// Handle timewarp by drawing the white (unwarped) position up to the bottom of the active timewarp track,
		// then draw the orange (warped) time from there
		const FVirtualGeometry Geometry = ActiveTimeWarpTrack->GetVirtualGeometry();
		const float UnwarpedScrubVerticalClip = Sequencer->GetUnderlyingSequencerWidget()->GetPinnedTreeView()->VirtualToPhysical(Geometry.GetTop() + Geometry.GetNestedHeight());

		const FQualifiedFrameTime UnwarpedScrubPosition = Sequencer->GetUnwarpedLocalTime();
		const FQualifiedFrameTime WarpedScrubPosition = Sequencer->GetLocalTime();
		const FSequencerTimeSliderController::FScrubberMetrics UnwarpedScrubMetrics
			= TimeSliderController->GetScrubPixelMetrics(UnwarpedScrubPosition, MouseDrawInput.RangeToScreen);
		const FSequencerTimeSliderController::FScrubberMetrics WarpedScrubMetrics
			= TimeSliderController->GetScrubPixelMetrics(WarpedScrubPosition, MouseDrawInput.RangeToScreen);

		if (UnwarpedScrubMetrics.bDrawExtents)
		{
			// Draw a box for the unwarped scrub position
			FSlateDrawElement::MakeBox(
				MouseDrawInput.DrawElements,
				MouseDrawInput.IncrementDrawLayer().LayerId,
				MouseDrawInput.Geometry.ToPaintGeometry(
					FVector2f(UnwarpedScrubMetrics.FrameExtentsPx.Size<float>(), UnwarpedScrubVerticalClip - 1.f),
					FSlateLayoutTransform(FVector2f(UnwarpedScrubMetrics.FrameExtentsPx.GetLowerBoundValue(), 0.f))
				),
				ScrubFillBrush,
				MouseDrawInput.DrawEffects,
				FLinearColor::White.CopyWithNewOpacity(.5f)
			);

			// Draw a box for the warped scrub position
			FSlateDrawElement::MakeBox(
				MouseDrawInput.DrawElements,
				MouseDrawInput.LayerId,
				MouseDrawInput.Geometry.ToPaintGeometry(
					FVector2f(WarpedScrubMetrics.FrameExtentsPx.Size<float>(), MouseDrawInput.Geometry.GetLocalSize().Y - UnwarpedScrubVerticalClip),
					FSlateLayoutTransform(FVector2f(WarpedScrubMetrics.FrameExtentsPx.GetLowerBoundValue(), UnwarpedScrubVerticalClip))
				),
				ScrubFillBrush,
				MouseDrawInput.DrawEffects,
				TimeWarpColor.CopyWithNewOpacity(.75f)
			);
		}

		if (bInDisplayScrubPosition)
		{
			const float UnwarpedScrubPixelX = MouseDrawInput.RangeToScreen.InputToLocalX(UnwarpedScrubPosition.AsSeconds());
			const float WarpedScrubPixelX = MouseDrawInput.RangeToScreen.InputToLocalX(WarpedScrubPosition.AsSeconds());
			const float OriginPixelX = MouseDrawInput.RangeToScreen.InputToLocalX(0.f);

			TArray<FVector2f> LinePoints;
			LinePoints.SetNumUninitialized(2);
			LinePoints[0] = FVector2f(UnwarpedScrubPixelX, 0.f);
			LinePoints[1] = FVector2f(UnwarpedScrubPixelX, UnwarpedScrubVerticalClip - 1.f);

			// Draw a white line for the unwarped scrub position
			FSlateDrawElement::MakeLines(
				MouseDrawInput.DrawElements,
				MouseDrawInput.IncrementDrawLayer().LayerId,
				MouseDrawInput.Geometry.ToPaintGeometry(),
				LinePoints,
				MouseDrawInput.DrawEffects,
				FLinearColor(1.f, 1.f, 1.f, .5f),
				false
			);

			LinePoints[0] = FVector2f(WarpedScrubPixelX, UnwarpedScrubVerticalClip);
			LinePoints[1] = FVector2f(WarpedScrubPixelX, MouseDrawInput.Geometry.GetLocalSize().Y);

			// Draw a line for the warped scrub position
			FSlateDrawElement::MakeLines(
				MouseDrawInput.DrawElements,
				MouseDrawInput.LayerId,
				MouseDrawInput.Geometry.ToPaintGeometry(),
				LinePoints,
				MouseDrawInput.DrawEffects,
				TimeWarpColor,
				false
			);

			// Optional cvar for drawing the dashed line
			const IConsoleVariable* const CVarShowTimeWarpScrubberLink = IConsoleManager::Get().FindConsoleVariable(TEXT("Sequencer.ShowTimeWarpScrubberLink"));
			if (CVarShowTimeWarpScrubberLink && CVarShowTimeWarpScrubberLink->GetBool())
			{
				const float StartLine = WarpedScrubPixelX < UnwarpedScrubPixelX ? WarpedScrubPixelX : UnwarpedScrubPixelX;
				const float EndLine = WarpedScrubPixelX < UnwarpedScrubPixelX ? UnwarpedScrubPixelX : WarpedScrubPixelX;
				const float HalfPos = (EndLine + StartLine) * .5f;

				// Draw a dashed line to connect them
				LinePoints[0] = FVector2f(StartLine, UnwarpedScrubVerticalClip - 1.f);
				LinePoints[1] = FVector2f(FMath::Max(HalfPos - 10.f, StartLine), UnwarpedScrubVerticalClip - 1.f);

				FSlateDrawElement::MakeDashedLines(
					MouseDrawInput.DrawElements,
					MouseDrawInput.IncrementDrawLayer().LayerId,
					MouseDrawInput.Geometry.ToPaintGeometry(),
					CopyTemp(LinePoints),
					MouseDrawInput.DrawEffects,
					FColor(212, 147, 20),
					2.f,
					5.f,
					StartLine - OriginPixelX
				);

				LinePoints[0] = FVector2f(FMath::Min(HalfPos + 10.f, EndLine),   UnwarpedScrubVerticalClip - 1.f);
				LinePoints[1] = FVector2f(EndLine, UnwarpedScrubVerticalClip - 1.f);

				FSlateDrawElement::MakeDashedLines(
					MouseDrawInput.DrawElements,
					MouseDrawInput.LayerId,
					MouseDrawInput.Geometry.ToPaintGeometry(),
					CopyTemp(LinePoints),
					MouseDrawInput.DrawEffects,
					FColor(212, 147, 20),
					2.f,
					5.f,
					StartLine - OriginPixelX
				);

				FSlateDrawElement::MakeBox(
					MouseDrawInput.DrawElements,
					MouseDrawInput.LayerId,
					MouseDrawInput.Geometry.ToPaintGeometry(
						FVector2f(12.f, 12.f),
						FSlateLayoutTransform(FVector2f(HalfPos - 6.f, UnwarpedScrubVerticalClip - 6.f))
					),
					FAppStyle::GetBrush(TEXT("Sequencer.Tracks.TimeWarp")),
					MouseDrawInput.DrawEffects,
					TimeWarpColor.CopyWithNewOpacity(.75f)
				);
			}
		}
	}

	return MouseDrawInput.IncrementDrawLayer().LayerId;
}

} // UE::Sequencer::SimpleView
