// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerTimeSliderController.h"

namespace UE::Sequencer
{
	enum class EViewDependentCacheFlags : uint8;
	struct FKeyRenderer;
}

namespace UE::Sequencer::ToolableTimeline
{
	struct FMouseDrawInputData;
	struct FToolableTimelineKeyViewCacheState;
}

namespace UE::Sequencer::ToolableTimeline::Drawing
{

namespace Constants
{
	/** Vertical and horizontal padding for scrub frame text */
	const FVector2d ScrubTextPad = FVector2d(4., 0.);

	/** Offset for each tick frame label */
	const FVector2d FrameLabelOffset = FVector2d(5., 1.);

	/** Do not draw per-frame vertical lines when a frame is narrower than this many pixels */
	constexpr float MinPixelsPerFrameBackgroundLine = 2.f;

	constexpr float FrameAreaPadding = 2.f;
	constexpr float FrameAreaDoublePadding = FrameAreaPadding * 2.f;
}

void MirrorTransformY(FPaintGeometry& InPaintGeometry, const FGeometry& InGeometry, const bool bInMirrorLabels);

void GetFrameAreaGeometry(const FMouseDrawInputData& InMouseDrawInput, const FMargin& InPadding, FVector2f& OutSize, FVector2f& OutPosition);

float CalculateTickSize(const FMouseDrawInputData& InMouseDrawInput);

void GetMajorLinePy(const FMouseDrawInputData& InMouseDrawInput, const float InTickSize, float& OutMajorLinePy, float& OutHorizontalLineStartY);

int32 DrawBackground(FMouseDrawInputData& MouseDrawInput);

int32 DrawTicks(FMouseDrawInputData& MouseDrawInput, FSequencerTimeSliderController::FDrawTickArgs& InArgs);

int32 DrawKeys(FMouseDrawInputData& MouseDrawInput
	, const FKeyRenderer& InKeyRenderer
	, TOptional<FToolableTimelineKeyViewCacheState>& InViewCache
	, EViewDependentCacheFlags& InOutInvalidationFlags);

int32 DrawScrubHandle(FMouseDrawInputData& MouseDrawInput);

int32 DrawDragRangeIndicator(FMouseDrawInputData& MouseDrawInput);

int32 DrawAreaViewScrubPosition(FMouseDrawInputData& MouseDrawInput, const bool bInDisplayScrubPosition);

} // namespace UE::Sequencer::ToolableTimeline
