// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Events.h"
#include "Layout/Geometry.h"
#include "ToolableTimeline/ToolableTimeline.h"
#include "ToolableTimeSliderControllerMenuContext.generated.h"

UCLASS(MinimalAPI)
class UToolableTimeSliderControllerMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<UE::Sequencer::ToolableTimeline::FToolableTimeline> WeakTimeline;

	FGeometry Geometry;

	FPointerEvent PointerEvent;

	FFrameNumber FrameNumber;
};
