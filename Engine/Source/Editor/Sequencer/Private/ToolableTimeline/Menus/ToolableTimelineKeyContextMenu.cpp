// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimelineKeyContextMenu.h"
#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneTimeWarpChannel.h"
#include "Framework/Commands/GenericCommands.h"
#include "IKeyArea.h"
#include "ISequencer.h"
#include "ISequencerChannelInterface.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "SequencerContextMenus.h"
#include "SequencerToolMenuContext.h"
#include "SKeyEditInterface.h"
#include "Styling/AppStyle.h"
#include "ToolableTimeline/MouseInputData.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "ToolableTimelineKeyContextMenu"

namespace UE::Sequencer::ToolableTimeline
{

namespace KeyContextMenu
{
	struct FCurveInterpolationMenuExtension : TSharedFromThis<FCurveInterpolationMenuExtension>
	{
		FCurveInterpolationMenuExtension(const TWeakPtr<FSequencer>& InWeakSequencer)
			: WeakSequencer(InWeakSequencer)
		{
		}

		bool HasAnyChannels() const
		{
			return FloatChannels.Num() > 0 || DoubleChannels.Num() > 0 || TimeWarpChannels.Num() > 0;
		}

		/** A lot of this is similar to FSectionContextMenu::AddKeyInterpolationMenu or FSectionContextMenu::BuildKeyEditMenu,
		 * but we can't reuse those directly because this menu is driven by Toolable Timeline key selection, not the
		 * standard Sequencer section/context-menu selection path, and it needs to operate on the exact selected key handles
		 * across grouped channels rather than the broader section/key-area state those helpers assume.
		 * @TODO: Extract common key interpolation menu building logic into a reusable helper functions/class. */
		void ExtendMenu(FMenuBuilder& MenuBuilder)
		{
			const TSharedRef<FCurveInterpolationMenuExtension> SharedThis = AsShared();

			MenuBuilder.BeginSection(TEXT("SequencerInterpolation"), LOCTEXT("KeyInterpolationMenu", "Key Interpolation"));
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("SetKeyInterpolationSmartAuto", "Cubic (Smart Auto)"),
					LOCTEXT("SetKeyInterpolationSmartAutoTooltip", "Set key interpolation to smart auto"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.IconKeySmartAuto")),
					FUIAction(
						FExecuteAction::CreateLambda([SharedThis]
							{
								SharedThis->SetInterpTangentMode(RCIM_Cubic, RCTM_SmartAuto);
							}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([SharedThis]
							{
								return SharedThis->IsInterpTangentModeSelected(RCIM_Cubic, RCTM_SmartAuto);
							})),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("SetKeyInterpolationAuto", "Cubic (Auto)"),
					LOCTEXT("SetKeyInterpolationAutoTooltip", "Set key interpolation to auto"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.IconKeyAuto")),
					FUIAction(
						FExecuteAction::CreateLambda([SharedThis]
							{
								SharedThis->SetInterpTangentMode(RCIM_Cubic, RCTM_Auto);
							}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([SharedThis]
							{
								return SharedThis->IsInterpTangentModeSelected(RCIM_Cubic, RCTM_Auto);
							})),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("SetKeyInterpolationUser", "Cubic (User)"),
					LOCTEXT("SetKeyInterpolationUserTooltip", "Set key interpolation to user"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.IconKeyUser")),
					FUIAction(
						FExecuteAction::CreateLambda([SharedThis]
							{
								SharedThis->SetInterpTangentMode(RCIM_Cubic, RCTM_User);
							}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([SharedThis]
							{
								return SharedThis->IsInterpTangentModeSelected(RCIM_Cubic, RCTM_User);
							})),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("SetKeyInterpolationBreak", "Cubic (Break)"),
					LOCTEXT("SetKeyInterpolationBreakTooltip", "Set key interpolation to break"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.IconKeyBreak")),
					FUIAction(
						FExecuteAction::CreateLambda([SharedThis]
							{
								SharedThis->SetInterpTangentMode(RCIM_Cubic, RCTM_Break);
							}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([SharedThis]
							{
								return SharedThis->IsInterpTangentModeSelected(RCIM_Cubic, RCTM_Break);
							})),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("SetKeyInterpolationLinear", "Linear"),
					LOCTEXT("SetKeyInterpolationLinearTooltip", "Set key interpolation to linear"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.IconKeyLinear")),
					FUIAction(
						FExecuteAction::CreateLambda([SharedThis]
							{
								SharedThis->SetInterpTangentMode(RCIM_Linear, RCTM_Auto);
							}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([SharedThis]
							{
								return SharedThis->IsInterpTangentModeSelected(RCIM_Linear, RCTM_Auto);
							})),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("SetKeyInterpolationConstant", "Constant"),
					LOCTEXT("SetKeyInterpolationConstantTooltip", "Set key interpolation to constant"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.IconKeyConstant")),
					FUIAction(
						FExecuteAction::CreateLambda([SharedThis]
							{
								SharedThis->SetInterpTangentMode(RCIM_Constant, RCTM_Auto);
							}),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([SharedThis]
							{
								return SharedThis->IsInterpTangentModeSelected(RCIM_Constant, RCTM_Auto);
							})),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);
			}
			MenuBuilder.EndSection();
		}

		template<typename ChannelType>
		static bool IsInterpTangentModeSelectedForChannels(const TArray<FExtendKeyMenuParams>& InChannels
			, const ERichCurveInterpMode InterpMode, const ERichCurveTangentMode TangentMode)
		{
			for (const FExtendKeyMenuParams& Channel : InChannels)
			{
				ChannelType* const ChannelPtr = Channel.Channel.Cast<ChannelType>().Get();
				if (!ChannelPtr)
				{
					continue;
				}

				auto ChannelData = ChannelPtr->GetData();
				auto Values = ChannelData.GetValues();

				for (const FKeyHandle Handle : Channel.Handles)
				{
					const int32 KeyIndex = ChannelData.GetIndex(Handle);
					if (KeyIndex == INDEX_NONE
						|| Values[KeyIndex].InterpMode != InterpMode
						|| Values[KeyIndex].TangentMode != TangentMode)
					{
						return false;
					}
				}
			}

			return true;
		}

		bool IsInterpTangentModeSelected(ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode) const
		{
			return IsInterpTangentModeSelectedForChannels<FMovieSceneFloatChannel>(FloatChannels, InterpMode, TangentMode)
				&& IsInterpTangentModeSelectedForChannels<FMovieSceneDoubleChannel>(DoubleChannels, InterpMode, TangentMode)
				&& IsInterpTangentModeSelectedForChannels<FMovieSceneTimeWarpChannel>(TimeWarpChannels, InterpMode, TangentMode);
		}

		template<typename ChannelType>
		static void SetInterpTangentModeForChannels(const TArray<FExtendKeyMenuParams>& InChannels
			, const ERichCurveInterpMode InInterpMode
			, const ERichCurveTangentMode InTangentMode
			, TSet<UMovieSceneSignedObject*>& ModifiedOwners
			, bool& bOutAnythingChanged)
		{
			for (const FExtendKeyMenuParams& Channel : InChannels)
			{
				if (UMovieSceneSignedObject* const OwningObject = Cast<UMovieSceneSignedObject>(Channel.WeakOwner.Get()))
				{
					if (!ModifiedOwners.Contains(OwningObject))
					{
						OwningObject->Modify();
						ModifiedOwners.Add(OwningObject);
					}
				}

				ChannelType* const ChannelPtr = Channel.Channel.Cast<ChannelType>().Get();
				if (!ChannelPtr)
				{
					continue;
				}

				auto ChannelData = ChannelPtr->GetData();
				auto Values = ChannelData.GetValues();

				for (const FKeyHandle Handle : Channel.Handles)
				{
					const int32 KeyIndex = ChannelData.GetIndex(Handle);
					if (KeyIndex != INDEX_NONE)
					{
						Values[KeyIndex].InterpMode = InInterpMode;
						Values[KeyIndex].TangentMode = InTangentMode;
						bOutAnythingChanged = true;
					}
				}

				ChannelPtr->AutoSetTangents();
			}
		}

		void SetInterpTangentMode(const ERichCurveInterpMode InInterpMode, const ERichCurveTangentMode InTangentMode)
		{
			const FScopedTransaction Transaction(LOCTEXT("SetInterpTangentMode_Transaction", "Set Interpolation and Tangent Mode"));

			TSet<UMovieSceneSignedObject*> ModifiedOwners;
			bool bAnythingChanged = false;

			SetInterpTangentModeForChannels<FMovieSceneFloatChannel>(FloatChannels, InInterpMode, InTangentMode, ModifiedOwners, bAnythingChanged);
			SetInterpTangentModeForChannels<FMovieSceneDoubleChannel>(DoubleChannels, InInterpMode, InTangentMode, ModifiedOwners, bAnythingChanged);
			SetInterpTangentModeForChannels<FMovieSceneTimeWarpChannel>(TimeWarpChannels, InInterpMode, InTangentMode, ModifiedOwners, bAnythingChanged);

			const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
			if (Sequencer.IsValid() && bAnythingChanged)
			{
				Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
			}
		}

		TWeakPtr<FSequencer> WeakSequencer;

		TArray<FExtendKeyMenuParams> FloatChannels;
		TArray<FExtendKeyMenuParams> DoubleChannels;
		TArray<FExtendKeyMenuParams> TimeWarpChannels;
	};

	void CreateToolableTimelineKeyStructForSelection(const TWeakPtr<ISequencer>& InWeakSequencer
		, TSharedPtr<FStructOnScope>& OutKeyStruct, TWeakObjectPtr<UMovieSceneSection>& OutKeyStructSection
		, TWeakObjectPtr<UObject>& OutKeyStructOwningObject)
	{
		using namespace UE::Sequencer;

		const TSharedPtr<ISequencer> Sequencer = InWeakSequencer.Pin();
		if (!Sequencer.IsValid())
		{
			return;
		}

		const FKeySelection& SelectedKeys = Sequencer->GetViewModel()->GetSelection()->KeySelection;

		if (SelectedKeys.Num() == 1)
		{
			for (const FKeyHandle Key : SelectedKeys)
			{
				if (const TSharedPtr<FChannelModel> Channel = SelectedKeys.GetModelForKey(Key))
				{
					OutKeyStruct = Channel->GetKeyArea()->GetKeyStruct(Key);
					OutKeyStructSection = Channel->GetSection();
					if (const FMovieSceneChannelMetaData* MetaData = Channel->GetKeyArea()->GetChannel().GetMetaData())
					{
						OutKeyStructOwningObject = MetaData->WeakOwningObject;
					}
					return;
				}
			}
		}
		else
		{
			TArray<FKeyHandle> KeyHandles;
			UMovieSceneSection* CommonSection = nullptr;

			for (const FKeyHandle Key : SelectedKeys)
			{
				if (const TSharedPtr<FChannelModel> Channel = SelectedKeys.GetModelForKey(Key))
				{
					KeyHandles.Add(Key);

					if (!CommonSection)
					{
						CommonSection = Channel->GetSection();
					}
					else if (CommonSection != Channel->GetSection())
					{
						CommonSection = nullptr;

						return;
					}
				}
			}

			if (CommonSection)
			{
				OutKeyStruct = CommonSection->GetKeyStruct(KeyHandles);
				OutKeyStructSection = CommonSection;
			}
		}
	}

	TMap<FName, TArray<FExtendKeyMenuParams>> BuildSelectedKeyMenuParams(const TSharedRef<FSequencer>& InSequencer)
	{
		using namespace UE::Sequencer;

		TMap<FName, TArray<FExtendKeyMenuParams>> OutChannelParams;
		TMap<FName, TMap<const FMovieSceneChannel*, int32>> ChannelIndicesByType;

		const FKeySelection& SelectedKeys = InSequencer->GetViewModel()->GetSelection()->KeySelection;

		for (FKeyHandle KeyHandle : SelectedKeys)
		{
			const TSharedPtr<FChannelModel> ChannelModel = SelectedKeys.GetModelForKey(KeyHandle);
			if (!ChannelModel.IsValid())
			{
				continue;
			}

			const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea();
			if (!KeyArea.IsValid())
			{
				continue;
			}

			const FMovieSceneChannelHandle ChannelHandle = KeyArea->GetChannel();
			FMovieSceneChannel* const RawChannel = ChannelHandle.Get();
			if (!RawChannel)
			{
				continue;
			}

			const FName ChannelTypeName = ChannelHandle.GetChannelTypeName();

			TArray<FExtendKeyMenuParams>& ParamsArray = OutChannelParams.FindOrAdd(ChannelTypeName);
			TMap<const FMovieSceneChannel*, int32>& IndicesByChannel = ChannelIndicesByType.FindOrAdd(ChannelTypeName);

			if (const int32* ExistingIndex = IndicesByChannel.Find(RawChannel))
			{
				ParamsArray[*ExistingIndex].Handles.Add(KeyHandle);
				continue;
			}

			FExtendKeyMenuParams& Params = ParamsArray.AddDefaulted_GetRef();
			Params.Section = ChannelModel->GetSection();
			Params.WeakOwner = ChannelModel->GetSection();
			Params.Channel = ChannelHandle;
			Params.Handles.Add(KeyHandle);

			IndicesByChannel.Add(RawChannel, ParamsArray.Num() - 1);
		}

		return OutChannelParams;
	}
}

TSharedRef<SWidget> FToolableTimelineKeyContextMenu::GenerateWidget(const FMouseInputData& InMouseInput)
{
	using namespace KeyContextMenu;

	const TSharedPtr<FSequencer> Sequencer = InMouseInput.Timeline->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	const TWeakPtr<FSequencer> WeakSequencer = Sequencer;
	const TSharedRef<FCurveInterpolationMenuExtension> CurveInterpolationMenu =
		MakeShared<FCurveInterpolationMenuExtension>(WeakSequencer);

	const TSharedPtr<FExtender> MenuExtender = MakeShared<FExtender>();
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection*/true, Sequencer->GetCommandBindings(), MenuExtender);

	// 1) Properties submenu
	{
		TSharedPtr<FStructOnScope> KeyStruct;
		TWeakObjectPtr<UMovieSceneSection> KeyStructSection;
		TWeakObjectPtr<UObject> KeyStructOwningObject;
		CreateToolableTimelineKeyStructForSelection(WeakSequencer, KeyStruct, KeyStructSection, KeyStructOwningObject);

		if (KeyStruct.IsValid())
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("KeyProperties", "Properties"),
				LOCTEXT("KeyPropertiesTooltip", "Modify the key properties"),
				FNewMenuDelegate::CreateLambda([WeakSequencer](FMenuBuilder& SubMenuBuilder)
				{
					const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
					if (!Sequencer.IsValid())
					{
						return;
					}

					const TSharedRef<SKeyEditInterface> KeyEditWidget = SNew(SKeyEditInterface, Sequencer.ToSharedRef())
						.EditData_Lambda([WeakSequencer]
							{
								FKeyEditData EditData;
								CreateToolableTimelineKeyStructForSelection(WeakSequencer, EditData.KeyStruct, EditData.OwningSection, EditData.OwningObject);
								return EditData;
							});
					SubMenuBuilder.AddWidget(KeyEditWidget, FText::GetEmpty(), /*bInNoIndent=*/true);
				}),
				FUIAction(
					FExecuteAction(),
					FCanExecuteAction::CreateLambda([KeyStruct]() { return KeyStruct.IsValid(); })
				),
				NAME_None,
				EUserInterfaceActionType::Button);
		}
	}

	// 2) Register built-in per-channel key menu extensions using the ACTUAL selected keys.
	//    Curve channel interpolation is added once below so multi-type selections do not duplicate it.
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>(TEXT("Sequencer"));

	TMap<FName, TArray<FExtendKeyMenuParams>> ChannelAndHandlesByType = BuildSelectedKeyMenuParams(Sequencer.ToSharedRef());

	for (TPair<FName, TArray<FExtendKeyMenuParams>>& Pair : ChannelAndHandlesByType)
	{
		if (Pair.Key == FMovieSceneFloatChannel::StaticStruct()->GetFName())
		{
			CurveInterpolationMenu->FloatChannels = MoveTemp(Pair.Value);
			continue;
		}
		if (Pair.Key == FMovieSceneDoubleChannel::StaticStruct()->GetFName())
		{
			CurveInterpolationMenu->DoubleChannels = MoveTemp(Pair.Value);
			continue;
		}
		if (Pair.Key == FMovieSceneTimeWarpChannel::StaticStruct()->GetFName())
		{
			CurveInterpolationMenu->TimeWarpChannels = MoveTemp(Pair.Value);
			continue;
		}

		if (ISequencerChannelInterface* const ChannelInterface = SequencerModule.FindChannelEditorInterface(Pair.Key))
		{
			ChannelInterface->ExtendKeyMenu_Raw(MenuBuilder, MenuExtender, MoveTemp(Pair.Value), Sequencer);
		}
	}

	// 3) Add one interpolation section for the whole selected key set.
	if (CurveInterpolationMenu->HasAnyChannels())
	{
		CurveInterpolationMenu->ExtendMenu(MenuBuilder);
	}

	// 4) Add your own Edit section because you are NOT using FKeyHotspot.
	//    The built-in ToolMenu will not add this section for you in that case.
	MenuBuilder.BeginSection(TEXT("SequencerKeyEdit"), LOCTEXT("EditMenu", "Edit"));
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
	}
	MenuBuilder.EndSection();

	// 5) Stock Keys widget LAST
	//    We add ONLY the generated ToolMenu widget here, not FKeyContextMenu::BuildMenu,
	//    so Properties stays at the top and does not get inserted again.
	{
		if (!UToolMenus::Get()->IsMenuRegistered(TEXT("Sequencer.KeyContextMenu")))
		{
			const TSharedPtr<FExtender> RegistrationExtender = MakeShared<FExtender>();
			FMenuBuilder RegistrationBuilder(true, Sequencer->GetCommandBindings(), RegistrationExtender);
			FKeyContextMenu::BuildMenu(RegistrationBuilder, RegistrationExtender, WeakSequencer);
		}

		USequencerToolMenuContext* ContextObject = NewObject<USequencerToolMenuContext>();
		ContextObject->WeakSequencer = Sequencer;

		FToolMenuContext MenuContext(ContextObject);
		MenuContext.AppendCommandList(Sequencer->GetCommandBindings(ESequencerCommandBindings::Sequencer));
		MenuContext.AddExtender(MenuExtender);

		MenuBuilder.AddWidget(
			UToolMenus::Get()->GenerateWidget(TEXT("Sequencer.KeyContextMenu"), MenuContext),
			FText(),
			true,
			false
		);
	}

	return MenuBuilder.MakeWidget();
}

} // namespace UE::Sequencer::ToolableTimeline

#undef LOCTEXT_NAMESPACE
