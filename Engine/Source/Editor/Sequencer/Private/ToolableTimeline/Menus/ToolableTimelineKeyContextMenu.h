// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class SWidget;

namespace UE::Sequencer::ToolableTimeline
{

struct FMouseInputData;

/**  */
class FToolableTimelineKeyContextMenu
{
public:
	static TSharedRef<SWidget> GenerateWidget(const FMouseInputData& InMouseInput);

protected:

};

} // namespace UE::Sequencer::ToolableTimeline
