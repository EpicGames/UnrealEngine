// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimelineMenu.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "CurveEditor.h"
#include "CurveEditorCommands.h"
#include "Framework/Commands/GenericCommands.h"
#include "ISequencer.h"
#include "MovieScene.h"
#include "MVVM/Extensions/IClockExtension.h"
#include "MVVM/Selection/Selection.h"
#include "PropertyCustomizationHelpers.h"
#include "Sequencer.h"
#include "SequencerCommands.h"
#include "SequencerToolMenuContext.h"
#include "SSequencer.h"
#include "SSequencerPlayRateCombo.h"
#include "ToolableTimeline/Menus/ToolableTimeSliderControllerMenuContext.h"
#include "ToolableTimeline/MouseInputData.h"
#include "ToolableTimeline/ToolableTimelineCommands.h"
#include "ToolableTimeline/ToolableTimelineInstanceSettingsCustomization.h"
#include "ToolableTimeline/ToolableTimeline.h"
#include "ToolableTimeline/ToolableTimelineUtils.h"
#include "ToolableTimeline/ToolableTimeSliderController.h"
#include "Widgets/Sidebar/SMarkedFrameDetails.h"

#define LOCTEXT_NAMESPACE "TimelineMenu"

namespace UE::Sequencer::ToolableTimeline
{

const FName FToolableTimelineMenu::MenuName = TEXT("Sequencer.ToolableTimeline");

TSharedRef<SWidget> FToolableTimelineMenu::GenerateWidget(const FMouseInputData& InMouseInput)
{
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InMouseInput.Timeline->GetTimeSliderController();

	UToolableTimeSliderControllerMenuContext* const TimeSliderContext = NewObject<UToolableTimeSliderControllerMenuContext>();
	TimeSliderContext->WeakTimeline = InMouseInput.Timeline;
	TimeSliderContext->Geometry = InMouseInput.Geometry;
	TimeSliderContext->PointerEvent = InMouseInput.PointerEvent;
	TimeSliderContext->FrameNumber = TimeSliderController->ComputeMouseFrameTime(InMouseInput, /*bInCheckSnapping=*/true).FrameNumber;

	USequencerToolMenuContext* const SequencerContext = NewObject<USequencerToolMenuContext>();
	SequencerContext->WeakSequencer = InMouseInput.Timeline->GetSequencer();

	FToolMenuContext MenuContext(InMouseInput.Timeline->GetCommandList());
	MenuContext.AddObject(TimeSliderContext);
	MenuContext.AddObject(SequencerContext);

	return UToolMenus::Get()->GenerateWidget(MenuName, MenuContext);
}

void FToolableTimelineMenu::RegisterMenu()
{
	if (UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		return;
	}

	UToolMenu* const TimelineMenu = UToolMenus::Get()->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu);
	TimelineMenu->bSearchable = false;
	TimelineMenu->bShouldCloseWindowAfterMenuSelection = true;

	TimelineMenu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateStatic(PopulateMenu));
}

void FToolableTimelineMenu::PopulateMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	AddSettingsSection(InMenu);
	AddKeyDisplaySection(InMenu);
	AddSnapSection(InMenu);
	AddGeneralSection(InMenu);
}

void FToolableTimelineMenu::AddPlaybackRangeSection(UToolMenu* const InMenu)
{
	UToolableTimeSliderControllerMenuContext* const MenuContext = InMenu
		? InMenu->FindContext<UToolableTimeSliderControllerMenuContext>() : nullptr;
	if (!MenuContext)
	{
		return;
	}

	const TSharedPtr<FToolableTimeline> Timeline = MenuContext->WeakTimeline.Pin();
	if (!Timeline.IsValid())
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = Timeline->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = Timeline->GetTimeSliderController();
	const FTimeSliderArgs& TimeSliderArgs = TimeSliderController->GetTimeSliderArgs();
	const TRange<FFrameNumber> PlaybackRange = TimeSliderArgs.PlaybackRange.Get();
	const TOptional<TRange<FFrameNumber>> SubSequenceRange = TimeSliderArgs.SubSequenceRange.Get();
	const bool bReadOnly = Sequencer->IsReadOnly();
	const FFrameNumber& FrameNumber = MenuContext->FrameNumber;
	const FText CurrentTimeText = FText::FromString(Sequencer->GetNumericTypeInterface()->ToString(FrameNumber.Value));

	const TWeakPtr<FToolableTimeline> WeakTimeline = Timeline;

	FToolMenuSection& Section = InMenu->FindOrAddSection(TEXT("PlaybackRange")
		, FText::Format(LOCTEXT("PlaybackRangeTextFormat", "Playback Range ({0}):"), CurrentTimeText));

	Section.AddMenuEntry(TEXT("SetPlaybackStart"),
		LOCTEXT("SetPlaybackStart", "Set Start Time"),
		FText(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([WeakTimeline, FrameNumber]
			{
				if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
				{
					Timeline->GetTimeSliderController()->SetPlaybackRangeStart(FrameNumber);
				}
			}),
			FCanExecuteAction::CreateLambda([WeakTimeline, FrameNumber, PlaybackRange]
			{
				if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
				{
					return !Timeline->GetTimeSliderController()->GetTimeSliderArgs().IsPlaybackRangeLocked.Get()
						&& FrameNumber < MovieScene::DiscreteExclusiveUpper(PlaybackRange);
				}
				return false;
			})
		));

	Section.AddMenuEntry(TEXT("SetPlaybackEnd"),
		LOCTEXT("SetPlaybackEnd", "Set End Time"),
		FText(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([WeakTimeline, FrameNumber]
			{
				if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
				{
					Timeline->GetTimeSliderController()->SetPlaybackRangeEnd(FrameNumber);
				}
			}),
			FCanExecuteAction::CreateLambda([WeakTimeline, FrameNumber, PlaybackRange]
			{
				if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
				{
					return !Timeline->GetTimeSliderController()->GetTimeSliderArgs().IsPlaybackRangeLocked.Get()
						&& FrameNumber >= MovieScene::DiscreteInclusiveLower(PlaybackRange);
				}
				return false;
			})
		));

	Section.AddMenuEntry(TEXT("ConformToSubsequenceRange"),
		LOCTEXT("ConformToSubsequenceRange", "Conform to Range"),
		LOCTEXT("ConformToSubsequenceRangeTooltip", "Conform the start and end time to the extents of the subsequence range"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([WeakTimeline, SubSequenceRange]
			{
				if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
				{
					const TSharedRef<FToolableTimeSliderController> TimeSliderController = Timeline->GetTimeSliderController();
					TimeSliderController->SetPlaybackRangeStart(SubSequenceRange.GetValue().GetLowerBoundValue());
					TimeSliderController->SetPlaybackRangeEnd(SubSequenceRange.GetValue().GetUpperBoundValue());
				}
			}),
			FCanExecuteAction::CreateLambda([WeakTimeline, SubSequenceRange]
			{
				if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
				{
					return !Timeline->GetTimeSliderController()->GetTimeSliderArgs().IsPlaybackRangeLocked.Get()
						&& SubSequenceRange.IsSet();
				}
				return false;
			})
		));

	Section.AddMenuEntry(TEXT("ToggleLocked"),
		LOCTEXT("ToggleLocked", "Locked"),
		LOCTEXT("ToggleLockedTooltip", "Lock/Unlock the playback range"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([WeakTimeline]
			{
				if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
				{
					Timeline->GetTimeSliderController()->GetTimeSliderArgs().OnTogglePlaybackRangeLocked.ExecuteIfBound();
				}
			}),
			FCanExecuteAction::CreateLambda([bReadOnly]{ return !bReadOnly; }),
			FIsActionChecked::CreateLambda([WeakTimeline]
			{
				if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
				{
					return Timeline->GetTimeSliderController()->GetTimeSliderArgs().IsPlaybackRangeLocked.Get();
				}
				return false;
			})
		),
		EUserInterfaceActionType::ToggleButton);
}

void FToolableTimelineMenu::AddSelectionRangeSection(UToolMenu* const InMenu)
{
	UToolableTimeSliderControllerMenuContext* const MenuContext = InMenu
		? InMenu->FindContext<UToolableTimeSliderControllerMenuContext>() : nullptr;
	if (!MenuContext)
	{
		return;
	}

	const TSharedPtr<FToolableTimeline> Timeline = MenuContext->WeakTimeline.Pin();
	if (!Timeline.IsValid())
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = Timeline->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const FTimeSliderArgs& TimeSliderArgs = Timeline->GetTimeSliderController()->GetTimeSliderArgs();
	const TRange<FFrameNumber> SelectionRange = TimeSliderArgs.SelectionRange.Get();
	const FFrameNumber& FrameNumber = MenuContext->FrameNumber;
	const FText CurrentTimeText = FText::FromString(Sequencer->GetNumericTypeInterface()->ToString(FrameNumber.Value));

	const FToolableTimelineCommands& TimelineCommands = FToolableTimelineCommands::Get();
	const FSequencerCommands& SequencerCommands = FSequencerCommands::Get();
	const TSharedPtr<FUICommandList> SequencerCommandBindings = Sequencer->GetCommandBindings();

	const TWeakPtr<FToolableTimeline> WeakTimeline = Timeline;

	FToolMenuSection& Section = InMenu->FindOrAddSection(TEXT("SelectionRange")
		, FText::Format(LOCTEXT("SelectionRangeTextFormat", "Selection Range ({0}):"), CurrentTimeText));

	Section.AddMenuEntry(TimelineCommands.SetSelectionRangeFromToolRange);
	Section.AddMenuEntryWithCommandList(SequencerCommands.SetSelectionRangeStart, SequencerCommandBindings);
	Section.AddMenuEntryWithCommandList(SequencerCommands.SetSelectionRangeEnd, SequencerCommandBindings);
	Section.AddMenuEntryWithCommandList(SequencerCommands.ClearSelectionRange, SequencerCommandBindings);
}

void FToolableTimelineMenu::AddParentChainSection(UToolMenu* const InMenu)
{
	const TSharedPtr<FToolableTimeline> Timeline = GetTimelineFromMenu(InMenu);
	if (!Timeline.IsValid())
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = Timeline->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UMovieSceneCompiledDataManager* const CompiledDataManager = Sequencer->GetEvaluationTemplate().GetCompiledDataManager();
	if (!CompiledDataManager)
	{
		return;
	}

	const FMovieSceneSequenceHierarchy* const Hierarchy = CompiledDataManager->FindHierarchy(Sequencer->GetEvaluationTemplate().GetCompiledDataID());
	if (!Hierarchy)
	{
		return;
	}

	const FTimeSliderArgs& TimeSliderArgs = Timeline->GetTimeSliderController()->GetTimeSliderArgs();

	FToolMenuSection& Section = InMenu->FindOrAddSection("ParentChain", LOCTEXT("ParentChain", "Parent Chain"));

	for (const FMovieSceneSequenceID& ParentID : TimeSliderArgs.ScrubPositionParentChain.Get())
	{
		FText ParentText = Sequencer->GetRootMovieSceneSequence()->GetDisplayName();

		for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
		{
			if (Pair.Key == ParentID && Pair.Value.GetSequence())
			{
				ParentText = Pair.Value.GetSequence()->GetDisplayName();
				break;
			}
		}

		const TWeakPtr<FToolableTimeline> WeakTimeline = Timeline;

		Section.AddMenuEntry(NAME_None,
			ParentText,
			FText::Format(LOCTEXT("DisplayTimeSpace", "Display time in the space of {0}"), ParentText),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([WeakTimeline, ParentID]
				{
					if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
					{
						Timeline->GetTimeSliderController()->GetTimeSliderArgs().OnScrubPositionParentChanged.ExecuteIfBound(ParentID);
					}
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([WeakTimeline, ParentID]
				{
					if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
					{
						const FTimeSliderArgs& TimeSliderArgs = Timeline->GetTimeSliderController()->GetTimeSliderArgs();
						return TimeSliderArgs.ScrubPositionParent.Get() == MovieSceneSequenceID::Invalid
							? ParentID == TimeSliderArgs.ScrubPositionParentChain.Get().Last()
							: TimeSliderArgs.ScrubPositionParent.Get() == ParentID;
					}
					return false;
				})
			),
			EUserInterfaceActionType::RadioButton);
	}
}

void FToolableTimelineMenu::AddMarksSection(UToolMenu* const InMenu)
{
	UToolableTimeSliderControllerMenuContext* const MenuContext = InMenu
		? InMenu->FindContext<UToolableTimeSliderControllerMenuContext>() : nullptr;
	if (!MenuContext)
	{
		return;
	}

	const TSharedPtr<FToolableTimeline> Timeline = MenuContext->WeakTimeline.Pin();
	if (!Timeline.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = StaticCastSharedPtr<FSequencer>(Timeline->GetSequencer());
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = Timeline->GetTimeSliderController();
	const FTimeSliderArgs& TimeSliderArgs = TimeSliderController->GetTimeSliderArgs();

	const FToolableTimeSliderController::FScrubRangeToScreen RangeToScreen = FToolableTimeSliderController::FScrubRangeToScreen(TimeSliderController->GetViewRange(), MenuContext->Geometry.Size);
	const float MousePixel = MenuContext->Geometry.AbsoluteToLocal(MenuContext->PointerEvent.GetScreenSpacePosition()).X;

	const FFrameNumber& FrameNumber = MenuContext->FrameNumber;
	const FText CurrentTimeText = FText::FromString(Sequencer->GetNumericTypeInterface()->ToString(FrameNumber.Value));

	const bool bReadOnly = Sequencer->IsReadOnly();
	UMovieScene* const MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	const bool bHasMarks = MovieScene->GetMarkedFrames().Num() > 0;

	int32 MarkedIndex = INDEX_NONE;
	constexpr bool bTestLabelBox = true;
	TimeSliderController->HitTestMark(MenuContext->Geometry, RangeToScreen, MousePixel, bTestLabelBox, &MarkedIndex);

	FSequencerSelection& SequencerSelection = StaticCastSharedPtr<FSequencer>(Sequencer)->GetSelection();

	const TWeakPtr<FToolableTimeline> WeakTimeline = Timeline;

	FToolMenuSection& Section = InMenu->FindOrAddSection(TEXT("Marks")
		, FText::Format(LOCTEXT("MarkTextFormat", "Mark ({0}):"), CurrentTimeText));

	if (MarkedIndex != INDEX_NONE)
	{
		const bool bIsSelected = SequencerSelection.MarkedFrames.IsSelected(MarkedIndex);
		if (!bIsSelected)
		{
			SequencerSelection.MarkedFrames.Empty();
			SequencerSelection.MarkedFrames.Select(MarkedIndex);
		}

		const bool bLockedMarkedFrames = TimeSliderArgs.AreMarkedFramesLocked.Get();

		const TSharedRef<SMarkedFrameDetails> Widget = SNew(SMarkedFrameDetails, MarkedIndex, Sequencer.ToWeakPtr())
			.IsEnabled(!bLockedMarkedFrames);

		Section.AddEntry(FToolMenuEntry::InitWidget(NAME_None, Widget, FText::GetEmpty(), false));
	}

	Section.AddMenuEntry(TEXT("AddMark"),
		LOCTEXT("AddMarkAtPlayhead", "Add Mark at Playhead"),
		FText(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda( [WeakTimeline]
			{
				if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
				{
					const TSharedRef<FToolableTimeSliderController> Controller = Timeline->GetTimeSliderController();
					Controller->AddMarkAtFrame(Controller->GetTimeSliderArgs().ScrubPosition.Get().FloorToFrame());
				}
			}),
			FCanExecuteAction::CreateLambda([WeakTimeline, MarkedIndex]
			{
				if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
				{
					return Timeline.IsValid() && !Timeline->GetTimeSliderController()->GetTimeSliderArgs().AreMarkedFramesLocked.Get()
						&& MarkedIndex == INDEX_NONE;
				}
				return false;
			})
		));

	Section.AddMenuEntry(TEXT("DeleteMark"),
		LOCTEXT("DeleteSelectedMarks", "Delete Selected Marks"),
		FText(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([WeakTimeline]
			{
				if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
				{
					Timeline->GetTimeSliderController()->DeleteSelectedMarks();
				}
			}),
			FCanExecuteAction::CreateLambda([WeakTimeline]
			{
				if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
				{
					const TSharedPtr<FSequencer> Sequencer = StaticCastSharedPtr<FSequencer>(Timeline->GetSequencer());
					return Sequencer.IsValid()
						&& !Timeline->GetTimeSliderController()->GetTimeSliderArgs().AreMarkedFramesLocked.Get()
						&& Sequencer->GetSelection().MarkedFrames.Num() > 0;
				}
				return false;
			})
		));

	Section.AddMenuEntry(TEXT("DeleteAllMarks"),
		LOCTEXT("DeleteAllMarks", "Delete All Marks"),
		FText(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([WeakTimeline]
			{
				if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
				{
					Timeline->GetTimeSliderController()->DeleteAllMarks();
				}
			}),
			FCanExecuteAction::CreateLambda([WeakTimeline, bHasMarks]
			{
				if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
				{
					return !Timeline->GetTimeSliderController()->GetTimeSliderArgs().AreMarkedFramesLocked.Get()
						&& bHasMarks;
				}
				return false;
			})
		));

	Section.AddMenuEntry(TEXT("ToggleLockedMarks"),
		LOCTEXT("ToggleLockedMarks", "Locked"),
		LOCTEXT("ToggleLockedMarksTooltip", "Lock/Unlock all marked frames"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([WeakTimeline]
			{
				if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
				{
					Timeline->GetTimeSliderController()->GetTimeSliderArgs().OnToggleMarkedFramesLocked.ExecuteIfBound();
				}
			}),
			FCanExecuteAction::CreateLambda([bReadOnly]{ return !bReadOnly; }),
			FIsActionChecked::CreateLambda([WeakTimeline]
			{
				if (const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin())
				{
					return Timeline->GetTimeSliderController()->GetTimeSliderArgs().AreMarkedFramesLocked.Get();
				}
				return false;
			})
		),
		EUserInterfaceActionType::ToggleButton);
}

void FToolableTimelineMenu::AddGeneralSection(UToolMenu* const InMenu)
{
	const TSharedPtr<FToolableTimeline> Timeline = GetTimelineFromMenu(InMenu);
	if (!Timeline.IsValid())
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = Timeline->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const FToolableTimelineCommands& TimelineEditorCommands = FToolableTimelineCommands::Get();
	const FSequencerCommands& SequencerCommands = FSequencerCommands::Get();

	const TSharedPtr<FUICommandList> SequencerCommandList = Sequencer->GetCommandBindings();

	FToolMenuSection& TimelineOptionsSection = InMenu->FindOrAddSection(TEXT("TimelineOptions")
		, LOCTEXT("TimelineOptions", "Timeline Options"));

	TimelineOptionsSection.AddEntry(FToolMenuEntry::InitSubMenu(TEXT("PlaybackRange")
		, LOCTEXT("PlaybackRange_Label", "Playback Range")
		, LOCTEXT("PlaybackRange_Tooltip", "Playback Range Options")
		, FNewToolMenuDelegate::CreateStatic(&FToolableTimelineMenu::AddPlaybackRangeSection)
		, FUIAction()
		, EUserInterfaceActionType::None));

	TimelineOptionsSection.AddEntry(FToolMenuEntry::InitSubMenu(TEXT("SelectionRange")
		, LOCTEXT("SelectionRange_Label", "Selection Range")
		, LOCTEXT("SelectionRange_Tooltip", "Selection Range Options")
		, FNewToolMenuDelegate::CreateStatic(&FToolableTimelineMenu::AddSelectionRangeSection)
		, FUIAction()
		, EUserInterfaceActionType::None));

	TimelineOptionsSection.AddEntry(FToolMenuEntry::InitSubMenu(TEXT("Marks")
		, LOCTEXT("Marks_Label", "Marks")
		, LOCTEXT("Marks_Tooltip", "Marks")
		, FNewToolMenuDelegate::CreateStatic(&FToolableTimelineMenu::AddMarksSection)
		, FUIAction()
		, EUserInterfaceActionType::None));

	TimelineOptionsSection.AddSeparator(NAME_None);

	FToolMenuEntry& ZoomToFitEntry = TimelineOptionsSection.AddMenuEntry(TimelineEditorCommands.ZoomToFit);
	ZoomToFitEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.SimpleView.FrameAllKeys"));

	FToolMenuEntry& FocusPlaybackRangeEntry = TimelineOptionsSection.AddMenuEntryWithCommandList(SequencerCommands.ResetViewRange, SequencerCommandList);
	FocusPlaybackRangeEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.SimpleView.ResetViewRange"));

	FToolMenuEntry& ScrubToFrameEntry = TimelineOptionsSection.AddMenuEntry(TimelineEditorCommands.GoToFrame);
	ScrubToFrameEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.SimpleView.GoToFrame"));
}

void FToolableTimelineMenu::AddKeyDisplaySection(UToolMenu* const InMenu)
{
	const TSharedPtr<FToolableTimeline> Timeline = GetTimelineFromMenu(InMenu);
	if (!Timeline.IsValid())
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = Timeline->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const FToolableTimelineCommands& TimelineEditorCommands = FToolableTimelineCommands::Get();

	FToolMenuSection& KeyDisplaySection = InMenu->FindOrAddSection(TEXT("KeyDisplay")
		, LOCTEXT("KeyDisplay", "Key Display"));

	FToolMenuEntry& SetKeyDisplay_SelectedAndPinned = KeyDisplaySection.AddMenuEntry(TimelineEditorCommands.SetKeyDisplay_SelectedAndPinned);
	SetKeyDisplay_SelectedAndPinned.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.SimpleView.KeyDisplay.SelectedAndPinned"));

	FToolMenuEntry& SetKeyDisplay_Selected = KeyDisplaySection.AddMenuEntry(TimelineEditorCommands.SetKeyDisplay_Selected);
	SetKeyDisplay_Selected.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.SimpleView.KeyDisplay.Selected"));

	FToolMenuEntry& SetKeyDisplay_All = KeyDisplaySection.AddMenuEntry(TimelineEditorCommands.SetKeyDisplay_All);
	SetKeyDisplay_All.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.SimpleView.KeyDisplay.All"));
}

void FToolableTimelineMenu::AddSnapSection(UToolMenu* const InMenu)
{
	const TSharedPtr<FToolableTimeline> Timeline = GetTimelineFromMenu(InMenu);
	if (!Timeline.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = StaticCastSharedPtr<FSequencer>(Timeline->GetSequencer());
	if (!Sequencer.IsValid())
	{
		return;
	}

	const FSequencerCommands& SequencerCommands = FSequencerCommands::Get();
	const TSharedPtr<FUICommandList> CommandBindings = Sequencer->GetCommandBindings();

	FToolMenuSection& Section = InMenu->FindOrAddSection(TEXT("SnapOptions"), LOCTEXT("SnapOptions", "Snap Options"));

	Section.AddMenuEntryWithCommandList(SequencerCommands.ToggleIsSnapEnabled, CommandBindings);

	Section.AddMenuEntryWithCommandList(SequencerCommands.ToggleForceWholeFrames, CommandBindings);

	const TSharedPtr<SSequencer> SequencerWidget = Sequencer->GetUnderlyingSequencerWidget();
	const TWeakPtr<ISequencer> WeakSequencer = Sequencer;

	FToolMenuEntry PlayRateEntry = FToolMenuEntry::InitWidget(TEXT("PlayRate")
		, SNew(SSequencerPlayRateCombo, Sequencer, SequencerWidget)
			.Visibility_Lambda([WeakSequencer]()
				{
					if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
					{
						const TViewModelPtr<IClockExtension> ClockExtension = Sequencer->GetViewModel()->GetRootSequenceModel().ImplicitCast();
						return (!ClockExtension || ClockExtension->ShouldShowPlayRateCombo(Sequencer))
							? EVisibility::Visible : EVisibility::Collapsed;
					}
					return EVisibility::Collapsed;
				})
		, LOCTEXT("PlayRate", "Play Rate"));
	Section.AddEntry(PlayRateEntry);
}

void FToolableTimelineMenu::AddSettingsSection(UToolMenu* const InMenu)
{
	const TSharedPtr<FToolableTimeline> Timeline = GetTimelineFromMenu(InMenu);
	if (!Timeline.IsValid())
	{
		return;
	}

	FToolMenuSection& SettingsSection = InMenu->FindOrAddSection(TEXT("TimelineSettings"), LOCTEXT("TimelineSettingsSection", "Settings"));

	SettingsSection.AddEntry(FToolMenuEntry::InitSubMenu(TEXT("TimelineSettings")
		, LOCTEXT("TimelineSettings_Label", "Timeline Settings")
		, LOCTEXT("TimelineSettings_Tooltip", "Settings for the timeline")
		, FNewToolMenuDelegate::CreateLambda([](UToolMenu* const InMenu)
			{
				const TSharedPtr<FToolableTimeline> Timeline = GetTimelineFromMenu(InMenu);
				if (!Timeline.IsValid())
				{
					return;
				}

				InMenu->AddSection(TEXT("TimelineSettings"), LOCTEXT("TimelineSettings", "Timeline Settings"));
				InMenu->AddMenuEntry(TEXT("TimelineSettings")
					, FToolMenuEntry::InitWidget(TEXT("TimelineSettings")
					, CreateSettingsDetailsView(Timeline.ToSharedRef())
					, FText::GetEmpty()));
			})
		, FUIAction()
		, EUserInterfaceActionType::None
		, false
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Settings"))));
}

TSharedRef<IDetailsView> FToolableTimelineMenu::CreateSettingsDetailsView(const TSharedRef<FToolableTimeline>& InTimeline)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.ColumnWidth = .5f;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;

	const TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->RegisterInstancedCustomPropertyTypeLayout(FToolableTimelineInstanceSettings::StaticStruct()->GetFName()
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FToolableTimelineInstanceSettingsCustomization::MakeInstance));

	DetailsView->SetObject(&InTimeline->GetTimelineSettings());

	return DetailsView;
}

TSharedPtr<FToolableTimeline> FToolableTimelineMenu::GetTimelineFromMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return nullptr;
	}

	UToolableTimeSliderControllerMenuContext* const MenuContext = InMenu->FindContext<UToolableTimeSliderControllerMenuContext>();
	if (!MenuContext)
	{
		return nullptr;
	}

	return MenuContext->WeakTimeline.Pin();
}

} // namespace UE::Sequencer::ToolableTimeline

#undef LOCTEXT_NAMESPACE
