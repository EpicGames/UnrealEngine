// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class FName;
class IDetailsView;
class SWidget;
class UToolMenu;

namespace UE::Sequencer::ToolableTimeline
{
	struct FMouseInputData;
	class FToolableTimeline;
}

namespace UE::Sequencer::ToolableTimeline
{

/** Context menu displayed when right-clicking on the timeline area */
class FToolableTimelineMenu
{
public:
	static const FName MenuName;

	static TSharedRef<SWidget> GenerateWidget(const FMouseInputData& InMouseInput);

	static void RegisterMenu();

protected:
	static void PopulateMenu(UToolMenu* const InMenu);

	static void AddPlaybackRangeSection(UToolMenu* const InMenu);
	static void AddSelectionRangeSection(UToolMenu* const InMenu);
	static void AddParentChainSection(UToolMenu* const InMenu);
	static void AddMarksSection(UToolMenu* const InMenu);

	static void AddGeneralSection(UToolMenu* const InMenu);
	static void AddKeyDisplaySection(UToolMenu* const InMenu);
	static void AddSnapSection(UToolMenu* const InMenu);
	static void AddSettingsSection(UToolMenu* const InMenu);

	static TSharedRef<IDetailsView> CreateSettingsDetailsView(const TSharedRef<FToolableTimeline>& InTimeline);

	static TSharedPtr<FToolableTimeline> GetTimelineFromMenu(UToolMenu* const InMenu);
};

} // namespace UE::Sequencer::ToolableTimeline
