// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolableTimeline/ToolableTimeline.h"
#include "ToolableTimelineMenuContext.generated.h"

UCLASS(MinimalAPI)
class UToolableTimelineMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<UE::Sequencer::ToolableTimeline::FToolableTimeline> WeakTimeline;
};
