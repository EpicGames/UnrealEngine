// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Events.h"
#include "Layout/Geometry.h"
#include "Templates/SharedPointer.h"
#include "ToolableTimeline/ToolableTimeline.h"
#include "ToolableTimeline/ToolableTimeSliderController.h"
#include "ToolableTimeline/Widgets/SToolableTimeline.h"

namespace UE::Sequencer::ToolableTimeline
{

/**
 * Convenience helper for passing around mouse input data.
 * Transient struct that must NOT be stored or cached.
 * Holds references to data that are only valid during the current draw/input scope.
 */
struct FMouseInputData
{
	FMouseInputData(const TSharedRef<FToolableTimeline>& InTimeline
		, const FGeometry& InGeometry
		, const FPointerEvent& InPointerEvent)
		: Timeline(InTimeline)
		, OwnerWidget(InTimeline->GetWidget()->GetTimeSliderWidget().ToSharedRef())
		, Geometry(InGeometry)
		, PointerEvent(InPointerEvent)
		, KeyEvent(FKeyEvent())
		, LocalPosition(InGeometry.AbsoluteToLocal(InPointerEvent.GetScreenSpacePosition()))
		, RangeToScreen(Timeline->GetTimeSliderController()->GetViewRange(), InGeometry.Size)
	{}
	FMouseInputData(const TSharedRef<FToolableTimeline>& InTimeline
		, const FGeometry& InGeometry
		, const FKeyEvent& InKeyEvent)
		: Timeline(InTimeline)
		, OwnerWidget(InTimeline->GetWidget()->GetTimeSliderWidget().ToSharedRef())
		, Geometry(InGeometry)
		, PointerEvent(FPointerEvent())
		, KeyEvent(InKeyEvent)
		, LocalPosition(FVector2D::ZeroVector)
		, RangeToScreen(Timeline->GetTimeSliderController()->GetViewRange(), InGeometry.Size)
	{}

	double CalculatePixelsPerFrame() const
	{
		const FFrameRate DisplayRate = Timeline->GetTimeSliderController()->GetDisplayRate();
		return RangeToScreen.InputToLocalX(1.0 / DisplayRate.AsDecimal()) - RangeToScreen.InputToLocalX(0.0);
	}

	const TSharedRef<FToolableTimeline> Timeline;

	TSharedRef<SWidget> OwnerWidget;
	const FGeometry& Geometry;
	const FPointerEvent PointerEvent;
	const FKeyEvent KeyEvent;

	FVector2D LocalPosition = FVector2D::ZeroVector;

	FSequencerTimeSliderController::FScrubRangeToScreen RangeToScreen;
};

/**
 * Convenience helper for passing around mouse input data and associated draw data.
 * Transient struct that must NOT be stored or cached.
 * Holds references to data that are only valid during the current draw/input scope.
 */
struct FMouseDrawInputData : FMouseInputData
{
	FMouseDrawInputData(const TSharedRef<FToolableTimeline>& InTimeline
		, const FPointerEvent& InPointerEvent
		, const FGeometry& InGeometry
		, const FSlateRect& InCullingRect
		, FSlateWindowElementList& InDrawElements
		, int32 InLayerId
		, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None
		, const bool bInMirrorLabels = false)
		: FMouseInputData(InTimeline, InGeometry, InPointerEvent)
		, CullingRect(InCullingRect)
		, DrawElements(InDrawElements)
		, LayerId(InLayerId)
		, DrawEffects(InDrawEffects)
		, bMirrorLabels(bInMirrorLabels)
	{}

	FMouseDrawInputData(const FMouseInputData& InMouseInput
		, const FSlateRect& InCullingRect
		, FSlateWindowElementList& InDrawElements
		, int32 InLayerId
		, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None
		, const bool bInMirrorLabels = false)
		: FMouseDrawInputData(InMouseInput.Timeline
			, InMouseInput.PointerEvent
			, InMouseInput.Geometry
			, InCullingRect
			, InDrawElements
			, InLayerId
			, InDrawEffects
			, bInMirrorLabels)
	{}

	FMouseInputData ToMouseInputData() const
	{
		return FMouseInputData(Timeline, Geometry, PointerEvent);
	}

	FMouseDrawInputData& DrawLayer(const int32 InNewLayerId)
	{
		LayerId = InNewLayerId;
		return *this;
	}

	FMouseDrawInputData& IncrementDrawLayer(const int32 InIncrement = 1)
	{
		LayerId += InIncrement;
		return *this;
	}

	const FSlateRect& CullingRect;
	FSlateWindowElementList& DrawElements;
	int32 LayerId;
	ESlateDrawEffect DrawEffects;
	bool bMirrorLabels;
};

} // namespace UE::Sequencer::ToolableTimeline
