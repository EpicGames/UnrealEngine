// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimeSliderController.h"
#include "Brushes/SlateBoxBrush.h"
#include "Channels/BuiltInChannelEditors.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "IKeyArea.h"
#include "ISequencer.h"
#include "Misc/InputState.h"
#include "Misc/QualifiedFrameTime.h"
#include "MovieScene.h"
#include "MVVM/Extensions/IClockExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/Views/SOutlinerView.h"
#include "MVVM/Views/STrackAreaView.h"
#include "Rendering/DrawElements.h"
#include "SequencerChannelTraits.h"
#include "SequencerCommands.h"
#include "SequencerContextMenus.h"
#include "SequencerSettings.h"
#include "SequencerToolMenuContext.h"
#include "SSequencer.h"
#include "SSequencerSection.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "ToolableTimeline/Caches/ToolableTimelineKeyViewCacheState.h"
#include "ToolableTimeline/DragOperations/ToolableTimelineKeyAndMarkDragOperation.h"
#include "ToolableTimeline/Drawing/ToolableTimelineDrawer.h"
#include "ToolableTimeline/Menus/ToolableTimelineMenu.h"
#include "ToolableTimeline/Menus/ToolableTimelineKeyContextMenu.h"
#include "ToolableTimeline/MouseInputData.h"
#include "ToolableTimeline/ToolableTimelineSettings.h"
#include "ToolableTimeline/ToolableTimelineUtils.h"
#include "ToolableTimeline/Tools/ToolableTimelineBaseTool.h"
#include "ToolableTimeline/Tools/ToolableTimelineKeyRenderer.h"
#include "TrackEditors/TimeWarpTrackEditor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SToolableTimeline.h"

#define LOCTEXT_NAMESPACE "ToolableTimeSliderController"

namespace UE::Sequencer::ToolableTimeline
{

FToolableTimeSliderController::FToolableTimeSliderController(const FTimeSliderArgs& InTimeSliderArgs, FToolableTimeline& InTimeline)
	: FSequencerTimeSliderController(InTimeSliderArgs, StaticCastSharedPtr<FSequencer>(InTimeline.GetSequencer()))
	, Timeline(InTimeline)
{
}

FToolableTimeSliderController::~FToolableTimeSliderController()
{
	Timeline.GetTimelineSettings().OnSettingChanged().RemoveAll(this);

	if (const TSharedPtr<ISequencer> Sequencer = Timeline.GetSequencer())
	{
		Sequencer->OnGlobalTimeChanged().RemoveAll(this);
	}
}

double FToolableTimeSliderController::ComputeHeight() const
{
	if (const FStructProperty* const StructProperty
		= FindFProperty<FStructProperty>(UToolableTimelineSettings::StaticClass(), TEXT("Settings")))
	{
		int32 MinHeight = 24;
		int32 MaxHeight = 80;

		if (const FProperty* const HeightProperty = FindFProperty<FProperty>(StructProperty->Struct, TEXT("Height")))
		{
			const FString ClampMinString = HeightProperty->GetMetaData(TEXT("ClampMin"));
			if (!ClampMinString.IsEmpty())
			{
				MinHeight = FCString::Atoi(*ClampMinString);
			}
	
			const FString ClampMaxString = HeightProperty->GetMetaData(TEXT("ClampMax"));
			if (!ClampMaxString.IsEmpty())
			{
				MaxHeight = FCString::Atoi(*ClampMaxString);
			}
		}

		const UToolableTimelineSettings& TimelineSettings = Timeline.GetTimelineSettings();
		return FMath::Clamp<int32>(TimelineSettings.Settings.Height, MinHeight, MaxHeight);
	}

	return FSequencerTimeSliderController::ComputeHeight();
}

int32 FToolableTimeSliderController::OnPaintTimeSlider(const bool bInMirrorLabels
	, const FGeometry& InGeometry
	, const FSlateRect& InCullingRect
	, FSlateWindowElementList& OutDrawElements
	, int32 LayerId
	, const FWidgetStyle& InWidgetStyle
	, const bool bInParentEnabled) const
{
	const TSharedPtr<ISequencer> Sequencer = Timeline.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return LayerId;
	}

	const USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!SequencerSettings)
	{
		return LayerId;
	}

	const TRange<double> ViewRange = GetViewRange();
	const float LocalSequenceLength = ViewRange.GetUpperBoundValue() - ViewRange.GetLowerBoundValue();
	if (LocalSequenceLength <= 0.f)
	{
		return LayerId;
	}

	const UToolableTimelineSettings& TimelineSettings = Timeline.GetTimelineSettings();
	const ESlateDrawEffect DrawEffects = bInParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	FMouseDrawInputData MouseDrawInput(Timeline.AsShared(), FPointerEvent()
		, InGeometry, InCullingRect, OutDrawElements, LayerId, DrawEffects, bInMirrorLabels);

	Drawing::DrawBackground(MouseDrawInput);

	// draw tick marks
	float MajorTickHeight = Drawing::CalculateTickSize(MouseDrawInput);

	FDrawTickArgs DrawTickArgs;
	DrawTickArgs.AllottedGeometry = MouseDrawInput.Geometry;
	DrawTickArgs.bMirrorLabels = bInMirrorLabels;
	DrawTickArgs.bOnlyDrawMajorTicks = false;
	DrawTickArgs.TickColor = TimelineSettings.Settings.TickColor;
	DrawTickArgs.TickTextColor = TimelineSettings.Settings.TickTextColor;
	DrawTickArgs.CullingRect = InCullingRect;
	DrawTickArgs.DrawEffects = MouseDrawInput.DrawEffects;
	DrawTickArgs.StartLayer = MouseDrawInput.LayerId;
	DrawTickArgs.TickOffset = bInMirrorLabels ? 0.f : FMath::Abs(MouseDrawInput.Geometry.GetLocalSize().Y - MajorTickHeight);
	DrawTickArgs.MajorTickHeight = MajorTickHeight;

	MouseDrawInput.LayerId = Drawing::DrawTicks(MouseDrawInput, DrawTickArgs);

	// Draw playback and selection range
	static const FName PlayRangeTopL = TEXT("Sequencer.Timeline.PlayRange_Top_L");
	static const FName PlayRangeBottomL = TEXT("Sequencer.Timeline.PlayRange_Bottom_L");
	static const FName PlayRangeTopR = TEXT("Sequencer.Timeline.PlayRange_Top_R");
	static const FName PlayRangeBottomR = TEXT("Sequencer.Timeline.PlayRange_Bottom_R");

	FPaintPlaybackRangeArgs PlaybackRangeArgs(
		FAppStyle::GetBrush(PlayRangeTopL),
		FAppStyle::GetBrush(PlayRangeTopR),
		6.f
	);
	PlaybackRangeArgs.SolidFillOpacity = 0.f;

	MouseDrawInput.LayerId = DrawPlaybackRangeExcludedRegions(MouseDrawInput.Geometry,
		InCullingRect,
		MouseDrawInput.DrawElements,
		MouseDrawInput.IncrementDrawLayer().LayerId,
		MouseDrawInput.RangeToScreen,
		PlaybackRangeArgs
	);

	MouseDrawInput.LayerId = DrawSubSequenceRangeExcludedRegions(InGeometry,
		InCullingRect,
		MouseDrawInput.DrawElements,
		MouseDrawInput.IncrementDrawLayer().LayerId,
		MouseDrawInput.RangeToScreen,
		PlaybackRangeArgs
	);

	MouseDrawInput.LayerId = Drawing::DrawKeys(MouseDrawInput,
		KeyRenderer,
		KeyRendererCache,
		KeyRendererInvalidationFlags
	);

	// Draw top indicator brushes
	MouseDrawInput.LayerId = DrawPlaybackRangeMarkers(MouseDrawInput.Geometry,
		InCullingRect,
		MouseDrawInput.DrawElements,
		MouseDrawInput.IncrementDrawLayer().LayerId,
		MouseDrawInput.RangeToScreen,
		PlaybackRangeArgs
	);

	MouseDrawInput.LayerId = DrawSubSequenceRangeMarkers(InGeometry,
		InCullingRect,
		MouseDrawInput.DrawElements,
		MouseDrawInput.IncrementDrawLayer().LayerId,
		MouseDrawInput.RangeToScreen,
		PlaybackRangeArgs
	);

	MouseDrawInput.LayerId = DrawSubSequenceRangeHashMarks(InGeometry,
		InCullingRect,
		MouseDrawInput.DrawElements,
		MouseDrawInput.IncrementDrawLayer().LayerId,
		MouseDrawInput.RangeToScreen,
		PlaybackRangeArgs
	);

	const float PrevSoliFillOpacity = PlaybackRangeArgs.SolidFillOpacity;
	PlaybackRangeArgs.SolidFillOpacity = 0.05f;
	MouseDrawInput.LayerId = DrawSelectionRange(InGeometry,
		InCullingRect,
		MouseDrawInput.DrawElements,
		MouseDrawInput.IncrementDrawLayer().LayerId,
		MouseDrawInput.RangeToScreen,
		PlaybackRangeArgs
	);

	// Draw bottom indicator brushes
	PlaybackRangeArgs.StartBrush = FAppStyle::GetBrush(PlayRangeBottomL);
	PlaybackRangeArgs.EndBrush = FAppStyle::GetBrush(PlayRangeBottomR);
	PlaybackRangeArgs.SolidFillOpacity = PrevSoliFillOpacity;
	MouseDrawInput.LayerId = DrawPlaybackRangeMarkers(MouseDrawInput.Geometry,
		InCullingRect,
		MouseDrawInput.DrawElements,
		MouseDrawInput.IncrementDrawLayer().LayerId,
		MouseDrawInput.RangeToScreen,
		PlaybackRangeArgs
	);

	MouseDrawInput.LayerId = DrawSubSequenceRangeMarkers(InGeometry,
		InCullingRect,
		MouseDrawInput.DrawElements,
		MouseDrawInput.IncrementDrawLayer().LayerId,
		MouseDrawInput.RangeToScreen,
		PlaybackRangeArgs
	);

	MouseDrawInput.LayerId = DrawSubSequenceRangeHashMarks(InGeometry,
		InCullingRect,
		MouseDrawInput.DrawElements,
		MouseDrawInput.IncrementDrawLayer().LayerId,
		MouseDrawInput.RangeToScreen,
		PlaybackRangeArgs
	);

	PlaybackRangeArgs.SolidFillOpacity = 0.05f;
	MouseDrawInput.LayerId = DrawSelectionRange(InGeometry,
		InCullingRect,
		MouseDrawInput.DrawElements,
		MouseDrawInput.IncrementDrawLayer().LayerId,
		MouseDrawInput.RangeToScreen,
		PlaybackRangeArgs
	);

	if (SequencerSettings->GetShowMarkedFrames())
	{
		MouseDrawInput.LayerId = DrawMarkedFrames(InGeometry,
			MouseDrawInput.RangeToScreen,
			MouseDrawInput.DrawElements,
			MouseDrawInput.IncrementDrawLayer().LayerId,
			MouseDrawInput.DrawEffects,
			/*bInDrawLabels=*/true
		);
	}

	if (ActiveTool.IsValid())
	{
		MouseDrawInput.LayerId = ActiveTool->Paint(MouseDrawInput);
	}

	if (TimelineSettings.Settings.bShowDragRangeIndicator)
	{
		MouseDrawInput.LayerId = Drawing::DrawDragRangeIndicator(MouseDrawInput.IncrementDrawLayer());
	}

	if (!bHideScrubHandle)
	{
		MouseDrawInput.LayerId = Drawing::DrawScrubHandle(MouseDrawInput.IncrementDrawLayer());
	}

	return MouseDrawInput.IncrementDrawLayer().LayerId;
}

int32 FToolableTimeSliderController::OnPaintViewArea(const FGeometry& InGeometry
	, const FSlateRect& InCullingRect
	, FSlateWindowElementList& OutDrawElements
	, int32 LayerId
	, bool bInEnabled
	, const FPaintViewAreaArgs& InArgs) const
{
	const TSharedPtr<ISequencer> Sequencer = Timeline.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return LayerId;
	}

	const USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!SequencerSettings)
	{
		return LayerId;
	}

	const UToolableTimelineSettings& TimelineSettings = Timeline.GetTimelineSettings();
	const ESlateDrawEffect DrawEffects = bInEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	FMouseDrawInputData MouseDrawInput(Timeline.AsShared(), FPointerEvent()
		, InGeometry, InCullingRect, OutDrawElements, LayerId, DrawEffects);

	if (InArgs.PlaybackRangeArgs.IsSet())
	{
		MouseDrawInput.LayerId = DrawPlaybackRange(InGeometry,
			InCullingRect,
			OutDrawElements,
			MouseDrawInput.IncrementDrawLayer().LayerId,
			MouseDrawInput.RangeToScreen,
			InArgs.PlaybackRangeArgs.GetValue()
		);

		MouseDrawInput.LayerId = DrawSubSequenceRange(InGeometry,
			InCullingRect,
			OutDrawElements,
			MouseDrawInput.IncrementDrawLayer().LayerId,
			MouseDrawInput.RangeToScreen,
			InArgs.PlaybackRangeArgs.GetValue()
		);

		FPaintPlaybackRangeArgs PlaybackRangeArgsCopy = InArgs.PlaybackRangeArgs.GetValue();
		PlaybackRangeArgsCopy.SolidFillOpacity = 0.f;
		MouseDrawInput.LayerId = DrawSelectionRange(InGeometry,
			InCullingRect,
			OutDrawElements,
			MouseDrawInput.IncrementDrawLayer().LayerId,
			MouseDrawInput.RangeToScreen,
			PlaybackRangeArgsCopy
		);
	}

	if (InArgs.bDisplayTickLines)
	{
		// Draw major tick lines in the section area
		FDrawTickArgs DrawTickArgs;
		DrawTickArgs.AllottedGeometry = InGeometry;
		DrawTickArgs.bMirrorLabels = false;
		DrawTickArgs.bOnlyDrawMajorTicks = true;
		DrawTickArgs.TickColor = TimelineSettings.Settings.TickColor;
		DrawTickArgs.TickTextColor = TimelineSettings.Settings.TickTextColor;
		DrawTickArgs.CullingRect = InCullingRect;
		DrawTickArgs.DrawEffects = DrawEffects;
		// Draw major ticks under sections
		DrawTickArgs.StartLayer = MouseDrawInput.LayerId - 1;
		// Draw the tick the entire height of the section area
		DrawTickArgs.TickOffset = 0.f;
		DrawTickArgs.MajorTickHeight = InGeometry.GetLocalSize().Y;

		Drawing::DrawTicks(MouseDrawInput, DrawTickArgs);
	}

	if (SequencerSettings->GetShowMarkedFrames())
	{
		MouseDrawInput.LayerId = DrawMarkedFrames(InGeometry,
			MouseDrawInput.RangeToScreen,
			OutDrawElements,
			MouseDrawInput.IncrementDrawLayer().LayerId,
			DrawEffects,
			false
		);
	}

	MouseDrawInput.LayerId = DrawScalingAnchors(InGeometry,
		MouseDrawInput.RangeToScreen,
		OutDrawElements,
		MouseDrawInput.IncrementDrawLayer().LayerId,
		DrawEffects
	);

	MouseDrawInput.LayerId = DrawVerticalFrames(InGeometry,
		MouseDrawInput.RangeToScreen,
		OutDrawElements,
		MouseDrawInput.IncrementDrawLayer().LayerId,
		DrawEffects
	);

	MouseDrawInput.LayerId = Drawing::DrawAreaViewScrubPosition(MouseDrawInput, InArgs.bDisplayScrubPosition);

	return MouseDrawInput.LayerId;
}

FReply FToolableTimeSliderController::OnMouseButtonDown(SWidget& OwnerWidget
	, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	FReply OutReply = FReply::Unhandled();

	const FMouseInputData MouseInput(Timeline.AsShared(), InGeometry, InPointerEvent);
	const FKey EffectingButton = InPointerEvent.GetEffectingButton();
	const bool bIsLeftOrMiddleMouse = EffectingButton == EKeys::LeftMouseButton || EffectingButton == EKeys::MiddleMouseButton;
	const bool bIsNotRightMouse = EffectingButton != EKeys::RightMouseButton;
	const bool bIsScrubTimeCommandPressed = bIsLeftOrMiddleMouse
		&& CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport);

	ResetMouseDown(InGeometry, InPointerEvent);
	ResetPendingToolActivation();

	HoverTarget.Reset();
	ActiveDragTarget = bIsScrubTimeCommandPressed
		? TOptional<FResolvedDragTarget>()
		: ResolveDragTarget(MouseInput, /*bInFromTimeSlider=*/true, /*bInUseMouseDownPosition=*/true);

	const TSharedPtr<IToolableTimelineToolFactory> FactoryToActivate
		= FindToolFactoryToActivate(MouseInput, /*bInIsDoubleClick=*/false);

	const bool bControllerDragTarget = ActiveDragTarget.IsSet() && ActiveDragTarget->IsControllerOwned();
	const bool bControllerOwnedPress = FactoryToActivate.IsValid() || bControllerDragTarget;
	const bool bToolFactoryOverridesControllerTarget = FactoryToActivate.IsValid()
		&& ActiveDragTarget.IsSet() && ActiveDragTarget->IsScrubber();

	bool bToolWantsInputPrepass = ActiveTool.IsValid() && ActiveTool->WantsInputPrepass(MouseInput);

	// 1) Give the active tool a chance to handle the event
	if (bToolWantsInputPrepass)
	{
		OutReply = ActiveTool->OnMouseButtonDown(OwnerWidget, InGeometry, InPointerEvent);

		TryCloseActiveToolRequest();

		if (OutReply.IsEventHandled())
		{
			ResetControllerDragTargetForToolOwnership();
			return ActiveTool.IsValid() ? OutReply : FReply::Handled().ReleaseMouseCapture();
		}
	}

	// 2) Activate any new tool only if no current active tool handled the click
	if (FactoryToActivate.IsValid() && (!bControllerDragTarget || bToolFactoryOverridesControllerTarget))
	{
		const TSharedRef<FToolableTimelineBaseTool> NewPendingTool = FactoryToActivate->CreateTool(Timeline);

		if (NewPendingTool->WantsDeferredDragThresholdActivation())
		{
			PendingToolFactory = FactoryToActivate;
			PendingTool = NewPendingTool;

			// IMPORTANT:
			// once a pending tool has been armed, do NOT fall through into scrub handling
			return FReply::Handled()
				.CaptureMouse(OwnerWidget.AsShared())
				.UseHighPrecisionMouseMovement(OwnerWidget.AsShared())
				.PreventThrottling();
		}

		const FFrameTime InputTickTime = ComputeMouseFrameTime(MouseInput, /*bInCheckSnapping=*/true);

		ActivateTool(NewPendingTool, MouseInput, InputTickTime);

		OutReply = ActiveTool->OnMouseButtonDown(OwnerWidget, InGeometry, InPointerEvent);

		TryCloseActiveToolRequest();

		if (OutReply.IsEventHandled())
		{
			ResetControllerDragTargetForToolOwnership();
			return ActiveTool.IsValid() ? OutReply : FReply::Handled().ReleaseMouseCapture();
		}
	}

	// 3) Give active tool exactly one normal handling pass if it did not already pre-handle
	if (bIsNotRightMouse
		//!bControllerOwnedPress
		&& ActiveTool.IsValid()
		&& !bToolWantsInputPrepass)
	{
		OutReply = ActiveTool->OnMouseButtonDown(OwnerWidget, InGeometry, InPointerEvent);

		TryCloseActiveToolRequest();

		if (OutReply.IsEventHandled())
		{
			ResetControllerDragTargetForToolOwnership();
			return ActiveTool.IsValid() ? OutReply : FReply::Handled().ReleaseMouseCapture();
		}
	}

	// 4) Begin controller-owned mark/range drag
	if (bIsLeftOrMiddleMouse && ActiveDragTarget.IsSet())
	{
		if (ActiveDragTarget->IsMark() && !TimeSliderArgs.AreMarkedFramesLocked.Get())
		{
			OutReply = TryBeginMarkDrag(MouseInput);
			if (OutReply.IsEventHandled())
			{
				return OutReply;
			}
		}
		else if (ActiveDragTarget->IsScrubber())
		{
			OutReply = TryBeginScrub(MouseInput);
			if (OutReply.IsEventHandled())
			{
				return OutReply;
			}
		}
		else if (EffectingButton == EKeys::LeftMouseButton
			&& (ActiveDragTarget->IsSelectionRange() || ActiveDragTarget->IsPlaybackRange()))
		{
			// Do not set MouseDragType here.
			// Do not fire begin-drag delegates here.
			// Let the base controller own drag-mode selection and drag-begin notification.
			LastMousePosition = MouseDownPosition.GetValue();

			return FReply::Handled()
				.CaptureMouse(OwnerWidget.AsShared())
				.UseHighPrecisionMouseMovement(OwnerWidget.AsShared())
				.PreventThrottling();
		}
	}

	// 5) Only do normal scrub if tool did not handle
	if (bIsLeftOrMiddleMouse && !bControllerOwnedPress)
	{
		OutReply = TryBeginScrub(MouseInput);
		if (OutReply.IsEventHandled())
		{
			return OutReply;
		}
	}

	// 6) Fall back to controller behavior
	if (EffectingButton == EKeys::RightMouseButton)
	{
		return FSequencerTimeSliderController::OnMouseButtonDown(OwnerWidget, InGeometry, InPointerEvent);
	}

	return OutReply;
}

FReply FToolableTimeSliderController::OnMouseButtonUp(SWidget& OwnerWidget
	, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	FReply OutReply = FReply::Unhandled();

	const FMouseInputData MouseInput(Timeline.AsShared(), InGeometry, InPointerEvent);
	const FKey EffectingButton = InPointerEvent.GetEffectingButton();
	const bool bIsNotRightMouse = EffectingButton != EKeys::RightMouseButton;

	UpdateScrubHoveredKeys();

	// 1) Give the active tool a chance to handle the event
	if (bIsNotRightMouse && ActiveTool.IsValid())
	{
		OutReply = ActiveTool->OnMouseButtonUp(OwnerWidget, InGeometry, InPointerEvent);

		TryCloseActiveToolRequest();

		if (OutReply.IsEventHandled())
		{
			ResetInput();
			return ActiveTool.IsValid() ? OutReply : FReply::Handled().ReleaseMouseCapture();
		}
	}

	// 2) Open timeline context menu
	if (EffectingButton == EKeys::RightMouseButton)
	{
		OutReply = OnMouseButtonUp_ContextMenu(OwnerWidget, InGeometry, InPointerEvent);

		TryCloseActiveToolRequest();

		if (OutReply.IsEventHandled())
		{
			ResetInput();
			return ActiveTool.IsValid() ? OutReply : FReply::Handled().ReleaseMouseCapture();
		}
	}

	const bool bHasMouseCapture = OwnerWidget.HasMouseCapture();

	const bool bLeftMouseUp = EffectingButton == EKeys::LeftMouseButton && bHasMouseCapture;
	const bool bRightMouseUp = EffectingButton == EKeys::RightMouseButton && bHasMouseCapture && TimeSliderArgs.AllowZoom;
	const bool bMiddleMouseUp = EffectingButton == EKeys::MiddleMouseButton && bHasMouseCapture;
	const bool bAnyMouseButtonUp = bLeftMouseUp || bMiddleMouseUp || bRightMouseUp;

	const bool bHasControllerDrag = MouseDragType != DRAG_NONE
		|| ScrubDragOperation.IsSet()
		|| KeyAndMarkDragOperation.IsValid();
	const bool bHasTransactionalDrag = MouseDragType == DRAG_PLAYBACK_START
		|| MouseDragType == DRAG_PLAYBACK_END
		|| MouseDragType == DRAG_SELECTION_START
		|| MouseDragType == DRAG_SELECTION_END
		|| MouseDragType == DRAG_MARK;

	// 3) Key click/selection handling
	if (!bHasTransactionalDrag && HandleScrubReleaseKeySelection(MouseInput))
	{
		ResetInput();
		return FReply::Handled().ReleaseMouseCapture();
	}

	if (bAnyMouseButtonUp
		&& PendingTool.IsValid()
		&& ActiveDragTarget.IsSet()
		&& ActiveDragTarget->IsMark())
	{
		FSequencerTimeSliderController::HandleDragMark_NoDragType(ActiveDragTarget->MarkIndex);

		if (const TOptional<FFrameNumber> MarkTime = GetMarkFrameNumber(ActiveDragTarget->MarkIndex))
		{
			const bool bEvaluate = EffectingButton != EKeys::MiddleMouseButton;
			CommitScrubPosition(FFrameTime(MarkTime.GetValue()), /*bIsScrubbing=*/false, bEvaluate);
		}

		EndAnyInProgressTransactionalDrag();
		ResetInput();
		return FReply::Handled().ReleaseMouseCapture();
	}

	// 4) Base class fall through for state management and event broadcasting
	if (bAnyMouseButtonUp && bHasControllerDrag)
	{
		if (KeyAndMarkDragOperation.IsValid() && ActiveDragTarget.IsSet() && ActiveDragTarget->IsMark())
		{
			const FVector2D AccumulatedDelta = KeyAndMarkDragOperation->GetAccumulatedLocalDelta();
			const bool bDidActuallyDrag = !FMath::IsNearlyZero(AccumulatedDelta.X) || !FMath::IsNearlyZero(AccumulatedDelta.Y);
			if (!bDidActuallyDrag)
			{
				FSequencerTimeSliderController::HandleDragMark_NoDragType(ActiveDragTarget->MarkIndex);

				if (const TOptional<FFrameNumber> MarkTime = GetMarkFrameNumber(ActiveDragTarget->MarkIndex))
				{
					const bool bEvaluate = EffectingButton != EKeys::MiddleMouseButton;
					CommitScrubPosition(FFrameTime(MarkTime.GetValue()), /*bIsScrubbing=*/false, bEvaluate);
				}

				EndAnyInProgressTransactionalDrag();
				ResetInput();
				return FReply::Handled().ReleaseMouseCapture();
			}
		}

		if (bHasTransactionalDrag)
		{
			EndAnyInProgressTransactionalDrag();
			ResetInput();
			return FReply::Handled().ReleaseMouseCapture();
		}

		const FFrameTime MouseTime = ComputeMouseFrameTime(MouseInput, true);

		OutReply = OnMouseButtonUp_MouseDragType(OwnerWidget, InGeometry, InPointerEvent, MouseInput.RangeToScreen, MouseTime);

		if (OutReply.IsEventHandled())
		{
			ResetInput();
			return OutReply;
		}
	}

	// If Slate has already moved mouse capture elsewhere, we can still receive the
	// button-up after a transactional range/mark drag and must end the BeginDrag.
	if (!bAnyMouseButtonUp && EffectingButton == EKeys::LeftMouseButton && bHasTransactionalDrag)
	{
		EndAnyInProgressTransactionalDrag();
		ResetInput();
		return FReply::Handled().ReleaseMouseCapture();
	}

	ResetInput();
	return OutReply;
}

FReply FToolableTimeSliderController::OnMouseMove(SWidget& OwnerWidget
	, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	return OnMouseMoveImpl(OwnerWidget, InGeometry, InPointerEvent, true);
}

FReply FToolableTimeSliderController::OnMouseMoveImpl(SWidget& OwnerWidget
	, const FGeometry& InGeometry
	, const FPointerEvent& InPointerEvent
	, const bool bInFromTimeSlider)
{
	const USequencerSettings* const SequencerSettings = Timeline.GetSequencerSettings();
	if (!SequencerSettings)
	{
		return FReply::Unhandled();
	}

	const FMouseInputData MouseInput(Timeline.AsShared(), InGeometry, InPointerEvent);

	FReply OutReply = FReply::Unhandled();

	const bool bLeftMouseDown = InPointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
	const bool bRightMouseDown = InPointerEvent.IsMouseButtonDown(EKeys::RightMouseButton) && TimeSliderArgs.AllowZoom;
	const bool bMiddleMouseDown = InPointerEvent.IsMouseButtonDown(EKeys::MiddleMouseButton);

	// 1) If a tool candidate is pending, only activate it after drag threshold
	if (ShouldBeginPendingToolActivation(InPointerEvent))
	{
		const FFrameTime InputTickTime = ComputeMouseFrameTime(MouseInput, /*bInCheckSnapping=*/true);

		ActivateTool(PendingTool.ToSharedRef(), MouseInput, InputTickTime);

		const FReply BeginReply = ActiveTool->OnMouseButtonDown(OwnerWidget, InGeometry, InPointerEvent);
		if (!BeginReply.IsEventHandled())
		{
			DeactivateTool();
			return FReply::Unhandled();
		}

		// Once the tool has successfully taken over the mouse-down interaction, the
		// controller's original hit target becomes stale and must not keep steering
		// subsequent mouse-move routing.
		ResetControllerDragTargetForToolOwnership();

		// Intentionally fall through to below
	}

	// 2) Handle controller mouse drag
	if (bRightMouseDown)
	{
		OutReply = OnMouseMove_RightMouseDown(InGeometry, InPointerEvent, *SequencerSettings);
	}
	else if ((bLeftMouseDown || bMiddleMouseDown) && KeyAndMarkDragOperation.IsValid())
	{
		KeyAndMarkDragOperation->UpdateMarkDragAndKeys(MouseInput);

		LastMousePosition = InPointerEvent.GetScreenSpacePosition();

		OutReply = FReply::Handled()
			.CaptureMouse(MouseInput.OwnerWidget)
			.UseHighPrecisionMouseMovement(MouseInput.OwnerWidget)
			.PreventThrottling();
	}
	else if ((bLeftMouseDown || bMiddleMouseDown) && ScrubDragOperation.IsSet())
	{
		ScrubDragOperation->UpdateDrag(MouseInput);

		LastMousePosition = InPointerEvent.GetScreenSpacePosition();

		const FFrameTime ScrubTime = ResolveAndUpdateScrubTime(MouseInput);
		if (!LastCommittedScrubTime.IsSet() || LastCommittedScrubTime.GetValue() != ScrubTime)
		{
			CommitScrubPosition(ScrubTime, /*bIsScrubbing=*/true, /*bEvaluate=*/!bMiddleMouseDown);
			LastCommittedScrubTime = ScrubTime;
		}

		return FReply::Handled()
			.CaptureMouse(MouseInput.OwnerWidget)
			.UseHighPrecisionMouseMovement(MouseInput.OwnerWidget)
			.PreventThrottling();
	}
	else if (bLeftMouseDown || bMiddleMouseDown)
	{
		const bool bHasDragType = MouseDragType != DRAG_NONE;
		const bool bIsTargetRangeStartEnd = ActiveDragTarget.IsSet()
				&& (ActiveDragTarget->IsPlaybackRange() || ActiveDragTarget->IsSelectionRange());
		const bool bShouldRouteRangeDragToBase = MouseDownPosition.IsSet() && (bHasDragType || bIsTargetRangeStartEnd);

		if (bShouldRouteRangeDragToBase)
		{
			FSequencerTimeSliderController::OnMouseMove_LeftAndMiddleMouseDown(
				InGeometry,
				InPointerEvent,
				bInFromTimeSlider,
				MouseInput.RangeToScreen,
				*SequencerSettings
			);

			return FReply::Handled()
				.CaptureMouse(MouseInput.OwnerWidget)
				.UseHighPrecisionMouseMovement(MouseInput.OwnerWidget)
				.PreventThrottling();
		}
	}

	// 3) Only do hover/hit test when not actively dragging
	const bool bIsDragging = MouseDragType != DRAG_NONE
		|| KeyAndMarkDragOperation.IsValid()
		|| ScrubDragOperation.IsSet();

	if (!bIsDragging)
	{
		if (CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport))
		{
			HoverTarget.Reset();
			HoverMarkIndex = INDEX_NONE;
			Timeline.GetKeySelection().ClearHoveredKeys();
		}
		else
		{
			HoverTarget = ResolveDragTarget(MouseInput, bInFromTimeSlider, /*bInUseMouseDownPosition=*/false);
			HoverMarkIndex = HoverTarget.IsSet() ? HoverTarget->MarkIndex : INDEX_NONE;

			TSet<FSequencerSelectedKey> NewHoveredKeys;
			GetKeysUnderMouse(MouseInput, InPointerEvent.GetScreenSpacePosition(), NewHoveredKeys);
			Timeline.GetKeySelection().SetHoveredKeys(NewHoveredKeys);
		}
	}

	// 4) Give the active tool a chance to handle the event first
	if (ActiveTool.IsValid())
	{
		OutReply = ActiveTool->OnMouseMove(OwnerWidget, InGeometry, InPointerEvent);

		TryCloseActiveToolRequest();

		if (OutReply.IsEventHandled())
		{
			return ActiveTool.IsValid() ? OutReply : FReply::Handled().ReleaseMouseCapture();
		}
	}

	return OutReply;
}

FReply FToolableTimeSliderController::OnMouseButtonDoubleClick(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	FReply OutReply = FReply::Unhandled();

	// Give the active tool a chance to handle the event
	if (ActiveTool.IsValid())
	{
		OutReply = ActiveTool->OnMouseButtonDoubleClick(OwnerWidget, InGeometry, InPointerEvent);

		TryCloseActiveToolRequest();

		if (OutReply.IsEventHandled())
		{
			return ActiveTool.IsValid() ? OutReply : FReply::Handled().ReleaseMouseCapture();
		}
	}

	const FMouseInputData MouseInput(Timeline.AsShared(), InGeometry, InPointerEvent);

	if (const TSharedPtr<IToolableTimelineToolFactory> FactoryToActivate
		= FindToolFactoryToActivate(MouseInput, /*bInIsDoubleClick=*/true))
	{
		const TSharedRef<FToolableTimelineBaseTool> NewTool = FactoryToActivate->CreateTool(Timeline);
		const FFrameTime InputTickTime = ComputeMouseFrameTime(MouseInput, /*bInCheckSnapping=*/true);

		ActivateTool(NewTool, MouseInput, InputTickTime);

		OutReply = ActiveTool->OnMouseButtonDoubleClick(OwnerWidget, InGeometry, InPointerEvent);

		TryCloseActiveToolRequest();

		if (OutReply.IsEventHandled())
		{
			ResetControllerDragTargetForToolOwnership();

			return ActiveTool.IsValid() ? OutReply : FReply::Handled().ReleaseMouseCapture();
		}

		DeactivateTool();
	}

	return FSequencerTimeSliderController::OnMouseButtonDoubleClick(OwnerWidget, InGeometry, InPointerEvent);
}

void FToolableTimeSliderController::OnMouseEnter(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (ActiveTool.IsValid())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ActiveTool->OnMouseEnter(OwnerWidget, InGeometry, InPointerEvent);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		TryCloseActiveToolRequest();
	}
}

void FToolableTimeSliderController::OnMouseLeave(SWidget& OwnerWidget, const FPointerEvent& InPointerEvent)
{
	HoverTarget.Reset();
	HoverMarkIndex = INDEX_NONE;

	Timeline.GetKeySelection().ResetHoveredKeysAndSelectionPreview();

	InvalidateKeyRendererKeyState();

	if (ActiveTool.IsValid())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ActiveTool->OnMouseLeave(OwnerWidget, InPointerEvent);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		TryCloseActiveToolRequest();
	}
}

FCursorReply FToolableTimeSliderController::OnCursorQuery(const SWidget& OwnerWidget
	, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) const
{
	const bool bIsScrubDragging = InPointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton)
		&& CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport);
	if (bIsScrubDragging)
	{
		return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
	}

	if (ActiveTool.IsValid())
	{
		const FCursorReply Reply = ActiveTool->OnCursorQuery(OwnerWidget, InGeometry, InPointerEvent);
		if (Reply.IsEventHandled())
		{
			return Reply;
		}
	}

	if (KeyAndMarkDragOperation.IsValid())
	{
		return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
	}

	if (ActiveDragTarget.IsSet())
	{
		if (ActiveDragTarget->IsMark()
			|| ActiveDragTarget->IsPlaybackRange()
			|| ActiveDragTarget->IsSelectionRange())
		{
			return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
		}
	}

	if (HoverTarget.IsSet())
	{
		if (HoverTarget->IsMark()
			|| HoverTarget->IsPlaybackRange()
			|| HoverTarget->IsSelectionRange())
		{
			return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
		}
	}

	return FCursorReply::Unhandled();
}

FReply FToolableTimeSliderController::OnKeyDown(SWidget& OwnerWidget, const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (ActiveTool.IsValid())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const FReply Reply = ActiveTool->OnKeyDown(OwnerWidget, InGeometry, InKeyEvent);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		TryCloseActiveToolRequest();

		if (Reply.IsEventHandled())
		{
			return Reply;
		}
	}

	if (Timeline.GetCommandList()->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply FToolableTimeSliderController::OnKeyUp(SWidget& OwnerWidget, const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (ActiveTool.IsValid())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const FReply Reply = ActiveTool->OnKeyUp(OwnerWidget, InGeometry, InKeyEvent);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		TryCloseActiveToolRequest();

		if (Reply.IsEventHandled())
		{
			return Reply;
		}
	}

	return FReply::Unhandled();
}

void FToolableTimeSliderController::OnFocusLost(SWidget& OwnerWidget, const FFocusEvent& InFocusEvent)
{
	if (ActiveTool.IsValid())
	{
		ActiveTool->OnFocusLost(OwnerWidget, InFocusEvent);
	}
}

TSharedPtr<SWidget> FToolableTimeSliderController::OpenContextMenu(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	const FMouseInputData MouseInput(Timeline.AsShared(), InGeometry, InPointerEvent);

	if (const TSharedPtr<FSequencer> Sequencer = Timeline.GetSequencer())
	{
		TSet<FSequencerSelectedKey> KeysUnderMouse;
		GetKeysUnderMouse(MouseInput, InPointerEvent.GetScreenSpacePosition(), KeysUnderMouse);

		FToolableTimelineKeySelection& KeySelection = Timeline.GetKeySelection();
		const USequencerSettings* const SequencerSettings = Timeline.GetSequencerSettings();

		const bool bShouldSelectRightClickedKeys = SequencerSettings
			&& SequencerSettings->IsSimpleView()
			&& !Utils::ShouldUseControlModifierToMoveKeys()
			&& !KeysUnderMouse.IsEmpty();

		if (bShouldSelectRightClickedKeys)
		{
			const TSet<FSequencerSelectedKey> SelectedKeys = KeySelection.GetSelectedKeys();

			// Preserve existing multi-selection if the clicked key is already selected
			if (KeysUnderMouse.Intersect(SelectedKeys).Num() == 0)
			{
				KeySelection.SelectKeys(KeysUnderMouse, /*bInAddToSelection=*/false);
			}
		}

		// Only generate the key context menu if the keys under the mouse match the selected keys
		const TSet<FSequencerSelectedKey> SelectedKeys = Timeline.GetKeySelection().GetSelectedKeys();

		if (!KeysUnderMouse.IsEmpty() && KeysUnderMouse.Intersect(SelectedKeys).Num() > 0)
		{
			return FToolableTimelineKeyContextMenu::GenerateWidget(MouseInput);
		}
	}

	return FToolableTimelineMenu::GenerateWidget(MouseInput);
}

TSharedPtr<FToolableTimelineBaseTool> FToolableTimeSliderController::GetActiveTool() const
{
	return ActiveTool;
}

void FToolableTimeSliderController::ActivateTool(const TSharedRef<FToolableTimelineBaseTool>& InTool
	, const FMouseInputData& InMouseInput, const FFrameTime& InInputTickTime)
{
	if (ActiveTool.IsValid())
	{
		DeactivateTool();
	}

	ActiveTool = InTool;

	ActiveTool->BindCommands(Timeline.GetCommandList());
	ActiveTool->OnToolClosed().AddSP(this, &FToolableTimeSliderController::DeactivateTool);
	ActiveTool->NotifyToolActivated(InMouseInput, InInputTickTime);

	Timeline.NotifyToolActivated();

	bHideScrubHandle = ActiveTool->ShouldHideScrubberOnActivation();

	ResetPendingToolActivation();
}

void FToolableTimeSliderController::DeactivateTool()
{
	if (ActiveTool.IsValid())
	{
		ActiveTool->OnToolClosed().RemoveAll(this);
		ActiveTool->UnbindCommands(Timeline.GetCommandList());
		ActiveTool->NotifyToolDeactivated();

		Timeline.NotifyToolDeactivated();

		ActiveTool.Reset();

		Timeline.KeySelection.ClearHoveredKeys();
		Timeline.KeySelection.ClearSelectionPreview();
	}

	bHideScrubHandle = false;
}

double FToolableTimeSliderController::GetMajorTickDrawSize() const
{
	if (FontMeasureService.IsValid())
	{
		const double FontSize = Timeline.GetTimelineSettings().Settings.MajorTickFontSize;
		const FSlateFontInfo TickFrameFont = FCoreStyle::GetDefaultFontStyle(TEXT("Regular"), FontSize);
		const FVector2d TextDrawSize = FontMeasureService->Measure(TEXT("0"), TickFrameFont);
		return TextDrawSize.Y + (Drawing::Constants::FrameLabelOffset.Y * 2.);
	}
	return 0.;
}

bool FToolableTimeSliderController::IsDragging() const
{
	return KeyAndMarkDragOperation.IsValid()
		|| ScrubDragOperation.IsSet()
		|| MouseDragType != DRAG_NONE;
}

const TOptional<FToolableTimelineScrubDragOperation>& FToolableTimeSliderController::GetScrubDragOperation() const
{
	return ScrubDragOperation;
}

bool FToolableTimeSliderController::IsScrubHandleHidden() const
{
	return bHideScrubHandle;
}

bool FToolableTimeSliderController::GetGridMetrics(const FGeometry& InGeometry, double& OutMajorGridStep, int32& OutMinorDivisions) const
{
	const TSharedPtr<FSequencer> Sequencer = Timeline.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	const TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface = Sequencer->GetNumericTypeInterface();
	if (!NumericTypeInterface.IsValid())
	{
		return false;
	}

	const float LocalWidth = InGeometry.GetLocalSize().X;
	if (LocalWidth <= 0)
	{
		return false;
	}

	const UToolableTimelineSettings& TimelineSettings = Timeline.GetTimelineSettings();
	const FFrameRate FocusedDisplayRate = Sequencer->GetFocusedDisplayRate();
	const TRange<double> ViewRange = GetViewRange();

	const double ViewStart = ViewRange.GetLowerBoundValue();
	const double ViewEnd = ViewRange.GetUpperBoundValue();

	return FocusedDisplayRate.ComputeGridSpacing(
		LocalWidth / (ViewEnd - ViewStart),
		OutMajorGridStep,
		OutMinorDivisions,
		TimelineSettings.Settings.MinMinorTickSize,
		TimelineSettings.Settings.DesiredMajorTickSize
	);
}

bool FToolableTimeSliderController::IsSnapEnabled() const
{
	const USequencerSettings* const SequencerSettings = Timeline.GetSequencerSettings();
	return SequencerSettings ? SequencerSettings->GetIsSnapEnabled() : false;
}

FToolableTimeSliderController::FScrubberMetrics FToolableTimeSliderController::GetScrubPixelMetrics(const FQualifiedFrameTime& InScrubTime
	, const FScrubRangeToScreen& InRangeToScreen, const float InDilationPixels) const
{
	const TSharedPtr<FSequencer> Sequencer = Timeline.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return FScrubberMetrics();
	}

	static constexpr float MinScrubSize = 1.f;
	static constexpr float HalfMinScrubSize = MinScrubSize * .5f;

	const UToolableTimelineSettings& TimelineSettings = Timeline.GetTimelineSettings();
	const FFrameRate DisplayRate = GetDisplayRate();
	const FFrameNumber Frame = InScrubTime.ConvertTo(DisplayRate).FloorToFrame();

	float FrameStartPixel = InRangeToScreen.InputToLocalX(Frame / DisplayRate);
	float FrameEndPixel = InRangeToScreen.InputToLocalX((Frame + 1) / DisplayRate);

	const float RoundedStartPixel = FMath::RoundToInt(FrameStartPixel);

	FrameStartPixel = RoundedStartPixel;
	FrameEndPixel -= (FrameStartPixel - RoundedStartPixel);
	FrameEndPixel = FMath::Max(FrameEndPixel, FrameStartPixel + 1);

	FScrubberMetrics Metrics;

	// Store off the pixel width of the frame
	Metrics.FrameExtentsPx = TRange<float>(FrameStartPixel - InDilationPixels, FrameEndPixel + InDilationPixels);

	// Set the style of the scrub handle
	Metrics.Style = Sequencer->GetScrubStyle();

	// Always draw the extents on the section area for frame block styles
	Metrics.bDrawExtents = true;

	const float ScrubPixel = InRangeToScreen.InputToLocalX(InScrubTime.AsSeconds());

	Metrics.HandleRangePx = TimelineSettings.Settings.bDrawWholeFrameScrubber
		? Metrics.FrameExtentsPx
		: TRange<float>(ScrubPixel - HalfMinScrubSize - InDilationPixels
			, ScrubPixel + HalfMinScrubSize + InDilationPixels);

	return Metrics;
}

FReply FToolableTimeSliderController::TryBeginScrub(const FMouseInputData& InMouseInput)
{
	if (!bMouseDownInRegion || MouseDragType != DRAG_NONE)
	{
		return FReply::Unhandled();
	}

	if (ActiveDragTarget.IsSet())
	{
		if (ActiveDragTarget->IsSelectionRange()
			|| ActiveDragTarget->IsPlaybackRange()
			|| ActiveDragTarget->IsMark())
		{
			return FReply::Unhandled();
		}
	}

	// If scrub begins on empty space, clear key selection immediately.
	// If scrub begins over keys, keep selection so mouse-up can decide the key selection behavior.
	if (Timeline.GetTimelineSettings().Settings.bClearSelectionOnScrub)
	{
		TSet<FSequencerSelectedKey> KeysUnderMouse;
		GetKeysUnderMouse(InMouseInput, InMouseInput.PointerEvent.GetScreenSpacePosition(), KeysUnderMouse);
		if (KeysUnderMouse.IsEmpty())
		{
			Timeline.GetKeySelection().ClearSelectedKeys();
		}
	}

	ClearMarkSelection();

	LastMousePosition = InMouseInput.PointerEvent.GetScreenSpacePosition();
	MouseDragType = DRAG_SCRUBBING_TIME;
	bHideScrubHandle = false;
	ScrubDragOperation.Emplace(InMouseInput, GetViewRange());

	TimeSliderArgs.OnBeginScrubberMovement.ExecuteIfBound();

	return FReply::Handled()
		.CaptureMouse(InMouseInput.OwnerWidget)
		.UseHighPrecisionMouseMovement(InMouseInput.OwnerWidget)
		.PreventThrottling();
}

FReply FToolableTimeSliderController::TryBeginMarkDrag(const FMouseInputData& InMouseInput)
{
	const TSharedPtr<FSequencerSelection> SequencerSelection = GetSequencerSelection();
	if (!SequencerSelection.IsValid()
		|| !ActiveDragTarget.IsSet()
		|| !ActiveDragTarget->IsMark())
	{
		return FReply::Unhandled();
	}

	FMarkedFrameSelection& MarkedFrameSelection = SequencerSelection->MarkedFrames;

	const bool bToggleSelection = InMouseInput.PointerEvent.IsControlDown()
		|| InMouseInput.PointerEvent.IsShiftDown();
	const bool bMarkAlreadySelected = MarkedFrameSelection.IsSelected(ActiveDragTarget->MarkIndex);
	if (!bToggleSelection && !bMarkAlreadySelected)
	{
		MarkedFrameSelection.Empty();
		MarkedFrameSelection.Select(ActiveDragTarget->MarkIndex);
	}

	LastMousePosition = InMouseInput.PointerEvent.GetScreenSpacePosition();
	MouseDragType = DRAG_MARK;
	TimeSliderArgs.OnMarkBeginDrag.ExecuteIfBound();
	KeyAndMarkDragOperation = MakeUnique<FToolableTimelineKeyAndMarkDragOperation<FChannelKeyCache>>(InMouseInput, GetViewRange(), ActiveDragTarget->MarkIndex);

	return FReply::Handled()
		.CaptureMouse(InMouseInput.OwnerWidget)
		.UseHighPrecisionMouseMovement(InMouseInput.OwnerWidget)
		.PreventThrottling();
}

TSharedPtr<FSequencerEditorViewModel> FToolableTimeSliderController::GetSequencerViewModel() const
{
	const TSharedPtr<FSequencer> Sequencer = Timeline.GetSequencer();
	return Sequencer.IsValid() ? Sequencer->GetViewModel() : nullptr;
}

TSharedPtr<FSequencerSelection> FToolableTimeSliderController::GetSequencerSelection() const
{
	const TSharedPtr<FSequencerEditorViewModel> ViewModel = GetSequencerViewModel();
	return ViewModel.IsValid() ? ViewModel->GetSelection() : nullptr;
}

TSharedPtr<FSequenceModel> FToolableTimeSliderController::GetRootSequenceModel() const
{
	const TSharedPtr<FSequencerEditorViewModel> ViewModel = GetSequencerViewModel();
	return ViewModel.IsValid() ? ViewModel->GetRootSequenceModel() : nullptr;
}

TRange<FFrameNumber> FToolableTimeSliderController::GetScrubWholeFrameRange() const
{
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = Timeline.GetTimeSliderController();
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();
	const FFrameRate DisplayRate = TimeSliderController->GetDisplayRate();

	const FFrameTime ScrubTimeTick = TimeSliderController->GetScrubPosition();

	// Find the display frame under the scrubber
	const FFrameTime ScrubTimeDisplay = FFrameRate::TransformTime(ScrubTimeTick, TickResolution, DisplayRate);
	const FFrameNumber DisplayFrame = ScrubTimeDisplay.FloorToFrame();

	return Utils::MakeTickRangeFromDisplayFrame(DisplayFrame, DisplayRate, TickResolution);
}

FFrameTime FToolableTimeSliderController::SnapFrameTimeToDisplayFrame(const FFrameTime& InTickTime) const
{
	const FFrameRate TickResolution = GetTickResolution();
	const FFrameRate DisplayRate = GetDisplayRate();

	const FFrameTime DisplayTime = FFrameRate::TransformTime(InTickTime, TickResolution, DisplayRate);
	const FFrameNumber SnappedDisplayFrame = DisplayTime.RoundToFrame();

	return FFrameRate::TransformTime(FFrameTime(SnappedDisplayFrame), DisplayRate, TickResolution);
}

void FToolableTimeSliderController::ResetInput()
{
	ResetMouseInput();

	MouseDragType = DRAG_NONE;
	DistanceDragged = 0.f;

	ScrubDragOperation.Reset();
	KeyAndMarkDragOperation.Reset();

	bHideScrubHandle = false;

	ActiveDragTarget.Reset();
	HoverTarget.Reset();
	LastCommittedScrubTime.Reset();

	ResetPendingToolActivation();

	Timeline.GetKeySelection().ResetHoveredKeysAndSelectionPreview();
}

void FToolableTimeSliderController::EndAnyInProgressTransactionalDrag()
{
	switch (MouseDragType)
	{
	case DRAG_PLAYBACK_START:
	case DRAG_PLAYBACK_END:
		TimeSliderArgs.OnPlaybackRangeEndDrag.ExecuteIfBound();
		break;

	case DRAG_SELECTION_START:
	case DRAG_SELECTION_END:
		TimeSliderArgs.OnSelectionRangeEndDrag.ExecuteIfBound();
		break;

	case DRAG_MARK:
		TimeSliderArgs.OnMarkEndDrag.ExecuteIfBound();
		break;

	default:
		break;
	}

	MouseDragType = DRAG_NONE;
	bMouseDownInRegion = false;
}

FFrameTime FToolableTimeSliderController::ComputeMouseFrameTime(const FMouseInputData& InMouseInput, const bool bInCheckSnapping) const
{
	if (ScrubDragOperation.IsSet())
	{
		const double LocalX = ScrubDragOperation->GetVirtualLocalPosition().X;
		const double MouseValue = InMouseInput.RangeToScreen.LocalXToInput(LocalX);

		if (bInCheckSnapping)
		{
			if (const TSharedPtr<FSequencer> Sequencer = Timeline.GetSequencer())
			{
				const USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
				if (SequencerSettings && SequencerSettings->GetForceWholeFrames())
				{
					const FFrameNumber SnappedFrameNumber = (MouseValue * GetDisplayRate()).FloorToFrame();
					const FQualifiedFrameTime RoundedPlayFrame(SnappedFrameNumber, GetDisplayRate());
					return RoundedPlayFrame.ConvertTo(GetTickResolution());
				}
			}
		}

		return MouseValue * GetTickResolution();
	}

	return ComputeFrameTimeFromMouse(InMouseInput.Geometry
		, InMouseInput.PointerEvent.GetScreenSpacePosition()
		, InMouseInput.RangeToScreen
		, bInCheckSnapping);
}

bool FToolableTimeSliderController::TryCloseActiveToolRequest()
{
	bool bToolWasDeactivated = false;

	if (ActiveTool.IsValid() && ActiveTool->IsCloseRequested())
	{
		DeactivateTool();

		bToolWasDeactivated = true;
	}

	return bToolWasDeactivated;
}

FToolableTimeSliderController::FResolvedDragTarget FToolableTimeSliderController::ResolveDragTarget(const FMouseInputData& InMouseInput
	, const bool bInFromTimeSlider, const bool bUseMouseDownPosition) const
{
	FResolvedDragTarget Result;

	const FVector2D ScreenPosition = (bUseMouseDownPosition && MouseDownPosition.IsSet())
		? MouseDownPosition.GetValue() : InMouseInput.PointerEvent.GetScreenSpacePosition();
	const float MousePixel = InMouseInput.Geometry.AbsoluteToLocal(ScreenPosition).X;
	const bool bHitScrubber = GetHitTestScrubPixelMetrics(InMouseInput.RangeToScreen).HandleRangePx.Contains(MousePixel);

	// 1) Active tool
	if (ActiveTool.IsValid())
	{
		Result.bToolHasPriorityHit = ActiveTool->WantsPriorityOverControllerHit(InMouseInput);

		if (Result.bToolHasPriorityHit && ActiveTool->HasHoverHit())
		{
			Result.HitType = EResolvedMouseHit::Tool;
			return Result;
		}
	}

	const FFrameRate TickResolution = GetTickResolution();
	const TRange<double> SelectionRange = TimeSliderArgs.SelectionRange.Get() / TickResolution;

	if (!bHitScrubber && !SelectionRange.IsEmpty())
	{
		if (HitTestRangeEnd(InMouseInput.RangeToScreen, SelectionRange, MousePixel))
		{
			Result.HitType = EResolvedMouseHit::SelectionEnd;
			return Result;
		}

		if (HitTestRangeStart(InMouseInput.RangeToScreen, SelectionRange, MousePixel))
		{
			Result.HitType = EResolvedMouseHit::SelectionStart;
			return Result;
		}
	}

	const TRange<double> PlaybackRange = TimeSliderArgs.PlaybackRange.Get() / TickResolution;

	if (!bHitScrubber && !TimeSliderArgs.IsPlaybackRangeLocked.Get())
	{
		if (HitTestRangeEnd(InMouseInput.RangeToScreen, PlaybackRange, MousePixel))
		{
			Result.HitType = EResolvedMouseHit::PlaybackEnd;
			return Result;
		}

		if (HitTestRangeStart(InMouseInput.RangeToScreen, PlaybackRange, MousePixel))
		{
			Result.HitType = EResolvedMouseHit::PlaybackStart;
			return Result;
		}
	}

	if (!bHitScrubber
		&& !TimeSliderArgs.AreMarkedFramesLocked.Get()
		&& HitTestMark(InMouseInput.Geometry, InMouseInput.RangeToScreen, MousePixel, bInFromTimeSlider, &Result.MarkIndex))
	{
		Result.HitType = EResolvedMouseHit::Mark;
		return Result;
	}

	if (ActiveTool.IsValid() && ActiveTool->HasHoverHit())
	{
		Result.HitType = EResolvedMouseHit::Tool;
		return Result;
	}

	GetKeysAtPixelX(InMouseInput, MousePixel, Result.HoveredKeys);
	if (Result.HoveredKeys.Num() > 0)
	{
		Result.HitType = EResolvedMouseHit::Key;
		return Result;
	}

	if (bHitScrubber)
	{
		Result.HitType = EResolvedMouseHit::Scrubber;
		return Result;
	}

	return Result;
}

void FToolableTimeSliderController::ReinitializeKeyRenderer(const TSet<TWeakViewModelPtr<FChannelModel>>& InWeakViewModels)
{
	TSet<FWeakViewModelPtr> ViewModels;
	ViewModels.Reserve(InWeakViewModels.Num());

	TMap<uint32, uint32> GroupIDsByModelID;
	GroupIDsByModelID.Reserve(InWeakViewModels.Num());

	// Simple view draws multiple full view rows with one key renderer. Preserve full view partial key
	// semantics by evaluating each channel against its category row, or the channel row if uncategorized.
	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : InWeakViewModels)
	{
		if (const TViewModelPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin())
		{
			const TViewModelPtr<FCategoryModel> CategoryModel = ChannelModel->FindAncestorOfType<FCategoryModel>();
			const uint32 FullViewPartialKeyGroupID = CategoryModel.IsValid()
				? CategoryModel->GetModelID() : ChannelModel->GetModelID();

			ViewModels.Add(ChannelModel.AsModel());
			GroupIDsByModelID.Add(ChannelModel->GetModelID(), FullViewPartialKeyGroupID);
		}
	}

	KeyRenderer.InitializeGrouped(ViewModels, MoveTemp(GroupIDsByModelID));

	InvalidateKeyRendererCache();
}

void FToolableTimeSliderController::GetKeysUnderMouse(const FMouseInputData& InMouseInput
	, const FVector2D& InMousePosition, TSet<FSequencerSelectedKey>& OutKeys) const
{
	if (!KeyRendererCache.IsSet())
	{
		return;
	}

	const UToolableTimelineSettings& TimelineSettings = Timeline.GetTimelineSettings();
	const FVector2D MousePixel = InMouseInput.Geometry.AbsoluteToLocal(InMousePosition);
	const float LocalHeight = InMouseInput.Geometry.GetLocalSize().Y;

	float KeyTop = 0.f;
	float KeyBottom = LocalHeight;

	switch (TimelineSettings.Settings.KeyDrawStyle)
	{
	case EKeyRendererStyleConfig::Line:
		switch (TimelineSettings.Settings.LabelVerticalAlignment)
		{
		case VAlign_Fill:
			KeyTop = 0.f;
			break;
		case VAlign_Center:
			KeyTop = (LocalHeight - KeyRendererCache->KeySizePx.Y) * .5f;
			break;
		default:
		case VAlign_Top:
			KeyTop = LocalHeight - KeyRendererCache->KeySizePx.Y;
			break;
		case VAlign_Bottom:
			KeyTop = 0.f;
			break;
		}
		KeyBottom = KeyTop + KeyRendererCache->KeySizePx.Y;
		break;

	case EKeyRendererStyleConfig::FrameBlock:
		{
			static constexpr float BlockPadding = 1.f;

			switch (TimelineSettings.Settings.LabelVerticalAlignment)
			{
			default:
			case VAlign_Fill:
			case VAlign_Center:
				KeyTop = BlockPadding;
				break;
			case VAlign_Top:
				KeyTop = GetMajorTickDrawSize() + BlockPadding;
				break;
			case VAlign_Bottom:
				KeyTop = BlockPadding;
				break;
			}

			KeyBottom = KeyTop + KeyRendererCache->KeySizePx.Y;
			break;
		}

	default:
	case EKeyRendererStyleConfig::Circle:
	case EKeyRendererStyleConfig::Diamond:
		{
			float KeyCenterY = LocalHeight * .5f;

			switch (TimelineSettings.Settings.LabelVerticalAlignment)
			{
			default:
			case VAlign_Fill:
			case VAlign_Center:
				break;
			case VAlign_Top:
				KeyCenterY += GetMajorTickDrawSize() * .5f;
				break;
			case VAlign_Bottom:
				KeyCenterY -= GetMajorTickDrawSize() * .5f;
				break;
			}

			const float HalfKeyHeight = KeyRendererCache->KeySizePx.Y * .5f;
			KeyTop = KeyCenterY - HalfKeyHeight;
			KeyBottom = KeyCenterY + HalfKeyHeight;
			break;
		}
	}

	if (MousePixel.Y >= KeyTop && MousePixel.Y <= KeyBottom)
	{
		GetKeysAtPixelX(InMouseInput, MousePixel.X, OutKeys);
	}
}

void FToolableTimeSliderController::GetKeysAtPixelX(const FMouseInputData& InMouseInput
	, const float InLocalMousePixelX, TSet<FSequencerSelectedKey>& OutKeys) const
{
	const FFrameRate TickResolution = GetTickResolution();

	TArray<FKeyRenderer::FKeysForModel> Results;
	const bool bLeftAlignHit = Timeline.GetTimelineSettings().Settings.KeyDrawStyle == EKeyRendererStyleConfig::FrameBlock;

	// Convert pixel to frame using RangeToScreen instead of RelativeTimeToPixel
	const double MouseSeconds = InMouseInput.RangeToScreen.LocalXToInput(InLocalMousePixelX);
	const FFrameTime MouseFrame = MouseSeconds * TickResolution;

	KeyRenderer.HitTestKeys(MouseFrame, Results, bLeftAlignHit);

	OutKeys = Utils::KeyRendererResultToSelectedKeys(Results);
}

TRange<FFrameNumber> FToolableTimeSliderController::GetPlayheadScrubRange() const
{
	const TSharedPtr<ISequencer> Sequencer = Timeline.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return TRange<FFrameNumber>::Empty();
	}

	const USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!SequencerSettings)
	{
		return TRange<FFrameNumber>::Empty();
	}

	return SequencerSettings->GetPlayheadScrubbingRange(Sequencer);
}

bool FToolableTimeSliderController::ShouldUseVirtualScrubCoordinates(const FMouseInputData& InMouseInput
	, const TRange<FFrameNumber>& InScrubRange) const
{
	if (!Timeline.GetTimelineSettings().Settings.bUseVirtualScrub)
	{
		return false;
	}

	if (!FToolableTimelineBaseTool::IsValidToolRange(InScrubRange))
	{
		return true;
	}

	const FFrameRate TickResolution = GetTickResolution();
	const TRange<double> ViewRange = GetViewRange();

	const double ViewStartSeconds = ViewRange.GetLowerBoundValue();
	const double ViewEndSeconds = ViewRange.GetUpperBoundValue();

	const double RangeMinSeconds = InScrubRange.GetLowerBoundValue() / TickResolution;
	const double RangeMaxSeconds = InScrubRange.GetUpperBoundValue() / TickResolution;

	const bool bLeftBoundOffscreen = RangeMinSeconds < ViewStartSeconds;
	const bool bRightBoundOffscreen = RangeMaxSeconds > ViewEndSeconds;

	const float CursorDeltaX = InMouseInput.PointerEvent.GetCursorDelta().X;
	if (FMath::IsNearlyZero(CursorDeltaX))
	{
		return false;
	}

	const FVector2D LocalMousePosition =
		InMouseInput.Geometry.AbsoluteToLocal(InMouseInput.PointerEvent.GetScreenSpacePosition());

	const float LocalWidth = InMouseInput.Geometry.GetLocalSize().X;

	// Small pixel tolerance so the mouse does not need to land on the exact edge
	static constexpr float EdgeTolerancePx = 2.f;

	const bool bAtLeftEdge = LocalMousePosition.X <= EdgeTolerancePx;
	const bool bAtRightEdge = LocalMousePosition.X >= (LocalWidth - EdgeTolerancePx);

	if (CursorDeltaX > 0.f)
	{
		// Dragging right: only allow virtual scrub once the real cursor is at the
		// right edge and the scrub range still extends offscreen to the right
		return bRightBoundOffscreen && bAtRightEdge;
	}

	// Dragging left: only allow virtual scrub once the real cursor is at the
	// left edge and the scrub range still extends offscreen to the left
	return bLeftBoundOffscreen && bAtLeftEdge;
}

FFrameTime FToolableTimeSliderController::ResolveAndUpdateScrubTime(const FMouseInputData& InMouseInput)
{
	const TRange<FFrameNumber> ScrubRange = GetPlayheadScrubRange();

	if (const TOptional<FFrameTime> SelectedKeyTime = GetFirstSelectedHoveredKeyTime(Timeline))
	{
		FFrameTime ScrubTime = SelectedKeyTime.GetValue();

		if (FToolableTimelineBaseTool::IsValidToolRange(ScrubRange))
		{
			ScrubTime = FMath::Clamp(ScrubTime, FFrameTime(ScrubRange.GetLowerBoundValue()), FFrameTime(ScrubRange.GetUpperBoundValue()));
		}

		return ScrubTime;
	}

	FFrameTime ScrubTime;

	const bool bUseVirtualScrub = ShouldUseVirtualScrubCoordinates(InMouseInput, ScrubRange);
	if (bUseVirtualScrub)
	{
		LastMousePosition = ScrubDragOperation->GetLastScreenSpacePosition();

		const double PanSeconds = ScrubDragOperation->ComputePanSecondsFromVirtualPointerOverflow()
			* Timeline.GetTimelineSettings().Settings.VirtualScrubSpeed;
		if (!FMath::IsNearlyZero(PanSeconds))
		{
			double NewViewMin = ScrubDragOperation->GetInitialViewRange().GetLowerBoundValue() + PanSeconds;
			double NewViewMax = ScrubDragOperation->GetInitialViewRange().GetUpperBoundValue() + PanSeconds;

			ClampViewRange(NewViewMin, NewViewMax);
			SetViewRange(NewViewMin, NewViewMax, EViewRangeInterpolation::Immediate);
		}

		ScrubTime = ScrubDragOperation->ComputeDraggedTickTime(GetTickResolution(), GetDisplayRate(), /*bInSnapEnabled=*/false);
	}
	else
	{
		LastMousePosition = InMouseInput.PointerEvent.GetScreenSpacePosition();

		ScrubTime = ComputeMouseFrameTime(InMouseInput, /*bInCheckSnapping=*/false);
	}

	if (FToolableTimelineBaseTool::IsValidToolRange(ScrubRange))
	{
		ScrubTime = FMath::Clamp(ScrubTime, FFrameTime(ScrubRange.GetLowerBoundValue()), FFrameTime(ScrubRange.GetUpperBoundValue()));
	}

	return ScrubTime;
}

TOptional<FFrameTime> FToolableTimeSliderController::GetFirstSelectedHoveredKeyTime(const FToolableTimeline& InTimeline)
{
	const FToolableTimelineKeySelection& KeySelection = InTimeline.GetKeySelection();

	const TSet<FSequencerSelectedKey>& HoveredKeys = KeySelection.GetHoveredKeys();
	if (HoveredKeys.IsEmpty())
	{
		return TOptional<FFrameTime>();
	}

	const FSequencerSelectedKey& FirstHoveredKey = *HoveredKeys.CreateConstIterator();
	const TSet<FSequencerSelectedKey> SelectedKeys = KeySelection.GetSelectedKeys();

	if (!SelectedKeys.Contains(FirstHoveredKey))
	{
		return TOptional<FFrameTime>();
	}

	const TViewModelPtr<FChannelModel> ChannelModel = FirstHoveredKey.WeakChannel.Pin();
	if (!ChannelModel.IsValid())
	{
		return TOptional<FFrameTime>();
	}

	const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea();
	if (!KeyArea.IsValid())
	{
		return TOptional<FFrameTime>();
	}

	TArray<FKeyHandle> KeyHandles;
	KeyHandles.Add(FirstHoveredKey.KeyHandle);

	TArray<FFrameNumber> KeyTimes;
	KeyTimes.SetNumUninitialized(1);
	KeyArea->GetKeyTimes(KeyHandles, KeyTimes);

	return FFrameTime(KeyTimes[0]);
}

TSharedPtr<IToolableTimelineToolFactory> FToolableTimeSliderController::FindToolFactoryToActivate(const FMouseInputData& InMouseInput
	, const bool bInIsDoubleClick)
{
	if (ActiveTool.IsValid())
	{
		return nullptr;
	}

	const bool bMouseHitKeyFrame = ActiveDragTarget.IsSet() && ActiveDragTarget->IsKey();

	TSharedPtr<IToolableTimelineToolFactory> FoundFactory;
	float FoundPriority = 0.f;

	for (const TSharedRef<IToolableTimelineToolFactory>& Factory : Timeline.ToolFactories)
	{
		const bool bWantsToActivate = bInIsDoubleClick
			? Factory->WantsToActivateOnDoubleClick(InMouseInput, bMouseHitKeyFrame)
			: Factory->WantsToActivate(InMouseInput, bMouseHitKeyFrame);
		if (!bWantsToActivate)
		{
			continue;
		}

		const float Priority = Factory->GetPriority();
		if (!FoundFactory.IsValid() || Priority < FoundPriority)
		{
			FoundFactory = Factory;
			FoundPriority = Priority;
		}
	}

	return FoundFactory;
}

bool FToolableTimeSliderController::HasActiveToolKeySelection() const
{
	return ActiveTool.IsValid()
		&& ActiveTool->HasValidToolRange()
		&& ActiveTool->GetToolRangeKeys().Num() > 0;
}

bool FToolableTimeSliderController::ShouldBeginPendingToolActivation(const FPointerEvent& InPointerEvent) const
{
	if (ActiveTool.IsValid()
		|| !PendingToolFactory.IsValid()
		|| !PendingTool.IsValid()
		|| !MouseDownPosition.IsSet()
		|| !InPointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		return false;
	}

	return FSlateApplication::Get().HasTraveledFarEnoughToTriggerDrag(InPointerEvent, MouseDownPosition.GetValue());
}

void FToolableTimeSliderController::ResetPendingToolActivation()
{
	PendingToolFactory.Reset();
	PendingTool.Reset();
}

void FToolableTimeSliderController::ResetControllerDragTargetForToolOwnership()
{
	ActiveDragTarget.Reset();
}

void FToolableTimeSliderController::UpdateScrubHoveredKeys()
{
	if (ActiveTool.IsValid() && HasActiveToolKeySelection())
	{
		return;
	}

	const TSet<FSequencerSelectedKey> ScrubRangeKeys = Timeline.GetScrubRangeKeys();
	Timeline.GetKeySelection().SetHoveredKeys(ScrubRangeKeys);
}

bool FToolableTimeSliderController::HandleScrubReleaseKeySelection(const FMouseInputData& InMouseInput)
{
	TSet<FSequencerSelectedKey> KeysUnderMouse;
	GetKeysUnderMouse(InMouseInput, InMouseInput.PointerEvent.GetScreenSpacePosition(), KeysUnderMouse);
	if (KeysUnderMouse.IsEmpty())
	{
		return false;
	}

	const bool bShiftDown = InMouseInput.PointerEvent.IsShiftDown();
	const bool bControlDown = InMouseInput.PointerEvent.IsControlDown();
	const bool bShiftOrControlDown = bShiftDown || bControlDown;
	if (bShiftOrControlDown && InMouseInput.PointerEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		FToolableTimelineKeySelection& KeySelection = Timeline.GetKeySelection();
		const TSet<FSequencerSelectedKey> SelectedKeys = KeySelection.GetSelectedKeys();
		const TSet<FSequencerSelectedKey> SelectedKeysUnderMouse = KeysUnderMouse.Intersect(SelectedKeys);

		if (bShiftDown)
		{
			if (!SelectedKeysUnderMouse.IsEmpty())
			{
				TSet<FSequencerSelectedKey> NewSelectedKeys = SelectedKeys;

				for (const FSequencerSelectedKey& Key : SelectedKeysUnderMouse)
				{
					NewSelectedKeys.Remove(Key);
				}

				KeySelection.SetSelectedKeys(NewSelectedKeys, /*bInSync=*/true);
			}
			else
			{
				KeySelection.SelectKeys(KeysUnderMouse, /*bInAddToSelection=*/true);
			}

			return true;
		}

		if (bControlDown)
		{
			if (SelectedKeysUnderMouse.IsEmpty())
			{
				KeySelection.SelectKeys(KeysUnderMouse, /*bInAddToSelection=*/true);
			}

			return true;
		}
	}

	// Plain click over a key should scrub to it and not change selection
	if (const TOptional<FFrameTime> FirstKeyTime = Utils::GetFirstKeyTime(KeysUnderMouse))
	{
		if (Timeline.GetTimelineSettings().Settings.bSelectScrubFrameKeys)
		{
			Timeline.KeySelection.SetSelectedKeys(KeysUnderMouse, /*bInSync=*/true);
		}

		const bool bEvaluate = InMouseInput.PointerEvent.GetEffectingButton() != EKeys::MiddleMouseButton;
		CommitScrubPosition(FirstKeyTime.GetValue(), /*bIsScrubbing=*/false, /*bEvaluate=*/bEvaluate);

		return true;
	}

	return false;
}

void FToolableTimeSliderController::InvalidateKeyRendererCache()
{
	KeyRendererInvalidationFlags |= EViewDependentCacheFlags::DataChanged;
}

void FToolableTimeSliderController::InvalidateKeyRendererKeyState()
{
	KeyRendererInvalidationFlags |= EViewDependentCacheFlags::KeyStateChanged;
}

} // namespace UE::Sequencer::ToolableTimeline

#undef LOCTEXT_NAMESPACE
