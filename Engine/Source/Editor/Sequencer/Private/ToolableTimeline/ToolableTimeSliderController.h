// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "ITimeSlider.h"
#include "Layout/Geometry.h"
#include "MVVM/ViewModelPtr.h"
#include "SequencerSelectedKey.h"
#include "SequencerTimeSliderController.h"
#include "Templates/UniquePtr.h"
#include "TimeSliderArgs.h"
#include "ToolableTimeline/Caches/ToolableTimelineKeyViewCacheState.h"
#include "ToolableTimeline/DragOperations/ToolableTimelineScrubDragOperation.h"
#include "ToolableTimeline/ToolableTimeline.h"
#include "ToolableTimeline/Tools/Factories/IToolableTimelineToolFactory.h"
#include "Widgets/SWidget.h"

class FSequencer;
class FSlateWindowElementList;

namespace UE::Sequencer
{
	class FSequenceModel;
	class FSequencerEditorViewModel;
	class FSequencerSelection;
}

namespace UE::Sequencer::ToolableTimeline
{

class FToolableTimelineBaseTool;
struct FChannelKeyCache;
struct FMouseInputData;
template <typename TKeyCacheType> class FToolableTimelineKeyAndMarkDragOperation;

/**
 * A time slider controller for Sequencer that supports multiple tools and a single active tools.
 */
class FToolableTimeSliderController : public FSequencerTimeSliderController
{
public:
	FToolableTimeSliderController(const FTimeSliderArgs& InTimeSliderArgs, FToolableTimeline& InTimeline);
	virtual ~FToolableTimeSliderController() override;

	FToolableTimeline& GetTimeline() const { return Timeline; }

	//~ Begin ITimeSliderController

	virtual double ComputeHeight() const override;

	virtual int32 OnPaintTimeSlider(const bool bInMirrorLabels
		, const FGeometry& InGeometry
		, const FSlateRect& InCullingRect
		, FSlateWindowElementList& OutDrawElements
		, int32 LayerId
		, const FWidgetStyle& InWidgetStyle
		, const bool bInParentEnabled) const override;
	virtual int32 OnPaintViewArea(const FGeometry& InGeometry
		, const FSlateRect& InCullingRect
		, FSlateWindowElementList& OutDrawElements
		, int32 LayerId
		, bool bInEnabled
		, const FPaintViewAreaArgs& InArgs) const override;

	virtual FReply OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseMove(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseButtonDoubleClick(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;

	virtual void OnMouseEnter(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual void OnMouseLeave(SWidget& OwnerWidget, const FPointerEvent& InPointerEvent) override;

	virtual FCursorReply OnCursorQuery(const SWidget& OwnerWidget
		, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) const override;

	virtual FReply OnKeyDown(SWidget& OwnerWidget, const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyUp(SWidget& OwnerWidget, const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;

	virtual void OnFocusLost(SWidget& OwnerWidget, const FFocusEvent& InFocusEvent) override;

	//~ End ITimeSliderController

	//~ Begin FSequencerTimeSliderController

	virtual FScrubberMetrics GetScrubPixelMetrics(const FQualifiedFrameTime& InScrubTime
		, const FScrubRangeToScreen& InRangeToScreen, const float InDilationPixels = 0.f) const override;

	//~ End FSequencerTimeSliderController

	TSharedPtr<FToolableTimelineBaseTool> GetActiveTool() const;

	template<typename TToolType> 
	TSharedPtr<TToolType> ActiveToolAs() const
	{
		return StaticCastSharedPtr<TToolType>(ActiveTool);
	}

	void ActivateTool(const TSharedRef<FToolableTimelineBaseTool>& InTool
		, const FMouseInputData& InMouseInput, const FFrameTime& InInputTickTime);

	/** Closes any active tool */
	void DeactivateTool();

	/** @return Size to draw for a major tick */
	double GetMajorTickDrawSize() const;

	bool IsDragging() const;
	const TOptional<FToolableTimelineScrubDragOperation>& GetScrubDragOperation() const;
	bool IsScrubHandleHidden() const;

	bool GetGridMetrics(const FGeometry& InGeometry, double& OutMajorGridStep, int32& OutMinorDivisions) const;

	bool IsSnapEnabled() const;

	TRange<FFrameNumber> GetScrubWholeFrameRange() const;

	FFrameTime ComputeMouseFrameTime(const FMouseInputData& InMouseInput, const bool bInCheckSnapping) const;

	void ReinitializeKeyRenderer(const TSet<TWeakViewModelPtr<FChannelModel>>& InWeakViewModels);

	void GetKeysUnderMouse(const FMouseInputData& InMouseInput
		, const FVector2D& InMousePosition, TSet<FSequencerSelectedKey>& OutKeys) const;
	void GetKeysAtPixelX(const FMouseInputData& InMouseInput
		, const float InLocalMousePixelX, TSet<FSequencerSelectedKey>& OutKeys) const;

	bool HasActiveToolKeySelection() const;

	void InvalidateKeyRendererCache();
	void InvalidateKeyRendererKeyState();

protected:
	static constexpr float FrameAreaPadding = 2.f;
	static constexpr float FrameAreaDoublePadding = FrameAreaPadding * 2.f;

	//~ Begin FSequencerTimeSliderController

	virtual FReply OnMouseMoveImpl(SWidget& OwnerWidget
		, const FGeometry& InGeometry
		, const FPointerEvent& InPointerEvent
		, const bool bInFromTimeSlider) override;

	virtual TSharedPtr<SWidget> OpenContextMenu(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;

	//~ End FSequencerTimeSliderController

	FReply TryBeginScrub(const FMouseInputData& InMouseInput);

	FReply TryBeginMarkDrag(const FMouseInputData& InMouseInput);

	/** Helper Utility Functions */
	TSharedPtr<FSequencerEditorViewModel> GetSequencerViewModel() const;
	TSharedPtr<FSequencerSelection> GetSequencerSelection() const;
	TSharedPtr<FSequenceModel> GetRootSequenceModel() const;

	FFrameTime SnapFrameTimeToDisplayFrame(const FFrameTime& InTickTime) const;

	void ResetInput();
	void EndAnyInProgressTransactionalDrag();

	/** @return True if the active tool was deactivated */
	bool TryCloseActiveToolRequest();

	TRange<FFrameNumber> GetPlayheadScrubRange() const;
	bool ShouldUseVirtualScrubCoordinates(const FMouseInputData& InMouseInput
		, const TRange<FFrameNumber>& InScrubRange) const;
	FFrameTime ResolveAndUpdateScrubTime(const FMouseInputData& InMouseInput);

	static TOptional<FFrameTime> GetFirstSelectedHoveredKeyTime(const FToolableTimeline& InTimeline);

	TSharedPtr<IToolableTimelineToolFactory> FindToolFactoryToActivate(const FMouseInputData& InMouseInput
		, const bool bInIsDoubleClick = false);

	bool ShouldBeginPendingToolActivation(const FPointerEvent& InPointerEvent) const;
	void ResetPendingToolActivation();
	void ResetControllerDragTargetForToolOwnership();

	void UpdateScrubHoveredKeys();

	bool HandleScrubReleaseKeySelection(const FMouseInputData& InMouseInput);

	FToolableTimeline& Timeline;

private:
	enum class EResolvedMouseHit : uint8
	{
		None,

		Tool,

		Key,
		Mark,
		Scrubber,

		SelectionStart,
		SelectionEnd,

		PlaybackStart,
		PlaybackEnd,
	};

	struct FResolvedDragTarget
	{
		bool operator==(const FResolvedDragTarget& InOther) const
		{
			return HitType == InOther.HitType
				&& MarkIndex == InOther.MarkIndex;
		}

		bool IsValid() const { return HitType != EResolvedMouseHit::None; }

		bool IsKey() const { return HitType == EResolvedMouseHit::Key; }

		bool IsMark() const { return HitType == EResolvedMouseHit::Mark; }

		bool IsScrubber() const { return HitType == EResolvedMouseHit::Scrubber; }

		bool IsSelectionRange() const
		{
			return HitType == EResolvedMouseHit::SelectionStart
				|| HitType == EResolvedMouseHit::SelectionEnd;
		}

		bool IsPlaybackRange() const
		{
			return HitType == EResolvedMouseHit::PlaybackStart
				|| HitType == EResolvedMouseHit::PlaybackEnd;
		}

		bool IsTool() const { return HitType == EResolvedMouseHit::Tool; }

		bool IsControllerOwned() const
		{
			if (bToolHasPriorityHit)
			{
				return false;
			}
			return HitType == EResolvedMouseHit::Mark
				|| HitType == EResolvedMouseHit::Scrubber
				|| IsSelectionRange()
				|| IsPlaybackRange();
		}

		EResolvedMouseHit HitType = EResolvedMouseHit::None;

		int32 MarkIndex = INDEX_NONE;

		TSet<FSequencerSelectedKey> HoveredKeys;

		bool bToolHasPriorityHit = false;
	};

	FResolvedDragTarget ResolveDragTarget(const FMouseInputData& InMouseInput
		, const bool bInFromTimeSlider, const bool bUseMouseDownPosition) const;

	TSharedPtr<IToolableTimelineToolFactory> PendingToolFactory;
	TSharedPtr<FToolableTimelineBaseTool> PendingTool;

	TSharedPtr<FToolableTimelineBaseTool> ActiveTool;

	/** Hit targets for hovering and drag. */
	TOptional<FResolvedDragTarget> HoverTarget;
	TOptional<FResolvedDragTarget> ActiveDragTarget;
	TOptional<FFrameTime> LastCommittedScrubTime;

	TOptional<FToolableTimelineScrubDragOperation> ScrubDragOperation;
	TUniquePtr<FToolableTimelineKeyAndMarkDragOperation<FChannelKeyCache>> KeyAndMarkDragOperation;

	bool bHideScrubHandle = false;

	FKeyRenderer KeyRenderer;
	mutable TOptional<FToolableTimelineKeyViewCacheState> KeyRendererCache;
	mutable EViewDependentCacheFlags KeyRendererInvalidationFlags = EViewDependentCacheFlags::None;
};

} // namespace UE::Sequencer::ToolableTimeline
