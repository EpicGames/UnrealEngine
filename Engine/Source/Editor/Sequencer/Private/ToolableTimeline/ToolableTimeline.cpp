// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimeline.h"
#include "Algo/Unique.h"
#include "CurveEditor.h"
#include "Components/SceneComponent.h"
#include "CurveEditorCommands.h"
#include "Filters/SequencerFilterBar.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Docking/TabManager.h"
#include "IKeyArea.h"
#include "GameFramework/Actor.h"
#include "Misc/KeyHelperUtils.h"
#include "MovieSceneToolHelpers.h"
#include "MVVM/Extensions/IPinnableExtension.h"
#include "MVVM/Extensions/ITopLevelChannelHolderExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "Sequencer.h"
#include "SequencerCommands.h"
#include "SequencerSelectedKey.h"
#include "SequencerSettings.h"
#include "ToolableTimeline/Menus/ToolableTimelineMenu.h"
#include "ToolableTimeline/ToolableTimelineClipboard.h"
#include "ToolableTimeline/ToolableTimelineCommands.h"
#include "ToolableTimeline/ToolableTimelineUtils.h"
#include "ToolableTimeline/ToolableTimeSliderController.h"
#include "ToolableTimeline/Tools/ToolableTimelineBaseTool.h"
#include "ToolableTimeline/Widgets/SToolableTimeline.h"

#define LOCTEXT_NAMESPACE "ToolableTimeline"

namespace UE::Sequencer::ToolableTimeline
{

static const TSet<TWeakViewModelPtr<FChannelModel>>& GetEmptyChannelModelSet()
{
	static const TSet<TWeakViewModelPtr<FChannelModel>> EmptyChannelModels;
	return EmptyChannelModels;
}

static void AddChannelModelToKeyIfKeyable(const FToolableTimeline& InTimeline
	, const TViewModelPtr<FChannelModel>& InChannelModel
	, TSet<TViewModelPtr<FChannelModel>>& InOutVisitedChannelModels
	, TSet<TWeakViewModelPtr<FChannelModel>>& OutChannelModels)
{
	if (InChannelModel.IsValid()
		&& InChannelModel->GetKeyArea().IsValid()
		&& InTimeline.PassesChannelFilters(*InChannelModel)
		&& !InOutVisitedChannelModels.Contains(InChannelModel))
	{
		InOutVisitedChannelModels.Add(InChannelModel);
		OutChannelModels.Add(InChannelModel);
	}
}

static TMap<const FToolableTimeline*, FDelegateHandle>& GetChannelChangedDelegateHandles()
{
	static TMap<const FToolableTimeline*, FDelegateHandle> DelegateHandles;
	return DelegateHandles;
}

TSet<TWeakViewModelPtr<FChannelModel>> GetChannelModelsToKey(const FToolableTimeline& InTimeline)
{
	using namespace ToolableTimelineClipboard;

	const TSet<FSequencerSelectedKey>& SelectedKeys = InTimeline.GetKeySelection().GetSelectedKeys();
	if (SelectedKeys.IsEmpty())
	{
		return InTimeline.GetChannelModels();
	}

	TSet<TWeakViewModelPtr<FChannelModel>> ExplicitlySelectedChannels;
	TSet<FGuid> SelectedObjectBindingGuids;

	for (const FSequencerSelectedKey& SelectedKey : SelectedKeys)
	{
		const TSharedPtr<FChannelModel> ChannelModel = SelectedKey.WeakChannel.Pin();
		if (!ChannelModel.IsValid())
		{
			continue;
		}

		ExplicitlySelectedChannels.Add(ChannelModel);

		const TOptional<FGuid> ObjectBindingGuid = GetObjectBindingGuid(*ChannelModel);
		if (ObjectBindingGuid.IsSet())
		{
			SelectedObjectBindingGuids.Add(ObjectBindingGuid.GetValue());
		}
	}

	if (SelectedObjectBindingGuids.IsEmpty())
	{
		return ExplicitlySelectedChannels;
	}

	TSet<TWeakViewModelPtr<FChannelModel>> OutChannelModels;
	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : InTimeline.GetChannelModels())
	{
		const TViewModelPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
		if (!ChannelModel.IsValid())
		{
			continue;
		}

		const TOptional<FGuid> ObjectBindingGuid = GetObjectBindingGuid(*ChannelModel);
		if (ObjectBindingGuid.IsSet() && SelectedObjectBindingGuids.Contains(ObjectBindingGuid.GetValue()))
		{
			OutChannelModels.Add(WeakChannelModel);
		}
	}

	for (const TWeakViewModelPtr<FChannelModel>& WeakSelectedChannel : ExplicitlySelectedChannels)
	{
		OutChannelModels.Add(WeakSelectedChannel);
	}

	return OutChannelModels;
}

void GatherChannelModelsToKeyFromViewModel(const FToolableTimeline& InTimeline
	, const TViewModelPtr<FViewModel>& InViewModel
	, TSet<TViewModelPtr<FChannelModel>>& InOutVisitedChannelModels
	, TSet<TWeakViewModelPtr<FChannelModel>>& OutChannelModels)
{
	if (!InViewModel.IsValid())
	{
		return;
	}

	if (const FChannelModel* const ChannelModel = InViewModel->CastThis<FChannelModel>())
	{
		AddChannelModelToKeyIfKeyable(InTimeline, InViewModel->CastThisShared<FChannelModel>(), InOutVisitedChannelModels, OutChannelModels);
	}

	if (const FChannelGroupModel* const ChannelGroup = InViewModel->CastThis<FChannelGroupModel>())
	{
		for (const TWeakViewModelPtr<FChannelModel>& WeakChannel : ChannelGroup->GetChannels())
		{
			AddChannelModelToKeyIfKeyable(InTimeline, WeakChannel.Pin(), InOutVisitedChannelModels, OutChannelModels);
		}
	}

	// Some rows expose their keyable channels through top-level channel holders instead of
	// regular outliner children, such as component property channels.
	if (ITopLevelChannelHolderExtension* const TopLevelChannelHolder = InViewModel->CastThis<ITopLevelChannelHolderExtension>())
	{
		for (const TViewModelPtr<FChannelGroupModel>& TopLevelChannel : TopLevelChannelHolder->GetTopLevelChannels().IterateSubList<FChannelGroupModel>())
		{
			for (const TWeakViewModelPtr<FChannelModel>& WeakChannel : TopLevelChannel->GetChannels())
			{
				AddChannelModelToKeyIfKeyable(InTimeline, WeakChannel.Pin(), InOutVisitedChannelModels, OutChannelModels);
			}
		}
	}

	for (FViewModelListIterator ChildIt = InViewModel->GetChildren(EViewModelListType::Outliner); ChildIt; ++ChildIt)
	{
		if (const TViewModelPtr<IOutlinerExtension> ChildOutlinerItem = ChildIt->CastThisShared<IOutlinerExtension>())
		{
			GatherChannelModelsToKeyFromViewModel(InTimeline, ChildOutlinerItem, InOutVisitedChannelModels, OutChannelModels);
		}
	}
}

TSet<TWeakViewModelPtr<FChannelModel>> GetSelectedChannelModelsToKey(const FToolableTimeline& InTimeline)
{
	TSet<TWeakViewModelPtr<FChannelModel>> OutChannelModels;
	TSet<TViewModelPtr<FChannelModel>> VisitedChannelModels;

	const TSharedPtr<FSequencer> Sequencer = InTimeline.GetSequencer();
	const TSharedPtr<FSequencerEditorViewModel> ViewModel = Sequencer.IsValid() ? Sequencer->GetViewModel() : nullptr;
	const TSharedPtr<FSequencerSelection> Selection = ViewModel.IsValid() ? ViewModel->GetSelection() : nullptr;
	if (!Selection.IsValid())
	{
		return OutChannelModels;
	}

	for (const TViewModelPtr<IOutlinerExtension>& SelectedOutlinerItem : Selection->Outliner)
	{
		GatherChannelModelsToKeyFromViewModel(InTimeline, SelectedOutlinerItem, VisitedChannelModels, OutChannelModels);
	}

	return OutChannelModels;
}

void GatherPinnedChannelModelsToKey(const FToolableTimeline& InTimeline
	, const TViewModelPtr<FViewModel>& InViewModel
	, TSet<TViewModelPtr<FChannelModel>>& InOutVisitedChannelModels
	, TSet<TWeakViewModelPtr<FChannelModel>>& OutChannelModels)
{
	if (!InViewModel.IsValid())
	{
		return;
	}

	if (const IPinnableExtension* const PinnableExtension = InViewModel->CastThis<IPinnableExtension>())
	{
		if (PinnableExtension->IsPinned())
		{
			GatherChannelModelsToKeyFromViewModel(InTimeline, InViewModel, InOutVisitedChannelModels, OutChannelModels);
			return;
		}
	}

	for (FViewModelListIterator ChildIt = InViewModel->GetChildren(EViewModelListType::Outliner); ChildIt; ++ChildIt)
	{
		GatherPinnedChannelModelsToKey(InTimeline, *ChildIt, InOutVisitedChannelModels, OutChannelModels);
	}
}

TSet<TWeakViewModelPtr<FChannelModel>> GetDisplayModeChannelModelsToKey(const FToolableTimeline& InTimeline
	, const bool bInIncludeSelectedChannels)
{
	if (InTimeline.GetKeySelection().HasAnySelectedKeys())
	{
		return GetChannelModelsToKey(InTimeline);
	}

	switch (InTimeline.GetKeyDisplay())
	{
	case ETimelineKeyDisplay::All:
		return InTimeline.GetChannelModels();

	case ETimelineKeyDisplay::Selected:
		return bInIncludeSelectedChannels
			? GetSelectedChannelModelsToKey(InTimeline) : TSet<TWeakViewModelPtr<FChannelModel>>();

	case ETimelineKeyDisplay::SelectedAndPinned:
	{
		TSet<TWeakViewModelPtr<FChannelModel>> OutChannelModels = bInIncludeSelectedChannels
			? GetSelectedChannelModelsToKey(InTimeline) : TSet<TWeakViewModelPtr<FChannelModel>>();
		// Selected and pinned sources can reach the same channel through different view-model paths.
		// Seed the visited set with selected channels before gathering pinned channels to avoid duplicates.
		TSet<TViewModelPtr<FChannelModel>> VisitedChannelModels;
		for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : OutChannelModels)
		{
			const TViewModelPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
			if (ChannelModel.IsValid())
			{
				VisitedChannelModels.Add(ChannelModel);
			}
		}

		const TSharedPtr<FSequencer> Sequencer = InTimeline.GetSequencer();
		const TSharedPtr<FSequencerEditorViewModel> RootViewModel = Sequencer.IsValid() ? Sequencer->GetViewModel() : nullptr;
		if (RootViewModel.IsValid())
		{
			GatherPinnedChannelModelsToKey(InTimeline, RootViewModel->GetRootSequenceModel(), VisitedChannelModels, OutChannelModels);
		}

		return OutChannelModels;
	}

	default:
		return TSet<TWeakViewModelPtr<FChannelModel>>();
	}
}

bool HasSelectedActorsForAddKey()
{
	if (!GEditor)
	{
		return false;
	}

	USelection* const ActorSelection = GEditor->GetSelectedActors();
	if (ActorSelection && ActorSelection->Num() > 0)
	{
		return true;
	}

	for (FSelectedEditableComponentIterator It(GEditor->GetSelectedEditableComponentIterator()); It; ++It)
	{
		if (Cast<USceneComponent>(*It))
		{
			return true;
		}
	}
	return false;
}

bool IsBoundObjectMatchForAddKey(UObject& SelectedObject, UObject& BoundObject)
{
	if (&SelectedObject == &BoundObject)
	{
		return true;
	}

	if (AActor* const SelectedActor = Cast<AActor>(&SelectedObject))
	{
		if (USceneComponent* const BoundSceneComponent = Cast<USceneComponent>(&BoundObject))
		{
			return BoundSceneComponent->GetOwner() == SelectedActor;
		}
	}

	if (USceneComponent* const SelectedSceneComponent = Cast<USceneComponent>(&SelectedObject))
	{
		if (AActor* const BoundActor = Cast<AActor>(&BoundObject))
		{
			return SelectedSceneComponent->GetOwner() == BoundActor;
		}
	}

	return false;
}

bool IsSelectedObjectUnboundForAddKey(const FToolableTimeline& InTimeline, UObject& InObject, const TArray<FGuid>& InSelectedSequencerObjectGuids)
{
	const TSharedPtr<FSequencer> Sequencer = InTimeline.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	// Fast path: if Sequencer has no selected bound object guids, treat this as unbound.
	// This avoids the expensive global FindCachedObjectId scan across all bindings.
	if (InSelectedSequencerObjectGuids.IsEmpty())
	{
		return true;
	}

	for (const FGuid& SelectedSequencerObjectGuid : InSelectedSequencerObjectGuids)
	{
		for (const TWeakObjectPtr<>& WeakBoundObject : Sequencer->FindObjectsInCurrentSequence(SelectedSequencerObjectGuid))
		{
			if (UObject* const BoundObject = WeakBoundObject.Get())
			{
				if (IsBoundObjectMatchForAddKey(InObject, *BoundObject))
				{
					return false;
				}
			}
		}
	}

	return true;
}

bool HasSelectedUnboundObjectsForAddKey(const FToolableTimeline& InTimeline)
{
	if (!GEditor)
	{
		return false;
	}

	TArray<FGuid> SelectedSequencerObjectGuids;
	if (const TSharedPtr<FSequencer> Sequencer = InTimeline.GetSequencer())
	{
		Sequencer->GetSelectedObjects(SelectedSequencerObjectGuids);
	}

	// Determine whether the editor-selected object is already represented by the current
	// sequencer selection without doing a broad object-id lookup across every binding.
	USelection* const ActorSelection = GEditor->GetSelectedActors();
	if (ActorSelection && ActorSelection->Num() > 0)
	{
		for (FSelectionIterator It(*ActorSelection); It; ++It)
		{
			if (AActor* const Actor = Cast<AActor>(*It))
			{
				if (IsSelectedObjectUnboundForAddKey(InTimeline, *Actor, SelectedSequencerObjectGuids))
				{
					return true;
				}
			}
		}
		return false;
	}

	for (FSelectedEditableComponentIterator It(GEditor->GetSelectedEditableComponentIterator()); It; ++It)
	{
		if (USceneComponent* const SceneComponent = Cast<USceneComponent>(*It))
		{
			if (IsSelectedObjectUnboundForAddKey(InTimeline, *SceneComponent, SelectedSequencerObjectGuids))
			{
				return true;
			}
		}
	}

	return false;
}

bool ShouldKeySelectedActorsForDisplayMode(const FToolableTimeline& InTimeline)
{
	return !InTimeline.GetKeySelection().HasAnySelectedKeys()
		&& InTimeline.GetKeyDisplay() != ETimelineKeyDisplay::All
		&& HasSelectedActorsForAddKey();
}

bool IsValidFilteredInNode(const TViewModelPtr<IOutlinerExtension>& InNode)
{
	return InNode.IsValid() && !InNode->IsFilteredOut();
}

const FName FToolableTimeline::SequencerTabId = TEXT("Sequencer");

FToolableTimeline::FToolableTimeline(const FPrivateToken& InToken)
	: WeakSequencer(InToken.WeakSequencer)
	, TimeRangeArgs(InToken.TimeRangeArgs)
	, CommandList(MakeShared<FUICommandList>())
	, TimelineSettings(GetMutableDefault<UToolableTimelineSettings>())
	, TimeSliderController(MakeShared<FToolableTimeSliderController>(InToken.TimeSliderArgs, *this))
	, TrackAreaViewModel(MakeShared<FTrackAreaViewModel>())
	, KeySelection(*this)
{
	FToolableTimelineCommands::Register();
	FToolableTimelineMenu::RegisterMenu();
}

FToolableTimeline::~FToolableTimeline()
{
	Shutdown();
}

void FToolableTimeline::Initialize()
{
	BindCommands();

	if (const TSharedPtr<FSequencer> Sequencer = GetSequencer())
	{
		MovieSceneDataChangedDelegateHandle = Sequencer->OnMovieSceneDataChanged().AddSP(this, &FToolableTimeline::HandleMovieSceneDataChanged);

		// Curve-editor key edits (e.g. middle-mouse insert) only broadcast OnChannelChangedWithTime,
		// not OnMovieSceneDataChanged, so the Simple View renderer cache must also invalidate from here.
		const FDelegateHandle ChannelChangedHandle = Sequencer->OnChannelChangedWithTime().AddLambda(
			[this](const TArray<FFrameNumber>&, const FMovieSceneChannelMetaData*, UMovieSceneSection*)
			{
				TimeSliderController->InvalidateKeyRendererCache();
			});
		GetChannelChangedDelegateHandles().Add(this, ChannelChangedHandle);

		FilterBarChangedDelegateHandle = Sequencer->GetFilterBar()->OnFiltersChanged().AddLambda([this]
			(const ESequencerFilterChange InChangeType, const TSharedRef<FSequencerTrackFilter>& InFilter)
			{
				RecacheChannelModels();
			});

		const TSharedRef<ISequencerTrackFilters> FilterInterface = Sequencer->GetFilterInterface();
		FilterRequestUpdateDelegateHandle = FilterInterface->OnRequestUpdate().AddSP(this, &FToolableTimeline::RecacheChannelModels);
		FilterTextChangedDelegateHandle = FilterInterface->OnTextFilterTextChanged().AddSP(this, &FToolableTimeline::RecacheChannelModels);
		FilterMuteChangedDelegateHandle = FilterInterface->OnMuteFiltersChanged().AddLambda([this](const bool bInMuted)
			{
				RecacheChannelModels();
			});

		if (const TSharedPtr<FSequencerEditorViewModel> ViewModel = Sequencer->GetViewModel())
		{
			if (const TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection())
			{
				OutlinerSelectionChangedDelegateHandle = Selection->Outliner.OnChanged.AddSP(this, &FToolableTimeline::HandleOutlinerSelectionChanged);
			}
		}
	}

	PinStateChangedDelegateHandle = FOutlinerItemModelMixin::OnPinStateChanged().AddSP(this, &FToolableTimeline::HandlePinStateChanged);

	TimelineSettingsChangedDelegateHandle = TimelineSettings->OnSettingChanged().AddSP(this, &FToolableTimeline::HandleTimelineSettingsChanged);

	KeySelection.Initialize();
}

void FToolableTimeline::Shutdown()
{
	KeySelection.Shutdown();

	if (TimelineSettingsChangedDelegateHandle.IsValid())
	{
		TimelineSettings->OnSettingChanged().Remove(TimelineSettingsChangedDelegateHandle);
		TimelineSettingsChangedDelegateHandle.Reset();
	}

	if (PinStateChangedDelegateHandle.IsValid())
	{
		FOutlinerItemModelMixin::OnPinStateChanged().Remove(PinStateChangedDelegateHandle);
		PinStateChangedDelegateHandle.Reset();
	}

	TOptional<FDelegateHandle> ChannelChangedHandle;
	if (FDelegateHandle* const StoredChannelChangedHandle = GetChannelChangedDelegateHandles().Find(this))
	{
		ChannelChangedHandle = *StoredChannelChangedHandle;
		GetChannelChangedDelegateHandles().Remove(this);
	}

	if (const TSharedPtr<FSequencer> Sequencer = GetSequencer())
	{
		if (MovieSceneDataChangedDelegateHandle.IsValid())
		{
			Sequencer->OnMovieSceneDataChanged().Remove(MovieSceneDataChangedDelegateHandle);
			MovieSceneDataChangedDelegateHandle.Reset();
		}

		if (ChannelChangedHandle.IsSet())
		{
			Sequencer->OnChannelChangedWithTime().Remove(ChannelChangedHandle.GetValue());
		}

		if (FilterBarChangedDelegateHandle.IsValid())
		{
			Sequencer->GetFilterBar()->OnFiltersChanged().Remove(FilterBarChangedDelegateHandle);
			FilterBarChangedDelegateHandle.Reset();
		}

		const TSharedRef<ISequencerTrackFilters> FilterInterface = Sequencer->GetFilterInterface();
		if (FilterRequestUpdateDelegateHandle.IsValid())
		{
			FilterInterface->OnRequestUpdate().Remove(FilterRequestUpdateDelegateHandle);
			FilterRequestUpdateDelegateHandle.Reset();
		}

		if (FilterTextChangedDelegateHandle.IsValid())
		{
			FilterInterface->OnTextFilterTextChanged().Remove(FilterTextChangedDelegateHandle);
			FilterTextChangedDelegateHandle.Reset();
		}

		if (FilterMuteChangedDelegateHandle.IsValid())
		{
			FilterInterface->OnMuteFiltersChanged().Remove(FilterMuteChangedDelegateHandle);
			FilterMuteChangedDelegateHandle.Reset();
		}

		if (const TSharedPtr<FSequencerEditorViewModel> ViewModel = Sequencer->GetViewModel())
		{
			if (const TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection())
			{
				if (OutlinerSelectionChangedDelegateHandle.IsValid())
				{
					Selection->Outliner.OnChanged.Remove(OutlinerSelectionChangedDelegateHandle);
					OutlinerSelectionChangedDelegateHandle.Reset();
				}
			}
		}
	}
}

void FToolableTimeline::BindCommands()
{
	const FToolableTimelineCommands& TimelineEditorCommands = FToolableTimelineCommands::Get();
	const FSequencerCommands& SequencerCommands = FSequencerCommands::Get();
	const FCurveEditorCommands& CurveEditorCommands = FCurveEditorCommands::Get();
	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	CommandList->MapAction(CurveEditorCommands.FlattenTangents
		, FExecuteAction::CreateSP(this, &FToolableTimeline::FlattenTangents)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanFlattenTangents));

	CommandList->MapAction(CurveEditorCommands.StraightenTangents
		, FExecuteAction::CreateSP(this, &FToolableTimeline::StraightenTangents)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanStraightenTangents));

	CommandList->MapAction(CurveEditorCommands.SmartSnapKeys
		, FExecuteAction::CreateSP(this, &FToolableTimeline::SmartSnapKeys)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanSmartSnapKeys));

	CommandList->MapAction(GenericCommands.Cut
		, FExecuteAction::CreateSP(this, &FToolableTimeline::CutSelectedKeys)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanCutSelectedKeys));

	CommandList->MapAction(GenericCommands.Copy
		, FExecuteAction::CreateSP(this, &FToolableTimeline::CopySelectedKeys)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanCopySelectedKeys));

	CommandList->MapAction(GenericCommands.Paste
		, FExecuteAction::CreateSP(this, &FToolableTimeline::PasteSelectedKeys)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanPasteSelectedKeys));

	CommandList->MapAction(GenericCommands.Delete
		, FExecuteAction::CreateSP(this, &FToolableTimeline::DeleteSelectedKeys)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanDeleteSelectedKeys));

	CommandList->MapAction(GenericCommands.SelectAll
		, FExecuteAction::CreateSP(this, &FToolableTimeline::SelectAllKeys));

	CommandList->MapAction(SequencerCommands.AddTransformKey
		, FExecuteAction::CreateSP(this, &FToolableTimeline::SetTransformKey, EMovieSceneTransformChannel::All)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanSetTransformKey, EMovieSceneTransformChannel::All));
	CommandList->MapAction(SequencerCommands.AddTranslationKey
		, FExecuteAction::CreateSP(this, &FToolableTimeline::SetTransformKey, EMovieSceneTransformChannel::Translation)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanSetTransformKey, EMovieSceneTransformChannel::Translation));
	CommandList->MapAction(SequencerCommands.AddRotationKey
		, FExecuteAction::CreateSP(this, &FToolableTimeline::SetTransformKey, EMovieSceneTransformChannel::Rotation)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanSetTransformKey, EMovieSceneTransformChannel::Rotation));
	CommandList->MapAction(SequencerCommands.AddScaleKey
		, FExecuteAction::CreateSP(this, &FToolableTimeline::SetTransformKey, EMovieSceneTransformChannel::Scale)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanSetTransformKey, EMovieSceneTransformChannel::Scale));
	CommandList->MapAction(SequencerCommands.DeleteTransformKey
		, FExecuteAction::CreateSP(this, &FToolableTimeline::DeleteTransformKey, EMovieSceneTransformChannel::All)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanDeleteTransformKey, EMovieSceneTransformChannel::All));
	CommandList->MapAction(SequencerCommands.DeleteTranslationKey
		, FExecuteAction::CreateSP(this, &FToolableTimeline::DeleteTransformKey, EMovieSceneTransformChannel::Translation)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanDeleteTransformKey, EMovieSceneTransformChannel::Translation));
	CommandList->MapAction(SequencerCommands.DeleteRotationKey
		, FExecuteAction::CreateSP(this, &FToolableTimeline::DeleteTransformKey, EMovieSceneTransformChannel::Rotation)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanDeleteTransformKey, EMovieSceneTransformChannel::Rotation));
	CommandList->MapAction(SequencerCommands.DeleteScaleKey
		, FExecuteAction::CreateSP(this, &FToolableTimeline::DeleteTransformKey, EMovieSceneTransformChannel::Scale)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanDeleteTransformKey, EMovieSceneTransformChannel::Scale));

	CommandList->MapAction(SequencerCommands.SetKey
		, FExecuteAction::CreateSP(this, &FToolableTimeline::SetKey)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanSetKey));

	CommandList->MapAction(TimelineEditorCommands.SetKeyDisplay_SelectedAndPinned
		, FExecuteAction::CreateSP(this, &FToolableTimeline::SetKeyDisplay, ETimelineKeyDisplay::SelectedAndPinned)
		, FCanExecuteAction()
		, FIsActionChecked::CreateSP(this, &FToolableTimeline::IsKeyDisplay, ETimelineKeyDisplay::SelectedAndPinned));

	CommandList->MapAction(TimelineEditorCommands.SetKeyDisplay_Selected
		, FExecuteAction::CreateSP(this, &FToolableTimeline::SetKeyDisplay, ETimelineKeyDisplay::Selected)
		, FCanExecuteAction()
		, FIsActionChecked::CreateSP(this, &FToolableTimeline::IsKeyDisplay, ETimelineKeyDisplay::Selected));

	CommandList->MapAction(TimelineEditorCommands.SetKeyDisplay_All
		, FExecuteAction::CreateSP(this, &FToolableTimeline::SetKeyDisplay, ETimelineKeyDisplay::All)
		, FCanExecuteAction()
		, FIsActionChecked::CreateSP(this, &FToolableTimeline::IsKeyDisplay, ETimelineKeyDisplay::All));

	CommandList->MapAction(TimelineEditorCommands.ZoomToFit
		, FExecuteAction::CreateSP(this, &FToolableTimeline::ZoomToFit));

	CommandList->MapAction(TimelineEditorCommands.GoToFrame
		, FExecuteAction::CreateSP(this, &FToolableTimeline::ScrubToFrame));

	CommandList->MapAction(TimelineEditorCommands.DeactivateActiveTool
		, FExecuteAction::CreateSP(this, &FToolableTimeline::DeactivateActiveTool));

	CommandList->MapAction(TimelineEditorCommands.SetSelectionRangeFromToolRange
		, FExecuteAction::CreateSP(this, &FToolableTimeline::SetSelectionRangeFromToolRange)
		, FCanExecuteAction::CreateSP(this, &FToolableTimeline::CanSetSelectionRangeFromToolRange));
}

void FToolableTimeline::UnbindCommands()
{
	const FToolableTimelineCommands& TimelineEditorCommands = FToolableTimelineCommands::Get();
	const FSequencerCommands& SequencerCommands = FSequencerCommands::Get();
	const FCurveEditorCommands& CurveEditorCommands = FCurveEditorCommands::Get();
	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	CommandList->UnmapAction(CurveEditorCommands.FlattenTangents);
	CommandList->UnmapAction(CurveEditorCommands.StraightenTangents);
	CommandList->UnmapAction(CurveEditorCommands.SmartSnapKeys);

	CommandList->UnmapAction(GenericCommands.Cut);
	CommandList->UnmapAction(GenericCommands.Copy);
	CommandList->UnmapAction(GenericCommands.Paste);
	CommandList->UnmapAction(GenericCommands.Delete);
	CommandList->UnmapAction(GenericCommands.SelectAll);

	CommandList->UnmapAction(SequencerCommands.AddTransformKey);
	CommandList->UnmapAction(SequencerCommands.AddTranslationKey);
	CommandList->UnmapAction(SequencerCommands.AddRotationKey);
	CommandList->UnmapAction(SequencerCommands.AddScaleKey);
	CommandList->UnmapAction(SequencerCommands.DeleteTransformKey);
	CommandList->UnmapAction(SequencerCommands.DeleteTranslationKey);
	CommandList->UnmapAction(SequencerCommands.DeleteRotationKey);
	CommandList->UnmapAction(SequencerCommands.DeleteScaleKey);
	CommandList->UnmapAction(SequencerCommands.SetKey);

	CommandList->UnmapAction(TimelineEditorCommands.SetKeyDisplay_SelectedAndPinned);
	CommandList->UnmapAction(TimelineEditorCommands.SetKeyDisplay_Selected);
	CommandList->UnmapAction(TimelineEditorCommands.SetKeyDisplay_All);

	CommandList->UnmapAction(TimelineEditorCommands.ZoomToFit);
	CommandList->UnmapAction(TimelineEditorCommands.GoToFrame);

	CommandList->UnmapAction(TimelineEditorCommands.DeactivateActiveTool);
	CommandList->UnmapAction(TimelineEditorCommands.SetSelectionRangeFromToolRange);
}

void FToolableTimeline::RecacheChannelModels()
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	const bool bRestoreSimpleViewSelection = Sequencer.IsValid()
		&& Sequencer->IsSimpleView()
		&& KeySelection.HasAnySelectedKeys();

	KeySelection.ClearSelectedAndHoveredKeys(/*bInSync=*/!bRestoreSimpleViewSelection);

	const TSharedPtr<FToolableTimelineBaseTool> ActiveTool = TimeSliderController->GetActiveTool();
	const bool bKeepActiveTool = ActiveTool.IsValid()
		&& !ActiveTool->IsCloseRequested()
		&& ActiveTool->CanPersistThroughChannelRecache()
		&& !ActiveTool->IsDragging();

	if (ActiveTool.IsValid() && !bKeepActiveTool)
	{
		TimeSliderController->DeactivateTool();
	}

	ChannelCache = MakeShared<FToolableTimelineChannelCache>(*this);
	++ChannelModelsSerialNumber;

	const TSet<TWeakViewModelPtr<FChannelModel>>& WeakChannelModels = ChannelCache->GetChannelModels();

	TimeSliderController->ReinitializeKeyRenderer(WeakChannelModels);

	if (bRestoreSimpleViewSelection)
	{
		KeySelection.RefreshSelectedKeysFromCurveEditor();
	}

	ChannelModelsChangedEvent.Broadcast(WeakChannelModels);

	if (bKeepActiveTool && ActiveTool == TimeSliderController->GetActiveTool())
	{
		ActiveTool->OnChannelModelsRecached();
	}
}

TArray<FRegisteredChannelFilter>& FToolableTimeline::GetChannelFilters()
{
	return ChannelFilters;
}

const TArray<FRegisteredChannelFilter>& FToolableTimeline::GetChannelFilters() const
{
	return ChannelFilters;
}

FDelegateHandle FToolableTimeline::AddChannelFilter(FChannelFilterFunction& InFilterFunction)
{
	FRegisteredChannelFilter& Entry = GetChannelFilters().AddDefaulted_GetRef();
	Entry.Handle = FDelegateHandle(FDelegateHandle::GenerateNewHandle);
	Entry.FilterFunction = MoveTemp(InFilterFunction);

	RequestRecacheChannels();

	return Entry.Handle;
}

void FToolableTimeline::RemoveChannelFilter(const FDelegateHandle& InDelegateHandle)
{
	GetChannelFilters().RemoveAll([&InDelegateHandle](const FRegisteredChannelFilter& InFilter)
		{
			return InFilter.Handle == InDelegateHandle;
		});

	RequestRecacheChannels();
}

bool FToolableTimeline::PassesChannelFilters(const FChannelModel& InChannelModel) const
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	for (const FRegisteredChannelFilter& ChannelFilter : GetChannelFilters())
	{
		if (ChannelFilter.FilterFunction.IsSet() && !ChannelFilter.FilterFunction(*Sequencer, InChannelModel))
		{
			return false;
		}
	}

	return true;
}

TSharedPtr<FSequencer> FToolableTimeline::GetSequencer() const
{
	return WeakSequencer.Pin();
}

USequencerSettings* FToolableTimeline::GetSequencerSettings() const
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	return Sequencer.IsValid() ? Sequencer->GetSequencerSettings() : nullptr;
}

TSharedPtr<FTabManager> FToolableTimeline::GetTabManager() const
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return nullptr;
	}

	const TSharedPtr<IToolkitHost> ToolkitHost = Sequencer->GetToolkitHost();
	return ToolkitHost.IsValid() ? ToolkitHost->GetTabManager() : nullptr;
}

TSharedRef<SWidget> FToolableTimeline::GenerateWidget()
{
	if (!TimelineWidget.IsValid())
	{
		TimelineWidget = SNew(SToolableTimeline, SharedThis(this));
	}
	return TimelineWidget.ToSharedRef();
}

TSharedPtr<SToolableTimeline> FToolableTimeline::GetWidget() const
{
	return TimelineWidget;
}

void FToolableTimeline::HandleMovieSceneDataChanged(const EMovieSceneDataChangeType InChangeType)
{
	if (ShouldRecacheChannelsForDataChange(InChangeType))
	{
		// Update to include cases where a new object that is not part of sequencer has just been keyed
		RequestRecacheChannels();
	}
	else
	{
		TimeSliderController->InvalidateKeyRendererCache();
	}
}

void FToolableTimeline::HandlePinStateChanged(const TViewModelPtr<IPinnableExtension>& InPinnableExtension, const bool bInPinned)
{
	FPinnableExtensionShim::UpdateCachedPinnedState(InPinnableExtension.AsModel()->AsShared());
	RequestRecacheChannels();
}

void FToolableTimeline::HandleTimelineSettingsChanged(UObject* const InObject, FPropertyChangedEvent& InEvent)
{
	if (TimelineWidget.IsValid())
	{
		const FName MemberPropertyName = InEvent.MemberProperty ? InEvent.MemberProperty->GetFName() : NAME_None;
		const FName PropertyName = InEvent.Property ? InEvent.Property->GetFName() : NAME_None;

		if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UToolableTimelineSettings, Settings))
		{
			if (PropertyName == GET_MEMBER_NAME_CHECKED(FToolableTimelineInstanceSettings, KeyDisplay))
			{
				TimelineWidget->RequestRecacheChannels();
			}
		}
	}

	if (TimelineSettings.IsValid())
	{
		TimelineSettings->SaveConfig();
	}
}

void FToolableTimeline::HandleOutlinerSelectionChanged()
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	const TSharedPtr<FToolableTimelineBaseTool> ActiveTool = TimeSliderController->GetActiveTool();

	const bool bShouldDeactivateToolOnSelectionChange = !Sequencer.IsValid()
		|| !Sequencer->IsSimpleView()
		|| TimelineSettings->Settings.bDeactivateToolOnOutlinerSelectionChange;

	if (bShouldDeactivateToolOnSelectionChange
		&& ActiveTool.IsValid()
		&& ActiveTool->ShouldDeactivateOnOutlinerSelectionChanged())
	{
		TimeSliderController->DeactivateTool();
	}

	// "All" key display already gathers the full channel set, so selection changes should not rebuild it
	if (GetKeyDisplay() != ETimelineKeyDisplay::All)
	{
		RequestRecacheChannels();
	}
}

ETimelineKeyDisplay FToolableTimeline::GetKeyDisplay() const
{
	return TimelineSettings.IsValid() ? TimelineSettings->Settings.KeyDisplay : ETimelineKeyDisplay::SelectedAndPinned;
}

bool FToolableTimeline::IsKeyDisplay(const ETimelineKeyDisplay InKeyDisplay) const
{
	return GetKeyDisplay() == InKeyDisplay;
}

void FToolableTimeline::SetKeyDisplay(const ETimelineKeyDisplay InKeyDisplay)
{
	if (TimelineSettings.IsValid() && TimelineSettings->Settings.KeyDisplay != InKeyDisplay)
	{
		TimelineSettings->Settings.KeyDisplay = InKeyDisplay;

		RequestRecacheChannels();
	}
}

TArray<FFrameNumber> FToolableTimeline::GetAllChannelKeyTimes() const
{
	if (!ChannelCache.IsValid())
	{
		return {};
	}

	TArray<FFrameNumber> OutKeyTimes;
	TArray<FFrameNumber> ScratchKeyTimes;

	// Reserve a small amount up-front to avoid early reallocations without overcommitting memory.
	OutKeyTimes.Reserve(64);
	ScratchKeyTimes.Reserve(64);

	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : ChannelCache->GetChannelModels())
	{
		const TViewModelPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
		if (!ChannelModel.IsValid())
		{
			continue;
		}

		const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea();
		if (!KeyArea.IsValid())
		{
			continue;
		}

		ScratchKeyTimes.Reset();
		KeyArea->GetKeyTimes(ScratchKeyTimes);

		const int32 NumScratchKeys = ScratchKeyTimes.Num();
		if (NumScratchKeys == 0)
		{
			continue;
		}

		// Grow once per channel (amortizes better than repeated growth during Append)
		OutKeyTimes.Reserve(OutKeyTimes.Num() + NumScratchKeys);
		OutKeyTimes.Append(ScratchKeyTimes);
	}

	if (OutKeyTimes.Num() > 1)
	{
		OutKeyTimes.Sort();
		OutKeyTimes.SetNum(Algo::Unique(OutKeyTimes));
	}

	return OutKeyTimes;
}

void FToolableTimeline::ZoomToFit()
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	TArray<FFrameNumber> KeyTimes;

	const TSet<FSequencerSelectedKey>& SelectedKeys = KeySelection.GetSelectedKeys();
	if (!SelectedKeys.IsEmpty())
	{
		const TArray<FSequencerSelectedKey> SelectedKeysArray = SelectedKeys.Array();
		KeyTimes.SetNumUninitialized(SelectedKeysArray.Num());
		GetKeyTimes(SelectedKeysArray, KeyTimes);
	}
	else
	{
		KeyTimes = GetAllChannelKeyTimes();
	}

	if (KeyTimes.IsEmpty())
	{
		return;
	}

	TRange<FFrameNumber> MinMaxRange;
	if (!Utils::GetMinMaxKeyRange(KeyTimes, MinMaxRange))
	{
		return;
	}

	const double SecondsPadding = FMath::Max<double>(.0, TimelineSettings->Settings.FrameKeysSecondsPadding);
	const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();

	double StartSeconds = TickResolution.AsSeconds(MinMaxRange.GetLowerBoundValue()) - SecondsPadding;
	double EndSeconds = TickResolution.AsSeconds(MinMaxRange.GetUpperBoundValue()) + SecondsPadding;

	// Avoid inverted ranges that can happen if all keys are at the same time
	if (EndSeconds <= StartSeconds)
	{
		const double Center = .5 * (StartSeconds + EndSeconds);
		const double HalfRange = FMath::Max(SecondsPadding, .0);
		StartSeconds = Center - HalfRange;
		EndSeconds = Center + HalfRange;
	}

	Sequencer->SetViewRange(TRange<double>(StartSeconds, EndSeconds), EViewRangeInterpolation::Immediate);
}

void FToolableTimeline::ScrubToFrame()
{
	if (TimelineWidget.IsValid())
	{
		TimelineWidget->DoScrubToFrameInput();
	}
}

void FToolableTimeline::DeactivateActiveTool()
{
	TimeSliderController->DeactivateTool();
}

void FToolableTimeline::RequestRecacheChannels()
{
	if (TimelineWidget.IsValid())
	{
		TimelineWidget->RequestRecacheChannels();
	}
}

TSharedPtr<FToolableTimelineBaseTool> FToolableTimeline::GetActiveTool() const
{
	return TimeSliderController->GetActiveTool();
}

const TSet<TWeakViewModelPtr<FChannelModel>>& FToolableTimeline::GetChannelModels() const
{
	return ChannelCache.IsValid() ? ChannelCache->GetChannelModels() : GetEmptyChannelModelSet();
}

void FToolableTimeline::SyncSelectionToSequencerAndCurveEditor()
{
	KeySelection.SyncSelectionToSequencerAndCurveEditor();
}

TSet<FSequencerSelectedKey> FToolableTimeline::GetScrubRangeKeys() const
{
	const TRange<FFrameNumber> ScrubRange = TimeSliderController->GetScrubWholeFrameRange();

	TSet<FSequencerSelectedKey> OutKeys;

	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : GetChannelModels())
	{
		const TViewModelPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
		if (!ChannelModel.IsValid())
		{
			continue;
		}

		const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea();
		if (!KeyArea.IsValid())
		{
			continue;
		}

		UMovieSceneSection* const Section = ChannelModel->GetSection();
		if (!Section)
		{
			continue;
		}

		FMovieSceneChannel* const Channel = ChannelModel->GetChannel();
		if (!Channel)
		{
			continue;
		}

		TArray<FKeyHandle> ChannelKeyHandles;
		Channel->GetKeys(ScrubRange, nullptr, &ChannelKeyHandles);

		for (const FKeyHandle KeyHandle : ChannelKeyHandles)
		{
			OutKeys.Add(FSequencerSelectedKey(*Section, ChannelModel, KeyHandle));
		}
	}

	return OutKeys;
}

void FToolableTimeline::CenterFrameOnScreen(const double InNewValue, const bool bInCenterScrubPosition)
{
	const FFrameRate TickResolution = TimeSliderController->GetTickResolution();

	const FFrameTime NewFrameTime = FFrameTime::FromDecimal(InNewValue);
	const double NewTimeSeconds = TickResolution.AsSeconds(NewFrameTime);

	const double CurrentViewStart = TimeSliderController->GetViewRange().GetLowerBoundValue();
	const double CurrentViewEnd = TimeSliderController->GetViewRange().GetUpperBoundValue();
	const double ViewDuration = CurrentViewEnd - CurrentViewStart;

	const double NewViewStart = NewTimeSeconds - ViewDuration * 0.5;
	const double NewViewEnd = NewTimeSeconds + ViewDuration * 0.5;

	const double CurrentClampStart = TimeSliderController->GetClampRange().GetLowerBoundValue();
	const double CurrentClampEnd = TimeSliderController->GetClampRange().GetUpperBoundValue();

	const double NewClampStart = FMath::Min(CurrentClampStart, NewViewStart);
	const double NewClampEnd = FMath::Max(CurrentClampEnd, NewViewEnd);

	TimeSliderController->SetClampRange(NewClampStart, NewClampEnd);
	TimeSliderController->SetViewRange(NewViewStart, NewViewEnd, EViewRangeInterpolation::Animated);

	TimeSliderController->SetScrubPosition(NewFrameTime, true);
}

void FToolableTimeline::CutSelectedKeys()
{
	using namespace ToolableTimelineClipboard;

	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<FToolableTimelineClipboard> Clipboard = BuildClipboardFromTimelineKeys(*this, *Sequencer);
	if (!Clipboard.IsValid())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("CutSelectedKeys_Transaction", "Cut Selected Keys"));

	TimelineKeyClipboard = Clipboard;

	if (Internal_DeleteSelectedKeys())
	{
		KeySelection.ClearSelectedAndHoveredKeys(/*bInSync=*/false);
	}
}

bool FToolableTimeline::CanCutSelectedKeys() const
{
	return ToolableTimelineClipboard::HasAnyKeysForClipboardOperation(*this);
}

void FToolableTimeline::CopySelectedKeys()
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	TimelineKeyClipboard = ToolableTimelineClipboard::BuildClipboardFromTimelineKeys(*this, *Sequencer);
}

bool FToolableTimeline::CanCopySelectedKeys() const
{
	return ToolableTimelineClipboard::HasAnyKeysForClipboardOperation(*this);
}

void FToolableTimeline::PasteSelectedKeys()
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid() || Sequencer->IsReadOnly())
	{
		return;
	}

	if (!TimelineKeyClipboard.IsValid() || TimelineKeyClipboard->IsEmpty())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("PasteSelectedKeys_Transaction", "Paste Keys"));

	ToolableTimelineClipboard::PasteClipboardIntoCachedChannels(*this, *Sequencer, *TimelineKeyClipboard);
}

bool FToolableTimeline::CanPasteSelectedKeys() const
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	return Sequencer.IsValid()
		&& !Sequencer->IsReadOnly()
		&& TimelineKeyClipboard.IsValid()
		&& ToolableTimelineClipboard::CanPasteClipboardIntoCachedChannels(*this, *TimelineKeyClipboard);
}

void FToolableTimeline::DeleteSelectedKeys()
{
	const FScopedTransaction Transaction(LOCTEXT("DeleteSelectedKeys_Transaction", "Delete Selected Keys"));

	if (Internal_DeleteSelectedKeys())
	{
		KeySelection.ClearSelectedAndHoveredKeys(/*bInSync=*/false);
	}
}

bool FToolableTimeline::CanDeleteSelectedKeys() const
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid() || Sequencer->IsReadOnly())
	{
		return false;
	}

	return ToolableTimelineClipboard::HasAnyKeysForClipboardOperation(*this);
}

void FToolableTimeline::SelectAllKeys()
{
	TSet<FSequencerSelectedKey> AllKeys;

	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : GetChannelModels())
	{
		const TSharedPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
		if (!ChannelModel.IsValid())
		{
			continue;
		}

		UMovieSceneSection* const Section = ChannelModel->GetSection();
		if (!Section)
		{
			continue;
		}

		const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea();
		if (!KeyArea.IsValid())
		{
			continue;
		}

		TArray<FKeyHandle> KeyHandles;
		KeyArea->GetKeyHandles(KeyHandles);

		for (const FKeyHandle KeyHandle : KeyHandles)
		{
			AllKeys.Add(FSequencerSelectedKey(*Section, ChannelModel, KeyHandle));
		}
	}

	const bool bSelectionChanged = KeySelection.SetSelectedKeys(AllKeys, /*bInSync=*/false);
	const bool bSelectionStateChanged = KeySelection.ResetHoveredKeysAndSelectionPreview();

	if (bSelectionChanged || bSelectionStateChanged)
	{
		SyncSelectionToSequencerAndCurveEditor();
	}
}

void FToolableTimeline::ExecuteSetKeyCommand()
{
	SetKey();
}

bool FToolableTimeline::CanExecuteSetKeyCommand() const
{
	return CanSetKey();
}

void FToolableTimeline::ExecuteTransformKeyCommand()
{
	SetTransformKey(EMovieSceneTransformChannel::All);
}

void FToolableTimeline::ExecuteTransformKeyCommand(const EMovieSceneTransformChannel InChannel)
{
	SetTransformKey(InChannel);
}

bool FToolableTimeline::CanExecuteTransformKeyCommand() const
{
	return CanSetTransformKey(EMovieSceneTransformChannel::All);
}

bool FToolableTimeline::CanExecuteTransformKeyCommand(const EMovieSceneTransformChannel InChannel) const
{
	return CanSetTransformKey(InChannel);
}

void FToolableTimeline::ExecuteDeleteTransformKeyCommand(const EMovieSceneTransformChannel InChannel)
{
	DeleteTransformKey(InChannel);
}

bool FToolableTimeline::CanExecuteDeleteTransformKeyCommand(const EMovieSceneTransformChannel InChannel) const
{
	return CanDeleteTransformKey(InChannel);
}

bool FToolableTimeline::ExecuteCurveEditorAction(const TSharedPtr<const FUICommandInfo>& InCommand)
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	const TSharedPtr<FCurveEditor> CurveEditor = Utils::GetSequencerCurveEditor(*Sequencer);
	if (!CurveEditor.IsValid())
	{
		return false;
	}

	const TSharedPtr<FUICommandList> CurveEditorCommandList = CurveEditor->GetCommands();
	if (!CurveEditorCommandList.IsValid())
	{
		return false;
	}

	const FUIAction* const Action = CurveEditorCommandList->GetActionForCommand(InCommand);
	if (!Action)
	{
		return false;
	}

	SyncSelectionToSequencerAndCurveEditor();

	return Action->Execute();
}

bool FToolableTimeline::HasToolRangeKeys() const
{
	const TSharedPtr<FToolableTimelineBaseTool> ActiveTool = TimeSliderController->GetActiveTool();
	return ActiveTool.IsValid() && !ActiveTool->GetToolRangeKeys().IsEmpty();
}

void FToolableTimeline::SetKey()
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid() || Sequencer->IsReadOnly())
	{
		return;
	}

	// S/Enter should key the channels currently exposed by Simple View first. This lets
	// selected camera component properties key directly instead of being preempted by the
	// selected-actor transform fallback.
	const TSet<TWeakViewModelPtr<FChannelModel>> ChannelModels = GetDisplayModeChannelModelsToKey(*this, true);
	if (KeyHelperUtils::HasKeyableChannelModels(ChannelModels))
	{
		const FScopedTransaction Transaction(LOCTEXT("SetKey_Transaction", "Set Key"));

		const FFrameNumber KeyTime = Sequencer->GetLocalTime().Time.FrameNumber;
		FAddKeyOperation::FromChannelModels(ChannelModels).Commit(KeyTime, *Sequencer);
		return;
	}

	// If Simple View has no keyable channels yet, fall back to the legacy selected-object
	// transform keying path so unbound editor selections can still be auto-added and keyed.
	const bool bCanKeySelectedActors = ShouldKeySelectedActorsForDisplayMode(*this);
	if (bCanKeySelectedActors)
	{
		Sequencer->AddTransformKeysForSelectedObjectsInternal(EMovieSceneTransformChannel::All);
	}
}

bool FToolableTimeline::CanSetKey() const
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid() || Sequencer->IsReadOnly())
	{
		return false;
	}

	if (KeyHelperUtils::HasKeyableChannelModels(GetDisplayModeChannelModelsToKey(*this, true)))
	{
		return true;
	}

	return ShouldKeySelectedActorsForDisplayMode(*this)
		&& Sequencer->CanAddTransformKeysForSelectedObjectsInternal(EMovieSceneTransformChannel::All);
}

void FToolableTimeline::SetTransformKey(const EMovieSceneTransformChannel InChannel)
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid() || Sequencer->IsReadOnly())
	{
		return;
	}

	const bool bCanKeySelectedActors = ShouldKeySelectedActorsForDisplayMode(*this);
	const bool bUseSelectedActorKeyingPath = bCanKeySelectedActors
		&& (GetKeyDisplay() == ETimelineKeyDisplay::Selected || HasSelectedUnboundObjectsForAddKey(*this));

	if (bUseSelectedActorKeyingPath)
	{
		Sequencer->AddTransformKeysForSelectedObjectsInternal(InChannel);
		return;
	}

	const TSet<TWeakViewModelPtr<FChannelModel>> ChannelModels = GetDisplayModeChannelModelsToKey(*this, !bUseSelectedActorKeyingPath);
	if (!ChannelModels.IsEmpty())
	{
		const TSet<TWeakViewModelPtr<FChannelModel>> TransformChannelModels = KeyHelperUtils::GetTransformChannelModels(ChannelModels, InChannel);
		const TSet<FGuid> TargetObjectBindingGuids = TransformChannelModels.IsEmpty()
			? ToolableTimelineClipboard::GetObjectBindingGuidsFromChannelModels(ChannelModels) : TSet<FGuid>();

		if (!TransformChannelModels.IsEmpty() || !TargetObjectBindingGuids.IsEmpty())
		{
			const FScopedTransaction Transaction(LOCTEXT("SetTransformKey_Transaction", "Set Transform Key"));

			if (!TransformChannelModels.IsEmpty())
			{
				const FFrameNumber KeyTime = Sequencer->GetLocalTime().Time.FrameNumber;
				FAddKeyOperation::FromChannelModels(TransformChannelModels).Commit(KeyTime, *Sequencer);
			}
			else
			{
				for (const FGuid& ObjectBindingGuid : TargetObjectBindingGuids)
				{
					for (const TWeakObjectPtr<>& WeakObject : Sequencer->FindObjectsInCurrentSequence(ObjectBindingGuid))
					{
						if (UObject* const Object = WeakObject.Get())
						{
							MovieSceneToolHelpers::AddTransformKeysForObject(Object, InChannel);
						}
					}
				}
			}
		}

		return;
	}

	if (bCanKeySelectedActors)
	{
		Sequencer->AddTransformKeysForSelectedObjectsInternal(InChannel);
	}
}

bool FToolableTimeline::CanSetTransformKey(const EMovieSceneTransformChannel InChannel) const
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid() || Sequencer->IsReadOnly())
	{
		return false;
	}

	const bool bCanKeySelectedActors = ShouldKeySelectedActorsForDisplayMode(*this);
	const bool bUseSelectedActorKeyingPath = bCanKeySelectedActors
		&& (GetKeyDisplay() == ETimelineKeyDisplay::Selected || HasSelectedUnboundObjectsForAddKey(*this));
	const bool bCanAddSelectedActorTransformKey = bCanKeySelectedActors
		&& Sequencer->CanAddTransformKeysForSelectedObjectsInternal(InChannel);

	if (bUseSelectedActorKeyingPath && bCanAddSelectedActorTransformKey)
	{
		return true;
	}

	const TSet<TWeakViewModelPtr<FChannelModel>> KeyableChannelModels = GetDisplayModeChannelModelsToKey(*this, !bUseSelectedActorKeyingPath);
	const TSet<TWeakViewModelPtr<FChannelModel>> TransformChannelModels = KeyHelperUtils::GetTransformChannelModels(KeyableChannelModels, InChannel);
	if (KeyHelperUtils::HasKeyableChannelModels(TransformChannelModels))
	{
		return true;
	}

	const TSet<FGuid> ObjectBindingGuids = ToolableTimelineClipboard::GetObjectBindingGuidsFromChannelModels(KeyableChannelModels);
	return !ObjectBindingGuids.IsEmpty() || bCanAddSelectedActorTransformKey;
}

void FToolableTimeline::DeleteTransformKey(const EMovieSceneTransformChannel InChannel)
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSet<TWeakViewModelPtr<FChannelModel>> KeyableChannelModels = GetChannelModelsToKey(*this);
	if (KeyableChannelModels.IsEmpty())
	{
		return;
	}

	const TSet<TWeakViewModelPtr<FChannelModel>> TransformChannelModels = KeyHelperUtils::GetTransformChannelModels(KeyableChannelModels, InChannel);
	if (TransformChannelModels.IsEmpty())
	{
		return;
	}

	const FFrameNumber KeyTime = Sequencer->GetLocalTime().Time.FrameNumber;
	if (!KeyHelperUtils::HasKeysAtTime(TransformChannelModels, KeyTime))
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("DeleteTransformKey_Transaction", "Delete Transform Key"));
	KeyHelperUtils::DeleteKeysAtTime(*Sequencer, TransformChannelModels, KeyTime);
}

bool FToolableTimeline::CanDeleteTransformKey(const EMovieSceneTransformChannel InChannel) const
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid() || Sequencer->IsReadOnly())
	{
		return false;
	}

	const TSet<TWeakViewModelPtr<FChannelModel>> TransformChannelModels = KeyHelperUtils::GetTransformChannelModels(GetChannelModelsToKey(*this), InChannel);
	if (TransformChannelModels.IsEmpty())
	{
		return false;
	}

	return KeyHelperUtils::HasKeysAtTime(TransformChannelModels, Sequencer->GetLocalTime().Time.FrameNumber);
}

void FToolableTimeline::FlattenTangents()
{
	ExecuteCurveEditorAction(FCurveEditorCommands::Get().FlattenTangents);
}

bool FToolableTimeline::CanFlattenTangents() const
{
	return HasToolRangeKeys();
}

void FToolableTimeline::StraightenTangents()
{
	ExecuteCurveEditorAction(FCurveEditorCommands::Get().StraightenTangents);
}

bool FToolableTimeline::CanStraightenTangents() const
{
	return HasToolRangeKeys();
}

void FToolableTimeline::SmartSnapKeys()
{
	ExecuteCurveEditorAction(FCurveEditorCommands::Get().SmartSnapKeys);
}

bool FToolableTimeline::CanSmartSnapKeys() const
{
	return HasToolRangeKeys();
}

void FToolableTimeline::SetSelectionRangeFromToolRange()
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<FToolableTimelineBaseTool> ActiveTool = GetActiveTool();
	if (ActiveTool.IsValid() && ActiveTool->HasValidToolRange())
	{
		Sequencer->SetSelectionRange(ActiveTool->GetToolRange());
	}
}

bool FToolableTimeline::CanSetSelectionRangeFromToolRange() const
{
	const TSharedPtr<FToolableTimelineBaseTool> ActiveTool = GetActiveTool();
	return ActiveTool.IsValid() && ActiveTool->HasValidToolRange();
}

bool FToolableTimeline::Internal_DeleteSelectedKeys()
{
	const TSharedPtr<FSequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	const FFrameNumber CurrentTime = Sequencer->GetLocalTime().Time.FloorToFrame();
	TSet<UObject*> ModifiedObjects;
	bool bAnythingRemoved = false;

	const TSet<FSequencerSelectedKey>& SelectedKeys = GetKeySelection().GetSelectedKeys();
	if (!SelectedKeys.IsEmpty())
	{
		TMap<TSharedPtr<FChannelModel>, TArray<FKeyHandle>> KeysByChannel;

		for (const FSequencerSelectedKey& SelectedKey : SelectedKeys)
		{
			const TSharedPtr<FChannelModel> ChannelModel = SelectedKey.WeakChannel.Pin();
			if (!ChannelModel.IsValid())
			{
				continue;
			}

			KeysByChannel.FindOrAdd(ChannelModel).Add(SelectedKey.KeyHandle);
		}

		for (const TPair<TSharedPtr<FChannelModel>, TArray<FKeyHandle>>& Pair : KeysByChannel)
		{
			const TSharedPtr<FChannelModel>& ChannelModel = Pair.Key;
			if (!ChannelModel.IsValid())
			{
				continue;
			}

			UMovieSceneSection* const Section = ChannelModel->GetSection();
			if (!Section || Section->IsReadOnly())
			{
				continue;
			}

			const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea();
			if (!KeyArea.IsValid() || Pair.Value.IsEmpty())
			{
				continue;
			}

			UObject* Owner = ChannelModel->GetOwningObject();
			if (!Owner)
			{
				Owner = Section;
			}

			if (Owner && !ModifiedObjects.Contains(Owner))
			{
				Owner->Modify();
				ModifiedObjects.Add(Owner);
			}

			KeyArea->DeleteKeys(Pair.Value, CurrentTime);
			bAnythingRemoved = true;
		}

		if (bAnythingRemoved)
		{
			TimeSliderController->InvalidateKeyRendererCache();
		}

		return bAnythingRemoved;
	}

	const TRange<FFrameNumber> DeleteRange = TimeSliderController->GetScrubWholeFrameRange();

	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : GetChannelModels())
	{
		const TSharedPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
		if (!ChannelModel.IsValid())
		{
			continue;
		}

		UMovieSceneSection* const Section = ChannelModel->GetSection();
		if (!Section || Section->IsReadOnly())
		{
			continue;
		}

		const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea();
		if (!KeyArea.IsValid())
		{
			continue;
		}

		TArray<FKeyHandle> KeyHandles;
		KeyArea->GetKeyHandles(KeyHandles, DeleteRange);
		if (KeyHandles.IsEmpty())
		{
			continue;
		}

		TArray<FFrameNumber> KeyTimes;
		KeyTimes.SetNumUninitialized(KeyHandles.Num());
		KeyArea->GetKeyTimes(KeyHandles, KeyTimes);

		TArray<FKeyHandle> MatchingKeyHandles;
		for (int32 KeyIndex = 0; KeyIndex < KeyHandles.Num(); ++KeyIndex)
		{
			if (KeyTimes[KeyIndex] == CurrentTime)
			{
				MatchingKeyHandles.Add(KeyHandles[KeyIndex]);
			}
		}

		if (MatchingKeyHandles.IsEmpty())
		{
			continue;
		}

		UObject* Owner = ChannelModel->GetOwningObject();
		if (!Owner)
		{
			Owner = Section;
		}

		if (Owner && !ModifiedObjects.Contains(Owner))
		{
			Owner->Modify();
			ModifiedObjects.Add(Owner);
		}

		KeyArea->DeleteKeys(MatchingKeyHandles, CurrentTime);
		bAnythingRemoved = true;
	}

	if (bAnythingRemoved)
	{
		TimeSliderController->InvalidateKeyRendererCache();
	}

	return bAnythingRemoved;
}

bool FToolableTimeline::ShouldRecacheChannelsForDataChange(const EMovieSceneDataChangeType InChangeType) const
{
	switch (InChangeType)
	{
	case EMovieSceneDataChangeType::TrackValueChanged:
	case EMovieSceneDataChangeType::TrackValueChangedRefreshImmediately:
		if (!ChannelFilters.IsEmpty())
		{
			return true;
		}

		if (const TSharedPtr<FSequencer> Sequencer = GetSequencer())
		{
			return Sequencer->GetFilterBar()->ShouldUpdateOnTrackValueChanged();
		}

		return false;

	case EMovieSceneDataChangeType::MovieSceneStructureItemAdded:
	case EMovieSceneDataChangeType::MovieSceneStructureItemRemoved:
	case EMovieSceneDataChangeType::MovieSceneStructureItemsChanged:
	case EMovieSceneDataChangeType::ActiveMovieSceneChanged:
	case EMovieSceneDataChangeType::RefreshAllImmediately:
	case EMovieSceneDataChangeType::Unknown:
	case EMovieSceneDataChangeType::RefreshTree:
	default:
		return true;
	}
}

} // namespace UE::Sequencer::ToolableTimeline

#undef LOCTEXT_NAMESPACE
