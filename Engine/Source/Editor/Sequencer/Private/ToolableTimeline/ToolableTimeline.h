// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerWidgetsModule.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "MVVM/Views/KeyRenderer.h"
#include "SimpleView/SequencerSimpleViewExtender.h"
#include "SimpleView/SimpleViewTabAutosizeHelper.h"
#include "TimeSliderArgs.h"
#include "ToolableTimeline/Caches/ToolableTimelineChannelCache.h"
#include "ToolableTimeline/ToolableTimelineChannelFilter.h"
#include "ToolableTimeline/ToolableTimelineKeySelection.h"
#include "ToolableTimeline/ToolableTimelineSettings.h"
#include "ToolableTimeline/Tools/Factories/IToolableTimelineToolFactory.h"

#define UE_API SEQUENCER_API

class FSequencer;
class FTabManager;
class FUICommandInfo;
class FUICommandList;
class ISequencer;
class UMovieSceneSection;
class USequencerSettings;
enum class EMovieSceneDataChangeType;
enum class EMovieSceneTransformChannel : uint32;
struct FMovieSceneChannelMetaData;
struct FSequencerSelectedKey;
struct FTimeRangeArgs;
struct FTimeSliderArgs;

namespace UE::Sequencer
{
	class FChannelModel;
}

namespace UE::Sequencer::ToolableTimelineClipboard
{
	struct FToolableTimelineClipboard;
}

namespace UE::Sequencer::ToolableTimeline
{

class FToolableTimelineBaseTool;
class FToolableTimeSliderController;
class SToolableTimeline;

/**
 * Main class for a toolable timeline used to create the Sequencer simple view
 */
class FToolableTimeline : public TSharedFromThis<FToolableTimeline>
{
protected:
	struct FPrivateToken
	{ 
		explicit FPrivateToken(const TWeakPtr<FSequencer>& InWeakSequencer
			, const FTimeSliderArgs& InTimeSliderArgs
			, const FTimeRangeArgs& InTimeRangeArgs)
			: WeakSequencer(InWeakSequencer)
			, TimeSliderArgs(InTimeSliderArgs)
			, TimeRangeArgs(InTimeRangeArgs)
		{}

		TWeakPtr<FSequencer> WeakSequencer;
		FTimeSliderArgs TimeSliderArgs;
		FTimeRangeArgs TimeRangeArgs;
	};

public:
	static const FName SequencerTabId;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnChannelModelsChanged, const TSet<TWeakViewModelPtr<FChannelModel>>&)

	template<typename TToolType, typename = TEnableIf<TIsDerivedFrom<TToolType, FToolableTimeline>::IsDerived>::Type>
	static TSharedRef<TToolType> MakeInstance(const TWeakPtr<FSequencer>& InWeakSequencer
		, const FTimeSliderArgs& InTimeSliderArgs
		, const FTimeRangeArgs& InTimeRangeArgs)
	{
		const TSharedRef<TToolType> NewInstance = MakeShared<TToolType>(
			FPrivateToken(InWeakSequencer, InTimeSliderArgs, InTimeRangeArgs)
		);
		NewInstance->Initialize();
		return NewInstance;
	}

	explicit FToolableTimeline(const FPrivateToken& InToken);
	virtual ~FToolableTimeline();

	virtual void Initialize();
	virtual void Shutdown();

	virtual void BindCommands();
	virtual void UnbindCommands();

	virtual TSharedRef<SWidget> GenerateWidget();

	/**
	 * Searches for a suitable tool factory that matches the specified identifier.
	 *
	 * @return A shared pointer to the found tool factory if available, or nullptr if no matching factory is found.
	 */
	template<typename TToolType, typename = TEnableIf<TIsDerivedFrom<TToolType, IToolableTimelineToolFactory>::IsDerived>::Type>
	TSharedPtr<TToolType> FindToolFactory()
	{
		for (const TSharedRef<IToolableTimelineToolFactory>& Factory : ToolFactories)
		{
			if (Factory->GetIdentifier() == TToolType::StaticToolId)
			{
				return StaticCastSharedRef<TToolType>(Factory);
			}
		}
		return nullptr;
	}

	/**
	 * Determines if a valid tool factory of the specified type exists for this timeline.
	 *
	 * @tparam TToolType The type of the tool factory to search for. Must derive from IToolableTimelineToolFactory.
	 * @return True if a valid tool factory of type `TToolType` exists; false otherwise.
	 */
	template<typename TToolType, typename = TEnableIf<TIsDerivedFrom<TToolType, IToolableTimelineToolFactory>::IsDerived>::Type>
	bool HasToolFactory()
	{
		return FindToolFactory<TToolType>().IsValid();
	}

	/**
	 * Adds a new tool factory of the specified type to the timeline if it doesn't already exist.
	 *
	 * This method checks for the existence of a tool factory of the specified type `TToolType`.
	 * If not already present, it creates a new factory instance, adds it to the tool factories list,
	 * and sorts the list based on priority. The operation returns a flag indicating whether the
	 * addition was successful.
	 *
	 * @tparam TToolType The type of the tool factory to add. Must derive from IToolableTimelineToolFactory.
	 * @return True if the tool factory was successfully added; false if a tool factory of the
	 *         specified type already exists or the addition fails.
	 */
	template<typename TToolType, typename = TEnableIf<TIsDerivedFrom<TToolType, IToolableTimelineToolFactory>::IsDerived>::Type>
	bool AddToolFactory()
	{
		if (!HasToolFactory<TToolType>())
		{
			const TSharedRef<TToolType> NewFactory = MakeShared<TToolType>();
			const int32 NumAdded = ToolFactories.Add(NewFactory);

			ToolFactories.Sort([](const TSharedRef<IToolableTimelineToolFactory>& InA
					, const TSharedRef<IToolableTimelineToolFactory>& InB)
				{
					return InA->GetPriority() < InB->GetPriority();
				});

			return NumAdded > 0;
		}
		return false;
	}

	/**
	 * Removes a tool factory of the specified type from the timeline.
	 *
	 * This method searches for an existing tool factory of the given type `TToolType`
	 * and removes it from the list of tool factories associated with the timeline.
	 * If a tool factory is successfully removed, the method returns true; otherwise, false.
	 *
	 * @tparam TToolType The type of the tool factory to remove. Must derive from IToolableTimelineToolFactory.
	 * @return True if the tool factory was successfully removed; false otherwise.
	 */
	template<typename TToolType, typename = TEnableIf<TIsDerivedFrom<TToolType, IToolableTimelineToolFactory>::IsDerived>::Type>
	bool RemoveToolFactory()
	{
		if (const TSharedPtr<IToolableTimelineToolFactory> Factory = FindToolFactory<TToolType>())
		{
			const int32 NumRemoved = ToolFactories.Remove(Factory.ToSharedRef());
			return NumRemoved > 0;
		}
		return false;
	}

	virtual void NotifyToolActivated() {}
	virtual void NotifyToolDeactivated() {}

	virtual TArray<FRegisteredChannelFilter>& GetChannelFilters();
	virtual const TArray<FRegisteredChannelFilter>& GetChannelFilters() const;
	FDelegateHandle AddChannelFilter(FChannelFilterFunction& InFilterFunction);
	void RemoveChannelFilter(const FDelegateHandle& InDelegateHandle);
	bool PassesChannelFilters(const FChannelModel& InChannelModel) const;

	TSharedPtr<FSequencer> GetSequencer() const;
	USequencerSettings* GetSequencerSettings() const;
	TSharedPtr<FTabManager> GetTabManager() const;

	FTimeRangeArgs& GetTimeRangeArgs() { return TimeRangeArgs; }

	TSharedRef<FUICommandList> GetCommandList() const { return CommandList; }

	UToolableTimelineSettings& GetTimelineSettings() const { return *TimelineSettings.Get(); }

	TSharedRef<FToolableTimeSliderController> GetTimeSliderController() const { return TimeSliderController; }
	TSharedRef<FTrackAreaViewModel> GetTrackAreaViewModel() const { return TrackAreaViewModel; }

	TSharedPtr<SToolableTimeline> GetWidget() const;

	ETimelineKeyDisplay GetKeyDisplay() const;
	bool IsKeyDisplay(const ETimelineKeyDisplay InKeyDisplay) const;
	void SetKeyDisplay(const ETimelineKeyDisplay InKeyDisplay);

	void ZoomToFit();
	void ScrubToFrame();
	void DeactivateActiveTool();

	void RequestRecacheChannels();

	TSharedPtr<FToolableTimelineBaseTool> GetActiveTool() const;

	template<typename TToolType> 
	TSharedPtr<TToolType> ActiveToolAs() const
	{
		return StaticCastSharedPtr<TToolType>(GetActiveTool());
	}

	const TSet<TWeakViewModelPtr<FChannelModel>>& GetChannelModels() const;

	uint32 GetChannelModelsSerialNumber() const { return ChannelModelsSerialNumber; }

	FOnChannelModelsChanged& OnChannelModelsChanged() { return ChannelModelsChangedEvent; }

	virtual TSet<FSequencerSelectedKey> GetScrubRangeKeys() const;

	FToolableTimelineKeySelection& GetKeySelection() { return KeySelection; }
	const FToolableTimelineKeySelection& GetKeySelection() const { return KeySelection; }

	void SyncSelectionToSequencerAndCurveEditor();

	/**
	 * Centers a frame on the screen to ensure it is properly visible within the viewport.
	 * 
	 * @param InNewValue Frame to be centered on the screen
	 * @param bInCenterScrubPosition Optionally centers the scrub position as well
	 */
	void CenterFrameOnScreen(const double InNewValue, const bool bInCenterScrubPosition = false);

	void SelectAllKeys();

	void ExecuteSetKeyCommand();
	bool CanExecuteSetKeyCommand() const;

	void ExecuteTransformKeyCommand();
	void ExecuteTransformKeyCommand(const EMovieSceneTransformChannel InChannel);

	bool CanExecuteTransformKeyCommand() const;
	bool CanExecuteTransformKeyCommand(const EMovieSceneTransformChannel InChannel) const;

	void ExecuteDeleteTransformKeyCommand(const EMovieSceneTransformChannel InChannel);
	bool CanExecuteDeleteTransformKeyCommand(const EMovieSceneTransformChannel InChannel) const;

protected:
	friend class SToolableTimeline;

	void RecacheChannelModels();

	/**
	 * Retrieves all unique key times for every cached channel associated with the timeline.
	 *
	 * This method iterates through all cached channels, collects key times from valid key areas,
	 * ensures uniqueness, and sorts the key times. The result includes all key times from
	 * available channels.
	 *
	 * @return An array of sorted, unique frame numbers representing all key times within the timeline's channels.
	 */
	TArray<FFrameNumber> GetAllChannelKeyTimes() const;

	/**
	 * Sync curve editor selection to the tool range and executes a curve editor action
	 * corresponding to the provided UI command.
	 * 
	 * @param InCommand A shared pointer to the UI command information to execute.
	 * @return True if the command was executed successfully; false otherwise.
	 */
	bool ExecuteCurveEditorAction(const TSharedPtr<const FUICommandInfo>& InCommand);

	bool HasToolRangeKeys() const;

	void CutSelectedKeys();
	bool CanCutSelectedKeys() const;

	void CopySelectedKeys();
	bool CanCopySelectedKeys() const;

	void PasteSelectedKeys();
	bool CanPasteSelectedKeys() const;

	void DeleteSelectedKeys();
	bool CanDeleteSelectedKeys() const;

	void SetKey();
	bool CanSetKey() const;

	void SetTransformKey(const EMovieSceneTransformChannel InChannel = EMovieSceneTransformChannel::All);
	bool CanSetTransformKey(const EMovieSceneTransformChannel InChannel = EMovieSceneTransformChannel::All) const;

	void DeleteTransformKey(const EMovieSceneTransformChannel InChannel);
	bool CanDeleteTransformKey(const EMovieSceneTransformChannel InChannel) const;

	void FlattenTangents();
	bool CanFlattenTangents() const;

	void StraightenTangents();
	bool CanStraightenTangents() const;

	void SmartSnapKeys();
	bool CanSmartSnapKeys() const;

	void SetSelectionRangeFromToolRange();
	bool CanSetSelectionRangeFromToolRange() const;

	bool Internal_DeleteSelectedKeys();

	bool ShouldRecacheChannelsForDataChange(const EMovieSceneDataChangeType InChangeType) const;

	/** Parent sequencer that this timeline is created from */
	TWeakPtr<FSequencer> WeakSequencer;

	/** Delegate handles for lifetime-managed timeline bindings created during Initialize(). */
	FDelegateHandle MovieSceneDataChangedDelegateHandle;
	FDelegateHandle ChannelChangedDelegateHandle;
	FDelegateHandle OutlinerSelectionChangedDelegateHandle;
	FDelegateHandle PinStateChangedDelegateHandle;
	FDelegateHandle TimelineSettingsChangedDelegateHandle;

	/** Delegate handles for filter-related channel recache hooks owned by this timeline. */
	FDelegateHandle FilterBarChangedDelegateHandle;
	FDelegateHandle FilterRequestUpdateDelegateHandle;
	FDelegateHandle FilterTextChangedDelegateHandle;
	FDelegateHandle FilterMuteChangedDelegateHandle;

	/** List of tool factories for all tools that can be used by this timeline */
	TArray<TSharedRef<IToolableTimelineToolFactory>> ToolFactories;

	/** Time range arguments for the bottom time range widget */
	FTimeRangeArgs TimeRangeArgs;

	/** Commands for this timeline instance */
	TSharedRef<FUICommandList> CommandList;

	/** Settings object for this timeline */
	TStrongObjectPtr<UToolableTimelineSettings> TimelineSettings;

	/** Controller for the time slider associated with this timeline */
	TSharedRef<FToolableTimeSliderController> TimeSliderController;

	/** Cached channel models holding keys that are being processed */
	TSharedPtr<FToolableTimelineChannelCache> ChannelCache;

	uint32 ChannelModelsSerialNumber = 0;

	/** List of channel filter functions to extend filtering of the channel cache list */
	TArray<FRegisteredChannelFilter> ChannelFilters;

	/** View model for track area, used most to pass to both pinned and non pinned SSequencerTrackAreaView widgets */
	TSharedRef<FTrackAreaViewModel> TrackAreaViewModel;

	/** Event that broadcasts when channel models have changed */
	FOnChannelModelsChanged ChannelModelsChangedEvent;

	/** Selected keys in the timeline */
	FToolableTimelineKeySelection KeySelection;

	/** Clipboard for key operations driven directly by the toolable timeline */
	TSharedPtr<ToolableTimelineClipboard::FToolableTimelineClipboard> TimelineKeyClipboard;

 private:
	friend class FToolableTimeSliderController;

	void HandleMovieSceneDataChanged(const EMovieSceneDataChangeType InChangeType);

	void HandleChannelChanged(const TArray<FFrameNumber>& InFrames, const FMovieSceneChannelMetaData* InMetaData, UMovieSceneSection* InSection);

	void HandlePinStateChanged(const TViewModelPtr<IPinnableExtension>& InPinnableExtension, const bool bInPinned);

	void HandleTimelineSettingsChanged(UObject* const InObject, FPropertyChangedEvent& InEvent);

	void HandleOutlinerSelectionChanged();

	/** Main widget for this timeline */
	TSharedPtr<SToolableTimeline> TimelineWidget;
};

} // namespace UE::Sequencer::ToolableTimeline

#undef UE_API
