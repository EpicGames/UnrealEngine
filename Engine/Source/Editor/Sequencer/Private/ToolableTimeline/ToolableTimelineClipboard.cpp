// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimeline/ToolableTimelineClipboard.h"
#include "IKeyArea.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "Sequencer.h"
#include "SequencerClipboardReconciler.h"
#include "SequencerSelectedKey.h"
#include "ToolableTimeline/ToolableTimeline.h"
#include "ToolableTimeline/ToolableTimeSliderController.h"

namespace UE::Sequencer::ToolableTimelineClipboard
{

bool FToolableTimelineChannelClipboardIdentifier::operator==(const FToolableTimelineChannelClipboardIdentifier& InOther) const
{
	return Section == InOther.Section
		&& OwningObject == InOther.OwningObject
		&& ObjectBindingGuid == InOther.ObjectBindingGuid
		&& TrackClassName == InOther.TrackClassName
		&& ChannelName == InOther.ChannelName;
}

bool FToolableTimelineClipboard::IsEmpty() const
{
	return Entries.IsEmpty();
}

/** Cached clipboard operation context for a single validated channel. */
struct FClipboardChannelContext
{
	TSharedPtr<FChannelModel> ChannelModel;
	UMovieSceneSection* Section = nullptr;
	TSharedPtr<IKeyArea> KeyArea;
};

bool TryMakeClipboardChannelContext(const TSharedPtr<FChannelModel>& InChannelModel
	, const bool bInRequireWritableSection, FClipboardChannelContext& OutContext)
{
	if (!InChannelModel.IsValid())
	{
		return false;
	}

	UMovieSceneSection* const Section = InChannelModel->GetSection();
	if (!Section || (bInRequireWritableSection && Section->IsReadOnly()))
	{
		return false;
	}

	const TSharedPtr<IKeyArea> KeyArea = InChannelModel->GetKeyArea();
	if (!KeyArea.IsValid())
	{
		return false;
	}

	OutContext.ChannelModel = InChannelModel;
	OutContext.Section = Section;
	OutContext.KeyArea = KeyArea;

	return true;
}

template<typename TCallback>
void EnumerateClipboardChannels(const ToolableTimeline::FToolableTimeline& InTimeline
	, const bool bInRequireWritableSection, TCallback&& InCallback)
{
	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : InTimeline.GetChannelModels())
	{
		FClipboardChannelContext Context;
		if (TryMakeClipboardChannelContext(WeakChannelModel.Pin(), bInRequireWritableSection, Context))
		{
			InCallback(Context);
		}
	}
}

template<typename TCallback>
void EnumerateKeysByChannel(const TMap<TSharedPtr<FChannelModel>, TArray<FKeyHandle>>& InKeysByChannel, TCallback&& InCallback)
{
	for (const TPair<TSharedPtr<FChannelModel>, TArray<FKeyHandle>>& Pair : InKeysByChannel)
	{
		const TSharedPtr<FChannelModel>& ChannelModel = Pair.Key;
		if (!ChannelModel.IsValid())
		{
			continue;
		}

		const TArray<FKeyHandle>& KeyHandles = Pair.Value;
		if (KeyHandles.IsEmpty())
		{
			continue;
		}

		InCallback(ChannelModel, KeyHandles);
	}
}

TMap<TSharedPtr<FChannelModel>, TArray<FKeyHandle>> GetSelectedKeysByChannel(const ToolableTimeline::FToolableTimeline& InTimeline)
{
	TMap<TSharedPtr<FChannelModel>, TArray<FKeyHandle>> OutKeysByChannel;

	for (const FSequencerSelectedKey& SelectedKey : InTimeline.GetKeySelection().GetSelectedKeys())
	{
		const TSharedPtr<FChannelModel> ChannelModel = SelectedKey.WeakChannel.Pin();
		if (!ChannelModel.IsValid())
		{
			continue;
		}

		OutKeysByChannel.FindOrAdd(ChannelModel).Add(SelectedKey.KeyHandle);
	}

	return OutKeysByChannel;
}

UMovieSceneTrack* GetChannelTrack(const FChannelModel& InChannelModel)
{
	if (const TViewModelPtr<ITrackExtension> TrackModel = InChannelModel.FindAncestorOfType<ITrackExtension>(true))
	{
		return TrackModel->GetTrack();
	}

	if (const UMovieSceneSection* const Section = InChannelModel.GetSection())
	{
		return Section->GetTypedOuter<UMovieSceneTrack>();
	}

	return nullptr;
}

FToolableTimelineChannelClipboardIdentifier BuildChannelClipboardIdentifier(const FChannelModel& InChannelModel)
{
	FToolableTimelineChannelClipboardIdentifier Identifier;
	Identifier.Section = InChannelModel.GetSection();
	Identifier.OwningObject = InChannelModel.GetOwningObject();
	Identifier.ObjectBindingGuid = GetObjectBindingGuid(InChannelModel);
	Identifier.ChannelName = InChannelModel.GetKeyArea().IsValid()
		? InChannelModel.GetKeyArea()->GetName() : NAME_None;

	if (const UMovieSceneTrack* Track = GetChannelTrack(InChannelModel))
	{
		Identifier.TrackClassName = Track->GetClass()->GetFName();
	}

	return Identifier;
}

bool DoesClipboardIdentifierMatchChannel(const FToolableTimelineChannelClipboardIdentifier& InIdentifier, const FChannelModel& InChannelModel)
{
	return InIdentifier == BuildChannelClipboardIdentifier(InChannelModel);
}

TMap<TSharedPtr<FChannelModel>, TArray<FKeyHandle>> GetScrubFrameKeysByChannel(const ToolableTimeline::FToolableTimeline& InTimeline)
{
	const TRange<FFrameNumber> ScrubRange = InTimeline.GetTimeSliderController()->GetScrubWholeFrameRange();

	TMap<TSharedPtr<FChannelModel>, TArray<FKeyHandle>> OutKeysByChannel;

	EnumerateClipboardChannels(InTimeline, false
		, [&OutKeysByChannel, &ScrubRange](const FClipboardChannelContext& InContext)
		{
			TArray<FKeyHandle> KeyHandles;
			InContext.KeyArea->GetKeyHandles(KeyHandles, ScrubRange);

			if (!KeyHandles.IsEmpty())
			{
				OutKeysByChannel.Add(InContext.ChannelModel, MoveTemp(KeyHandles));
			}
		});

	return OutKeysByChannel;
}

TMap<TSharedPtr<FChannelModel>, TArray<FKeyHandle>> GetKeysByChannelForClipboardOperation(const ToolableTimeline::FToolableTimeline& InTimeline)
{
	const TSet<FSequencerSelectedKey>& SelectedKeys = InTimeline.GetKeySelection().GetSelectedKeys();
	return SelectedKeys.IsEmpty()
		? GetScrubFrameKeysByChannel(InTimeline)
		: GetSelectedKeysByChannel(InTimeline);
}

TOptional<FFrameNumber> FindCopyRelativeTime(const TMap<TSharedPtr<FChannelModel>, TArray<FKeyHandle>>& InKeysByChannel)
{
	TOptional<FFrameNumber> CopyRelativeTo;

	EnumerateKeysByChannel(InKeysByChannel, [&CopyRelativeTo]
		(const TSharedPtr<FChannelModel>& InChannelModel, const TArray<FKeyHandle>& InKeyHandles)
		{
			const TSharedPtr<IKeyArea> KeyArea = InChannelModel->GetKeyArea();
			if (!KeyArea.IsValid())
			{
				return;
			}

			TArray<FFrameNumber> KeyTimes;
			KeyTimes.SetNumUninitialized(InKeyHandles.Num());
			KeyArea->GetKeyTimes(InKeyHandles, KeyTimes);

			for (const FFrameNumber KeyTime : KeyTimes)
			{
				CopyRelativeTo = CopyRelativeTo.IsSet()
					? TOptional<FFrameNumber>(FMath::Min(CopyRelativeTo.GetValue(), KeyTime))
					: TOptional<FFrameNumber>(KeyTime);
			}
		});

	return CopyRelativeTo;
}

const FMovieSceneClipboardKeyTrack* FindSingleKeyTrack(const FMovieSceneClipboard& InClipboard)
{
	const FMovieSceneClipboardKeyTrack* KeyTrack = nullptr;

	for (const TArray<FMovieSceneClipboardKeyTrack>& Group : InClipboard.GetKeyTrackGroups())
	{
		for (const FMovieSceneClipboardKeyTrack& Track : Group)
		{
			if (KeyTrack)
			{
				return nullptr;
			}

			KeyTrack = &Track;
		}
	}

	return KeyTrack;
}

TOptional<FGuid> GetObjectBindingGuid(const FChannelModel& InChannelModel)
{
	const TViewModelPtr<IObjectBindingExtension> ObjectBinding = InChannelModel.FindAncestorOfType<IObjectBindingExtension>(true);
	return ObjectBinding.IsValid() ? ObjectBinding->GetObjectGuid() : TOptional<FGuid>();
}

TSet<FGuid> GetObjectBindingGuidsFromChannelModels(const TSet<TWeakViewModelPtr<FChannelModel>>& InChannelModels)
{
	TSet<FGuid> OutObjectBindingGuids;

	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : InChannelModels)
	{
		const TViewModelPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
		if (!ChannelModel.IsValid())
		{
			continue;
		}

		if (const TOptional<FGuid> ObjectBindingGuid = GetObjectBindingGuid(*ChannelModel))
		{
			OutObjectBindingGuids.Add(ObjectBindingGuid.GetValue());
		}
	}

	return OutObjectBindingGuids;
}

bool HasAnyKeysForClipboardOperation(const ToolableTimeline::FToolableTimeline& InTimeline)
{
	if (InTimeline.GetKeySelection().HasAnySelectedKeys())
	{
		return true;
	}

	const TRange<FFrameNumber> KeyRange = InTimeline.GetTimeSliderController()->GetScrubWholeFrameRange();

	bool bHasAnyKeys = false;

	EnumerateClipboardChannels(InTimeline, false
		, [&bHasAnyKeys, &KeyRange](const FClipboardChannelContext& InContext)
		{
			if (bHasAnyKeys)
			{
				return;
			}

			TArray<FKeyHandle> KeyHandles;
			InContext.KeyArea->GetKeyHandles(KeyHandles, KeyRange);

			bHasAnyKeys = !KeyHandles.IsEmpty();
		});

	return bHasAnyKeys;
}

TSharedPtr<FToolableTimelineClipboard> BuildClipboardFromTimelineKeys(const ToolableTimeline::FToolableTimeline& InTimeline
	, const FSequencer& InSequencer)
{
	const TMap<TSharedPtr<FChannelModel>, TArray<FKeyHandle>> KeysByChannel = GetKeysByChannelForClipboardOperation(InTimeline);
	if (KeysByChannel.IsEmpty())
	{
		return nullptr;
	}

	const TOptional<FFrameNumber> CopyRelativeTo = FindCopyRelativeTime(KeysByChannel);
	if (!CopyRelativeTo.IsSet())
	{
		return nullptr;
	}

	const TSharedPtr<FToolableTimelineClipboard> Clipboard = MakeShared<FToolableTimelineClipboard>();

	EnumerateKeysByChannel(KeysByChannel, [&Clipboard, &CopyRelativeTo, &InSequencer]
		(const TSharedPtr<FChannelModel>& InChannelModel, const TArray<FKeyHandle>& InKeyHandles)
		{
			const TSharedPtr<IKeyArea> KeyArea = InChannelModel->GetKeyArea();
			if (!KeyArea.IsValid())
			{
				return;
			}

			FMovieSceneClipboardBuilder Builder;
			KeyArea->CopyKeys(Builder, InKeyHandles);

			FMovieSceneClipboard ChannelClipboard = Builder.Commit(CopyRelativeTo);
			if (ChannelClipboard.GetKeyTrackGroups().Num() == 0)
			{
				return;
			}

			ChannelClipboard.GetEnvironment().TickResolution = InSequencer.GetFocusedTickResolution();
			ChannelClipboard.GetEnvironment().TimeTransform = InSequencer.GetFocusedMovieSceneSequenceTransform().LinearTransform;

			FToolableTimelineClipboardEntry& Entry = Clipboard->Entries.AddDefaulted_GetRef();
			Entry.Identifier = BuildChannelClipboardIdentifier(*InChannelModel);
			Entry.Clipboard = MoveTemp(ChannelClipboard);
		});

	return Clipboard->IsEmpty() ? nullptr : Clipboard;
}

TMultiMap<FName, TSharedPtr<FChannelModel>> BuildCandidateChannelsByName(const ToolableTimeline::FToolableTimeline& InTimeline)
{
	TMultiMap<FName, TSharedPtr<FChannelModel>> ChannelsByName;

	EnumerateClipboardChannels(InTimeline, true
		, [&ChannelsByName](const FClipboardChannelContext& InContext)
		{
			ChannelsByName.Add(InContext.KeyArea->GetName(), InContext.ChannelModel);
		});

	return ChannelsByName;
}

TSharedPtr<FChannelModel> FindMatchingDestinationChannel(const TMultiMap<FName, TSharedPtr<FChannelModel>>& InChannelsByName
	, const FToolableTimelineClipboardEntry& InEntry)
{
	TArray<TSharedPtr<FChannelModel>> CandidateChannels;
	InChannelsByName.MultiFind(InEntry.Identifier.ChannelName, CandidateChannels);

	const TSharedPtr<FChannelModel>* const MatchingChannel = CandidateChannels.FindByPredicate(
		[&InEntry](const TSharedPtr<FChannelModel>& CandidateChannel)
		{
			return CandidateChannel.IsValid() && DoesClipboardIdentifierMatchChannel(InEntry.Identifier, *CandidateChannel);
		});

	return MatchingChannel ? *MatchingChannel : nullptr;
}

bool CanPasteClipboardIntoCachedChannels(const ToolableTimeline::FToolableTimeline& InTimeline, const FToolableTimelineClipboard& InClipboard)
{
	if (InClipboard.IsEmpty())
	{
		return false;
	}

	const TMultiMap<FName, TSharedPtr<FChannelModel>> ChannelsByName = BuildCandidateChannelsByName(InTimeline);
	for (const FToolableTimelineClipboardEntry& Entry : InClipboard.Entries)
	{
		if (FindMatchingDestinationChannel(ChannelsByName, Entry).IsValid() && FindSingleKeyTrack(Entry.Clipboard))
		{
			return true;
		}
	}

	return false;
}

bool PasteClipboardIntoCachedChannels(ToolableTimeline::FToolableTimeline& InTimeline, FSequencer& InSequencer, const FToolableTimelineClipboard& InClipboard)
{
	TSet<FSequencerSelectedKey> PastedKeys;
	const TMultiMap<FName, TSharedPtr<FChannelModel>> ChannelsByName = BuildCandidateChannelsByName(InTimeline);

	FSequencerPasteEnvironment PasteEnvironment;
	PasteEnvironment.TickResolution = InSequencer.GetFocusedTickResolution();
	PasteEnvironment.CardinalTime = InSequencer.GetLocalTime().Time;
	PasteEnvironment.TimeTransform = InSequencer.GetFocusedMovieSceneSequenceTransform().LinearTransform;
	PasteEnvironment.OnKeyPasted = [&PastedKeys](const FKeyHandle InHandle, const TSharedPtr<FChannelModel>& InChannel)
		{
			if (InChannel.IsValid())
			{
				if (UMovieSceneSection* const Section = InChannel->GetSection())
				{
					PastedKeys.Add(FSequencerSelectedKey(*Section, InChannel, InHandle));
				}
			}
		};

	bool bAnythingPasted = false;

	for (const FToolableTimelineClipboardEntry& Entry : InClipboard.Entries)
	{
		const TSharedPtr<FChannelModel> MatchingChannel = FindMatchingDestinationChannel(ChannelsByName, Entry);
		if (!MatchingChannel.IsValid())
		{
			continue;
		}

		const TSharedPtr<IKeyArea> KeyArea = MatchingChannel->GetKeyArea();
		if (!KeyArea.IsValid())
		{
			continue;
		}

		const FMovieSceneClipboardKeyTrack* const SourceKeyTrack = FindSingleKeyTrack(Entry.Clipboard);
		if (!SourceKeyTrack)
		{
			continue;
		}

		const TArray<FKeyHandle> NewlyPastedKeys = KeyArea->PasteKeys(*SourceKeyTrack, Entry.Clipboard.GetEnvironment(), PasteEnvironment);
		if (NewlyPastedKeys.IsEmpty())
		{
			continue;
		}

		bAnythingPasted = true;
	}

	if (bAnythingPasted)
	{
		InTimeline.GetKeySelection().SetSelectedKeys(PastedKeys, true);
	}

	return bAnythingPasted;
}

} // namespace UE::Sequencer::ToolableTimelineClipboard
