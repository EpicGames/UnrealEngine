// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Set.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "MovieSceneClipboard.h"
#include "MVVM/ViewModelPtr.h"
#include "Templates/SharedPointer.h"

class FSequencer;
class UMovieSceneSection;

namespace UE::Sequencer
{
	class FChannelModel;
}

namespace UE::Sequencer::ToolableTimeline
{
	class FToolableTimeline;
}

namespace UE::Sequencer::ToolableTimelineClipboard
{

/**
 * Identifies a cached channel for toolable timeline clipboard operations.
 * The identifier intentionally captures multiple ownership signals so copied channel data can
 * be matched back to the correct cached channel during paste, even when channel names alone
 * are not unique.
 */
struct FToolableTimelineChannelClipboardIdentifier
{
	TWeakObjectPtr<UMovieSceneSection> Section;
	TWeakObjectPtr<UObject> OwningObject;
	TOptional<FGuid> ObjectBindingGuid;
	FName TrackClassName = NAME_None;
	FName ChannelName = NAME_None;

	bool operator==(const FToolableTimelineChannelClipboardIdentifier& InOther) const;
};

/** Clipboard payload for a single cached channel. */
struct FToolableTimelineClipboardEntry
{
	FToolableTimelineChannelClipboardIdentifier Identifier;
	FMovieSceneClipboard Clipboard;
};

/** Cache-driven clipboard owned by the toolable timeline. */
struct FToolableTimelineClipboard
{
	TArray<FToolableTimelineClipboardEntry> Entries;

	bool IsEmpty() const;
};

/**
 * Resolves the nearest object binding GUID that owns the supplied channel model.
 *
 * @param InChannelModel Channel model to inspect.
 * @return The owning object binding GUID if one exists.
 */
TOptional<FGuid> GetObjectBindingGuid(const FChannelModel& InChannelModel);

/**
 * Collects object binding GUIDs for a set of channel models.
 * Invalid channel models and channels without object bindings are skipped.
 *
 * @param InChannelModels Channel models to inspect.
 * @return Set of resolved object binding GUIDs.
 */
TSet<FGuid> GetObjectBindingGuidsFromChannelModels(const TSet<TWeakViewModelPtr<FChannelModel>>& InChannelModels);

/**
 * Checks whether the current toolable timeline clipboard source set contains any keys.
 * If the timeline has selected keys, this checks that selection. Otherwise, it checks whether the
 * current cached channel set contains any keys on the scrub head frame.
 *
 * @param InTimeline Timeline whose current clipboard source set should be inspected.
 * @return True if a copy/cut/delete operation would have any keys to operate on.
 */
bool HasAnyKeysForClipboardOperation(const ToolableTimeline::FToolableTimeline& InTimeline);

/**
 * Builds a cache-driven clipboard from the timeline's current clipboard source keys.
 *
 * Selected keys are copied when present. Otherwise, keys are copied from the scrub head frame
 * in the current cached channel set.
 *
 * @param InTimeline Timeline providing the source keys and channel cache.
 * @param InSequencer Sequencer used to supply clipboard timing environment data.
 * @return A populated clipboard, or nullptr if there were no copyable keys.
 */
TSharedPtr<FToolableTimelineClipboard> BuildClipboardFromTimelineKeys(const ToolableTimeline::FToolableTimeline& InTimeline
	, const FSequencer& InSequencer);

/**
 * Returns whether any clipboard entry can resolve to one of the current cached channels.
 *
 * @param InTimeline Timeline providing the destination cached channels.
 * @param InClipboard Clipboard payload to test.
 * @return True if at least one entry has a valid destination channel match.
 */
bool CanPasteClipboardIntoCachedChannels(const ToolableTimeline::FToolableTimeline& InTimeline, const FToolableTimelineClipboard& InClipboard);

/**
 * Pastes a toolable timeline clipboard back into the current cached channels.
 * Clipboard entries are matched back to cached channels by identifier, then pasted through each
 * matching key area. Any newly pasted keys are reselected in the timeline on success.
 *
 * @param InTimeline Timeline providing the destination cached channels and key selection.
 * @param InSequencer Sequencer used to supply paste timing environment data.
 * @param InClipboard Clipboard payload to paste.
 * @return True if any clipboard entry pasted successfully.
 */
bool PasteClipboardIntoCachedChannels(ToolableTimeline::FToolableTimeline& InTimeline, FSequencer& InSequencer, const FToolableTimelineClipboard& InClipboard);

} // namespace UE::Sequencer::ToolableTimelineClipboard
