// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimelineCommands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "ToolableTimelineCommands"

UE_DEFINE_TCOMMANDS(FToolableTimelineCommands)

void FToolableTimelineCommands::RegisterCommands()
{
	UI_COMMAND(SetKeyDisplay_SelectedAndPinned
		, "Selected And Pinned"
		, "Show keys for selected and pinned objects on the timeline"
		, EUserInterfaceActionType::RadioButton
		, FInputChord(EKeys::One))

	UI_COMMAND(SetKeyDisplay_Selected
		, "Selected"
		, "Show keys for selected objects on the timeline"
		, EUserInterfaceActionType::RadioButton
		, FInputChord(EKeys::Two))

	UI_COMMAND(SetKeyDisplay_All
		, "All"
		, "Show all keys for all objects on the timeline"
		, EUserInterfaceActionType::RadioButton
		, FInputChord(EKeys::Three))

	UI_COMMAND(ZoomToFit
		, "Zoom to Fit"
		, "If keys are selected, fits the Sequencer view range to the keys. "
		  "If no keys are selected, sets the view range to include all keys."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::F))

	UI_COMMAND(GoToFrame
		, "Go To Frame"
		, "Scrubs immediately to a specified frame and centers it on the timeline"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Control, EKeys::Enter))

	UI_COMMAND(DeactivateActiveTool
		, "Deactivate Tool"
		, "Deactivates the currently active tool"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Escape))

	UI_COMMAND(SetSelectionRangeFromToolRange
		, "Set Selection Range from Tool Range"
		, "Sets the Sequencer selection range from the active tool range"
		, EUserInterfaceActionType::Button
		, FInputChord())
}

#undef LOCTEXT_NAMESPACE
