// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FUICommandInfo;

UE_DECLARE_TCOMMANDS(class FToolableTimelineCommands, SEQUENCER_API)

class FToolableTimelineCommands : public TCommands<FToolableTimelineCommands>
{
public:
	FToolableTimelineCommands()
		: TCommands<FToolableTimelineCommands>(TEXT("ToolableTimelineEditor"),
			NSLOCTEXT("Contexts", "ToolableTimelineEditor", "Toolable Timeline Editor"),
			NAME_None,
			FAppStyle::GetAppStyleSetName())
	{}

	SEQUENCER_API virtual void RegisterCommands() override;

	/** Show keys for selected and pinned objects on the timeline */
	TSharedPtr<FUICommandInfo> SetKeyDisplay_SelectedAndPinned;

	/** Show keys for selected objects on the timeline */
	TSharedPtr<FUICommandInfo> SetKeyDisplay_Selected;

	/** Show all keys for all objects on the timeline */
	TSharedPtr<FUICommandInfo> SetKeyDisplay_All;

	/** If keys are selected, fits the Sequencer view range to the keys.
	 * If no keys are selected, sets the view range to include all keys. */
	TSharedPtr<FUICommandInfo> ZoomToFit;

	/** Scrubs immediately to a specific frame using a numeric entry box that gets displayed */
	TSharedPtr<FUICommandInfo> GoToFrame;

	/** Deactivate the currently active tool */
	TSharedPtr<FUICommandInfo> DeactivateActiveTool;

	/** Sets the Sequencer selection range from the active tool range */
	TSharedPtr<FUICommandInfo> SetSelectionRangeFromToolRange;
};
