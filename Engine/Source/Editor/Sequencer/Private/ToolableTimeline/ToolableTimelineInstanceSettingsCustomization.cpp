// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimelineInstanceSettingsCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "ToolableTimeline/ToolableTimelineSettings.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ToolableTimelineInstanceSettingsCustomization"

namespace UE::Sequencer::ToolableTimeline
{

namespace InstanceSettingsCustomization
{
	static const TCHAR* const EnableSimpleViewHiddenConfigCVarName = TEXT("Sequencer.SimpleView.HiddenConfig");
	static TAutoConsoleVariable<bool> CVarEnableSimpleViewHiddenConfig(
		EnableSimpleViewHiddenConfigCVarName,
		false,
		TEXT("When true, enables view of additional configuration settings.\n")
		TEXT("0: Disabled (default), 1: Enabled"),
		ECVF_Default);
}

FToolableTimelineInstanceSettingsCustomization::~FToolableTimelineInstanceSettingsCustomization()
{
	InstanceSettingsCustomization::CVarEnableSimpleViewHiddenConfig->OnChangedDelegate().Remove(ViewHiddenConfigCVarChangedHandle);
}

TSharedRef<IPropertyTypeCustomization> FToolableTimelineInstanceSettingsCustomization::MakeInstance()
{
	return MakeShared<FToolableTimelineInstanceSettingsCustomization>();
}

void FToolableTimelineInstanceSettingsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle
	, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FToolableTimelineInstanceSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle
	, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	StructHandle = InPropertyHandle;

	if (!ViewHiddenConfigCVarChangedHandle.IsValid())
	{
		ViewHiddenConfigCVarChangedHandle = InstanceSettingsCustomization::CVarEnableSimpleViewHiddenConfig->OnChangedDelegate().AddSP(this
			, &FToolableTimelineInstanceSettingsCustomization::HandleHiddenConfigToggled);
	}

	PropertyUtilities = CustomizationUtils.GetPropertyUtilities();

	const TWeakPtr<FToolableTimelineInstanceSettingsCustomization> ThisWeak = SharedThis(this);
	const FText TooltipText = LOCTEXT("ResetToDefault", "Reset To Default");

	StructBuilder.AddCustomRow(TooltipText)
		.WholeRowContent()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.IsEnabled(this, &FToolableTimelineInstanceSettingsCustomization::CanStructResetToDefaults)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(2.f, 0.f)
			[
				SNew(SButton)
				.IsFocusable(false)
				.ToolTipText(TooltipText)
				.ContentPadding(0)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Lambda([ThisWeak]()
					{
						if (const TSharedPtr<FToolableTimelineInstanceSettingsCustomization> This = ThisWeak.Pin())
						{
							This->StructResetToDefaults();
							return FReply::Handled();
						}
						return FReply::Unhandled();
					})
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(LOCTEXT("ResetToDefaults", "Reset To Defaults"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(6.f, 0.f, 0.f, 0.f)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(TEXT("PropertyWindow.DiffersFromDefault")))
					]
				]
			]
		];

	uint32 NumChildren;
	InPropertyHandle->GetNumChildren(NumChildren);

	// Map to keep track of groups we've already created
	TMap<FName, IDetailGroup*> CategoryGroups;

	for (uint32 i = 0; i < NumChildren; ++i)
	{
		const TSharedRef<IPropertyHandle> ChildHandle = InPropertyHandle->GetChildHandle(i).ToSharedRef();

		if (!InstanceSettingsCustomization::CVarEnableSimpleViewHiddenConfig.GetValueOnAnyThread()
			&& ChildHandle->HasMetaData(TEXT("HiddenConfig")))
		{
			continue;
		}

		// Get the category name from the property metadata
		const FString CategoryString = ChildHandle->GetProperty()->GetMetaData(TEXT("Category"));
		const FName CategoryName = CategoryString.IsEmpty() ? FName(TEXT("General")) : *CategoryString;

		// Create or find the group for this category
		IDetailGroup* Group = nullptr;

		if (CategoryGroups.Contains(CategoryName))
		{
			Group = CategoryGroups[CategoryName];
		}
		else
		{
			Group = &StructBuilder.AddGroup(CategoryName, FText::FromName(CategoryName), true);
			CategoryGroups.Add(CategoryName, Group);
		}

		Group->AddPropertyRow(ChildHandle)
			.OverrideResetToDefault(FResetToDefaultOverride::Create(
				FIsResetToDefaultVisible::CreateSP(this, &FToolableTimelineInstanceSettingsCustomization::CanPropertyResetToDefault),
				FResetToDefaultHandler::CreateSP(this, &FToolableTimelineInstanceSettingsCustomization::ResetPropertyToDefault))
			);
	}
}

bool FToolableTimelineInstanceSettingsCustomization::CanStructResetToDefaults() const
{
	if (!StructHandle.IsValid())
	{
		return false;
	}

	const FStructProperty* const StructProperty = CastField<FStructProperty>(StructHandle->GetProperty());
	if (!StructProperty || !StructProperty->Struct)
	{
		return false;
	}

	const UToolableTimelineDefaultSettings* const DefaultSettings = GetDefault<UToolableTimelineDefaultSettings>();
	const void* const DefaultStructPtr = &DefaultSettings->Settings;

	TArray<void*> RawData;
	StructHandle->AccessRawData(RawData);

	for (void* const RawDataPtr : RawData)
	{
		if (RawDataPtr && !StructProperty->Struct->CompareScriptStruct(RawDataPtr, DefaultStructPtr, 0))
		{
			return true;
		}
	}

	return false;
}

void FToolableTimelineInstanceSettingsCustomization::StructResetToDefaults()
{
	if (!StructHandle.IsValid())
	{
		return;
	}

	const FStructProperty* const StructProperty = CastField<FStructProperty>(StructHandle->GetProperty());
	if (!StructProperty || !StructProperty->Struct)
	{
		return;
	}

	const UToolableTimelineDefaultSettings* const DefaultSettings = GetDefault<UToolableTimelineDefaultSettings>();
	const void* const DefaultStructPtr = &DefaultSettings->Settings;

	TArray<void*> RawData;
	StructHandle->AccessRawData(RawData);

	StructHandle->NotifyPreChange();

	for (void* const RawDataPtr : RawData)
	{
		if (RawDataPtr)
		{
			StructProperty->Struct->CopyScriptStruct(RawDataPtr, DefaultStructPtr);
		}
	}

	StructHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	StructHandle->NotifyFinishedChangingProperties();
}

bool FToolableTimelineInstanceSettingsCustomization::CanPropertyResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle) const
{
	FProperty* const ChildProperty = InPropertyHandle->GetProperty();
	if (!ChildProperty)
	{
		return false;
	}

	const UToolableTimelineDefaultSettings* const DefaultSettings = GetDefault<UToolableTimelineDefaultSettings>();
	uint8* const DefaultSettingsPtr = reinterpret_cast<uint8*>(const_cast<FToolableTimelineInstanceSettings*>(&DefaultSettings->Settings));
	uint8* const DefaultValueAddr = DefaultSettingsPtr + ChildProperty->GetOffset_ForInternal();

	TArray<void*> RawData;
	InPropertyHandle->AccessRawData(RawData);

	for (void* const RawDataPtr : RawData)
	{
		if (RawDataPtr && !ChildProperty->Identical(RawDataPtr, DefaultValueAddr))
		{
			return true;
		}
	}

	return false;
}

void FToolableTimelineInstanceSettingsCustomization::ResetPropertyToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	FProperty* const ChildProperty = InPropertyHandle->GetProperty();
	if (!ChildProperty)
	{
		return;
	}

	const UToolableTimelineDefaultSettings* const DefaultSettings = GetDefault<UToolableTimelineDefaultSettings>();
	uint8* const DefaultSettingsPtr = reinterpret_cast<uint8*>(const_cast<FToolableTimelineInstanceSettings*>(&DefaultSettings->Settings));
	uint8* const DefaultValueAddr = DefaultSettingsPtr + ChildProperty->GetOffset_ForInternal();

	TArray<void*> RawData;
	InPropertyHandle->AccessRawData(RawData);

	StructHandle->NotifyPreChange();

	for (void* const RawDataPtr : RawData)
	{
		if (RawDataPtr)
		{
			ChildProperty->CopySingleValue(RawDataPtr, DefaultValueAddr);
		}
	}

	StructHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	StructHandle->NotifyFinishedChangingProperties();
}
	
void FToolableTimelineInstanceSettingsCustomization::HandleHiddenConfigToggled(IConsoleVariable* const InConsoleVariable)
{
	if (PropertyUtilities.IsValid())
	{
		PropertyUtilities->RequestForceRefresh();
	}
}

} // namespace UE::Sequencer::ToolableTimeline

#undef LOCTEXT_NAMESPACE
