// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class IPropertyHandle;

namespace UE::Sequencer::ToolableTimeline
{

class FToolableTimelineInstanceSettingsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual ~FToolableTimelineInstanceSettingsCustomization() override;

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle
		, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle
		, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ End IPropertyTypeCustomization

protected:
	bool CanStructResetToDefaults() const;
	void StructResetToDefaults();

	bool CanPropertyResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle) const;
	void ResetPropertyToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);

	void HandleHiddenConfigToggled(IConsoleVariable* const InConsoleVariable);

	TSharedPtr<IPropertyHandle> StructHandle;

	FDelegateHandle ViewHiddenConfigCVarChangedHandle;

	TSharedPtr<IPropertyUtilities> PropertyUtilities;
};

} // namespace UE::Sequencer::ToolableTimeline
