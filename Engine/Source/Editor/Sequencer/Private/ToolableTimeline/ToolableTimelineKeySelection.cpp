// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimelineKeySelection.h"
#include "CurveEditor.h"
#include "CurveEditorSelection.h"
#include "ISequencer.h"
#include "MouseInputData.h"
#include "MVVM/Selection/Selection.h"
#include "Sequencer.h"
#include "ToolableTimeline/ToolableTimelineUtils.h"
#include "ToolableTimeSliderController.h"

namespace UE::Sequencer::ToolableTimeline
{

FToolableTimelineKeySelection::FCurveModelToChannelModelMap BuildCurveModelToChannelModelMap(const FToolableTimeline& InTimeline, FCurveEditor& InCurveEditor)
{
	FToolableTimelineKeySelection::FCurveModelToChannelModelMap OutCurveModelToChannelModel;
	OutCurveModelToChannelModel.Reserve(InCurveEditor.GetCurves().Num());

	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : InTimeline.GetChannelModels())
	{
		const TSharedPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
		if (!ChannelModel.IsValid())
		{
			continue;
		}

		const TViewModelPtr<IOutlinerExtension> LinkedOutlinerItem = ChannelModel->GetLinkedOutlinerItem();
		if (!LinkedOutlinerItem)
		{
			continue;
		}

		const TViewModelPtr<ICurveEditorTreeItemExtension> CurveEditorItem = LinkedOutlinerItem.ImplicitCast();
		if (!CurveEditorItem)
		{
			continue;
		}

		const FCurveEditorTreeItemID CurveEditorItemID = CurveEditorItem->GetCurveEditorItemID();
		if (!CurveEditorItemID.IsValid())
		{
			continue;
		}

		const FCurveEditorTreeItem* TreeItem = InCurveEditor.FindTreeItem(CurveEditorItemID);
		if (!TreeItem)
		{
			continue;
		}

		for (const FCurveModelID CurveModelID : TreeItem->GetCurves())
		{
			OutCurveModelToChannelModel.Add(CurveModelID, ChannelModel);
		}
	}

	return OutCurveModelToChannelModel;
}

TSet<FSequencerSelectedKey> BuildSelectedKeysFromCurveEditorSelection(const TMap<FCurveModelID, FKeyHandleSet>& InSelectedCurveKeys
	, const FToolableTimelineKeySelection::FCurveModelToChannelModelMap& InCurveModelToChannelModel)
{
	TSet<FSequencerSelectedKey> OutKeys;

	int32 NumSelectedKeys = 0;
	for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : InSelectedCurveKeys)
	{
		NumSelectedKeys += Pair.Value.Num();
	}
	OutKeys.Reserve(NumSelectedKeys);

	for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : InSelectedCurveKeys)
	{
		const TSharedPtr<FChannelModel>* ChannelModel = InCurveModelToChannelModel.Find(Pair.Key);
		if (!ChannelModel || !ChannelModel->IsValid())
		{
			continue;
		}

		for (const FKeyHandle KeyHandle : Pair.Value.AsArray())
		{
			FSequencerSelectedKey SelectedKey;
			SelectedKey.WeakChannel = *ChannelModel;
			SelectedKey.Section = (*ChannelModel)->GetSection();
			SelectedKey.KeyHandle = KeyHandle;
			OutKeys.Add(SelectedKey);
		}
	}

	return OutKeys;
}

bool FToolableTimelineKeySelection::AreSelectedKeySetsEqual(const TSet<FSequencerSelectedKey>& InA, const TSet<FSequencerSelectedKey>& InB)
{
	return InA.Num() == InB.Num() && InA.Includes(InB);
}

FToolableTimelineKeySelection::FToolableTimelineKeySelection(FToolableTimeline& InTimeline)
	: Timeline(InTimeline)
{
}

FToolableTimelineKeySelection::~FToolableTimelineKeySelection()
{
	Shutdown();
}

void FToolableTimelineKeySelection::Initialize()
{
	if (const TSharedPtr<FSequencer> Sequencer = Timeline.GetSequencer())
	{
		FSequencerSelection& SequencerSelection = Sequencer->GetSelection();
		SequencerKeySelectionChangedDelegate = SequencerSelection.KeySelection.OnChanged.AddRaw(this, &FToolableTimelineKeySelection::HandleSequencerKeySelectionChanged);

		EnsureCurveEditorSelectionDelegateBound(Utils::GetSequencerCurveEditor(*Sequencer));
	}
}

void FToolableTimelineKeySelection::Shutdown()
{
	UnbindCurveEditorSelectionDelegate();

	if (const TSharedPtr<FSequencer> Sequencer = Timeline.GetSequencer())
	{
		FSequencerSelection& SequencerSelection = Sequencer->GetSelection();
		SequencerSelection.KeySelection.OnChanged.Remove(SequencerKeySelectionChangedDelegate);
		SequencerKeySelectionChangedDelegate.Reset();
	}
}

void FToolableTimelineKeySelection::EnsureCurveEditorSelectionDelegateBound(const TSharedPtr<FCurveEditor>& InCurveEditor)
{
	if (!InCurveEditor.IsValid())
	{
		UnbindCurveEditorSelectionDelegate();
		return;
	}

	if (WeakCurveEditor.Pin() == InCurveEditor && CurveEditorKeySelectionChangedDelegate.IsValid())
	{
		return;
	}

	UnbindCurveEditorSelectionDelegate();

	CurveEditorKeySelectionChangedDelegate = InCurveEditor->GetSelection().OnSelectionChanged().AddRaw(this
			, &FToolableTimelineKeySelection::HandleCurveEditorKeySelectionChanged);

	WeakCurveEditor = InCurveEditor;
}

void FToolableTimelineKeySelection::UnbindCurveEditorSelectionDelegate()
{
	if (const TSharedPtr<FCurveEditor> CurveEditor = WeakCurveEditor.Pin())
	{
		CurveEditor->GetSelection().OnSelectionChanged().Remove(CurveEditorKeySelectionChangedDelegate);
	}

	CurveEditorKeySelectionChangedDelegate.Reset();
	WeakCurveEditor.Reset();
}

void FToolableTimelineKeySelection::HandleSequencerKeySelectionChanged()
{
	if (bSyncDisabled || !IsSimpleView())
	{
		return;
	}

	TGuardValue<bool> Guard(bSyncDisabled, true);

	const TSet<FSequencerSelectedKey> NewSelectedKeys = BuildSelectedKeysFromSequencer();

	if (AreSelectedKeySetsEqual(NewSelectedKeys, SelectedKeys))
	{
		return;
	}

	SelectedKeys = NewSelectedKeys;

	ResetHoveredKeysAndSelectionPreview();
	SyncKeySelectionToCurveEditor();
}

void FToolableTimelineKeySelection::HandleCurveEditorKeySelectionChanged()
{
	if (bSyncDisabled || bIgnoreCurveEditorSelectionChanged || !IsSimpleView())
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = Timeline.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<FCurveEditor> CurveEditor = Utils::GetSequencerCurveEditor(*Sequencer);
	if (!CurveEditor.IsValid() || !CurveEditor->IsBroadcasting())
	{
		return;
	}

	TGuardValue<bool> Guard(bSyncDisabled, true);

	if (!ApplySelectedKeysFromCurveEditor(BuildSelectedKeysFromCurveEditor()))
	{
		return;
	}

	SyncKeySelectionToSequencer();
}

bool FToolableTimelineKeySelection::ApplySelectedKeysFromCurveEditor(const TSet<FSequencerSelectedKey>& InSelectedKeys)
{
	const bool bSelectedChanged = !AreSelectedKeySetsEqual(SelectedKeys, InSelectedKeys);
	const bool bHoveredChanged = !HoveredKeys.IsEmpty();
	const bool bPreviewChanged = !SelectionPreview.IsEmpty();

	if (!bSelectedChanged && !bHoveredChanged && !bPreviewChanged)
	{
		return false;
	}

	SelectedKeys = InSelectedKeys;
	HoveredKeys.Reset();
	SelectionPreview.Reset();

	Timeline.GetTimeSliderController()->InvalidateKeyRendererKeyState();

	return true;
}

TSet<FSequencerSelectedKey> FToolableTimelineKeySelection::BuildSelectedKeysFromSequencer() const
{
	TSet<FSequencerSelectedKey> OutKeys;

	const TSharedPtr<FSequencer> Sequencer = Timeline.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return OutKeys;
	}

	const FKeySelection& SequencerKeySelection = Sequencer->GetSelection().KeySelection;

	OutKeys.Reserve(SequencerKeySelection.Num());

	for (const FKeyHandle KeyHandle : SequencerKeySelection)
	{
		if (const TViewModelPtr<FChannelModel> KeyModel = SequencerKeySelection.GetModelForKey(KeyHandle))
		{
			FSequencerSelectedKey SelectedKey;
			SelectedKey.WeakChannel = KeyModel;
			SelectedKey.Section = KeyModel->GetSection();
			SelectedKey.KeyHandle = KeyHandle;
			OutKeys.Add(SelectedKey);
		}
	}

	return OutKeys;
}

TSet<FSequencerSelectedKey> FToolableTimelineKeySelection::BuildSelectedKeysFromCurveEditor() const
{
	const TSharedPtr<ISequencer> Sequencer = Timeline.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return {};
	}

	const TSharedPtr<FCurveEditor> CurveEditor = Utils::GetSequencerCurveEditor(*Sequencer);
	if (!CurveEditor.IsValid())
	{
		return {};
	}

	const TMap<FCurveModelID, FKeyHandleSet>& AllSelectedCurveKeys = CurveEditor->GetSelection().GetAll();
	const FCurveModelToChannelModelMap& CurveModelToChannelModel = GetCurveModelToChannelModelMap(*CurveEditor);

	return BuildSelectedKeysFromCurveEditorSelection(AllSelectedCurveKeys, CurveModelToChannelModel);
}

const FToolableTimelineKeySelection::FCurveModelToChannelModelMap& FToolableTimelineKeySelection::GetCurveModelToChannelModelMap(FCurveEditor& InCurveEditor) const
{
	const uint32 ActiveCurvesSerialNumber = InCurveEditor.GetActiveCurvesSerialNumber();
	const uint32 ChannelModelsSerialNumber = Timeline.GetChannelModelsSerialNumber();

	const bool bNeedsRebuild = CachedCurveEditor != &InCurveEditor
		|| CachedCurveEditorActiveCurvesSerialNumber != ActiveCurvesSerialNumber
		|| CachedChannelModelsSerialNumber != ChannelModelsSerialNumber;

	if (bNeedsRebuild)
	{
		CachedCurveModelToChannelModelMap = BuildCurveModelToChannelModelMap(Timeline, InCurveEditor);
		CachedCurveEditor = &InCurveEditor;
		CachedCurveEditorActiveCurvesSerialNumber = ActiveCurvesSerialNumber;
		CachedChannelModelsSerialNumber = ChannelModelsSerialNumber;
	}

	return CachedCurveModelToChannelModelMap;
}

bool FToolableTimelineKeySelection::IsSelectionPreviewEqual(const TMap<FSequencerSelectedKey, ESelectionPreviewState>& InPreview) const
{
	if (SelectionPreview.Num() != InPreview.Num())
	{
		return false;
	}

	for (const TPair<FSequencerSelectedKey, ESelectionPreviewState>& Pair : SelectionPreview)
	{
		const ESelectionPreviewState* OtherState = InPreview.Find(Pair.Key);
		if (!OtherState || *OtherState != Pair.Value)
		{
			return false;
		}
	}

	return true;
}

bool FToolableTimelineKeySelection::HasAnySelectedKeys() const
{
	return !SelectedKeys.IsEmpty();
}

const TSet<FSequencerSelectedKey>& FToolableTimelineKeySelection::GetSelectedKeys() const
{
	return SelectedKeys;
}

bool FToolableTimelineKeySelection::ClearSelectedKeys(const bool bInSync)
{
	if (SelectedKeys.IsEmpty())
	{
		return false;
	}

	SelectedKeys.Reset();

	Timeline.GetTimeSliderController()->InvalidateKeyRendererKeyState();

	if (bInSync)
	{
		SyncSelectionToSequencerAndCurveEditor();
	}

	return true;
}

bool FToolableTimelineKeySelection::SetSelectedKeys(const TSet<FSequencerSelectedKey>& InKeys, const bool bInSync)
{
	if (AreSelectedKeySetsEqual(SelectedKeys, InKeys))
	{
		return false;
	}

	SelectedKeys = InKeys;

	Timeline.GetTimeSliderController()->InvalidateKeyRendererKeyState();

	if (bInSync)
	{
		SyncSelectionToSequencerAndCurveEditor();
	}

	return true;
}

bool FToolableTimelineKeySelection::AddSelectedKeys(const TSet<FSequencerSelectedKey>& InKeys, const bool bInSync)
{
	const int32 OldNum = SelectedKeys.Num();

	for (const FSequencerSelectedKey& Key : InKeys)
	{
		SelectedKeys.Add(Key);
	}

	if (SelectedKeys.Num() == OldNum)
	{
		return false;
	}

	Timeline.GetTimeSliderController()->InvalidateKeyRendererKeyState();

	if (bInSync)
	{
		SyncSelectionToSequencerAndCurveEditor();
	}

	return true;
}

bool FToolableTimelineKeySelection::HasAnyHoveredKeys() const
{
	return !HoveredKeys.IsEmpty();
}

const TSet<FSequencerSelectedKey>& FToolableTimelineKeySelection::GetHoveredKeys() const
{
	return HoveredKeys;
}

bool FToolableTimelineKeySelection::ClearHoveredKeys(const bool bInSync)
{
	if (HoveredKeys.IsEmpty())
	{
		return false;
	}

	HoveredKeys.Reset();

	Timeline.GetTimeSliderController()->InvalidateKeyRendererKeyState();

	if (bInSync)
	{
		SyncSelectionToSequencerAndCurveEditor();
	}

	return true;
}

bool FToolableTimelineKeySelection::SetHoveredKeys(const TSet<FSequencerSelectedKey>& InKeys)
{
	if (AreSelectedKeySetsEqual(HoveredKeys, InKeys))
	{
		return false;
	}

	HoveredKeys = InKeys;

	Timeline.GetTimeSliderController()->InvalidateKeyRendererKeyState();

	return true;
}

bool FToolableTimelineKeySelection::ClearSelectedAndHoveredKeys(const bool bInSync)
{
	const bool bSelectedChanged = !SelectedKeys.IsEmpty();
	const bool bHoveredChanged = !HoveredKeys.IsEmpty();
	const bool bPreviewChanged = !SelectionPreview.IsEmpty();

	if (!bSelectedChanged && !bHoveredChanged && !bPreviewChanged)
	{
		return false;
	}

	SelectedKeys.Reset();
	HoveredKeys.Reset();
	SelectionPreview.Reset();

	Timeline.GetTimeSliderController()->InvalidateKeyRendererKeyState();

	if (bInSync)
	{
		SyncSelectionToSequencerAndCurveEditor();
	}

	return true;
}

bool FToolableTimelineKeySelection::SetSelectedAndHoveredKeys(const TSet<FSequencerSelectedKey>& InSelectedKeys, const bool bInSync)
{
	const bool bSelectedChanged = !AreSelectedKeySetsEqual(SelectedKeys, InSelectedKeys);
	const bool bHoveredChanged = !AreSelectedKeySetsEqual(HoveredKeys, InSelectedKeys);
	const bool bPreviewChanged = !SelectionPreview.IsEmpty();

	if (!bSelectedChanged && !bHoveredChanged && !bPreviewChanged)
	{
		return false;
	}

	SelectedKeys = InSelectedKeys;
	HoveredKeys = InSelectedKeys;
	SelectionPreview.Reset();

	Timeline.GetTimeSliderController()->InvalidateKeyRendererKeyState();

	if (bInSync)
	{
		SyncSelectionToSequencerAndCurveEditor();
	}

	return true;
}

bool FToolableTimelineKeySelection::HasAnyPreviewSelectedKeys() const
{
	return !SelectionPreview.IsEmpty();
}

const TMap<FSequencerSelectedKey, ESelectionPreviewState>& FToolableTimelineKeySelection::GetSelectionPreview() const
{
	return SelectionPreview;
}

bool FToolableTimelineKeySelection::ClearSelectionPreview()
{
	if (SelectionPreview.IsEmpty())
	{
		return false;
	}

	SelectionPreview.Reset();

	Timeline.GetTimeSliderController()->InvalidateKeyRendererKeyState();

	return true;
}

bool FToolableTimelineKeySelection::SetSelectionPreview(const TMap<FSequencerSelectedKey, ESelectionPreviewState>& InPreview)
{
	if (IsSelectionPreviewEqual(InPreview))
	{
		return false;
	}

	SelectionPreview = InPreview;

	Timeline.GetTimeSliderController()->InvalidateKeyRendererKeyState();

	return true;
}

bool FToolableTimelineKeySelection::ResetHoveredKeysAndSelectionPreview()
{
	const bool bHoveredChanged = !HoveredKeys.IsEmpty();
	const bool bPreviewChanged = !SelectionPreview.IsEmpty();

	if (!bHoveredChanged && !bPreviewChanged)
	{
		return false;
	}

	HoveredKeys.Reset();
	SelectionPreview.Reset();

	Timeline.GetTimeSliderController()->InvalidateKeyRendererKeyState();

	return true;
}

bool FToolableTimelineKeySelection::IsKeySelected(const FSequencerSelectedKey& InKey) const
{
	return SelectedKeys.Contains(InKey);
}

bool FToolableTimelineKeySelection::IsKeySelected(const TViewModelPtr<IKeyExtension>& InOwner, const FKeyHandle InKey) const
{
	const TViewModelPtr<FChannelModel> ChannelModel = InOwner.ImplicitCast();
	if (!ChannelModel.IsValid())
	{
		return false;
	}

	UMovieSceneSection* const Section = ChannelModel->GetSection();
	if (!Section)
	{
		return false;
	}

	const FSequencerSelectedKey SelectedKey(*Section, ChannelModel, InKey);
	return SelectedKeys.Contains(SelectedKey);
}

bool FToolableTimelineKeySelection::IsKeyHovered(const FSequencerSelectedKey& InKey) const
{
	return HoveredKeys.Contains(InKey);
}

bool FToolableTimelineKeySelection::IsKeyHovered(const TViewModelPtr<IKeyExtension>& InOwner, const FKeyHandle InKey) const
{
	const TViewModelPtr<FChannelModel> ChannelModel = InOwner.ImplicitCast();
	if (!ChannelModel.IsValid())
	{
		return false;
	}

	UMovieSceneSection* const Section = ChannelModel->GetSection();
	if (!Section)
	{
		return false;
	}

	const FSequencerSelectedKey SelectedKey(*Section, ChannelModel, InKey);
	return HoveredKeys.Contains(SelectedKey);
}

ESelectionPreviewState FToolableTimelineKeySelection::GetKeyPreviewSelectionState(const FSequencerSelectedKey& InKey) const
{
	const ESelectionPreviewState* KeySelectionPreview = SelectionPreview.Find(InKey);
	return KeySelectionPreview ? *KeySelectionPreview : ESelectionPreviewState::Undefined;
}

EKeySelectionPreviewState FToolableTimelineKeySelection::GetPreviewSelectionState(const TViewModelPtr<IKeyExtension>& InOwner, const FKeyHandle InKey) const
{
	const TViewModelPtr<FChannelModel> ChannelModel = InOwner.ImplicitCast();
	if (!ChannelModel.IsValid())
	{
		return EKeySelectionPreviewState::Undefined;
	}

	UMovieSceneSection* const Section = ChannelModel->GetSection();
	if (!Section)
	{
		return EKeySelectionPreviewState::Undefined;
	}

	const ESelectionPreviewState KeyPreviewSelectionState = GetKeyPreviewSelectionState(FSequencerSelectedKey(*Section, ChannelModel, InKey));
	switch(KeyPreviewSelectionState)
	{
	default:
	case ESelectionPreviewState::Undefined: return EKeySelectionPreviewState::Undefined;
	case ESelectionPreviewState::Selected: return EKeySelectionPreviewState::Selected;
	case ESelectionPreviewState::NotSelected: return EKeySelectionPreviewState::NotSelected;
	}
}

bool FToolableTimelineKeySelection::SelectKeys(const TSet<FSequencerSelectedKey>& InKeys, const bool bInAddToSelection)
{
	bool bSelectionChanged = false;

	if (InKeys.IsEmpty())
	{
		bSelectionChanged |= ClearSelectedKeys(false);
		bSelectionChanged |= ClearHoveredKeys(false);
		bSelectionChanged |= ClearSelectionPreview();
	}
	else
	{
		if (bInAddToSelection)
		{
			bSelectionChanged |= AddSelectedKeys(InKeys, false);
		}
		else
		{
			bSelectionChanged |= SetSelectedKeys(InKeys, false);
		}

		bSelectionChanged |= SetHoveredKeys(InKeys);
	}

	if (bSelectionChanged)
	{
		Timeline.SyncSelectionToSequencerAndCurveEditor();
	}

	return bSelectionChanged;
}

bool FToolableTimelineKeySelection::SelectKeysUnderMouse(const FMouseInputData& InMouseInput, const bool bInAddToSelection)
{
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = InMouseInput.Timeline->GetTimeSliderController();

	TSet<FSequencerSelectedKey> KeysUnderMouse;
	TimeSliderController->GetKeysUnderMouse(InMouseInput, InMouseInput.PointerEvent.GetScreenSpacePosition(), KeysUnderMouse);

	return SelectKeys(KeysUnderMouse, bInAddToSelection);
}

bool FToolableTimelineKeySelection::RefreshSelectedKeysFromCurveEditor()
{
	if (!IsSimpleView())
	{
		return false;
	}

	const TSet<FSequencerSelectedKey> CurveEditorSelectedKeys = BuildSelectedKeysFromCurveEditor();
	return ApplySelectedKeysFromCurveEditor(CurveEditorSelectedKeys);
}

void FToolableTimelineKeySelection::SyncKeySelectionToSequencer()
{
	if (!IsSimpleView())
	{
		return;
	}

	TGuardValue<bool> Guard(bSyncDisabled, true);

	const TSharedPtr<FSequencer> Sequencer = Timeline.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	FKeySelection& KeySelection = Sequencer->GetSelection().KeySelection;

	// Simple view maintains its own cache-driven key selection. Exporting that selection into
	// Sequencer's global FKeySelection causes SequencerSelection to prune outliner selection down
	// to the owners of the selected keys, which breaks multi-object key selection in
	// SelectedAndPinned mode by removing the other cached channels. Keep Sequencer's key selection
	// empty here and let simple view + Curve Editor remain the authority for local key selection.
	if (KeySelection.Num() > 0)
	{
		const FSelectionEventSuppressor SuppressEvents(&Sequencer->GetSelection());
		KeySelection.Empty();
	}
}

void FToolableTimelineKeySelection::SyncKeySelectionToCurveEditor()
{
	if (!IsSimpleView())
	{
		return;
	}

	TGuardValue<bool> Guard(bSyncDisabled, true);

	const TSharedPtr<ISequencer> Sequencer = Timeline.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<FCurveEditor> CurveEditor = Utils::GetSequencerCurveEditor(*Sequencer);
	if (!CurveEditor.IsValid())
	{
		UnbindCurveEditorSelectionDelegate();
		return;
	}

	// Always ensure the curve editor key selection change delegate is bound since it may not be available on initialization
	EnsureCurveEditorSelectionDelegateBound(CurveEditor);

	TMap<TSharedPtr<FChannelModel>, TArray<FKeyHandle>> SelectedKeysByChannel;
	SelectedKeysByChannel.Reserve(SelectedKeys.Num());

	for (const FSequencerSelectedKey& SelectedKey : SelectedKeys)
	{
		const TSharedPtr<FChannelModel> ChannelModel = SelectedKey.WeakChannel.Pin();
		if (!ChannelModel.IsValid())
		{
			continue;
		}

		SelectedKeysByChannel.FindOrAdd(ChannelModel).Add(SelectedKey.KeyHandle);
	}

	FCurveEditorSelection& CurveEditorSelection = CurveEditor->GetSelection();

	// Suspend Curve Editor model broadcasts while we mutate key selection so Sequencer's
	// deferred curve-tree/outliner sync does not re-enter us mid-update.
	CurveEditor->SuspendBroadcast();
	ON_SCOPE_EXIT
	{
		CurveEditor->ResumeBroadcast();
	};

	const FScopedIgnoreCurveEditorSelectionChanged IgnoreCurveEditorSelectionChanged(*this);

	TMap<FCurveModelID, TArray<FKeyHandle>> SelectedHandlesByCurveModel;
	const FCurveModelToChannelModelMap& CurveModelToChannelModel = GetCurveModelToChannelModelMap(*CurveEditor);
	for (const TPair<FCurveModelID, TSharedPtr<FChannelModel>>& Pair : CurveModelToChannelModel)
	{
		if (!Pair.Value.IsValid())
		{
			continue;
		}

		const TArray<FKeyHandle>* SelectedHandles = SelectedKeysByChannel.Find(Pair.Value);
		if (!SelectedHandles)
		{
			continue;
		}

		SelectedHandlesByCurveModel.Add(Pair.Key, *SelectedHandles);
	}

	CurveEditorSelection.Replace(SelectedHandlesByCurveModel, ECurvePointType::Key);
}

void FToolableTimelineKeySelection::SyncSelectionToSequencerAndCurveEditor()
{
	if (bSyncDisabled || !IsSimpleView())
	{
		return;
	}

	TGuardValue<bool> Guard(bSyncDisabled, true);

	SyncKeySelectionToSequencer();
	SyncKeySelectionToCurveEditor();
}

bool FToolableTimelineKeySelection::IsSimpleView() const
{
	const TSharedPtr<FSequencer> Sequencer = Timeline.GetSequencer();
	return Sequencer.IsValid() && Sequencer->IsSimpleView();
}

} // namespace UE::Sequencer::ToolableTimeline
