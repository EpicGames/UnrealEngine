// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Views/KeyRenderer.h"
#include "SequencerSelectedKey.h"
#include "SequencerSelectionPreview.h"

class FCurveEditor;
struct FCurveModelID;
struct FFrameNumber;

namespace UE::Sequencer::ToolableTimeline
{

class FToolableTimeline;
struct FMouseInputData;

/**
 * Maintains simple-view key selection state for the toolable timeline and synchronizes that
 * local selection with Curve Editor when possible, while intentionally avoiding Sequencer's
 * global key-selection path when it would collapse the current cached channel set.
 */
class FToolableTimelineKeySelection : public IKeyRendererInterface
{
public:
	using FCurveModelToChannelModelMap = TMap<FCurveModelID, TSharedPtr<FChannelModel>>;

	static bool AreSelectedKeySetsEqual(const TSet<FSequencerSelectedKey>& InA, const TSet<FSequencerSelectedKey>& InB);

	explicit FToolableTimelineKeySelection(FToolableTimeline& InTimeline);
	virtual ~FToolableTimelineKeySelection() override;

	void Initialize();

	void Shutdown();

	const TSet<FSequencerSelectedKey>& GetSelectedKeys() const;
	bool ClearSelectedKeys(const bool bInSync = true);
	bool SetSelectedKeys(const TSet<FSequencerSelectedKey>& InKeys, const bool bInSync = true);
	bool AddSelectedKeys(const TSet<FSequencerSelectedKey>& InKeys, const bool bInSync = true);

	const TSet<FSequencerSelectedKey>& GetHoveredKeys() const;
	bool ClearHoveredKeys(const bool bInSync = true);
	bool SetHoveredKeys(const TSet<FSequencerSelectedKey>& InKeys);

	bool ClearSelectedAndHoveredKeys(const bool bInSync = true);
	bool SetSelectedAndHoveredKeys(const TSet<FSequencerSelectedKey>& InSelectedKeys, const bool bInSync = true);

	const TMap<FSequencerSelectedKey, ESelectionPreviewState>& GetSelectionPreview() const;
	bool ClearSelectionPreview();
	bool SetSelectionPreview(const TMap<FSequencerSelectedKey, ESelectionPreviewState>& InPreview);

	bool ResetHoveredKeysAndSelectionPreview();

	bool IsKeySelected(const FSequencerSelectedKey& InKey) const;
	bool IsKeyHovered(const FSequencerSelectedKey& InKey) const;

	ESelectionPreviewState GetKeyPreviewSelectionState(const FSequencerSelectedKey& InKey) const;

	bool SelectKeys(const TSet<FSequencerSelectedKey>& InKeys, const bool bInAddToSelection = false);
	bool SelectKeysUnderMouse(const FMouseInputData& InMouseInput, const bool bInAddToSelection = false);

	bool RefreshSelectedKeysFromCurveEditor();

	void SyncSelectionToSequencerAndCurveEditor();

	//~ Begin IKeyRendererInterface

	virtual bool HasAnySelectedKeys() const override;
	virtual bool HasAnyPreviewSelectedKeys() const override;
	virtual bool HasAnyHoveredKeys() const override;

	virtual bool IsKeySelected(const TViewModelPtr<IKeyExtension>& InOwner, const FKeyHandle InKey) const override;
	virtual bool IsKeyHovered(const TViewModelPtr<IKeyExtension>& InOwner, const FKeyHandle InKey) const override;

	virtual EKeySelectionPreviewState GetPreviewSelectionState(const TViewModelPtr<IKeyExtension>& InOwner, const FKeyHandle InKey) const override;

	//~ End IKeyRendererInterface

private:
	class FScopedIgnoreCurveEditorSelectionChanged
	{
	public:
		explicit FScopedIgnoreCurveEditorSelectionChanged(FToolableTimelineKeySelection& InOwner)
			: Owner(InOwner)
		{
			++Owner.IgnoreCurveEditorSelectionChangedCount;
			Owner.bIgnoreCurveEditorSelectionChanged = true;
		}

		~FScopedIgnoreCurveEditorSelectionChanged()
		{
			const int32 NewCount = --Owner.IgnoreCurveEditorSelectionChangedCount;
			check(NewCount >= 0);

			if (NewCount == 0)
			{
				Owner.bIgnoreCurveEditorSelectionChanged = false;
			}
		}

	private:
		FToolableTimelineKeySelection& Owner;
	};

	void EnsureCurveEditorSelectionDelegateBound(const TSharedPtr<FCurveEditor>& InCurveEditor);
	void UnbindCurveEditorSelectionDelegate();

	void HandleSequencerKeySelectionChanged();
	void HandleCurveEditorKeySelectionChanged();

	bool ApplySelectedKeysFromCurveEditor(const TSet<FSequencerSelectedKey>& InSelectedKeys);

	TSet<FSequencerSelectedKey> BuildSelectedKeysFromSequencer() const;
	TSet<FSequencerSelectedKey> BuildSelectedKeysFromCurveEditor() const;

	const FCurveModelToChannelModelMap& GetCurveModelToChannelModelMap(FCurveEditor& InCurveEditor) const;

	bool IsSelectionPreviewEqual(const TMap<FSequencerSelectedKey, ESelectionPreviewState>& InPreview) const;

	void SyncKeySelectionToSequencer();
	void SyncKeySelectionToCurveEditor();

	bool IsSimpleView() const;

	FToolableTimeline& Timeline;

	bool bSyncDisabled = false;
	bool bIgnoreCurveEditorSelectionChanged = false;
	int32 IgnoreCurveEditorSelectionChangedCount = 0;

	TSet<FSequencerSelectedKey> SelectedKeys;
	TSet<FSequencerSelectedKey> HoveredKeys;
	TMap<FSequencerSelectedKey, ESelectionPreviewState> SelectionPreview;

	/** Last curve editor instance that delegates are bound for syncing to */
	TWeakPtr<FCurveEditor> WeakCurveEditor;
	mutable FCurveModelToChannelModelMap CachedCurveModelToChannelModelMap;
	mutable const FCurveEditor* CachedCurveEditor = nullptr;
	mutable uint32 CachedCurveEditorActiveCurvesSerialNumber = MAX_uint32;
	mutable uint32 CachedChannelModelsSerialNumber = MAX_uint32;

	FDelegateHandle CurveEditorKeySelectionChangedDelegate;
	FDelegateHandle SequencerKeySelectionChangedDelegate;
};

} // namespace UE::Sequencer::ToolableTimeline
