// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerSection.h"
#include "Engine/DeveloperSettings.h"
#include "Styling/StyleColors.h"
#include "ToolableTimeline/Caches/ToolableTimelineChannelCache.h"
#include "ToolableTimelineSettings.generated.h"

/** The style to draw a key. This needs to map exactly to UE::Sequencer::EKeyRendererStyle values. */
UENUM()
enum class EKeyRendererStyleConfig : uint8
{
	Circle,
	Diamond,
	Line,
	FrameBlock
};

/** Style to draw the scrubber head */
UENUM()
enum class EToolableTimelineScrubHeadStyle : uint8
{
	OldTextOnly,
	FrameBubble,
	NewCenteredBlock
};

USTRUCT()
struct FToolableTimelineInstanceSettings 
{
	GENERATED_BODY()

	/** Determines what keys are displayed on the timeline */
	UPROPERTY(Config)
	ETimelineKeyDisplay KeyDisplay = ETimelineKeyDisplay::SelectedAndPinned;

	/** Draw the frame labels on the bottom of the timeline instead of the top */
	UPROPERTY(Config, EditAnywhere, Category = "General")
	TEnumAsByte<EVerticalAlignment> LabelVerticalAlignment = VAlign_Top;

	/** Draw the scrubber over the whole frame instead of just a line at the beginning of the frame */
	UPROPERTY(Config, EditAnywhere, Category = "General")
	bool bDrawWholeFrameScrubber = true;

	/** Style to draw the scrubber head */
	UPROPERTY(Config, EditAnywhere, Category = "General")
	EToolableTimelineScrubHeadStyle ScrubHeadStyle = EToolableTimelineScrubHeadStyle::NewCenteredBlock;

	/** Style to draw the timeline keys */
	UPROPERTY(Config, EditAnywhere, Category = "General")
	EKeyRendererStyleConfig KeyDrawStyle = EKeyRendererStyleConfig::FrameBlock;

	/** Padding in seconds to apply to the min and max range when framing all keys */
	UPROPERTY(Config, EditAnywhere, Category = "General", meta = (ClampMin = "0", UIMin = "0"))
	double FrameKeysSecondsPadding = .25;

	/** Show the drag range indicator on the timeline */
	UPROPERTY(Config, EditAnywhere, Category = "General")
	bool bShowDragRangeIndicator = true;

	/** Require holding Control to click and drag keys in the timeline. Disable to allow direct key dragging. */
	UPROPERTY(Config, EditAnywhere, Category = "Behavior")
	bool bUseControlModifierToMoveKeys = true;

	/** Deactivate the active tool when the Sequencer outliner selection changes */
	UPROPERTY(Config, EditAnywhere, Category = "Behavior")
	bool bDeactivateToolOnOutlinerSelectionChange = true;

	/** Uses virtual mouse deltas to continue scrubbing outside the view area when the mouse cursor
	 * is no longer visually moving on the screen. If false, locks the scrub position to always stay
	 * within the visible view range. */
	UPROPERTY(Config, EditAnywhere, Category = "Behavior")
	bool bUseVirtualScrub = true;

	/** Speed multiplier for virtual scrubbing when the mouse cursor is no longer visually moving on the screen */
	UPROPERTY(Config, EditAnywhere, Category = "Behavior", meta = (ClampMin = "0.1", ClampMax = "1.5", UIMin = "0.1", UIMax = "1.5"))
	float VirtualScrubSpeed = 0.5f;

	/** Select keys on the scrub frame when clicking a key to scrub to its time */
	UPROPERTY(Config, EditAnywhere, Category = "Behavior")
	bool bSelectScrubFrameKeys = false;

	/** Clear the key selection when beginning a scrub from empty timeline space */
	UPROPERTY(Config, EditAnywhere, Category = "Behavior")
	bool bClearSelectionOnScrub = true;

	/** Throttle expensive key time updates while dragging timeline tools */
	UPROPERTY(Config, EditAnywhere, Category = "Behavior")
	bool bUseThrottledUpdates = true;

	/** If true, draw grid color blocks */
	UPROPERTY(Config, EditAnywhere, Category = "Color Block Grid")
	bool bUseGridColorBlocks = false;

	/** Size in frames to use for each grid color block. Every GridColorBlockSize frames, the color will alternate. */
	UPROPERTY(Config, EditAnywhere, Category = "Color Block Grid", meta = (ClampMin = "1", ClampMax = "1000", UIMin = "1", UIMax = "1000"))
	int32 GridColorBlockSize = 10;

	/** Color to use for grid blocks */
	UPROPERTY(Config, EditAnywhere, Category = "Color Block Grid")
	FLinearColor GridBlockColor = FColor(36, 36, 36);

	/** Alternate color to use for grid blocks */
	UPROPERTY(Config, EditAnywhere, Category = "Color Block Grid")
	FLinearColor GridBlockAlternateColor = FColor(56, 56, 56);

	/** Height of the timeline (not including the toolbar above and time range slider below) */
	UPROPERTY(Config, EditAnywhere, Category = "Sizes", meta = (ClampMin = "28", ClampMax = "48", UIMin = "28", UIMax = "48"))
	int32 Height = 40;

	/** Font size to use for major ticks */
	UPROPERTY(Config, EditAnywhere, Category = "Sizes", meta = (ClampMin = "5", ClampMax = "10", UIMin = "5", UIMax = "10"))
	float MajorTickFontSize = 8.f;

	/** Font size to use for the current scrubbing frame */
	UPROPERTY(Config, EditAnywhere, Category = "Sizes", meta = (ClampMin = "5", ClampMax = "16", UIMin = "5", UIMax = "16"))
	float ScrubFontSize = 9.f;

	/** If true, uses bold font for the scrub text */
	UPROPERTY(Config, EditAnywhere, Category = "Sizes")
	bool ScrubTextBold = false;

	/** Font size to use for the frame number bubble text */
	UPROPERTY(Config, EditAnywhere, Category = "Sizes", meta = (ClampMin = "5", ClampMax = "16", UIMin = "5", UIMax = "16"))
	float FrameNumberBubbleFontSize = 9.f;

	/** Padding to use for the frame number bubble text */
	UPROPERTY(Config, EditAnywhere, Category = "Sizes", meta = (ClampMin = "0", ClampMax = "16", UIMin = "0", UIMax = "16"))
	FMargin FrameNumberBubblePadding = FMargin(3.f, 2.f);

	/** Mimimum tick size to use for a minor tick before the minor ticks are hidden */
	UPROPERTY(Config, EditAnywhere, Category = "Sizes", meta = (ClampMin = "3", ClampMax = "100", UIMin = "3", UIMax = "100"))
	float MinMinorTickSize = 30.f;

	/** Desired major tick size */
	UPROPERTY(Config, EditAnywhere, Category = "Sizes", meta = (ClampMin = "5", ClampMax = "1000", UIMin = "5", UIMax = "1000"))
	float DesiredMajorTickSize = 120.f;

	/** Color used for the background area where keys are drawn */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor FrameBackgroundColor = FStyleColors::Panel.GetSpecifiedColor();

	/** Color used for the label area where tick marks are drawn */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor LabelBackgroundColor = FStyleColors::Background.GetSpecifiedColor();

	/** Color used for the scrub handle */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor ScrubFrameOverlayColor = FLinearColor(.27f, .27f, .27f, .3f);

	/** Color used for the scrub handle text */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor ScrubTextColor = FLinearColor(1.f, 1.f, 1.f, 1.f);

	/** Color used for the scrub frame line accents and text bubble background */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor ScrubColor = FStyleColors::AccentRed.GetSpecifiedColor();

	/** Color used for the non-evaluating scrub handle */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor NonEvaluatingScrubColor = FStyleColors::AccentYellow.GetSpecifiedColor();

	/** Color used for the non-evaluating scrub handle text */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor NonEvaluatingScrubTextColor = FLinearColor(0.f, 0.f, 0.f, 1.f);

	/** Color used for the time-warped scrub handle */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor TimeWarpedScrubColor = FStyleColors::AccentOrange.GetSpecifiedColor();

	/** Color used for the time-warped scrub handle text */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor TimeWarpedScrubTextColor = FLinearColor(0.f, 0.f, 0.f, 1.f);

	/** Default frame tick color */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor TickColor = FLinearColor(1.f, 1.f, 1.f, 1.f);

	/** Default frame tick text color */
    UPROPERTY(Config, EditAnywhere, Category = "Colors")
    FLinearColor TickTextColor = FLinearColor(.6f, .6f, .6f, 1.f);

	/** Playback range start color */
	UPROPERTY(Config, EditAnywhere, Category = "Colors", meta = (HiddenConfig))
	FLinearColor PlaybackRangeStartColor = FLinearColor(.1f, .5f, .1f, 1.f);

	/** Playback range end color */
	UPROPERTY(Config, EditAnywhere, Category = "Colors", meta = (HiddenConfig))
	FLinearColor PlaybackRangeEndColor = FLinearColor(.5f, .01f, .01f, 1.f);

	/** Fill color to use to display keys on the timeline */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor KeyColor = FColor(134, 138, 253);

	/** Fill color to use to blend with the KeyColor to display hovered keys on the timeline */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor HoveredKeyColor = FLinearColor::White;

	/** Fill color to use to display selected keys on the timeline */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor KeySelectionColor = FLinearColor(.851f, .851f, .851f, 1.f);

	/** Fill color to use to display partial keys on the timeline */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor PartialKeyColor = FColor(63, 64, 118);

	/** Fill color to use to display selected partial keys on the timeline */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor PartialKeySelectionColor = KeySelectionColor;

	/** Color used for the tool handles */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor ToolHandleColor = FStyleColors::Select.GetSpecifiedColor();

	/** Color used to display the area the tool operates on */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor ToolRangeColor = FStyleColors::Select.GetSpecifiedColor().CopyWithNewOpacity(.2f);

	/** Color used for the frame number bubble */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor FrameNumberBubbleColor = FLinearColor::White;

	/** Color used for the frame number bubble text */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor FrameNumberBubbleTextColor = FLinearColor::Black;

	/** Color used for the drag indicator lines */
	UPROPERTY(Config, EditAnywhere, Category = "Colors")
	FLinearColor DragRangeIndicatorLineColor = FLinearColor(1.0f, .0f, .0f, .5f);
};

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, DefaultConfig)
class UToolableTimelineDefaultSettings : public UDeveloperSettings
{
public:
	GENERATED_BODY()

	/** Saved defaults for new timeline tool instances */
	UPROPERTY()
	FToolableTimelineInstanceSettings Settings;
};

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, DefaultConfig)
class UToolableTimelineSettings : public UDeveloperSettings
{
public:
	GENERATED_BODY()

	UPROPERTY(Config, EditAnywhere, Category="Timeline Settings", meta = (ShowOnlyInnerProperties))
	FToolableTimelineInstanceSettings Settings;
};
