// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimelineUtils.h"
#include "Channels/MovieSceneChannel.h"
#include "CurveEditor.h"
#include "IKeyArea.h"
#include "ISequencer.h"
#include "MovieSceneSection.h"
#include "MVVM/CurveEditorExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "Sequencer.h"
#include "SequencerSelectedKey.h"
#include "ToolableTimeline/ToolableTimeline.h"

#define LOCTEXT_NAMESPACE "ToolableTimelineUtils"

namespace UE::Sequencer::ToolableTimeline::Utils
{

using namespace UE::Sequencer;

TSet<FSequencerSelectedKey> KeyRendererResultToSelectedKeys(const TArrayView<const FKeyRenderer::FKeysForModel>& InKeys)
{
	TSet<FSequencerSelectedKey> OutSelectedKeys;

	if (InKeys.IsEmpty())
	{
		return OutSelectedKeys;
	}

	// Reserve once to avoid repeated growth/reserve inside the loop
	int32 TotalKeys = 0;
	for (const FKeyRenderer::FKeysForModel& Result : InKeys)
	{
		TotalKeys += Result.Keys.Num();
	}

	OutSelectedKeys.Reserve(TotalKeys);

	for (const FKeyRenderer::FKeysForModel& Result : InKeys)
	{
		if (Result.Keys.IsEmpty())
		{
			continue;
		}

		const TViewModelPtr<FChannelModel> Channel = Result.Model.ImplicitCast();
		if (!Channel.IsValid())
		{
			continue;
		}

		UMovieSceneSection* const InSectionObject = Channel->GetSection();
		if (!InSectionObject)
		{
			continue;
		}

		for (const FKeyHandle Key : Result.Keys)
		{
			if (Key.IsValid())
			{
				OutSelectedKeys.Emplace(FSequencerSelectedKey(*InSectionObject, Channel, Key));
			}
		}
	}

	return OutSelectedKeys;
}

TOptional<FFrameTime> GetFirstKeyTime(const TSet<FSequencerSelectedKey>& InKeys)
{
	if (InKeys.IsEmpty())
	{
		return TOptional<FFrameTime>();
	}

	const FSequencerSelectedKey FirstKey = InKeys.Array()[0];

	const TViewModelPtr<FChannelModel> ChannelModel = FirstKey.WeakChannel.Pin();
	if (!ChannelModel.IsValid())
	{
		return TOptional<FFrameTime>();
	}

	const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea();
	if (!KeyArea.IsValid())
	{
		return TOptional<FFrameTime>();
	}

	TArray<FKeyHandle> KeyHandles;
	KeyHandles.Add(FirstKey.KeyHandle);

	TArray<FFrameNumber> KeyTimes;
	KeyTimes.SetNumUninitialized(1);
	KeyArea->GetKeyTimes(KeyHandles, KeyTimes);

	return FFrameTime(KeyTimes[0]);
}

bool GetMinMaxKeyRange(const TArray<FFrameNumber>& InKeyTimes, TRange<FFrameNumber>& OutRange)
{
	if (InKeyTimes.IsEmpty())
	{
		return false;
	}

	FFrameNumber MinTime = InKeyTimes[0];
	FFrameNumber MaxTime = InKeyTimes[0];

	for (const FFrameNumber& Time : InKeyTimes)
	{
		MinTime = FMath::Min(MinTime, Time);
		MaxTime = FMath::Max(MaxTime, Time);
	}

	OutRange = TRange<FFrameNumber>(MinTime, MaxTime);

	return MinTime != MaxTime;
}

TSharedPtr<FCurveEditor> GetSequencerCurveEditor(const ISequencer& InSequencer)
{
	const FCurveEditorExtension* const CurveEditorExtension = InSequencer.GetViewModel()->CastDynamic<FCurveEditorExtension>();
	return CurveEditorExtension ? CurveEditorExtension->GetCurveEditor() : nullptr;
}

TRange<FFrameNumber> NormalizeRange(const TRange<FFrameNumber>& InRange)
{
	FFrameNumber StartInclusive;
	FFrameNumber EndExclusive;

	if (!InRange.GetLowerBound().IsOpen())
	{
		StartInclusive = InRange.GetLowerBoundValue();
		if (InRange.GetLowerBound().IsExclusive())
		{
			StartInclusive = StartInclusive + 1;
		}
	}

	if (!InRange.GetUpperBound().IsOpen())
	{
		EndExclusive = InRange.GetUpperBoundValue();
		if (InRange.GetUpperBound().IsInclusive())
		{
			EndExclusive = EndExclusive + 1;
		}
	}
	else
	{
		EndExclusive = StartInclusive + 1;
	}

	if (EndExclusive <= StartInclusive)
	{
		EndExclusive = StartInclusive + 1;
	}

	return TRange<FFrameNumber>(StartInclusive, EndExclusive);
}

FString TickFrameToString(const ISequencer& InSequencer
	, const FFrameNumber& InFrame, const bool bInRemoveLeadingZeros, const bool bInUnsigned)
{
	const TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface = InSequencer.GetNumericTypeInterface();
	if (!NumericTypeInterface.IsValid())
	{
		return FString();
	}

	const FString TickFrameString = NumericTypeInterface->ToString(InFrame.Value);

	return CleanTickFrameString(InSequencer, TickFrameString, bInRemoveLeadingZeros, bInUnsigned);
}

FString CleanTickFrameString(const ISequencer& InSequencer
	, const FString& InString, const bool bInRemoveLeadingZeros, const bool bInUnsigned)
{
	FString OutString = InString;

	const bool bIsNegative = !bInUnsigned && OutString.StartsWith(TEXT("-"));

	OutString.ReplaceInline(TEXT("*"), TEXT(""));
	OutString.ReplaceInline(TEXT(" "), TEXT(""));
	OutString.ReplaceInline(TEXT("-"), TEXT(""));

	const USequencerSettings* const SequencerSettings = InSequencer.GetSequencerSettings();
	if (SequencerSettings && bInRemoveLeadingZeros)
	{
		RemoveFrameStringLeadingZeros(InSequencer, OutString);
	}

	if (bIsNegative)
	{
		OutString.InsertAt(0, TEXT("-"));
	}

	return OutString;
}

bool RemoveFrameStringLeadingZeros(const ISequencer& InSequencer, FString& InOutString)
{
	const TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface = InSequencer.GetNumericTypeInterface();
	if (!NumericTypeInterface.IsValid())
	{
		return false;
	}

	const USequencerSettings* const SequencerSettings = InSequencer.GetSequencerSettings();
	if (!SequencerSettings)
	{
		return false;
	}

	bool bChanged = false;

	switch (SequencerSettings->GetTimeDisplayFormat())
	{
	case EFrameNumberDisplayFormats::DropFrameTimecode:
	case EFrameNumberDisplayFormats::NonDropFrameTimecode:
		{
			FString Stripped = InOutString;
			bChanged |= Stripped.ReplaceInline(TEXT("["), TEXT("")) > 0;
			bChanged |= Stripped.ReplaceInline(TEXT("]"), TEXT("")) > 0;

			TArray<FString> Parts;
			Stripped.ParseIntoArray(Parts, TEXT(":"), false);

			while (Parts.Num() > 1 && Parts[0] == TEXT("00"))
			{
				Parts.RemoveAt(0);
				bChanged = true;
			}

			Stripped = FString::Join(Parts, TEXT(":"));
			InOutString = TEXT("[") + Stripped + TEXT("]");
		}
		break;
	default:
	case EFrameNumberDisplayFormats::Frames:
		while (InOutString.StartsWith(TEXT("0")) && InOutString.Len() > 1)
		{
			bChanged |= InOutString.RemoveFromStart(TEXT("0"));
		}
		break;
	case EFrameNumberDisplayFormats::Seconds:
		break;
	}

	return bChanged;
}

TRange<FFrameNumber> MakeTickRangeFromDisplayFrame(const FFrameNumber InDisplayFrame
	, const FFrameRate& InDisplayRate, const FFrameRate& InTickResolution)
{
	const FFrameTime StartTick = FFrameRate::TransformTime(FFrameTime(InDisplayFrame)
		, InDisplayRate, InTickResolution);

	const FFrameTime EndExclusiveTick = FFrameRate::TransformTime(FFrameTime(InDisplayFrame + 1)
		, InDisplayRate, InTickResolution);

	return TRange<FFrameNumber>(
		TRangeBound<FFrameNumber>::Inclusive(StartTick.FloorToFrame()),
		TRangeBound<FFrameNumber>::Exclusive(EndExclusiveTick.CeilToFrame())
	);
}

TSet<FKeyHandle> GetAllCurveModelKeyHandles(FCurveModel& InCurveModel)
{
	TArray<FKeyHandle> KeyHandles;
	InCurveModel.GetAllKeys(KeyHandles);

	TSet<FKeyHandle> Result;
	for (const FKeyHandle KeyHandle : KeyHandles)
	{
		Result.Add(KeyHandle);
	}

	return Result;
}

TSet<FKeyHandle> GetAllChannelKeyHandles(FMovieSceneChannel& InChannel)
{
	TArray<FKeyHandle> KeyHandles;
	InChannel.GetKeys(TRange<FFrameNumber>::All(), nullptr, &KeyHandles);

	TSet<FKeyHandle> Result;
	for (const FKeyHandle KeyHandle : KeyHandles)
	{
		Result.Add(KeyHandle);
	}

	return Result;
}

bool AreKeyHandleSetsEqual(const TSet<FKeyHandle>& InA, const TSet<FKeyHandle>& InB)
{
	if (InA.Num() != InB.Num())
	{
		return false;
	}

	for (const FKeyHandle KeyHandle : InA)
	{
		if (!InB.Contains(KeyHandle))
		{
			return false;
		}
	}

	return true;
}

bool ShouldUseControlModifierToMoveKeys()
{
	const UToolableTimelineSettings* const Settings = GetDefault<UToolableTimelineSettings>();
	return !Settings || Settings->Settings.bUseControlModifierToMoveKeys;
}

} // namespace UE::Sequencer::ToolableTimeline

#undef LOCTEXT_NAMESPACE
