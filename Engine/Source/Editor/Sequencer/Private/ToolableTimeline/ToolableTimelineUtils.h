// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Views/KeyRenderer.h"
#include "MVVM/ViewModelPtr.h"

class FCurveEditor;
class ISequencer;
class FSequencer;
struct FSequencerSelectedKey;

namespace UE::Sequencer
{
	class FChannelModel;

	namespace ToolableTimeline
	{
		class FToolableTimeline;
	}
}

namespace UE::Sequencer::ToolableTimeline::Utils
{

/** Convert a SequencerCore FKeyRenderer hittest query to FSequencerSelectedKeys */
TSet<FSequencerSelectedKey> KeyRendererResultToSelectedKeys(const TArrayView<const FKeyRenderer::FKeysForModel>& InKeys);

/** @return Time of the first key in a set of selected keys */
TOptional<FFrameTime> GetFirstKeyTime(const TSet<FSequencerSelectedKey>& InKeys);

/**
 * Determines the minimum and maximum key times from a given array of key times.
 * 
 * @param InKeyTimes The array of key times to process.
 * @param OutRange Reference to a variable that will be set to the range of key times.
 * 
 * @return True if the minimum and maximum times are different, false otherwise.
 */
bool GetMinMaxKeyRange(const TArray<FFrameNumber>& InKeyTimes, TRange<FFrameNumber>& OutRange);

/** Attempts to the the curve editor instance for a sequencer instance */
TSharedPtr<FCurveEditor> GetSequencerCurveEditor(const ISequencer& InSequencer);

/** Makes sure exclusive and inclusive frames are adjusted accordingly */
TRange<FFrameNumber> NormalizeRange(const TRange<FFrameNumber>& InRange);

/**
 * Converts a sequencer tick frame number to its string representation based on the provided formatting options.
 * 
 * @param InSequencer The sequencer instance providing context for numeric type interface and settings.
 * @param InFrame The frame number to be converted to a string.
 * @param bInRemoveLeadingZeros If true, leading zeros will be removed from the resulting string.
 * @param bInUnsigned If true, the resulting string will be treated as an unsigned value (no negative sign).
 * 
 * @return A string representation of the frame number, formatted according to the specified options.
 *         Returns an empty string if the numeric type interface is invalid.
 */
FString TickFrameToString(const ISequencer& InSequencer
	, const FFrameNumber& InFrame, const bool bInRemoveLeadingZeros, const bool bInUnsigned);

/**
 * Cleans and modifies a tick frame string by removing unwanted characters and optionally removing leading zeros.
 * 
 * @param InSequencer A reference to the sequencer instance that provides settings for formatting the string.
 * @param InString The input string representing a tick frame.
 * @param bInRemoveLeadingZeros A boolean indicating whether to remove leading zeros from the string.
 * @param bInUnsigned A boolean specifying whether the number should be treated as unsigned.
 * 
 * @return The cleaned and formatted tick frame string.
 */
FString CleanTickFrameString(const ISequencer& InSequencer
	, const FString& InString, const bool bInRemoveLeadingZeros, const bool bInUnsigned);

/**
 * Removes leading zeros from a string representation of a frame, based on the sequencer's time display format settings.
 * 
 * @param InSequencer Reference to the sequencer containing settings and numeric type interfaces.
 * @param InOutString The string to be processed. Leading zeros are removed and the string is updated in place.
 * 
 * @return True if the string was modified, false otherwise.
 */
bool RemoveFrameStringLeadingZeros(const ISequencer& InSequencer, FString& InOutString);

/**
 * Generates a tick range that corresponds to a specific display frame.
 * 
 * @param InDisplayFrame The frame number in the display rate to generate the tick range for.
 * @param InDisplayRate The display frame rate used to interpret the display frame.
 * @param InTickResolution The tick resolution frame rate to which the display frame is transformed.
 * 
 * @return The tick range in terms of frame numbers at the specified tick resolution, spanning from
 *         the inclusive start of the target frame to the exclusive end of the next frame.
 */
TRange<FFrameNumber> MakeTickRangeFromDisplayFrame(const FFrameNumber InDisplayFrame
	, const FFrameRate& InDisplayRate, const FFrameRate& InTickResolution);

bool ShouldUseControlModifierToMoveKeys();

} // namespace UE::Sequencer::ToolableTimeline
