// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace UE::Sequencer::ToolableTimeline
{

class FToolableTimeline;
class FToolableTimelineBaseTool;
struct FMouseInputData;

/**
 * Represents a factory interface responsible for creating toolable timeline tool instances.
 */
class IToolableTimelineToolFactory
{
public:
	virtual ~IToolableTimelineToolFactory() = default;

	/**
	 * Retrieves the unique identifier of the tool factory. This identifier is used to distinguish
	 * the factory from others and can be used for tool selection or activation processes.
	 *
	 * @return The unique identifier of the tool factory.
	 */
	virtual FName GetIdentifier() const = 0;

	/**
	 * Retrieves the priority level of the tool factory, which is used to determine the relative order
	 * of the factory when compared with others.
	 *
	 * @return A floating-point value representing the priority of this tool factory. Lower values
	 *         indicate higher priority.
	 */
	virtual float GetPriority() const = 0;

	/**  */
	virtual bool RequiresDragThresholdToActivate() const { return false; }

	/**
	 * Determines whether the tool factory wishes to activate a tool based on the specified mouse input data.
	 *
	 * @param InMouseInput A reference to the data describing the current mouse input, including position and state.
	 * @param bInMouseHitKeyFrame Indicates whether the mouse hit is on a frame that contains a key.
	 * @return True if the tool factory wants to activate a tool in response to the mouse input; otherwise, false.
	 */
	virtual bool WantsToActivate(const FMouseInputData& InMouseInput, const bool bInMouseHitKeyFrame) const = 0;

	/**
	 * Determines whether the tool factory wishes to activate a tool from a mouse double-click.
	 *
	 * @param InMouseInput A reference to the data describing the current mouse input, including position and state.
	 * @param bInMouseHitKeyFrame Indicates whether the mouse hit is on a frame that contains a key.
	 * @return True if the tool factory wants to activate a tool in response to the mouse double-click; otherwise false.
	 */
	virtual bool WantsToActivateOnDoubleClick(const FMouseInputData& InMouseInput, const bool bInMouseHitKeyFrame) const
	{
		return false;
	}

	/**
	 * Creates a new instance of the tool for the specified timeline.
	 *
	 * @param InTimeline A reference to the toolable timeline for which the tool instance is created.
	 * @return A shared reference to the newly created tool instance.
	 */
	virtual TSharedRef<FToolableTimelineBaseTool> CreateTool(FToolableTimeline& InTimeline) const = 0;
};

} // namespace UE::Sequencer::ToolableTimeline
