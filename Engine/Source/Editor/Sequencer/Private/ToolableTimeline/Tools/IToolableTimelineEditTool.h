// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerInputHandler.h"
#include "Misc/FrameTime.h"
#include "Templates/SharedPointer.h"

class FUICommandList;
class FText;
struct FFrameNumber;
struct FSequencerSelectedKey;

namespace UE::Sequencer::ToolableTimeline
{

struct FMouseInputData;
struct FMouseDrawInputData;

/**
 * Editor for a toolable timeline. Only one editor can be active at a time.
 */
class IToolableTimelineEditTool : public ISequencerInputHandler, public TSharedFromThis<IToolableTimelineEditTool>
{
public:
	virtual ~IToolableTimelineEditTool() override = default;

	bool operator==(const IToolableTimelineEditTool& InOther) const
	{
		return GetIdentifier().IsEqual(InOther.GetIdentifier());
	}

	/** @return Unique identifier for this tool */
	virtual FName GetIdentifier() const = 0;

	/** @return Label for this tool to use for UX. */
	virtual FText GetLabel() const = 0;

	/** @return Description for this tool to use for UX. */
	virtual FText GetDescription() const = 0;

	/**
	 * Allows the tool to bind commands.
	 * @param InCommandBindings The existing command bindings to map to.
	 */
	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandBindings) {}

	/**
	 * Allows the tool to unbind commands.
	 */
	virtual void UnbindCommands(const TSharedRef<FUICommandList>& InCommandBindings) {}

	/**
	 * Called when the tool is activated by switching from another tool.
	 * The current tool (if any) will have OnToolDeactivated called first before the new tool has OnToolActivated called.
	 */
	virtual void NotifyToolActivated(const FMouseInputData& InMouseInput, const FFrameTime& InInputTickTime) {}

	/**
	 * Called when the tool is deactivated by switching to another tool.
	 * Called before the new tool has OnToolActivated called.
	 */
	virtual void NotifyToolDeactivated() {}

	/** @return True if the tool has requested to close. */
	virtual bool IsCloseRequested() const = 0;

	/**
	 * Requests the tool to close, either marking it for closure or canceling the request.
	 * @param bInClose Specifies whether to request closure (true) or cancel a closure request (false). Default is true.
	 */
	virtual void RequestClose(const bool bInClose = true) = 0;
	
	/** Broadcast an event to handle closing of the tool */
	//void CloseTool() const { OnCloseDelegate.Broadcast(); }
	/** @return Event that is broadcast to when the tool has been closed */
	FSimpleMulticastDelegate& OnToolClosed() { return OnCloseDelegate; };

	/**  */
	virtual bool WantsDeferredDragThresholdActivation() const { return false; }

	/**
	 * Determines if this tool wants to take priority over standard controller hit detection for the specified mouse input.
	 * Useful for scenarios where the tool introduces custom interaction behavior that overrides default controller behavior.
	 * @param InMouseInput The mouse input data to evaluate.
	 * @return True if the tool should take priority over the controller hit, false otherwise.
	 */
	virtual bool WantsPriorityOverControllerHit(const FMouseInputData& InMouseInput) const
	{
		return false;
	}

	/**
	 * @return True if the tool wants to intercept the current mouse state in an input
	 * prepass before standard hit testing and interaction dispatch by the controller.
	 * This is primarily for alternate interaction modes driven by modifiers/buttons,
	 * where the tool needs to present different cursor or drag behavior than its normal
	 * resolved target would imply.
	 */
	virtual bool WantsInputPrepass(const FMouseInputData& InMouseInput) const = 0;

	/**  */
	virtual int32 Paint(FMouseDrawInputData& MouseDrawInput) const = 0;

	/** Returns the range in tick resolution frames that the tool is currently operating on. */
	virtual TRange<FFrameNumber> GetToolRange() const = 0;

	/** @return Array of keys that the tool is currently operating on. */
	virtual TSet<FSequencerSelectedKey> GetToolRangeKeys() const = 0;

	/** @return Tick range of the drag indicator based on the provided mouse input data. */
	virtual TRange<FFrameTime> GetDragIndicatorTickRange(const FMouseInputData& InMouseInput) const
	{
		return TRange<FFrameTime>();
	}

	/** @return True if the scrubber should be hidden on activation. */
	virtual bool ShouldHideScrubberOnActivation() const { return true; }

	/** @return True if the drag range indicator should be shown when the tool is dragging. */
	virtual bool ShouldShowDragRangeIndicator() const { return false; }

	/** @return True if this tool should be deactivated when outliner selection changes. */
	virtual bool ShouldDeactivateOnOutlinerSelectionChanged() const { return false; }

private:
	FSimpleMulticastDelegate OnCloseDelegate;
};

} // namespace UE::Sequencer::ToolableTimeline
