// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimelineBaseTool.h"
#include "Channels/MovieSceneChannel.h"
#include "SequencerSelectionPreview.h"
#include "ToolableTimeline/MouseInputData.h"
#include "ToolableTimeline/ToolableTimeline.h"

#define LOCTEXT_NAMESPACE "ToolableTimelineBaseTool"

namespace UE::Sequencer::ToolableTimeline
{

FToolableTimelineBaseTool::FToolableTimelineBaseTool(const TSharedRef<FToolableTimeline>& InTimeline)
	: WeakTimeline(InTimeline)
{
}

void FToolableTimelineBaseTool::NotifyToolActivated(const FMouseInputData& InMouseInput, const FFrameTime& InInputTickTime)
{
}

void FToolableTimelineBaseTool::NotifyToolDeactivated()
{
	FToolableTimelineKeySelection& TimelineKeySelection = GetTimelineKeySelection();
	TimelineKeySelection.ResetHoveredKeysAndSelectionPreview();
	TimelineKeySelection.ClearSelectedKeys();
}

bool FToolableTimelineBaseTool::IsCloseRequested() const
{
	return bCloseRequested;
}

void FToolableTimelineBaseTool::RequestClose(const bool bInClose)
{
	bCloseRequested = bInClose;
}

TSet<FSequencerSelectedKey> FToolableTimelineBaseTool::GetToolRangeKeys() const
{
	const TSharedRef<FToolableTimeline> Timeline = GetTimelineChecked();
	const TRange<FFrameNumber> ToolRange = GetToolRange();

	TSet<FSequencerSelectedKey> OutKeys;

	const TSet<TWeakViewModelPtr<FChannelModel>>& ChannelModels = Timeline->GetChannelModels();
	for (const TWeakViewModelPtr<FChannelModel>& WeakChannelModel : ChannelModels)
	{
		const TViewModelPtr<FChannelModel> ChannelModel = WeakChannelModel.Pin();
		if (!ChannelModel.IsValid())
		{
			continue;
		}

		const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea();
		if (!KeyArea.IsValid())
		{
			continue;
		}

		UMovieSceneSection* const Section = ChannelModel->GetSection();
		if (!Section)
		{
			continue;
		}

		FMovieSceneChannel* const Channel = ChannelModel->GetChannel();
		if (!Channel)
		{
			continue;
		}

		TArray<FKeyHandle> ChannelKeyHandles;
		Channel->GetKeys(ToolRange, nullptr, &ChannelKeyHandles);

		for (const FKeyHandle KeyHandle : ChannelKeyHandles)
		{
			OutKeys.Add(FSequencerSelectedKey(*Section, ChannelModel, KeyHandle));
		}
	}

	return OutKeys;
}

TSharedRef<FToolableTimeline> FToolableTimelineBaseTool::GetTimelineChecked() const
{
	const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin();
	check(Timeline.IsValid());
	return Timeline.ToSharedRef();
}

FToolableTimelineKeySelection& FToolableTimelineBaseTool::GetTimelineKeySelection() const
{
	return GetTimelineChecked()->GetKeySelection();
}

bool FToolableTimelineBaseTool::HitTestLabelArea(const FMouseInputData& InMouseInput) const
{
	const TSharedPtr<FToolableTimeline> Timeline = WeakTimeline.Pin();
	if (!Timeline.IsValid())
	{
		return false;
	}

	static constexpr double DragToleranceSlateUnits = 0.0;//2.0;
	static constexpr double MouseTolerance = 0.0;//2.0;
	constexpr double TotalTolerance = DragToleranceSlateUnits + MouseTolerance;

	const FVector2D PointerPosition = InMouseInput.PointerEvent.GetScreenSpacePosition();
	const FVector2D HitTestPixel = InMouseInput.Geometry.AbsoluteToLocal(PointerPosition);

	const double HeightPx = InMouseInput.Geometry.GetLocalSize().Y;
	const double TickSize = Timeline->GetTimeSliderController()->GetMajorTickDrawSize();
	const double HalfTickSize = TickSize * 0.5;

	auto HitTestTop = [&HitTestPixel, TotalTolerance](const double InHeight) -> bool
	{
		return HitTestPixel.Y <= InHeight + TotalTolerance;
	};
	auto HitTestBottom = [&HitTestPixel, TotalTolerance, HeightPx](const double InHeight) -> bool
	{
		return HitTestPixel.Y >= (HeightPx - InHeight) - TotalTolerance;
	};

	switch (Timeline->GetTimelineSettings().Settings.LabelVerticalAlignment)
	{
	case VAlign_Top:
		if (HitTestTop(TickSize))
		{
			return true;
		}
		break;
	case VAlign_Bottom:
		if (HitTestBottom(TickSize))
		{
			return true;
		}
		break;
	case VAlign_Center:
		if (HitTestTop(HalfTickSize) || HitTestBottom(HalfTickSize))
		{
			return true;
		}
		break;
	default:
	case VAlign_Fill:
		break;
	}

	return false;
}

bool FToolableTimelineBaseTool::IsValidToolRange(const TRange<FFrameNumber>& InRange)
{
	return !InRange.IsEmpty()
		&& InRange.HasLowerBound()
		&& InRange.HasUpperBound();
}

bool FToolableTimelineBaseTool::HasValidToolRange() const
{
	return IsValidToolRange(GetToolRange());
}

} // namespace UE::Sequencer::ToolableTimeline

#undef LOCTEXT_NAMESPACE
