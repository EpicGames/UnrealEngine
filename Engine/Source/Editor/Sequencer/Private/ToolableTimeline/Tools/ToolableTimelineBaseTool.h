// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScopedTransaction.h"
#include "ToolableTimeline/Tools/IToolableTimelineEditTool.h"

struct FFrameTime;

namespace UE::Sequencer::ToolableTimeline
{
	class FToolableTimeline;
	class FToolableTimelineKeySelection;
	struct FMouseInputData;
}

namespace UE::Sequencer::ToolableTimeline
{

/** Base tool to subclass for a toolable timeline tool. */
class FToolableTimelineBaseTool : public IToolableTimelineEditTool
{
public:
	explicit FToolableTimelineBaseTool(const TSharedRef<FToolableTimeline>& InTimeline);
	virtual ~FToolableTimelineBaseTool() override = default;

	//~ Begin IToolableTimelineEditTool

	virtual void NotifyToolActivated(const FMouseInputData& InMouseInput, const FFrameTime& InInputTickTime) override;
	virtual void NotifyToolDeactivated() override;

	virtual bool IsCloseRequested() const override;
	virtual void RequestClose(const bool bInClose = true) override;

	virtual TSet<FSequencerSelectedKey> GetToolRangeKeys() const override;

	//~ End IToolableTimelineEditTool

	TSharedRef<FToolableTimeline> GetTimelineChecked() const;

	virtual bool IsDragging() const { return false; }

	virtual bool HasHoverHit() const { return false; }

	/** @return True if this tool should remain active while channel models are rebuilt */
	virtual bool CanPersistThroughChannelRecache() const { return false; }

	/** Called after channel models are rebuilt while this tool remains active */
	virtual void OnChannelModelsRecached() {}

	static bool IsValidToolRange(const TRange<FFrameNumber>& InRange);
	bool HasValidToolRange() const;

protected:
	virtual void OnDragStart(const FMouseInputData& InMouseInput) = 0;
	virtual void OnDrag(const FMouseInputData& InMouseInput) = 0;
	virtual void OnDragEnd(const bool bInCommit = true) = 0;

	FToolableTimelineKeySelection& GetTimelineKeySelection() const;

	bool HitTestLabelArea(const FMouseInputData& InMouseInput) const;

	TWeakPtr<FToolableTimeline> WeakTimeline;

	TUniquePtr<FScopedTransaction> ActiveTransaction;

private:
	bool bCloseRequested = false;
};

} // namespace UE::Sequencer::ToolableTimeline
