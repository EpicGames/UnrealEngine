// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolableTimelineKeyRenderer.h"
#include "SequencerSelectedKey.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/Views/KeyRenderer.h"
#include "SequencerSelectionPreview.h"
#include "ToolableTimeline/ToolableTimelineKeySelection.h"

#define LOCTEXT_NAMESPACE "ToolableTimelineKeyRenderer"

namespace UE::Sequencer::ToolableTimeline
{

FToolableTimelineKeyRenderer::FToolableTimelineKeyRenderer(const FToolableTimelineKeySelection& InKeySelection)
{
	// Copy selection state so key rendering uses a stable snapshot for this paint pass
	SelectedKeys = InKeySelection.GetSelectedKeys();
	HoveredKeys = InKeySelection.GetHoveredKeys();
	SelectionPreview = InKeySelection.GetSelectionPreview();
}

bool FToolableTimelineKeyRenderer::HasAnySelectedKeys() const
{
	return SelectedKeys.Num() != 0;
}

bool FToolableTimelineKeyRenderer::HasAnyPreviewSelectedKeys() const
{
	return !SelectionPreview.IsEmpty();
}

bool FToolableTimelineKeyRenderer::HasAnyHoveredKeys() const
{
	return HoveredKeys.Num() != 0;
}

bool FToolableTimelineKeyRenderer::IsKeySelected(const TViewModelPtr<IKeyExtension>& InOwner, const FKeyHandle InKey) const
{
	const TViewModelPtr<FChannelModel> ChannelModel = InOwner.ImplicitCast();
	if (!ChannelModel.IsValid())
	{
		return false;
	}

	UMovieSceneSection* const Section = ChannelModel->GetSection();
	if (!Section)
	{
		return false;
	}

	const FSequencerSelectedKey SelectedKey(*Section, ChannelModel, InKey);
	return SelectedKeys.Contains(SelectedKey);
}

bool FToolableTimelineKeyRenderer::IsKeyHovered(const TViewModelPtr<IKeyExtension>& InOwner, const FKeyHandle InKey) const
{
	const TViewModelPtr<FChannelModel> ChannelModel = InOwner.ImplicitCast();
	if (!ChannelModel.IsValid())
	{
		return false;
	}

	UMovieSceneSection* const Section = ChannelModel->GetSection();
	if (!Section)
	{
		return false;
	}

	const FSequencerSelectedKey SelectedKey(*Section, ChannelModel, InKey);
	return HoveredKeys.Contains(SelectedKey);
}

EKeySelectionPreviewState FToolableTimelineKeyRenderer::GetPreviewSelectionState(const TViewModelPtr<IKeyExtension>& InOwner, const FKeyHandle InKey) const
{
	if (SelectionPreview.IsEmpty())
	{
		return EKeySelectionPreviewState::Undefined;
	}

	const TViewModelPtr<FChannelModel> ChannelModel = InOwner.ImplicitCast();
	if (!ChannelModel.IsValid())
	{
		return EKeySelectionPreviewState::Undefined;
	}

	UMovieSceneSection* const Section = ChannelModel->GetSection();
	if (!Section)
	{
		return EKeySelectionPreviewState::Undefined;
	}

	if (const ESelectionPreviewState* State = SelectionPreview.Find(FSequencerSelectedKey(*Section, ChannelModel, InKey)))
	{
		switch(*State)
		{
		case ESelectionPreviewState::Undefined:   return EKeySelectionPreviewState::Undefined;
		case ESelectionPreviewState::Selected:    return EKeySelectionPreviewState::Selected;
		case ESelectionPreviewState::NotSelected: return EKeySelectionPreviewState::NotSelected;
		}
	}

	return EKeySelectionPreviewState::Undefined;
}

} // namespace UE::Sequencer::ToolableTimeline

#undef LOCTEXT_NAMESPACE
