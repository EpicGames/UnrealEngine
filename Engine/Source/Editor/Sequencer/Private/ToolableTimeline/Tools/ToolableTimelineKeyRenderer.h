// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Views/KeyRenderer.h"
#include "SequencerSelectedKey.h"

class FSequencer;
enum class ESelectionPreviewState;

namespace UE::Sequencer::ToolableTimeline
{

class FToolableTimelineKeySelection;

struct FToolableTimelineKeyRenderer : IKeyRendererInterface
{
	FToolableTimelineKeyRenderer(const FToolableTimelineKeySelection& InKeySelection);

	//~ Begin IKeyRendererInterface

	virtual bool HasAnySelectedKeys() const override;
	virtual bool HasAnyPreviewSelectedKeys() const override;
	virtual bool HasAnyHoveredKeys() const override;

	virtual bool IsKeySelected(const TViewModelPtr<IKeyExtension>& InOwner, const FKeyHandle InKey) const override;
	virtual bool IsKeyHovered(const TViewModelPtr<IKeyExtension>& InOwner, const FKeyHandle InKey) const override;

	virtual EKeySelectionPreviewState GetPreviewSelectionState(const TViewModelPtr<IKeyExtension>& InOwner, const FKeyHandle InKey) const override;

	//~ End IKeyRendererInterface

private:
	TSet<FSequencerSelectedKey> HoveredKeys;

	TMap<FSequencerSelectedKey, ESelectionPreviewState> SelectionPreview;

	TSet<FSequencerSelectedKey> SelectedKeys;
};

} // namespace UE::Sequencer::ToolableTimeline
