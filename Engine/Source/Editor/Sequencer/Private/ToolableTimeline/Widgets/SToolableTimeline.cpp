// Copyright Epic Games, Inc. All Rights Reserved.

#include "SToolableTimeline.h"
#include "Framework/Application/SlateApplication.h"
#include "ISequencerWidgetsModule.h"
#include "ITimeSlider.h"
#include "MVVM/CurveEditorExtension.h"
#include "MVVM/Views/ITrackAreaHotspot.h"
#include "SimpleView/Tools/SimpleViewMoveFrameKeysTool.h"
#include "SSequencer.h"
#include "ToolableTimeline/ToolableTimeline.h"
#include "ToolableTimeline/ToolableTimeSliderController.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"

#define LOCTEXT_NAMESPACE "SToolableTimelineAreaView"

namespace UE::Sequencer::ToolableTimeline
{

void SToolableTimeline::Construct(const FArguments& InArgs, const TSharedRef<FToolableTimeline>& InTimeline)
{
	Timeline = InTimeline;

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = Timeline->GetTimeSliderController();

	InputStack.AddHandler(&TimeSliderController.Get());

	TimeToPixel = MakeShared<FTimeToPixelSpace>(100.f, TimeSliderController->GetViewRange(), TimeSliderController->GetTickResolution());

	ISequencerWidgetsModule& SequencerWidgets = FModuleManager::Get().LoadModuleChecked<ISequencerWidgetsModule>(TEXT("SequencerWidgets"));
	const bool bMirrorLabels = Timeline->GetTimelineSettings().Settings.LabelVerticalAlignment == VAlign_Top;
	TimeSliderWidget = SequencerWidgets.CreateTimeSlider(TimeSliderController, bMirrorLabels);

	ChildSlot
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.ZOrder(0)
		[
			TimeSliderWidget.ToSharedRef()
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.ZOrder(1)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("PlainBorder")))
			.BorderBackgroundColor(FStyleColors::Error.GetSpecifiedColor().CopyWithNewOpacity(.5f))
			.Visibility(this, &SToolableTimeline::GetLockedAnimationBorderVisibility)
		]
	];
}

void SToolableTimeline::RequestRecacheChannels()
{
	bRequestRecacheChannels = true;
}

FVector2D SToolableTimeline::ComputeDesiredSize(float) const
{
	if (!Timeline.IsValid())
	{
		return FVector2D::ZeroVector;
	}

	const TSharedPtr<SWidget> ParentWidget = GetParentWidget();
	if (!ParentWidget.IsValid())
	{
		return FVector2D::ZeroVector;
	}

	return FVector2D(ParentWidget->GetDesiredSize().X, Timeline->GetTimeSliderController()->ComputeHeight());
}

FReply SToolableTimeline::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) 
{
	if (TextInputWidget.IsValid() && InKeyEvent.GetKey() == EKeys::Escape)
	{
		CloseInput();

		return FReply::Handled();
	}

	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		// @TODO: deselect all visibibly selected keys?
	}

	if (Timeline->GetCommandList()->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	// If CanDeleteSelectedKeys() returns false for the local GenericCommands.Delete binding,
	// do not fall through to Sequencer's global GenericCommands.Delete binding.
	if (InKeyEvent.GetKey() == EKeys::Delete)
	{
		return FReply::Handled();
	}

	if (const TSharedPtr<ISequencer> Sequencer = Timeline->GetSequencer())
	{
		if (const TSharedPtr<FUICommandList> CommandBindings = Sequencer->GetCommandBindings())
		{
			if (CommandBindings->ProcessCommandBindings(InKeyEvent))
			{
				return FReply::Handled();
			}
		}

		if (FCurveEditorExtension* const CurveEditorExtension = Sequencer->GetViewModel()->CastDynamic<FCurveEditorExtension>())
		{
			if (const TSharedPtr<FCurveEditor> CurveEditor = CurveEditorExtension->GetCurveEditor())
			{
				if (const TSharedPtr<FUICommandList> CurveEditorCommands = CurveEditor->GetCommands())
				{
					if (CurveEditorCommands->ProcessCommandBindings(InKeyEvent))
					{
						return FReply::Handled();
					}
				}
			}
		}
	}

	return SCompoundWidget::OnKeyDown(InGeometry, InKeyEvent);
}

void SToolableTimeline::Tick(const FGeometry& InGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(InGeometry, InCurrentTime, InDeltaTime);

	if (bRequestRecacheChannels)
	{
		Timeline->RecacheChannelModels();
		bRequestRecacheChannels = false;
	}

	// Cache the size of the window and rescale the timeline if the size changes
	const FVector2D LocalSize = InGeometry.GetLocalSize();
	if (SizeLastFrame.IsSet() && LocalSize.X != SizeLastFrame->X)
	{
		OnResized(SizeLastFrame.GetValue(), LocalSize);
	}
	SizeLastFrame = LocalSize;

	// Make sure we close text input if we no longer have focus
	if (TextInputWidget.IsValid() && !TextInputWidget->HasKeyboardFocus())
	{
		CloseInput();
	}
}

int32 SToolableTimeline::OnPaint(const FPaintArgs& InArgs
	, const FGeometry& InGeometry
	, const FSlateRect& InCullingRect
	, FSlateWindowElementList& OutDrawElements
	, int32 InLayerId
	, const FWidgetStyle& InWidgetStyle
	, const bool bInParentEnabled) const
{
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = Timeline->GetTimeSliderController();

	// Reassign the time <-> pixel space for this frame
	*TimeToPixel = FTimeToPixelSpace(InGeometry, TimeSliderController->GetViewRange(), TimeSliderController->GetTickResolution());

	return SCompoundWidget::OnPaint(InArgs
		, InGeometry
		, InCullingRect
		, OutDrawElements
		, InLayerId
		, InWidgetStyle
		, bInParentEnabled);
}

void SToolableTimeline::OnResized(const FVector2D& InOldSize, const FVector2D& InNewSize)
{
	if (!Timeline.IsValid())
	{
		return;
	}

	const TSharedRef<FToolableTimeSliderController> TimeSliderController = Timeline->GetTimeSliderController();

	// Zoom by the difference in horizontal size
	const float Difference = InNewSize.X - InOldSize.X;
	const TRange<double> OldRange = TimeSliderController->GetViewRange().GetAnimationTarget();

	const double NewRangeMin = OldRange.GetLowerBoundValue();
	const double NewRangeMax = OldRange.GetUpperBoundValue() + (Difference * OldRange.Size<double>() / SizeLastFrame->X);

	const TRange<double> ClampRange = TimeSliderController->GetClampRange();

	if (NewRangeMin < ClampRange.GetLowerBoundValue() || NewRangeMax > ClampRange.GetUpperBoundValue())
	{
		double NewClampRangeMin = NewRangeMin < ClampRange.GetLowerBoundValue() ? NewRangeMin : ClampRange.GetLowerBoundValue();
		double NewClampRangeMax = NewRangeMax > ClampRange.GetUpperBoundValue() ? NewRangeMax : ClampRange.GetUpperBoundValue();

		TimeSliderController->SetClampRange(NewClampRangeMin, NewClampRangeMax);
	}

	TimeSliderController->SetViewRange(NewRangeMin, NewRangeMax, EViewRangeInterpolation::Immediate);
}

double SToolableTimeline::GetScrubToFrameValue() const
{
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = Timeline->GetTimeSliderController();
	return TimeSliderController->GetScrubPosition().GetFrame().Value;
}

void SToolableTimeline::HandleScrubToFrameCommitted(const double InNewValue, const ETextCommit::Type InTextCommit)
{
	if (InTextCommit == ETextCommit::OnEnter)
	{
		Timeline->CenterFrameOnScreen(InNewValue, true);
	}

	CloseInput();
}

double SToolableTimeline::GetScrubToFrameDelta() const
{
	const TSharedRef<FToolableTimeSliderController> TimeSliderController = Timeline->GetTimeSliderController();
	return TimeSliderController->GetTickResolution().AsDecimal()
		* TimeSliderController->GetDisplayRate().AsInterval();
}

void SToolableTimeline::CreateInputPopup(const TSharedRef<SWidget>& InWidget, const FText& InLabel)
{
	TextInputWidget = InWidget;

	TSharedRef<SWidget> OutContent = InWidget;

	if (!InLabel.IsEmptyOrWhitespace())
	{
		OutContent = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FStyleColors::Foreground)
				.Text(InLabel)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(6.f, 0.f, 0.f, 0.f)
			[
				InWidget
			];
	}

	const TSharedRef<SWidget> PopupWindowContent = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(TEXT("Menu.Background")))
		.Padding(10.f, 6.f)
		.VAlign(VAlign_Center)
		[
			OutContent
		];

	const FVector2D LocalSize = GetCachedGeometry().GetLocalSize();
	const FVector2D LocalCenter = LocalSize * .5f;
	const FVector2D ScreenCenter = GetCachedGeometry().LocalToAbsolute(LocalCenter);

	ActiveInputMenu = FSlateApplication::Get().PushMenu(
		AsShared(),
		FWidgetPath(),
		PopupWindowContent,
		ScreenCenter,
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup),
		/*bFocusImmediately=*/false
	);

	FSlateApplication::Get().SetKeyboardFocus(InWidget, EFocusCause::SetDirectly);
}

EVisibility SToolableTimeline::GetLockedAnimationBorderVisibility() const
{
	const TSharedPtr<FSequencer> Sequencer = Timeline->GetSequencer();
	return Sequencer.IsValid() && Sequencer->IsReadOnly() ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
}

void SToolableTimeline::CloseInput()
{
	if (ActiveInputMenu.IsValid())
	{
		ActiveInputMenu->Dismiss();
		ActiveInputMenu.Reset();
	}

	TextInputWidget.Reset();
}

void SToolableTimeline::DoScrubToFrameInput()
{
	const TSharedPtr<ISequencer> Sequencer = Timeline->GetSequencer();
	
	const FText Label = LOCTEXT("ScrubToFrameLabel", "Scrub To Frame");

	const TSharedRef<SSpinBox<double>> ScrubToFrameSpinBox = SNew(SSpinBox<double>)
		.ToolTipText(Label)
		.TypeInterface(Sequencer->GetNumericTypeInterface())
		.MinDesiredWidth(40.f)
		.LinearDeltaSensitivity(25)
		.ClearKeyboardFocusOnCommit(true)
		.MinValue(TOptional<double>())
		.MaxValue(TOptional<double>())
		.Value(this, &SToolableTimeline::GetScrubToFrameValue)
		.OnValueCommitted(this, &SToolableTimeline::HandleScrubToFrameCommitted)
		.Delta(this, &SToolableTimeline::GetScrubToFrameDelta);

	CreateInputPopup(ScrubToFrameSpinBox, Label);
}

} // namespace UE::Sequencer::ToolableTimeline

#undef LOCTEXT_NAMESPACE
