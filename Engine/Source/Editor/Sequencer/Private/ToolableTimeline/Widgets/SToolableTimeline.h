// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "MVVM/Views/KeyRenderer.h"
#include "MVVM/Views/SequencerInputHandlerStack.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SCompoundWidget.h"

class ITimeSlider;

namespace UE::Sequencer::ToolableTimeline
{

class FToolableTimeline;

class SToolableTimeline : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SToolableTimeline)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FToolableTimeline>& InTimeline);

	//~ Begin SWidget
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual void Tick(const FGeometry& InGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	virtual int32 OnPaint(const FPaintArgs& InArgs
		, const FGeometry& InGeometry
		, const FSlateRect& InCullingRect
		, FSlateWindowElementList& OutDrawElements
		, int32 InLayerId
		, const FWidgetStyle& InWidgetStyle
		, const bool bInParentEnabled) const override;
	//~ End SWidget

	void RequestRecacheChannels();

	void OnResized(const FVector2D& InOldSize, const FVector2D& InNewSize);

	TSharedPtr<FTimeToPixelSpace> GetTimeToPixel() const { return TimeToPixel; }

	/** Closes and resets any active text input popup menu. */
	void CloseInput();

	/** Displays an input popup allowing the user to scrub to a specific frame in the timeline. */
	void DoScrubToFrameInput();

	TSharedPtr<ITimeSlider> GetTimeSliderWidget() const { return TimeSliderWidget; }

private:
	/**  */
	void CreateInputPopup(const TSharedRef<SWidget>& InWidget, const FText& InLabel = FText::GetEmpty());

	EVisibility GetLockedAnimationBorderVisibility() const;

	double GetScrubToFrameValue() const;
	void HandleScrubToFrameCommitted(const double InNewValue, const ETextCommit::Type InTextCommit);
	double GetScrubToFrameDelta() const;

	TSharedPtr<FToolableTimeline> Timeline;

	TSharedPtr<ITimeSlider> TimeSliderWidget;

	/** Flag indicating if a channel model recache is requested on next tick */
	bool bRequestRecacheChannels = false;

	/** Input handler stack responsible for routing input to the different handlers */
	FInputHandlerStack InputStack;

	/**  */
	mutable TSharedPtr<FTimeToPixelSpace> TimeToPixel;

	/** Keep a hold of the size of the area so we can maintain zoom levels. */
	TOptional<FVector2D> SizeLastFrame;

	/** Active input menu receiving input */
	TSharedPtr<IMenu> ActiveInputMenu;

	/** Active input widget receiving input */
	TSharedPtr<SWidget> TextInputWidget;
};

} // namespace UE::Sequencer::ToolableTimeline
