// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragOperation_Stretch.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/Selection/Selection.h"
#include "SequencerSettings.h"

#include "ISequencer.h"
#include "Sequencer.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/VirtualTrackArea.h"

#define LOCTEXT_NAMESPACE "FEditToolDragOperation_Stretch"

namespace UE
{
namespace Sequencer
{

FEditToolDragOperation_Stretch::FEditToolDragOperation_Stretch(ISequencer* InSequencer, EStretchConstraint InStretchConstraint, FFrameNumber InDragStartPosition)
	: Sequencer(InSequencer)
	, DragStartPosition(InDragStartPosition)
	, StretchConstraint(InStretchConstraint)
{
	GlobalParameters.Anchor = GlobalParameters.Handle = InDragStartPosition.Value;
}

bool FEditToolDragOperation_Stretch::InitiateStretch(TSharedPtr<FViewModel> Controller, TSharedPtr<IStretchableExtension> Target, int32 Priority, const FStretchParameters& InParams)
{
	if (!ensure(InParams.IsValid()))
	{
		return false;
	}

	if (FStretchTarget* ExistingTarget = StretchTargets.Find(Target))
	{
		if (ExistingTarget->Priority < Priority)
		{
			ExistingTarget->Params = InParams;
			return true;
		}
	}
	else
	{
		StretchTargets.Add(Target, FStretchTarget{ InParams, Priority });
		return true;
	}

	return false;
}

void FEditToolDragOperation_Stretch::DoNotSnapTo(TSharedPtr<FViewModel> Model)
{
	if (FSectionModel* SectionModel = Model->CastThis<FSectionModel>())
	{
		SnapExclusion.Add(SectionModel->GetSection());
	}
}

void FEditToolDragOperation_Stretch::OnBeginDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const FVirtualTrackArea& VirtualTrackArea)
{
	TArray<IStretchableExtension*> AllStretchables;

	for (FViewModelPtr SelectedModel : Sequencer->GetViewModel()->GetSelection()->TrackArea)
	{
		if (TViewModelPtr<IStretchableExtension> Stretchable = SelectedModel.ImplicitCast())
		{
			Stretchable->OnInitiateStretch(*this, StretchConstraint, &GlobalParameters);
		}
	}

	Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("StretchTransaction", "Stretch"));

	FStretchScreenParameters ScreenParameters;
	ScreenParameters.DragStartPosition   = DragStartPosition;
	ScreenParameters.CurrentDragPosition = DragStartPosition;
	ScreenParameters.LocalMousePos       = LocalMousePos;
	ScreenParameters.MouseEvent          = &MouseEvent;
	ScreenParameters.VirtualTrackArea    = &VirtualTrackArea;

	FStretchParameters TempParams;
	const bool bUseGlobalParams = MouseEvent.IsAltDown() == false;

	for (auto TargetIt = StretchTargets.CreateIterator(); TargetIt; ++TargetIt)
	{
		FStretchParameters* Params = &TargetIt.Value().Params;
		if (bUseGlobalParams)
		{
			// Ensure the TempParams match the global ones (in-case something overwrote them)
			TempParams = GlobalParameters;
			Params = &TempParams;
		}

		EStretchResult Result = TargetIt.Key()->OnBeginStretch(*this, ScreenParameters, Params);

		// Don't drag if the parameters are invalid
		if (Result == EStretchResult::Failure)
		{
			TargetIt.RemoveCurrent();
		}
	}

	SnapField = MakeUnique<FSequencerSnapField>();
	SnapField->Initialize(*static_cast<FSequencer*>(Sequencer), *this);
	SnapField->AddExplicitSnap(FSnapPoint{FSnapPoint::CustomSection, DragStartPosition });
	SnapField->Finalize();
}

void FEditToolDragOperation_Stretch::OnDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const FVirtualTrackArea& VirtualTrackArea)
{
	UE::MovieScene::FScopedSignedObjectModifyDefer DeferMarkAsChanged(true);

	FFrameTime CurrentTime = VirtualTrackArea.PixelToFrame(LocalMousePos.X);

	if (Sequencer->GetSequencerSettings()->GetIsSnapEnabled())
	{
		const float PixelSnapWidth = 10.f;

		const float SnapThresholdPx = VirtualTrackArea.PixelToSeconds(PixelSnapWidth) - VirtualTrackArea.PixelToSeconds(0.f);
		const int32 SnapThreshold = (SnapThresholdPx * Sequencer->GetFocusedTickResolution()).FloorToFrame().Value;

		TOptional<FSequencerSnapField::FSnapResult> SnappedResult = SnapField->Snap(CurrentTime.RoundToFrame(), SnapThreshold);
		if (SnappedResult.IsSet())
		{
			CurrentTime = SnappedResult->SnappedTime;
		}
	}

	if (Sequencer->GetSequencerSettings()->GetForceWholeFrames())
	{
		CurrentTime = FFrameRate::Snap(CurrentTime, Sequencer->GetFocusedTickResolution(), Sequencer->GetFocusedDisplayRate());
	}

	FStretchScreenParameters ScreenParameters;
	ScreenParameters.DragStartPosition   = DragStartPosition;
	ScreenParameters.CurrentDragPosition = CurrentTime;
	ScreenParameters.LocalMousePos       = LocalMousePos;
	ScreenParameters.MouseEvent          = &MouseEvent;
	ScreenParameters.VirtualTrackArea    = &VirtualTrackArea;

	FStretchParameters TempParams;
	const bool bUseGlobalParams = MouseEvent.IsAltDown() == false;

	for (TPair<TSharedPtr<IStretchableExtension>, FStretchTarget> Pair : StretchTargets)
	{
		FStretchParameters* Params = &Pair.Value.Params;
		if (bUseGlobalParams)
		{
			// Ensure the TempParams match the global ones (in-case something overwrote them)
			TempParams = GlobalParameters;
			Params = &TempParams;
		}
		Pair.Key->OnStretch(*this, ScreenParameters, Params);
	}
}

void FEditToolDragOperation_Stretch::OnEndDrag( const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const FVirtualTrackArea& VirtualTrackArea)
{
	FFrameTime CurrentTime = VirtualTrackArea.PixelToFrame(LocalMousePos.X);

	if (Sequencer->GetSequencerSettings()->GetIsSnapEnabled())
	{
		const float PixelSnapWidth = 10.f;

		const float SnapThresholdPx = VirtualTrackArea.PixelToSeconds(PixelSnapWidth) - VirtualTrackArea.PixelToSeconds(0.f);
		const int32 SnapThreshold = (SnapThresholdPx * Sequencer->GetFocusedTickResolution()).FloorToFrame().Value;

		TOptional<FSequencerSnapField::FSnapResult> SnappedResult = SnapField->Snap(CurrentTime.RoundToFrame(), SnapThreshold);
		if (SnappedResult.IsSet())
		{
			CurrentTime = SnappedResult->SnappedTime;
		}
	}

	if (Sequencer->GetSequencerSettings()->GetForceWholeFrames())
	{
		CurrentTime = FFrameRate::Snap(CurrentTime, Sequencer->GetFocusedTickResolution(), Sequencer->GetFocusedDisplayRate());
	}

	FStretchScreenParameters ScreenParameters;
	ScreenParameters.DragStartPosition   = DragStartPosition;
	ScreenParameters.CurrentDragPosition = VirtualTrackArea.PixelToFrame(LocalMousePos.X);
	ScreenParameters.LocalMousePos       = LocalMousePos;
	ScreenParameters.MouseEvent          = &MouseEvent;
	ScreenParameters.VirtualTrackArea    = &VirtualTrackArea;

	FStretchParameters TempParams;
	const bool bUseGlobalParams = MouseEvent.IsAltDown() == false;

	for (TPair<TSharedPtr<IStretchableExtension>, FStretchTarget> Pair : StretchTargets)
	{
		FStretchParameters* Params = &Pair.Value.Params;
		if (bUseGlobalParams)
		{
			// Ensure the TempParams match the global ones (in-case something overwrote them)
			TempParams = GlobalParameters;
			Params = &TempParams;
		}

		Pair.Key->OnEndStretch(*this, ScreenParameters, Params);
	}

	Transaction.Reset();
}

bool FEditToolDragOperation_Stretch::IsKeyApplicable(FKeyHandle KeyHandle, const FViewModelPtr& Owner) const
{
	TSharedPtr<FChannelModel> Channel = Owner.ImplicitCast();
	return !Channel || !SnapExclusion.Contains(Channel->GetSection());
}

bool FEditToolDragOperation_Stretch::AreSectionBoundsApplicable(UMovieSceneSection* Section) const
{
	return !SnapExclusion.Contains(Section);
}

bool FEditToolDragOperation_Stretch::AreSectionCustomSnapsApplicable(UMovieSceneSection* Section) const
{
	return !SnapExclusion.Contains(Section);
}

FCursorReply FEditToolDragOperation_Stretch::GetCursor() const
{
	return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
}

int32 FEditToolDragOperation_Stretch::OnPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	return LayerId;
}

} // namespace Sequencer
} // namespace UE

#undef LOCTEXT_NAMESPACE

