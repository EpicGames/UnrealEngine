// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ISequencerEditTool.h"
#include "MVVM/Extensions/IStretchableExtension.h"
#include "Tools/SequencerSnapField.h"

#include "ScopedTransaction.h"

class ISequencer;
class FSequencerSnapField;

namespace UE
{
namespace Sequencer
{


class FEditToolDragOperation_Stretch : public ISequencerEditToolDragOperation, public IStretchOperation, public ISnapCandidate
{
public:

	FEditToolDragOperation_Stretch(ISequencer* InSequencer, EStretchConstraint InStretchConstraint, FFrameNumber InDragStartPosition);

	void DoNotSnapTo(TSharedPtr<FViewModel> Model) override;
	bool InitiateStretch(TSharedPtr<FViewModel> Controller, TSharedPtr<IStretchableExtension> Target, int32 Priority, const FStretchParameters& InParams) override;

	void OnBeginDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const FVirtualTrackArea& VirtualTrackArea) override;
	void OnDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const FVirtualTrackArea& VirtualTrackArea) override;
	void OnEndDrag( const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const FVirtualTrackArea& VirtualTrackArea) override;
	FCursorReply GetCursor() const override;
	int32 OnPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const override;

	bool IsKeyApplicable(FKeyHandle KeyHandle, const FViewModelPtr& Owner) const;
	bool AreSectionBoundsApplicable(UMovieSceneSection* Section) const;
	bool AreSectionCustomSnapsApplicable(UMovieSceneSection* Section) const;

private:

	struct FStretchTarget
	{
		FStretchParameters Params;
		int32 Priority;
	};
	TMap<TSharedPtr<IStretchableExtension>, FStretchTarget> StretchTargets;

	TSet<UMovieSceneSection*> SnapExclusion;

	TUniquePtr<FSequencerSnapField> SnapField;

	/** Scoped transaction for this drag operation */
	TUniquePtr<FScopedTransaction> Transaction;

	FStretchParameters GlobalParameters;

	ISequencer* Sequencer;

	FFrameNumber DragStartPosition;

	EStretchConstraint StretchConstraint;
};

} // namespace Sequencer
} // namespace UE

