// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/EditToolDragOperations.h"
#include "ISequencer.h"
#include "ISequencerSection.h"
#include "MovieSceneTrack.h"
#include "MovieSceneSequence.h"
#include "Sequencer.h"
#include "SSequencer.h"
#include "SequencerSettings.h"
#include "SequencerCommonHelpers.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/Extensions/IDraggableTrackAreaExtension.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "MVVM/ViewModels/VirtualTrackArea.h"
#include "Algo/AllOf.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "Modules/ModuleManager.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "ISequencerModule.h"

struct FInvalidKeyAndSectionSnappingCandidates : UE::Sequencer::ISnapCandidate
{
	/**
	 * Keys and Sections added to this ISnapField will be ignored as potential candidates for snapping.
	 */
	FInvalidKeyAndSectionSnappingCandidates(const TSet<FSequencerSelectedKey>& InKeysToIgnore, const TSet<UMovieSceneSection*>& InSectionsToIgnore)
	{
		KeysToExclude = InKeysToIgnore;
		SectionsToExclude = InSectionsToIgnore;
	}

	virtual bool IsKeyApplicable(FKeyHandle KeyHandle, const UE::Sequencer::FViewModelPtr& Owner) const override
	{
		using namespace UE::Sequencer;

		TSharedPtr<FChannelModel> Channel = Owner.ImplicitCast();
		return !Channel || (!KeysToExclude.Contains(FSequencerSelectedKey(*Channel->GetSection(), Channel, KeyHandle)) && !SectionsToExclude.Contains(Channel->GetSection()));
	}

	virtual bool AreSectionBoundsApplicable(UMovieSceneSection* Section) const override
	{
		return !SectionsToExclude.Contains(Section);
	}
	
protected:
	TSet<FSequencerSelectedKey> KeysToExclude;
	TSet<UMovieSceneSection*> SectionsToExclude;
};


/** How many pixels near the mouse has to be before snapping occurs */
const float PixelSnapWidth = 20.f;



FEditToolDragOperation::FEditToolDragOperation( FSequencer& InSequencer )
	: Sequencer(InSequencer)
{
	Settings = Sequencer.GetSequencerSettings();
}

FCursorReply FEditToolDragOperation::GetCursor() const
{
	return FCursorReply::Cursor( EMouseCursor::Default );
}

int32 FEditToolDragOperation::OnPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	return LayerId;
}

void FEditToolDragOperation::BeginTransaction( TSet<UMovieSceneSection*>& Sections, const FText& TransactionDesc )
{
	// Begin an editor transaction and mark the section as transactional so it's state will be saved
	Transaction.Reset( new FScopedTransaction(TransactionDesc) );

	for (auto It = Sections.CreateIterator(); It; ++It)
	{
		UMovieSceneSection* SectionObj = *It;

		SectionObj->SetFlags( RF_Transactional );
		// Save the current state of the section
		if (!SectionObj->TryModify())
		{
			It.RemoveCurrent();
		}
	}
}

void FEditToolDragOperation::EndTransaction()
{
	Transaction.Reset();
	Sequencer.NotifyMovieSceneDataChanged( EMovieSceneDataChangeType::TrackValueChanged );
}

TRange<FFrameNumber> FEditToolDragOperation::GetSectionBoundaries(const UMovieSceneSection* Section)
{
	using namespace UE::Sequencer;

	// Find the borders of where you can drag to
	FFrameNumber LowerBound = TNumericLimits<int32>::Lowest(), UpperBound = TNumericLimits<int32>::Max();

	// Find the track node for this section
	TSharedPtr<FSectionModel> SectionHandle = Sequencer.GetNodeTree()->GetSectionModel(Section);
	if (SectionHandle)
	{
		// Get the closest borders on either side
		TViewModelPtr<ITrackAreaExtension> TrackModel = SectionHandle->FindAncestorOfType<ITrackAreaExtension>();
		for (const TViewModelPtr<FSectionModel>& SectionModel : TrackModel->GetTrackAreaModelListAs<FSectionModel>())
		{
			const UMovieSceneSection* TestSection = SectionModel->GetSection();
			TArray<UMovieSceneSection*> Sections;
			GetSections(Sections);
			if (!TestSection || Sections.Contains(TestSection))
			{
				continue;
			}

			if (TestSection->HasEndFrame() && Section->HasStartFrame() && TestSection->GetExclusiveEndFrame() <= Section->GetInclusiveStartFrame() && TestSection->GetExclusiveEndFrame() > LowerBound)
			{
				LowerBound = TestSection->GetExclusiveEndFrame();
			}
			if (TestSection->HasStartFrame() && Section->HasEndFrame() && TestSection->GetInclusiveStartFrame() >= Section->GetExclusiveEndFrame() && TestSection->GetInclusiveStartFrame() < UpperBound)
			{
				UpperBound = TestSection->GetInclusiveStartFrame();
			}
		}
	}

	return TRange<FFrameNumber>(LowerBound, UpperBound);
}

FResizeSection::FResizeSection( FSequencer& InSequencer, bool bInDraggingByEnd, bool bInIsSlipping )
	: FEditToolDragOperation( InSequencer )
	, bDraggingByEnd(bInDraggingByEnd)
	, bIsSlipping(bInIsSlipping)
	, MouseDownTime(0)
{
	TSet<UMovieSceneSection*> SelectedSections = Sequencer.GetViewModel()->GetSelection()->GetSelectedSections();
	Sections.Reserve(SelectedSections.Num());

	for (TWeakObjectPtr<UMovieSceneSection> WeakSection : SelectedSections)
	{
		UMovieSceneSection* Section = WeakSection.Get();
		if (Section)
		{
			if (bDraggingByEnd && Section->HasEndFrame())
			{
				Sections.Add(Section);
			}
			else if (!bDraggingByEnd && Section->HasStartFrame())
			{
				Sections.Add(Section);
			}
		}
	}
}

void FResizeSection::OnBeginDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea)
{
	using namespace UE::Sequencer;

	UE::MovieScene::FScopedSignedObjectModifyDefer DeferMarkAsChanged;

	BeginTransaction( Sections, NSLOCTEXT("Sequencer", "DragSectionEdgeTransaction", "Resize section") );

	MouseDownTime = VirtualTrackArea.PixelToFrame(LocalMousePos.X);

	// Construct a snap field of unselected sections
	TSet<FSequencerSelectedKey> EmptyKeySet;
	FInvalidKeyAndSectionSnappingCandidates SnapCandidates(EmptyKeySet, Sections);
	SnapField = FSequencerSnapField(Sequencer, SnapCandidates, ESequencerEntity::Section | ESequencerEntity::Key);

	SectionInitTimes.Empty();

	bool bIsDilating = MouseEvent.IsControlDown();
	PreDragSectionData.Empty();

	for (UMovieSceneSection* Section : Sections)
	{
		if (bIsDilating)
		{
			// Populate the resize data for this section
			FPreDragSectionData ResizeData; 
			ResizeData.MovieSection = Section;
			ResizeData.InitialRange = Section->GetRange();

			TSharedPtr<FSectionModel> SectionHandle = Sequencer.GetNodeTree()->GetSectionModel(Section);
			if (SectionHandle)
			{
				//Tell section that may not have keys it's starting to dilate (e.g. skeletal tracks will cache play rate).
				SectionHandle->GetSectionInterface()->BeginDilateSection();
				ResizeData.SequencerSection = SectionHandle->GetSectionInterface().Get();
			}

			// Add the key times for all keys of all channels on this section
			FMovieSceneChannelProxy& Proxy = Section->GetChannelProxy();
			for (const FMovieSceneChannelEntry& Entry : Proxy.GetAllEntries())
			{
				TArrayView<FMovieSceneChannel* const> ChannelPtrs = Entry.GetChannels();
				for (int32 Index = 0; Index < ChannelPtrs.Num(); ++Index)
				{
					// Populate the cached state of this channel
					FPreDragChannelData& ChannelData = ResizeData.Channels[ResizeData.Channels.Emplace()];
					ChannelData.Channel = Proxy.MakeHandle(Entry.GetChannelTypeName(), Index);

					ChannelPtrs[Index]->GetKeys(TRange<FFrameNumber>::All(), &ChannelData.FrameNumbers, &ChannelData.Handles);
				}
			}
			PreDragSectionData.Emplace(ResizeData);
		}
		else if (TSharedPtr<FSectionModel> SectionHandle = Sequencer.GetNodeTree()->GetSectionModel(Section))
		{
			if (bIsSlipping)
			{
				SectionHandle->GetSectionInterface()->BeginSlipSection();
			}
			else
			{
				SectionHandle->GetSectionInterface()->BeginResizeSection();
			}
		}

		SectionInitTimes.Add(Section, bDraggingByEnd ? Section->GetExclusiveEndFrame() : Section->GetInclusiveStartFrame());
	}
}

void FResizeSection::OnEndDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea)
{
	bool bIsDilating = MouseEvent.IsControlDown();

	TOptional<EMovieSceneDataChangeType> ChangeNotification;

	if (!bIsDilating)
	{
		TSet<UMovieSceneTrack*> Tracks;
		FMovieSceneSectionMovedParams SectionMovedParams(EPropertyChangeType::ValueSet);
		EMovieSceneSectionMovedResult SectionMovedResult(EMovieSceneSectionMovedResult::None);
		for (UMovieSceneSection* Section : Sections)
		{
			if (UMovieSceneTrack* OuterTrack = Section->GetTypedOuter<UMovieSceneTrack>())
			{
				OuterTrack->Modify();
				SectionMovedResult |= OuterTrack->OnSectionMoved(*Section, SectionMovedParams);

				Tracks.Add(OuterTrack);
			}
		}
		for (UMovieSceneTrack* Track : Tracks)
		{
			Track->UpdateEasing();
		}
		if (SectionMovedResult != EMovieSceneSectionMovedResult::None)
		{
			ChangeNotification = EMovieSceneDataChangeType::MovieSceneStructureItemsChanged;
		}
	}

	EndTransaction();

	if (ChangeNotification.IsSet())
	{
		Sequencer.NotifyMovieSceneDataChanged(ChangeNotification.GetValue());
	}
}

void FResizeSection::OnDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea)
{
	using namespace UE::Sequencer;

	UE::MovieScene::FScopedSignedObjectModifyDefer DeferMarkAsChanged(true);

	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");

	bool bIsDilating = MouseEvent.IsControlDown();

	ESequencerScrubberStyle ScrubStyle = Sequencer.GetScrubStyle();

	FFrameRate   TickResolution  = Sequencer.GetFocusedTickResolution();
	FFrameRate   DisplayRate     = Sequencer.GetFocusedDisplayRate();

	// Convert the current mouse position to a time
	FFrameNumber DeltaTime = (VirtualTrackArea.PixelToFrame(LocalMousePos.X) - MouseDownTime).RoundToFrame();

	// Snapping
	const bool bSnapEnabled = (Settings->GetIsSnapEnabled() || MouseEvent.IsShiftDown());
	const bool bSnapToInterval = Settings->GetForceWholeFrames();
	const bool bSnapToLikeTypes = bSnapEnabled && Settings->GetSnapSectionTimesToElements();
	
	SnapField.GetValue().SetSnapToInterval(bSnapToInterval);
	SnapField.GetValue().SetSnapToLikeTypes(bSnapToLikeTypes);

	if (bSnapEnabled || bSnapToInterval)
	{
		TArray<FFrameTime> SectionTimes;
		for (UMovieSceneSection* Section : Sections)
		{
			SectionTimes.Add(SectionInitTimes[Section] + DeltaTime);
		}

		float SnapThresholdPx = VirtualTrackArea.PixelToSeconds(PixelSnapWidth) - VirtualTrackArea.PixelToSeconds(0.f);
		int32 SnapThreshold   = ( SnapThresholdPx * TickResolution ).FloorToFrame().Value;

		TOptional<FSequencerSnapField::FSnapResult> SnappedTime = SnapField->Snap(SectionTimes, SnapThreshold);

		if (SnappedTime.IsSet())
		{
			// Add the snapped amount onto the delta
			DeltaTime += (SnappedTime->SnappedTime - SnappedTime->OriginalTime).RoundToFrame();
		}
	}

	auto GetMovementMaximums = [this](const FPointerEvent& MouseEvent, const UMovieSceneSection* Section, TOptional<FFrameNumber>& LeftMovementMaximum, TOptional<FFrameNumber>& RightMovementMaximum)
	{
		// We'll calculate this section's borders and clamp the possible delta time to be less than that

		if (!Section->GetBlendType().IsValid())
		{
			TRange<FFrameNumber> SectionBoundaries = GetSectionBoundaries(Section);
			LeftMovementMaximum = UE::MovieScene::DiscreteInclusiveLower(SectionBoundaries);
			RightMovementMaximum = UE::MovieScene::DiscreteExclusiveUpper(SectionBoundaries);
		}
	};
	
	/********************************************************************/
	EMovieSceneSectionMovedResult SectionMovedResult(EMovieSceneSectionMovedResult::None);
	if (bIsDilating)
	{
		for(FPreDragSectionData Data: PreDragSectionData)
		{
			// It is only valid to dilate a fixed bound. Tracks can have mixed bounds types (ie: infinite upper, closed lower)
			check(bDraggingByEnd ? Data.InitialRange.GetUpperBound().IsClosed() : Data.InitialRange.GetLowerBound().IsClosed());

			TOptional<FFrameNumber> LeftMovementMaximum;
			TOptional<FFrameNumber> RightMovementMaximum;
			GetMovementMaximums(MouseEvent, Data.MovieSection, LeftMovementMaximum, RightMovementMaximum);

			FFrameNumber StartPosition  = bDraggingByEnd ? UE::MovieScene::DiscreteExclusiveUpper(Data.InitialRange) : UE::MovieScene::DiscreteInclusiveLower(Data.InitialRange);

			FFrameNumber DilationOrigin;
			if (bDraggingByEnd)
			{
				if (Data.InitialRange.GetLowerBound().IsClosed())
				{
					DilationOrigin = UE::MovieScene::DiscreteInclusiveLower(Data.InitialRange);
				}
				else
				{
					// We're trying to dilate a track that has an infinite lower bound as its origin.
					// Sections already compute an effective range for UMG's auto-playback range, so we'll use that to have it handle finding either the
					// uppermost key or the overall length of the section.
					DilationOrigin = Data.MovieSection->ComputeEffectiveRange().GetLowerBoundValue();
				}
			}
			else
			{
				if (Data.InitialRange.GetUpperBound().IsClosed())
				{
					DilationOrigin = UE::MovieScene::DiscreteExclusiveUpper(Data.InitialRange);
				}
				else
				{
					// We're trying to dilate a track that has an infinite upper bound as its origin. 
					DilationOrigin = Data.MovieSection->ComputeEffectiveRange().GetUpperBoundValue();

				}
			}

			// Because we can have an one-sided infinite data range, we calculate a new range using our clamped values. 
			TRange<FFrameNumber> DataRange;
			DataRange.SetLowerBound(TRangeBound<FFrameNumber>(DilationOrigin < StartPosition ? DilationOrigin : StartPosition));
			DataRange.SetUpperBound(TRangeBound<FFrameNumber>(DilationOrigin > StartPosition ? DilationOrigin : StartPosition));

			FFrameNumber NewPosition    = bDraggingByEnd ? FMath::Clamp(StartPosition + DeltaTime, DilationOrigin, RightMovementMaximum.Get(TNumericLimits<int32>::Max()))
														 : FMath::Clamp(StartPosition + DeltaTime, LeftMovementMaximum.Get(TNumericLimits<int32>::Lowest()), DilationOrigin);

			float DilationFactor = FMath::Abs(NewPosition.Value - DilationOrigin.Value) / static_cast<float>(DataRange.Size<FFrameNumber>().Value);

			if (bDraggingByEnd)
			{
				if (Data.SequencerSection)
				{
					Data.SequencerSection->DilateSection(TRange<FFrameNumber>(Data.MovieSection->GetRange().GetLowerBound(), TRangeBound<FFrameNumber>::Exclusive(NewPosition)), DilationFactor);
				}
				else
				{
					Data.MovieSection->SetRange(TRange<FFrameNumber>(Data.MovieSection->GetRange().GetLowerBound(), TRangeBound<FFrameNumber>::Exclusive(NewPosition)));
				}
			}
			else
			{
				if (Data.SequencerSection)
				{
					Data.SequencerSection->DilateSection(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(NewPosition), Data.MovieSection->GetRange().GetUpperBound()), DilationFactor);
				}
				else
				{
					Data.MovieSection->SetRange(TRange<FFrameNumber>(TRangeBound<FFrameNumber>::Inclusive(NewPosition), Data.MovieSection->GetRange().GetUpperBound()));
				}
			}


			TArray<FFrameNumber> NewFrameNumbers;
			for (const FPreDragChannelData& ChannelData : Data.Channels)
			{
				// Compute new frame times for each key
				NewFrameNumbers.Reset(ChannelData.FrameNumbers.Num());
				for (FFrameNumber StartFrame : ChannelData.FrameNumbers)
				{
					FFrameNumber NewTime = DilationOrigin + FFrameNumber(FMath::FloorToInt((StartFrame - DilationOrigin).Value * DilationFactor));
					NewFrameNumbers.Add(NewTime);
				}

				// Apply the key times to the channel
				FMovieSceneChannel* Channel = ChannelData.Channel.Get();
				if (Channel)
				{
					Channel->SetKeyTimes(ChannelData.Handles, NewFrameNumbers);
				}
			}
		}
	}
	/********************************************************************/
	else for (UMovieSceneSection* Section : Sections)
	{
		TSharedPtr<FSectionModel> SectionHandle = Sequencer.GetNodeTree()->GetSectionModel(Section);
		if (!SectionHandle)
		{
			continue;
		}

		TOptional<FFrameNumber> LeftMovementMaximum;
		TOptional<FFrameNumber> RightMovementMaximum;
		GetMovementMaximums(MouseEvent, Section, LeftMovementMaximum, RightMovementMaximum);

		TSharedPtr<ISequencerSection> SectionInterface = SectionHandle->GetSectionInterface();

		FFrameNumber NewTime = SectionInitTimes[Section] + DeltaTime;

		if( bDraggingByEnd )
		{
			FFrameNumber MinFrame = Section->HasStartFrame() ? Section->GetInclusiveStartFrame() : TNumericLimits<int32>::Lowest();

			if (Settings->GetForceWholeFrames())
			{
				int32 IntervalSnapThreshold = FMath::RoundToInt( ( TickResolution / DisplayRate ).AsDecimal() );
				MinFrame = MinFrame + IntervalSnapThreshold;
			}

			FFrameNumber MaxFrame = RightMovementMaximum.Get(TNumericLimits<int32>::Max());

			// Dragging the end of a section
			// Ensure we aren't shrinking past the start time or into another section if we can't blend
			NewTime = FMath::Clamp( NewTime, MinFrame, MaxFrame );
			if (bIsSlipping)
			{
				SectionInterface->SlipSection( NewTime );
			}
			else
			{
				SectionInterface->ResizeSection( SSRM_TrailingEdge, NewTime );
			}
		}
		else
		{
			FFrameNumber MaxFrame = Section->HasEndFrame() ? Section->GetExclusiveEndFrame()-1 : TNumericLimits<int32>::Max();

			if (Settings->GetForceWholeFrames())
			{
				int32 IntervalSnapThreshold = FMath::RoundToInt( ( TickResolution / DisplayRate ).AsDecimal() );
				MaxFrame = MaxFrame - IntervalSnapThreshold;
			}

			FFrameNumber MinFrame = LeftMovementMaximum.Get(TNumericLimits<int32>::Lowest());

			// Dragging the start of a section
			// Ensure we arent expanding past the end time or into another section if we can't blend
			NewTime = FMath::Clamp( NewTime, MinFrame, MaxFrame );

			if (bIsSlipping)
			{
				SectionInterface->SlipSection( NewTime );
			}
			else
			{
				SectionInterface->ResizeSection( SSRM_LeadingEdge, NewTime );
			}
		}

		UMovieSceneTrack* OuterTrack = Section->GetTypedOuter<UMovieSceneTrack>();
		if (OuterTrack)
		{
			OuterTrack->Modify();
			SectionMovedResult |= OuterTrack->OnSectionMoved(*Section, EPropertyChangeType::Interactive);
		}
	}

	{
		TSet<UMovieSceneTrack*> Tracks;
		for (UMovieSceneSection* Section : Sections)
		{
			if (UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>())
			{
				Tracks.Add(Track);
			}
		}
		for (UMovieSceneTrack* Track : Tracks)
		{
			Track->UpdateEasing();
		}
	}

	if (SectionMovedResult != EMovieSceneSectionMovedResult::None)
	{
		Sequencer.NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	}
	else
	{
		Sequencer.NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}
}

void FDuplicateKeysAndSections::OnBeginDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea)
{
	// Begin an editor transaction and mark the section as transactional so it's state will be saved
	BeginTransaction( Sections, NSLOCTEXT("Sequencer", "DuplicateKeysTransaction", "Duplicate Keys or Sections") );

	// Call Modify on all of the sections that own keys we have selected so that when we duplicate keys we can restore them properly.
	ModifyNonSelectedSections();

	// We're going to take our current selection and make a duplicate of each item in it and leave those items behind.
	// This means our existing selection will still refer to the same keys, so we're duplicating and moving the originals.
	// This saves us from modifying the user's selection when duplicating. We can't move the duplicates as we can't get
	// section handles for sections until the tree is rebuilt.
	TArray<FKeyHandle> NewKeyHandles;
	NewKeyHandles.SetNumZeroed(KeysAsArray.Num());

	// Duplicate our keys into the NewKeyHandles array. Duplicating keys automatically updates their sections,
	// so we don't need to actually use the new key handles.
	DuplicateKeys(KeysAsArray, NewKeyHandles);

	for (UMovieSceneSection* SectionToDuplicate : Sections)
	{
		if (!SectionToDuplicate)
		{
			continue;
		}

		UMovieSceneSection* DuplicatedSection = DuplicateObject<UMovieSceneSection>(SectionToDuplicate, SectionToDuplicate->GetOuter());
		UMovieSceneTrack* OwningTrack = SectionToDuplicate->GetTypedOuter<UMovieSceneTrack>();
		OwningTrack->Modify();
		OwningTrack->AddSection(*DuplicatedSection);
	}

	// Now start the move drag
	FMoveKeysAndSections::OnBeginDrag(MouseEvent, LocalMousePos, VirtualTrackArea);
}

void FDuplicateKeysAndSections::OnEndDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea)
{
	FMoveKeysAndSections::OnEndDrag(MouseEvent, LocalMousePos, VirtualTrackArea);

	EndTransaction();
}

FManipulateSectionEasing::FManipulateSectionEasing( FSequencer& InSequencer, TWeakObjectPtr<UMovieSceneSection> InSection, bool _bEaseIn )
	: FEditToolDragOperation(InSequencer)
	, WeakSection(InSection)
	, bEaseIn(_bEaseIn)
	, MouseDownTime(0)
{
}

void FManipulateSectionEasing::OnBeginDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea)
{
	using namespace UE::Sequencer;

	Transaction.Reset( new FScopedTransaction(NSLOCTEXT("Sequencer", "DragSectionEasing", "Change Section Easing")) );

	UMovieSceneSection* Section = WeakSection.Get();
	if (Section == nullptr)
	{
		return;
	}

	Section->SetFlags( RF_Transactional );
	Section->Modify();

	MouseDownTime = VirtualTrackArea.PixelToFrame(LocalMousePos.X);

	// Construct a snap field of all section bounds
	ISnapCandidate SnapCandidates;
	SnapField = FSequencerSnapField(Sequencer, SnapCandidates, ESequencerEntity::Section);

	InitValue = bEaseIn ? Section->Easing.GetEaseInDuration() : Section->Easing.GetEaseOutDuration();
}

void FManipulateSectionEasing::OnDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea)
{
	UE::MovieScene::FScopedSignedObjectModifyDefer DeferMarkAsChanged(true);

	ESequencerScrubberStyle ScrubStyle = Sequencer.GetScrubStyle();

	FFrameRate TickResolution  = Sequencer.GetFocusedTickResolution();
	FFrameRate DisplayRate     = Sequencer.GetFocusedDisplayRate();

	// Convert the current mouse position to a time
	FFrameTime  DeltaTime = VirtualTrackArea.PixelToFrame(LocalMousePos.X) - MouseDownTime;

	UMovieSceneSection* Section = WeakSection.Get();
	if (Section == nullptr)
	{
		return;
	}

	// Snapping
	const bool bSnapEnabled = Settings->GetIsSnapEnabled() || MouseEvent.IsShiftDown();
	const bool bSnapToInterval = Settings->GetForceWholeFrames();
	const bool bSnapToLikeTypes = bSnapEnabled && Settings->GetSnapSectionTimesToElements();

	SnapField.GetValue().SetSnapToInterval(bSnapToInterval);
	SnapField.GetValue().SetSnapToLikeTypes(bSnapToLikeTypes);

	if (bSnapEnabled || bSnapToInterval)
	{
		TArray<FFrameTime> SnapTimes;
		if (bEaseIn)
		{
			FFrameNumber DesiredTime = (DeltaTime + Section->GetInclusiveStartFrame() + InitValue.Get(0)).RoundToFrame();
			SnapTimes.Add(DesiredTime);
		}
		else
		{
			FFrameNumber DesiredTime = (Section->GetExclusiveEndFrame() - InitValue.Get(0) + DeltaTime).RoundToFrame();
			SnapTimes.Add(DesiredTime);
		}

		float SnapThresholdPx = VirtualTrackArea.PixelToSeconds(PixelSnapWidth) - VirtualTrackArea.PixelToSeconds(0.f);
		int32 SnapThreshold   = ( SnapThresholdPx * TickResolution ).FloorToFrame().Value;

		TOptional<FSequencerSnapField::FSnapResult> SnappedTime = SnapField->Snap(SnapTimes, SnapThreshold);

		if (SnappedTime.IsSet())
		{
			// Add the snapped amount onto the delta
			DeltaTime += SnappedTime->SnappedTime - SnappedTime->OriginalTime;
		}
	}

	const int32 MaxEasingDuration = Section->HasStartFrame() && Section->HasEndFrame() ? UE::MovieScene::DiscreteSize(Section->GetRange()) : TNumericLimits<int32>::Max() / 2;

	Section->Modify();

	if (bEaseIn)
	{
		Section->Easing.bManualEaseIn = true;
		Section->Easing.ManualEaseInDuration  = FMath::Clamp(InitValue.Get(0) + DeltaTime.RoundToFrame().Value, 0, MaxEasingDuration);
	}
	else
	{
		Section->Easing.bManualEaseOut = true;
		Section->Easing.ManualEaseOutDuration = FMath::Clamp(InitValue.Get(0) - DeltaTime.RoundToFrame().Value, 0, MaxEasingDuration);
	}
}

void FManipulateSectionEasing::OnEndDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea)
{
	EndTransaction();
}


FMoveKeysAndSections::FMoveKeysAndSections(FSequencer& InSequencer, ESequencerMoveOperationType MoveType)
	: FEditToolDragOperation(InSequencer)
{
	using namespace UE::Sequencer;

	TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = InSequencer.GetViewModel()->CastThisShared<FSequencerEditorViewModel>();
	bAllowVerticalMovement = HotspotCast<FSectionHotspotBase>(SequencerViewModel->GetHotspot()) != nullptr;

	if (EnumHasAnyFlags(MoveType, ESequencerMoveOperationType::MoveKeys))
	{
		// Filter out the keys on sections that are read only
		const FKeySelection& KeySelection = Sequencer.GetViewModel()->GetSelection()->KeySelection;

		Keys.Reserve(KeySelection.Num());

		for (FKeyHandle Key : KeySelection)
		{
			TSharedPtr<FChannelModel> Channel = KeySelection.GetModelForKey(Key);
			UMovieSceneSection*       Section = Channel ? Channel->GetSection() : nullptr;
			if (Section && !Section->IsReadOnly())
			{
				Keys.Emplace(FSequencerSelectedKey(*Section, Channel, Key));
			}
		}

		KeysAsArray = Keys.Array();
	}

	if (EnumHasAnyFlags(MoveType, ESequencerMoveOperationType::MoveSections))
	{
		for (TViewModelPtr<IDraggableTrackAreaExtension> DraggableItem : Sequencer.GetViewModel()->GetSelection()->TrackArea.Filter<IDraggableTrackAreaExtension>())
		{
			DraggableItem->OnBeginDrag(*this);
		}
	}

	// Always move selected marked frames along with keys and/or sections.
	MarkedFrames = Sequencer.GetViewModel()->GetSelection()->MarkedFrames.GetSelected();
}

void FMoveKeysAndSections::AddSnapTime(FFrameNumber SnapTime)
{
	RelativeSnapOffsets.Add(SnapTime);
}

void FMoveKeysAndSections::AddModel(TSharedPtr<UE::Sequencer::FViewModel> Model)
{
	using namespace UE::Sequencer;

	if (FSectionModel* SectionModel = Model->CastThis<FSectionModel>())
	{
		if (UMovieSceneSection* Section = SectionModel->GetSection())
		{
			Sections.Add(Section);

			if (UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>())
			{
				// If the section is in a group, we also want to add the sections it is grouped with
				if (const FMovieSceneSectionGroup* SectionGroup = MovieScene->GetSectionGroup(*Section))
				{
					for (TWeakObjectPtr<UMovieSceneSection> WeakGroupedSection : *SectionGroup)
					{
						// Verify sections are still valid, and are not infinite.
						if (WeakGroupedSection.IsValid())
						{
							Sections.Add(WeakGroupedSection.Get());
						}
					}
				}
			}
		}
	}
	if (TSharedPtr<IDraggableTrackAreaExtension> DraggableItem = Model->CastThisShared<IDraggableTrackAreaExtension>())
	{
		DraggedItems.Add(DraggableItem);

		// Cache the drag owner track model if not already set (used for hover suppression during drag)
		if (!CachedDragOwnerTrackModel.IsValid())
		{
			if (TViewModelPtr<ISectionDragOwnerExtension> TrackExt = Model->FindAncestorOfType<ISectionDragOwnerExtension>(true))
			{
				CachedDragOwnerTrackModel = TrackExt.AsModel();
			}
		}
	}
}

TSharedPtr<UE::Sequencer::FViewModel> FMoveKeysAndSections::GetDragOwnerTrackModel() const
{
	return CachedDragOwnerTrackModel.Pin();
}

void FMoveKeysAndSections::OnBeginDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea)
{
	using namespace UE::Sequencer;

	// Early out if we've somehow started a drag operation without any sections or keys. This prevents an empty Undo/Redo Transaction from being created.
	if (!Sections.Num() && !Keys.Num())
	{
		return;
	}

	BeginTransaction(Sections, NSLOCTEXT("Sequencer", "MoveKeyAndSectionTransaction", "Move Keys or Sections"));

	// Tell the Snap Field to ignore our currently selected keys and sections. We can snap to the edges of non-selected
	// sections and keys. The actual snapping field will add other sequencer data (play ranges, playheads, etc.) as snap targets.
	FInvalidKeyAndSectionSnappingCandidates AvoidSnapCanidates(Keys, Sections);
	SnapField = FSequencerSnapField(Sequencer, AvoidSnapCanidates );

	// Store the frame time of the mouse so we can see how far we've moved from the starting point.
	MouseTimePrev = VirtualTrackArea.PixelToFrame(LocalMousePos.X).RoundToFrame();

	// Convert initial snapoffsets so they are relative to the mouse time
	for (FFrameNumber& RelativeTime : RelativeSnapOffsets)
	{
		RelativeTime = (RelativeTime - MouseTimePrev).FloorToFrame();
	}

	// Sections can be dragged vertically to adjust their row up or down, so we need to store what row each section is currently on. A section
	// can be dragged above all other sections - this is accomplished by moving all other sections down. We store the row indices for all sections
	// in all tracks that we're modifying so we can get them later to move them.
	TSet<UMovieSceneTrack*> Tracks;
	for (UMovieSceneSection* Section : Sections)
	{
		Tracks.Add(Section->GetTypedOuter<UMovieSceneTrack>());
	}
	for (UMovieSceneTrack* Track : Tracks)
	{
		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			InitialSectionRowIndicies.Add(FInitialRowIndex{ Section, Section->GetRowIndex() });
		}
	}

	// Notify all drag owners that drag is beginning
	FSectionModelStorageExtension* SectionStorage = Sequencer.GetViewModel()->GetRootModel()->CastDynamic<FSectionModelStorageExtension>();
	if (SectionStorage)
	{
		TSet<TSharedPtr<FViewModel>> ProcessedDragOwners;
		for (UMovieSceneSection* Section : Sections)
		{
			TSharedPtr<FSectionModel> SectionModel = SectionStorage->FindModelForSection(Section);
			TViewModelPtr<ISectionOwnerExtension> SectionOwner = SectionModel ? SectionModel->GetParentSectionOwnerModel() : nullptr;
			if (!SectionOwner)
			{
				continue;
			}

			TViewModelPtr<ISectionDragOwnerExtension> DragOwner = SectionOwner.ImplicitCast();
			if (!DragOwner)
			{
				DragOwner = SectionOwner.AsModel()->FindAncestorOfType<ISectionDragOwnerExtension>();
			}

			if (DragOwner && !ProcessedDragOwners.Contains(DragOwner.AsModel()))
			{
				DragOwner->OnBeginSectionVerticalDrag();
				ProcessedDragOwners.Add(DragOwner.AsModel());
			}
		}
	}

	// Our Key Handles don't store their times so we need to convert the handles into an array of times
	// so that we can store the relative offset to each one.
	TArray<FFrameNumber> KeyTimes;
	KeyTimes.SetNum(Keys.Num());
	GetKeyTimes(KeysAsArray, KeyTimes);

	const int32 StartNum = RelativeSnapOffsets.Num();
	RelativeSnapOffsets.SetNumUninitialized(StartNum + KeyTimes.Num());
	for (int32 Index = 0; Index < KeyTimes.Num(); ++Index)
	{
		RelativeSnapOffsets[StartNum + Index] = (KeyTimes[Index] - MouseTimePrev).RoundToFrame();
	}

	// Keys can be moved within sections without the section itself being moved, so we need to call Modify on any section that owns a key that isn't also being moved.
	ModifyNonSelectedSections();
}

void FMoveKeysAndSections::OnDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea)
{
	UE::MovieScene::FScopedSignedObjectModifyDefer DeferMarkAsChanged(true);

	if (!Sections.Num() && !Keys.Num())
	{
		return;
	}
	
	// Convert the current mouse position to a time
	FVector2D  VirtualMousePos = VirtualTrackArea.PhysicalToVirtual(LocalMousePos);
	FFrameTime MouseTime = VirtualTrackArea.PixelToFrame(LocalMousePos.X);

	// Calculate snapping first which modifies our MouseTime to reflect where it would have to be for the closest snap to work.
	FFrameTime SnapThreshold = VirtualTrackArea.PixelDeltaToFrame(PixelSnapWidth);

	// The edge of each bounded section as well as each individual key is a valid marker to try and snap to intervals/sections/etc.
	// We take our stored offsets and add them to our current time to figure out where on the timeline the are currently.
	TArray<FFrameTime> ValidSnapMarkers;

	// If they have both keys and settings selected then we snap to the interval if either one of them is enabled, otherwise respect the individual setting.
	const bool bSnapEnabled = (Settings->GetIsSnapEnabled() || MouseEvent.IsShiftDown());
	const bool bSnapToInterval = (Settings->GetForceWholeFrames() && (KeysAsArray.Num() > 0 || Sections.Num()));
	const bool bSnapToLikeTypes = bSnapEnabled && ((KeysAsArray.Num() > 0 && Settings->GetSnapKeyTimesToElements()) || (Sections.Num() > 0 && Settings->GetSnapSectionTimesToElements()));

	SnapField.GetValue().SetSnapToInterval(bSnapToInterval);
	SnapField.GetValue().SetSnapToLikeTypes(bSnapToLikeTypes);

	// RelativeSnapOffsets contains both our sections and our keys, and we add them all as potential things that can snap to stuff.
	if (bSnapEnabled || bSnapToInterval)
	{
		ValidSnapMarkers.SetNumUninitialized(RelativeSnapOffsets.Num());
		for (int32 Index = 0; Index < RelativeSnapOffsets.Num(); ++Index)
		{
			ValidSnapMarkers[Index] = (RelativeSnapOffsets[Index] + MouseTime);
		}
	}

	// Now we'll try and snap all of these points to the closest valid snap marker (which may be a section or interval)
	TOptional<FSequencerSnapField::FSnapResult> SnappedTime;

	if (bSnapEnabled || bSnapToInterval)
	{
		// This may or may not set the SnappedTime depending on if there are any sections within the threshold.
		SnappedTime = SnapField->Snap(ValidSnapMarkers, SnapThreshold);
	}

	// If they actually snapped to something (snapping may be on but settings might dictate nothing to snap to) add the difference
	// to our current MouseTime so that MouseTime reflects the amount needed to move to get to the whole snap point.
	if (SnappedTime.IsSet())
	{
		// Add the snapped amount onto the mouse time so the resulting delta brings us in alignment.
		MouseTime += (SnappedTime->SnappedTime - SnappedTime->OriginalTime);
	}

	// We'll calculate a DeltaX based on limits on movement (snapping, section collision) and then use them on keys and sections below.
	TOptional<FFrameNumber> MaxDeltaX = GetMovementDeltaX(MouseEvent, MouseTime);

	FFrameNumber MouseDeltaTime = (MouseTime - MouseTimePrev).FloorToFrame();
	MouseTimePrev = MouseTimePrev + MaxDeltaX.Get(MouseDeltaTime);

	// Move sections horizontally (limited by our calculated delta) and vertically based on mouse cursor.
	bool bSectionMovementModifiedStructure = HandleSectionMovement(MouseTime, VirtualMousePos, LocalMousePos, MaxDeltaX, MouseDeltaTime);

	// Update our key times by moving them by our delta.
	HandleKeyMovement(MouseEvent, MaxDeltaX, MouseDeltaTime);

	// Update our marked frames by moving them by our delta.
	HandleMarkedFrameMovement(MaxDeltaX, MouseDeltaTime);
	
	// Get a list of the unique tracks in this selection and update their easing so previews draw interactively as you drag.
	TSet<UMovieSceneTrack*> Tracks;
	FMovieSceneSectionMovedParams SectionMovedParams(EPropertyChangeType::Interactive);
	EMovieSceneSectionMovedResult SectionMovedResult = EMovieSceneSectionMovedResult::None;
	for (UMovieSceneSection* Section : Sections)
	{
		if (UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>())
		{
			SectionMovedResult |= Track->OnSectionMoved(*Section, SectionMovedParams);
			Tracks.Add(Track);
		}
	}

	for (UMovieSceneTrack* Track : Tracks)
	{
		Track->UpdateEasing();
	}

	// If we changed the layout by rearranging sections we need to tell the Sequencer to rebuild things, otherwise just re-evaluate existing tracks.
	if (EnumHasAnyFlags(SectionMovedResult, EMovieSceneSectionMovedResult::SectionsChanged))
	{
		bSectionMovementModifiedStructure = true;
	}
	if (bSectionMovementModifiedStructure)
	{
		Sequencer.NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	}
	else
	{
		Sequencer.NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}
}

void FMoveKeysAndSections::OnEndDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea)
{
	using namespace UE::Sequencer;

	UE::MovieScene::FScopedSignedObjectModifyDefer DeferMarkAsChanged;

	if (!Sections.Num() && !Keys.Num())
	{
		return;
	}

	TArray<FKeyHandle> KeyHandles;
	for (const FSequencerSelectedKey& Key : KeysAsArray)
	{
		KeyHandles.Add(Key.KeyHandle);
	}
	const FKeySelection& KeySelection = Sequencer.GetViewModel()->GetSelection()->KeySelection;
	SequencerHelpers::RemoveDuplicateKeys(KeySelection, MakeArrayView(KeyHandles));

	InitialSectionRowIndicies.Empty();
	ModifiedNonSelectedObjects.Empty();

	// Tracks can tell us if the row indexes for any sections were changed during our drag/drop operation.
	TSet<UMovieSceneTrack*> Tracks;

	for (UMovieSceneSection* Section : Sections)
	{
		// Grab only unique tracks as multiple sections can reside on the same track.
		Tracks.Add(Section->GetTypedOuter<UMovieSceneTrack>());
	}

	for (UMovieSceneTrack* Track : Tracks)
	{
		// Ensure all of the tracks have updated the row indices for their sections
		Track->FixRowIndices();
	}

	FMovieSceneSectionMovedParams SectionMovedParams(EPropertyChangeType::ValueSet);
	for (UMovieSceneSection* Section : Sections)
	{
		UMovieSceneTrack* OuterTrack = Section->GetTypedOuter<UMovieSceneTrack>();
		if (OuterTrack)
		{
			OuterTrack->Modify();
			OuterTrack->OnSectionMoved(*Section, SectionMovedParams);
		}
	}

	for (UMovieSceneTrack* Track : Tracks)
	{
		Track->UpdateEasing();
	}

	for (TWeakPtr<IDraggableTrackAreaExtension> WeakDraggableItem : DraggedItems)
	{
		if (TSharedPtr<IDraggableTrackAreaExtension> DraggableItem = WeakDraggableItem.Pin())
		{
			DraggableItem->OnEndDrag(*this);
		}
	}

	// Notify track models that vertical drag has ended
	if (bAllowVerticalMovement && Sections.Num() > 0)
	{
		FSectionModelStorageExtension* SectionStorage = Sequencer.GetViewModel()->GetRootModel()->CastDynamic<FSectionModelStorageExtension>();
		if (SectionStorage)
		{
			TSet<TSharedPtr<FViewModel>> ProcessedDragOwners;

			// Notify all unique drag owners that drag ended
			for (UMovieSceneSection* Section : Sections)
			{
				TSharedPtr<FSectionModel> SectionModel = SectionStorage->FindModelForSection(Section);
				TViewModelPtr<ISectionOwnerExtension> SectionOwner = SectionModel ? SectionModel->GetParentSectionOwnerModel() : nullptr;
				if (!SectionOwner)
				{
					continue;
				}

				// Find the drag owner (track model that implements ISectionDragOwnerExtension)
				TViewModelPtr<ISectionDragOwnerExtension> DragOwner = SectionOwner.ImplicitCast();
				if (!DragOwner)
				{
					DragOwner = SectionOwner.AsModel()->FindAncestorOfType<ISectionDragOwnerExtension>();
				}

				if (DragOwner && !ProcessedDragOwners.Contains(DragOwner.AsModel()))
				{
					// Build context for the end of drag (preview-based tracks need this to apply changes)
					FSectionVerticalDragContext EndDragContext;
					EndDragContext.Section = Section;
					EndDragContext.DraggedSections = &Sections;
					EndDragContext.InitialSectionRowIndices = &InitialSectionRowIndicies;

					// Notify the drag owner that drag ended
					DragOwner->OnEndSectionVerticalDrag(EndDragContext);
					ProcessedDragOwners.Add(DragOwner.AsModel());
				}
			}
		}
	}

	EndTransaction();
}

int32 FMoveKeysAndSections::OnPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	using namespace UE::Sequencer;

	// Let drag owners paint their drop indicators
	if (bAllowVerticalMovement && Sections.Num() > 0)
	{
		// Get the VirtualTrackArea for coordinate conversion
		TSharedRef<SSequencer> SequencerWidget = Sequencer.GetUnderlyingSequencerWidget();
		FVirtualTrackArea VirtualTrackArea = SequencerWidget->GetVirtualTrackArea();

		FSectionModelStorageExtension* SectionStorage = Sequencer.GetViewModel()->GetRootModel()->CastDynamic<FSectionModelStorageExtension>();
		if (SectionStorage)
		{
			TSet<TSharedPtr<FViewModel>> ProcessedDragOwners;

			// Find unique drag owners
			for (UMovieSceneSection* Section : Sections)
			{
				TSharedPtr<FSectionModel> SectionModel = SectionStorage->FindModelForSection(Section);
				TViewModelPtr<ISectionOwnerExtension> SectionOwner = SectionModel ? SectionModel->GetParentSectionOwnerModel() : TViewModelPtr<ISectionOwnerExtension>();
				if (!SectionOwner)
				{
					continue;
				}

				// Find the drag owner (track model that implements ISectionDragOwnerExtension)
				TViewModelPtr<ISectionDragOwnerExtension> DragOwner = SectionOwner.ImplicitCast();
				if (!DragOwner)
				{
					DragOwner = SectionOwner.AsModel()->FindAncestorOfType<ISectionDragOwnerExtension>();
				}

				if (DragOwner && !ProcessedDragOwners.Contains(DragOwner.AsModel()))
				{
					// Let the drag owner paint its preview indicators
					LayerId = DragOwner->OnPaintSectionDragPreview(
						AllottedGeometry,
						VirtualTrackArea,
						MyCullingRect,
						OutDrawElements,
						LayerId,
						FWidgetStyle(),
						true
					);

					ProcessedDragOwners.Add(DragOwner.AsModel());
				}
			}
		}
	}

	return LayerId;
}

void FMoveKeysAndSections::ModifyNonSelectedSections()
{
	using namespace UE::Sequencer;

	for (const FSequencerSelectedKey& Key : Keys)
	{
		TSharedPtr<FChannelModel> Channel = Key.WeakChannel.Pin();
		UObject* OwningObject = Channel ? Channel->GetOwningObject() : nullptr;

		if (!OwningObject || ModifiedNonSelectedObjects.Contains(OwningObject))
		{
			continue;
		}

		if (UMovieSceneSection* OwningSection = Cast<UMovieSceneSection>(OwningObject))
		{
			if (Sections.Contains(OwningSection) || OwningSection->IsReadOnly())
			{
				continue;
			}
		}

		OwningObject->SetFlags(RF_Transactional);
		OwningObject->Modify();
		ModifiedNonSelectedObjects.Add(OwningObject);
	}
}

TOptional<FFrameNumber> FMoveKeysAndSections::GetMovementDeltaX(const FPointerEvent& MouseEvent, FFrameTime MouseTime)
{
	TOptional<FFrameNumber> DeltaX;

	// The delta of the mouse is the difference in the current mouse time vs when we started dragging
	const FFrameNumber MouseDeltaTime = (MouseTime - MouseTimePrev).FloorToFrame();

	// Disallow movement if any of the sections can't move
	for (UMovieSceneSection* Section : Sections)
	{
		// If we're moving a section that is blending with something then it's OK if it overlaps stuff, the blend amount will get updated at the end.
		if (!Section)
		{
			continue;
		}

		TOptional<FFrameNumber> LeftMovementMaximum;
		TOptional<FFrameNumber> RightMovementMaximum;

		// We'll calculate this section's borders and clamp the possible delta time to be less than that
		
		if (!Section->GetBlendType().IsValid())
		{
			TRange<FFrameNumber> SectionBoundaries = GetSectionBoundaries(Section);
			LeftMovementMaximum = UE::MovieScene::DiscreteInclusiveLower(SectionBoundaries);
			RightMovementMaximum = UE::MovieScene::DiscreteExclusiveUpper(SectionBoundaries);
		}
		
		if (LeftMovementMaximum.IsSet())
		{
			if (Section->HasStartFrame())
			{
				FFrameNumber NewStartTime = Section->GetInclusiveStartFrame() + MouseDeltaTime;
				if (NewStartTime < LeftMovementMaximum.GetValue())
				{
					FFrameNumber ClampedDeltaTime = LeftMovementMaximum.GetValue() - Section->GetInclusiveStartFrame();
					if (!DeltaX.IsSet() || DeltaX.GetValue() > ClampedDeltaTime)
					{
						DeltaX = ClampedDeltaTime;
					}
				}
			}
		}

		if (RightMovementMaximum.IsSet())
		{
			if (Section->HasEndFrame())
			{
				FFrameNumber NewEndTime = Section->GetExclusiveEndFrame() + MouseDeltaTime;
				if (NewEndTime > RightMovementMaximum.GetValue())
				{
					FFrameNumber ClampedDeltaTime = RightMovementMaximum.GetValue() - Section->GetExclusiveEndFrame();
					if (!DeltaX.IsSet() || DeltaX.GetValue() > ClampedDeltaTime)
					{
						DeltaX = ClampedDeltaTime;
					}
				}
			}
		}
	}

	return DeltaX;
}

bool FMoveKeysAndSections::HandleSectionMovement(FFrameTime MouseTime, FVector2D VirtualMousePos, FVector2D LocalMousePos, TOptional<FFrameNumber> MaxDeltaX, FFrameNumber DesiredDeltaX)
{
	using namespace UE::Sequencer;

	// Don't try to process moving sections if we don't have any sections.
	if (Sections.Num() == 0)
	{
		return false;
	}

	// If sections are all on different rows or from different tracks, don't set row indices for anything because it leads to odd behavior.
	bool bSectionsAreOnDifferentRows = false;
	TOptional<int32> LowestRowIndex;
	TOptional<int32> HighestRowIndex;
	UMovieSceneTrack* FirstTrack = nullptr;

	for (UMovieSceneSection* Section : Sections)
	{
		UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>();
		if (!LowestRowIndex.IsSet() || LowestRowIndex.GetValue() < Section->GetRowIndex())
		{
			LowestRowIndex = Section->GetRowIndex();
		}
		if (!HighestRowIndex.IsSet() || HighestRowIndex.GetValue() > Section->GetRowIndex())
		{
			HighestRowIndex = Section->GetRowIndex();
		}
		if (FirstTrack)
		{
			if (FirstTrack != Track)
			{
				bSectionsAreOnDifferentRows = true;
			}
		}
		else
		{
			FirstTrack = Track;
		}
	}

	if (LowestRowIndex.IsSet() && HighestRowIndex.IsSet() && LowestRowIndex.GetValue() != HighestRowIndex.GetValue())
	{
		bSectionsAreOnDifferentRows = true;
	}

	TArray<TSharedPtr<FViewModel>> Tracks;

	// @todo_sequencer_mvvm: really this code should all be operating on models rather than looking directly at UMovieSceneSections and tracks
	FSectionModelStorageExtension* SectionStorage = Sequencer.GetViewModel()->GetRootModel()->CastDynamic<FSectionModelStorageExtension>();

	bool bRowIndexChanged = false;
	for (UMovieSceneSection* Section : Sections)
	{
		TSharedPtr<FSectionModel>      SectionModel  = SectionStorage->FindModelForSection(Section);
		TViewModelPtr<ITrackExtension> TrackExtModel = SectionModel ? SectionModel->GetParentTrackModel() : nullptr;
		if (!SectionModel || !TrackExtModel)
		{
			continue;
		}

		UMovieSceneTrack* Track = TrackExtModel->GetTrack();

		const TArray<UMovieSceneSection*>& AllSections = Track->GetAllSections();

		TArray<UMovieSceneSection*> NonDraggedSections;
		for (UMovieSceneSection* TrackSection : AllSections)
		{
			if (!Sections.Contains(TrackSection))
			{
				NonDraggedSections.Add(TrackSection);
			}
		}

		Tracks.AddUnique(TrackExtModel.AsModel());

		// Get the drag owner (track model that implements ISectionDragOwnerExtension)
		TViewModelPtr<ISectionDragOwnerExtension> DragOwner = TrackExtModel.ImplicitCast();
		if (!DragOwner)
		{
			DragOwner = TrackExtModel.AsModel()->FindAncestorOfType<ISectionDragOwnerExtension>();
		}

		// @todo_sequencer_mvvm: need to go through this dragging code and figure out what's going on and what it means
		//                       now that models and views have separated concerns

		// Horizontal movement
		bool bDeltaX = DesiredDeltaX != 0;
		if (bDeltaX)
		{
			Section->MoveSection(MaxDeltaX.Get(DesiredDeltaX));
		}

		// Vertical movement - delegate to drag owner
		if (DragOwner.IsValid())
		{
			// Build context for vertical drag
			FSectionVerticalDragContext DragContext;
			DragContext.Section = Section;
			DragContext.VirtualMousePos = VirtualMousePos;
			DragContext.LocalMousePos = LocalMousePos;
			DragContext.DraggedSections = &Sections;
			DragContext.InitialSectionRowIndices = &InitialSectionRowIndicies;
			DragContext.bSectionsAreOnDifferentRows = bSectionsAreOnDifferentRows;
			DragContext.LowestDraggedRow = LowestRowIndex;
			DragContext.HighestDraggedRow = HighestRowIndex;
			DragContext.PrevMousePosY = PrevMousePosY;

			bool bSectionRowChanged = DragOwner->OnSectionVerticalDrag(DragContext);
			bRowIndexChanged |= bSectionRowChanged;
		}
	}

	if (bRowIndexChanged)
	{
		PrevMousePosY = LocalMousePos.Y;

		// @todo_sequence_mvvm: Fix this code as well - is it necessary if we no longer destructively re-populate the tree when data changes?
		// Expand track node if it wasn't already expanded. This ensures that multi row tracks will show multiple rows if regenerated
		//for (TSharedPtr<FViewModel> TrackModel : Tracks)
		//{
		//	IOutlinerExtension* OutlinerExtension = TrackModel->CastThis<IOutlinerExtension>(TrackModel);
		//	bool bIsTrackExpanded = (!OutlinerExtension || OutlinerExtension->IsExpanded());
		//	if (!bIsTrackExpanded && TrackModel->GetSubTrackMode() == FSequencerTrackNode::ESubTrackMode::None)
		//	{
		//		TArray<TSharedRef<ISequencerSection> > TrackNodeSections = TrackModel->GetSections();
		//		if (TrackNodeSections.Num() && TrackNodeSections[0]->GetSectionObject())
		//		{
		//			int32 SectionFirstRowIndex = TrackNodeSections[0]->GetSectionObject()->GetRowIndex();

		//			for (TSharedRef<ISequencerSection> TrackNodeSection : TrackNodeSections)
		//			{
		//				if (TrackNodeSection->GetSectionObject() && SectionFirstRowIndex != TrackNodeSection->GetSectionObject()->GetRowIndex())
		//				{
		//					TrackNode->SetExpansion(true);
		//					break;
		//				}
		//			}
		//		}
		//	}
		//}
	}

	return bRowIndexChanged;
}

void FMoveKeysAndSections::HandleKeyMovement(const FPointerEvent& MouseEvent, TOptional<FFrameNumber> MaxDeltaX, FFrameNumber DesiredDeltaX)
{
	if (KeysAsArray.Num() == 0)
	{
		return;
	}

	// Apply the delta to our key times. We need to get our key time so that we can add the delta
	// to each one so that we come up with a new absolute time for it.
	TArray<FFrameNumber> CurrentKeyTimes;
	CurrentKeyTimes.SetNum(KeysAsArray.Num());
	GetKeyTimes(KeysAsArray, CurrentKeyTimes);

	for (int32 Index = 0; Index < CurrentKeyTimes.Num(); ++Index)
	{
		FSequencerSelectedKey& SelectedKey = KeysAsArray[Index];
		const bool bOwningSectionIsSelected = Sections.Contains(SelectedKey.Section);

		// We don't want to apply delta if we have the key's section selected as well, otherwise they get double
		// transformed (moving the section moves the keys + we add the delta to the key positions).
		if (!bOwningSectionIsSelected)
		{
			const FFrameNumber NewKeyTime = CurrentKeyTimes[Index] + MaxDeltaX.Get(DesiredDeltaX);

			const UMovieSceneTrack* Track = SelectedKey.Section->GetTypedOuter<UMovieSceneTrack>();
			check(Track);

			if (Track->bAutoExpandToKeyframes) // Auto expand allows keyframes to be dragged outside sections and sections will resize to keyframes
			{
				CurrentKeyTimes[Index] = NewKeyTime;
			}
			// Without auto expand, ensure that keyframe is within section bounds. We include exclusive end frame to allow keyframes to visually align with section rendering
			else if (NewKeyTime <= SelectedKey.Section->GetExclusiveEndFrame() && NewKeyTime >= SelectedKey.Section->GetInclusiveStartFrame())
			{
				CurrentKeyTimes[Index] = NewKeyTime;
			}
		}
	}

	// Now set the times back to the keys.
	SetKeyTimes(KeysAsArray, CurrentKeyTimes);

	// Expand any sections containing those keys to encompass their new location
	for (int32 Index = 0; Index < CurrentKeyTimes.Num(); ++Index)
	{
		FSequencerSelectedKey SelectedKey = KeysAsArray[Index];

		UMovieSceneSection* Section = SelectedKey.Section;
		if (ModifiedNonSelectedObjects.Contains(Section))
		{
			// If the key moves outside of the section resize the section to fit the key
			// @todo Sequencer - Doesn't account for hitting other sections 
			const FFrameNumber   NewKeyTime = CurrentKeyTimes[Index];
			TRange<FFrameNumber> SectionRange = Section->GetRange();

			const UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>();
			check(Track);

			if (!SectionRange.Contains(NewKeyTime) && Track->bAutoExpandToKeyframes) // If we can auto expand and we are out of bounds, expand to new range
			{
				TRange<FFrameNumber> NewRange = TRange<FFrameNumber>::Hull(SectionRange, TRange<FFrameNumber>(NewKeyTime));
				Section->SetRange(NewRange);
			}
		}
	}


	// Snap the play time to the new dragged key time if all the keyframes were dragged to the same time
	const bool bSnapPlayTimeToDraggedKey = 
		((Settings->GetIsSnapEnabled() && Settings->GetSnapPlayTimeToDraggedKey()) || (MouseEvent.IsShiftDown() && MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))) &&
		CurrentKeyTimes.Num();
	if (bSnapPlayTimeToDraggedKey)
	{
		FFrameNumber FirstFrame = CurrentKeyTimes[0];
		auto         EqualsFirstFrame = [=](FFrameNumber In)
		{
			return In == FirstFrame;
		};

		if (Algo::AllOf(CurrentKeyTimes, EqualsFirstFrame))
		{
			Sequencer.SetLocalTime(FirstFrame);
		}
	}

	// Explicitly mark everything as changed to ensure that the UI is responsive during a drag
	for (UObject* OwningObject : ModifiedNonSelectedObjects)
	{
		if (UMovieSceneSignedObject* SignedObject = Cast<UMovieSceneSignedObject>(OwningObject))
		{
			SignedObject->MarkAsChanged();
			SignedObject->BroadcastChanged();
		}
	}
}

void FMoveKeysAndSections::HandleMarkedFrameMovement(TOptional<FFrameNumber> MaxDeltaX, FFrameNumber DesiredDeltaX)
{
	if (MarkedFrames.Num() == 0)
	{
		return;
	}

	const FFrameNumber EffectiveDelta = MaxDeltaX.Get(DesiredDeltaX);
	UMovieScene* FocusedMovieScene = Sequencer.GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!FocusedMovieScene || FocusedMovieScene->AreMarkedFramesLocked())
	{
		return;
	}

	FocusedMovieScene->Modify();

	const TArray<FMovieSceneMarkedFrame>& AllMarkedFrames = FocusedMovieScene->GetMarkedFrames();

	for (TSet<int32>::TConstIterator It = MarkedFrames.CreateConstIterator(); It; ++It)
	{
		const int32 MarkIndex = *It;
		const FMovieSceneMarkedFrame& MarkedFrame = AllMarkedFrames[MarkIndex];
		const FFrameNumber NewMarkTime = MarkedFrame.FrameNumber + EffectiveDelta;
		FocusedMovieScene->SetMarkedFrame(MarkIndex, NewMarkTime);
	}

	FocusedMovieScene->MarkAsChanged();
	FocusedMovieScene->BroadcastChanged();
}

