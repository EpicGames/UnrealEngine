// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/CursorReply.h"
#include "Curves/KeyHandle.h"
#include "SequencerSelectedKey.h"
#include "ISequencerEditTool.h"
#include "SequencerHotspots.h"
#include "ScopedTransaction.h"
#include "Tools/SequencerSnapField.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "MVVM/Extensions/IDraggableTrackAreaExtension.h"
#include "MVVM/Extensions/ISectionDragOwnerExtension.h"


class FSequencer;
class FSlateWindowElementList;
class USequencerSettings;

namespace UE
{
namespace Sequencer
{

class IDraggableTrackAreaExtension;
class FSectionModel;
class FTrackModel;
class FVirtualTrackArea;

} // namespace Sequencer
} // namespace UE



enum class ESequencerMoveOperationType
{
	MoveKeys     = 1<<0,
	MoveSections = 1<<1,
};
ENUM_CLASS_FLAGS(ESequencerMoveOperationType)

/**
 * Abstract base class for drag operations that handle an operation for an edit tool.
 */
class FEditToolDragOperation
	: public UE::Sequencer::ISequencerEditToolDragOperation
{
public:

	/** Create and initialize a new instance. */
	FEditToolDragOperation( FSequencer& InSequencer );

public:

	// ISequencerEditToolDragOperation interface

	virtual FCursorReply GetCursor() const override;
	virtual int32 OnPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const override;

protected:

	/** begin a new scoped transaction for this drag */
	void BeginTransaction( TSet<UMovieSceneSection*>& Sections, const FText& TransactionDesc );

	/** End an existing scoped transaction if one exists */
	void EndTransaction();

	virtual void GetSections(TArray<UMovieSceneSection*>& OutSections) {}

	/** Get the bounds within which the specified section can be resized based on its surrounding sections */
	TRange<FFrameNumber> GetSectionBoundaries(const UMovieSceneSection* Section);

protected:

	/** Scoped transaction for this drag operation */
	TUniquePtr<FScopedTransaction> Transaction;

	/** The current sequencer settings, cached on construction */
	const USequencerSettings* Settings;

	/** Reference to the sequencer itself */
	FSequencer& Sequencer;
};


/**
 * An operation to resize a section by dragging its left or right edge
 */
class FResizeSection
	: public FEditToolDragOperation
{
public:

	/** Create and initialize a new instance. */
	FResizeSection( FSequencer& InSequencer, bool bInDraggingByEnd, bool bIsSlipping );

public:

	// FEditToolDragOperation interface

	virtual void OnBeginDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override;
	virtual void OnDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override;
	virtual void OnEndDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override;
	virtual FCursorReply GetCursor() const override { return FCursorReply::Cursor( EMouseCursor::ResizeLeftRight ); }

private:

	/** The sections we are interacting with */
	TSet<UMovieSceneSection*> Sections;

	/********************************************************/
	struct FPreDragChannelData
	{
		/** Weak handle to the base channel ptr */
		FMovieSceneChannelHandle Channel;

		/** Array of all the handles in the section at the start of the drag */
		TArray<FKeyHandle> Handles;
		/** Array of all the above handle's times, one per index of Handles */
		TArray<FFrameNumber> FrameNumbers;
	};

	struct FPreDragSectionData
	{
		/** Pointer to the ISequencerSection in the drag*/
		ISequencerSection* SequencerSection;

		/** Pointer to the movie section, this section is only valid during a drag operation*/
		UMovieSceneSection * MovieSection;
		/** The initial range of the section before it was resized */
		TRange<FFrameNumber> InitialRange;
		/** Array of all the channels in the section before it was resized */
		TArray<FPreDragChannelData> Channels;
	};
	TArray<FPreDragSectionData> PreDragSectionData;

	/** true if dragging  the end of the section, false if dragging the start */
	bool bDraggingByEnd;

	/** true if slipping, adjust only the start offset */
	bool bIsSlipping;

	/** Time where the mouse is pressed */
	FFrameTime MouseDownTime;

	/** The section start or end times when the mouse is pressed */
	TMap<TWeakObjectPtr<UMovieSceneSection>, FFrameNumber> SectionInitTimes;

	/** Optional snap field to use when dragging */
	TOptional<FSequencerSnapField> SnapField;

protected:
	void GetSections(TArray<UMovieSceneSection*>& OutSections) override { OutSections = Sections.Array(); }
};

/**
 * This drag operation handles moving both keys and sections depending on what you have selected.
 */
class FMoveKeysAndSections
	: public FEditToolDragOperation
	, public UE::Sequencer::IDragOperation
{
public:

	FMoveKeysAndSections(FSequencer& InSequencer, ESequencerMoveOperationType InMoveType);

	// FEditToolDragOperation interface
	virtual void OnBeginDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override;
	virtual void OnDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override;
	virtual void OnEndDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override;
	virtual FCursorReply GetCursor() const override { return FCursorReply::Cursor(EMouseCursor::CardinalCross); }
	virtual int32 OnPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const override;
	// ~FEditToolDragOperation interface

	/* UE::Sequencer:IDragOperation Interface */
	virtual void AddSnapTime(FFrameNumber SnapTime) override;
	virtual void AddModel(TSharedPtr<UE::Sequencer::FViewModel> Model) override;

	/* ISequencerEditToolDragOperation */
	virtual TSharedPtr<UE::Sequencer::FViewModel> GetDragOwnerTrackModel() const override;

protected:
	/** Calculate the possible horizontal movement we can, constrained by sections running into things. */
	TOptional<FFrameNumber> GetMovementDeltaX(const FPointerEvent& MouseEvent, FFrameTime MouseTime);

	/** Move selected sections, if any. */
	bool HandleSectionMovement(FFrameTime MouseTime, FVector2D VirtualMousePos, FVector2D LocalMousePos, TOptional<FFrameNumber> MaxDeltaX, FFrameNumber DesiredDeltaX);
	/** Move selected keys, if any. */
	void HandleKeyMovement(const FPointerEvent& MouseEvent, TOptional<FFrameNumber> MaxDeltaX, FFrameNumber DesiredDeltaX);
	/** Move selected marked frames, if any. */
	void HandleMarkedFrameMovement(TOptional<FFrameNumber> MaxDeltaX, FFrameNumber DesiredDeltaX);

	void OnSequencerNodeTreeUpdated();

	/** Calls Modify on sections that own keys we're moving, as the need to be notified the data is about to change too. */
	void ModifyNonSelectedSections();

protected:
	/** Array of models that we're moving. */
	TSet<TWeakPtr<UE::Sequencer::IDraggableTrackAreaExtension>> DraggedItems;

	/** Array of sections that we're moving. */
	TSet<UMovieSceneSection*> Sections;

	/** Set of keys that are being moved. */
	TSet<FSequencerSelectedKey> Keys;
	TArray<FSequencerSelectedKey> KeysAsArray;

	/** Set of marked frames that are being moved. */
	TSet<int32> MarkedFrames;

	/** What was the time of the mouse for the previous frame? Used to calculate a per-frame delta. */
	FFrameTime MouseTimePrev;

	/** The position of the mouse when the last section move occurred */
	TOptional<float> PrevMousePosY;

	/** Array of relative offsets for each selected item. Keys + Sections are both added to this array. */
	TArray<FFrameNumber> RelativeSnapOffsets;

	/** Store the row each section starts on when we start dragging. */
	TArray<UE::Sequencer::FInitialRowIndex> InitialSectionRowIndicies;

	/** Array of sections and owners that we called Modify on because we're editing keys that belong to these sections, but not actually moving these sections. */
	TSet<UObject*> ModifiedNonSelectedObjects;

	/** Optional snap field to use when dragging */
	TOptional<FSequencerSnapField> SnapField;

	/** If the user is moving them via clicking on the Section then we'll allow vertical re-arranging, otherwise not. */
	bool bAllowVerticalMovement;

	/** Cached drag owner track model (computed in AddModel) */
	TWeakPtr<UE::Sequencer::FViewModel> CachedDragOwnerTrackModel;

protected:
	void GetSections(TArray<UMovieSceneSection*>& OutSections) override { OutSections = Sections.Array(); }
};

/**
 * Operation to drag-duplicate the currently selected keys and sections.
 */
class FDuplicateKeysAndSections : public FMoveKeysAndSections
{
public:

	FDuplicateKeysAndSections( FSequencer& InSequencer, ESequencerMoveOperationType Type)
		: FMoveKeysAndSections(InSequencer, Type)
	{}

public:

	virtual void OnBeginDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override;
	virtual void OnEndDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override;
};

/**
 * An operation to change a section's ease in/out by dragging its left or right handle
 */
class FManipulateSectionEasing
	: public FEditToolDragOperation
{
public:

	/** Create and initialize a new instance. */
	FManipulateSectionEasing( FSequencer& InSequencer, TWeakObjectPtr<UMovieSceneSection> InSection, bool bEaseIn );

public:

	// FEditToolDragOperation interface

	virtual void OnBeginDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override;
	virtual void OnDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override;
	virtual void OnEndDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override;
	virtual FCursorReply GetCursor() const override { return FCursorReply::Cursor( EMouseCursor::ResizeLeftRight ); }

private:

	/** The sections we are interacting with */
	TWeakObjectPtr<UMovieSceneSection> WeakSection;

	/** true if editing the section's ease in, false for ease out */
	bool bEaseIn;

	/** Time where the mouse is pressed */
	FFrameTime MouseDownTime;

	/** The section ease in/out when the mouse was pressed */
	TOptional<int32> InitValue;

	/** Optional snap field to use when dragging */
	TOptional<FSequencerSnapField> SnapField;
};
