// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/MotionTrailMenuHelpers.h"

#include "Tools/MotionTrailOptions.h"
#include "Tools/TrailCategory.h"
#include "ToolMenus.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "MovieSceneCommonHelpers.h"
#include "Widgets/Input/SComboBox.h"
#include "LevelEditor.h"
#include "SSocketChooser.h"
#include "ActorPickerMode.h"
#include "Components/SkeletalMeshComponent.h"
#include "ILevelEditor.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "MotionTrailMenuHelpers"

namespace UE::Sequencer::MotionTrailMenu
{

namespace Private
{

void OffsetActionExecuteAction(const FToolMenuContext& InContext, UMotionTrailToolOptions* Settings, int32 Index)
{
	if (UMotionTrailToolOptions::FPinnedTrail* Trail = Settings->GetPinnedTrail(Index))
	{
		Settings->SetHasOffset(Index, !Trail->bHasOffset);
	}
}

ECheckBoxState OffsetActionGetActionCheckState(const FToolMenuContext& Context, UMotionTrailToolOptions* Settings, int32 Index)
{
	if (UMotionTrailToolOptions::FPinnedTrail* Trail = Settings->GetPinnedTrail(Index))
	{
		return (Trail->bHasOffset) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

void SpaceActionExecuteAction(const FToolMenuContext& InContext, UMotionTrailToolOptions* Settings, int32 Index)
{
	if (UMotionTrailToolOptions::FPinnedTrail* Trail = Settings->GetPinnedTrail(Index))
	{
		if (Trail->SpaceName.IsSet() == false)
		{
			// FIXME temp approach for selecting the parent
			FSlateApplication::Get().DismissAllMenus();

			static const FActorPickerModeModule& ActorPickerMode = FModuleManager::Get().GetModuleChecked<FActorPickerModeModule>("ActorPickerMode");

			ActorPickerMode.BeginActorPickingMode(
				FOnGetAllowedClasses(),
				FOnShouldFilterActor::CreateLambda([](const AActor* InActor)
					{
						return true; //todo make sure in sequencer
					}),
				FOnActorSelected::CreateLambda([Settings, Index](AActor* InActor)
					{
						FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
						TSharedPtr< ILevelEditor > LevelEditor = LevelEditorModule.GetFirstLevelEditor();

						if (USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(InActor->GetComponentByClass(USkeletalMeshComponent::StaticClass())))
						{
							TSharedPtr<SWidget> MenuWidget =
								SNew(SSocketChooserPopup)
								.SceneComponent(Component)
								.OnSocketChosen_Lambda([Settings, Index, InActor](FName InSocketName) mutable
									{
										Settings->PutPinnnedInSpace(Index, InActor, InSocketName);
									}
								);
							FSlateApplication::Get().PushMenu(
								LevelEditor.ToSharedRef(),
								FWidgetPath(),
								MenuWidget.ToSharedRef(),
								FSlateApplication::Get().GetCursorPos(),
								FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
							);
						}
						else
						{
							// No skeletal mesh component, just use the actor directly with no socket
							Settings->PutPinnnedInSpace(Index, InActor, NAME_None);
						}
					})
			);
		}
		else
		{
			Settings->PutPinnnedInSpace(Index, nullptr, NAME_None);
		}
	}
}

ECheckBoxState SpaceActionGetActionCheckState(const FToolMenuContext& Context, UMotionTrailToolOptions* Settings, int32 Index)
{
	if (UMotionTrailToolOptions::FPinnedTrail* Trail = Settings->GetPinnedTrail(Index))
	{
		return (Trail->SpaceName.IsSet()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

void CreatePinnedMenuDelegate(UToolMenu* SubMenu, UMotionTrailToolOptions* Settings, int32 Index)
{
	FToolMenuSection& Section = SubMenu->AddSection(NAME_None);

	FToolUIAction OffsetAction;
	OffsetAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&OffsetActionExecuteAction, Settings, Index);
	OffsetAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateStatic(&OffsetActionGetActionCheckState, Settings, Index);

	FToolMenuEntry OffsetEntry = FToolMenuEntry::InitMenuEntry(
		"Offset",
		LOCTEXT("OffsetLabel", "Offset"),
		LOCTEXT("OffsetLabelTooltip", "Toggle offset on selects the curve in the viewport, and allows you to move it like shift select does. Toggling it off will remove any offset."),
		FSlateIcon(),
		OffsetAction,
		EUserInterfaceActionType::Check
	);
	Section.AddEntry(OffsetEntry);

	FToolUIAction SpaceAction;
	SpaceAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&SpaceActionExecuteAction, Settings, Index);
	SpaceAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateStatic(&SpaceActionGetActionCheckState, Settings, Index);

	FToolMenuEntry SpaceEntry = FToolMenuEntry::InitMenuEntry(
		"Space",
		LOCTEXT("SpaceLabel", "Space"),
		LOCTEXT("SpaceLabelTooltip", "Toggling on space will put you into eye drop selection mode to pick the scene compponent/socket that you want to have this trail in. Toggling it off puts it back in world space."),
		FSlateIcon(),
		SpaceAction,
		EUserInterfaceActionType::Check
	);
	Section.AddEntry(SpaceEntry);
}

void CreatePinnedItems(UMotionTrailToolOptions* Settings, FToolMenuSection& PinnedTrails, ETrailCategory Category)
{
	const int32  NumPinned = Settings->GetNumPinned();
	if (NumPinned > 0)
	{
		for (int32 Index = 0; Index < NumPinned; ++Index)
		{
			if (UMotionTrailToolOptions::FPinnedTrail* Trail = Settings->GetPinnedTrail(Index))
			{
				if (!EnumHasAnyFlags(Trail->Category, Category))
				{
					continue;
				}
				FNewToolMenuDelegate MakeMenuDelegate = FNewToolMenuDelegate::CreateStatic(&CreatePinnedMenuDelegate, Settings, Index);
				FUIAction TogglePinnedAction(
					FExecuteAction::CreateLambda([Settings, Index]()
						{
							if (UMotionTrailToolOptions::FPinnedTrail* Trail = Settings->GetPinnedTrail(Index))
							{
								Settings->DeletePinned(Index);
							}
						}
					),
					FCanExecuteAction()
				);

				FText Label = Trail->TrailName;
				FName Name = FName(*Label.ToString());
				FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
					Name,
					Label,
					// TODO: Update this and other labels/tooltips in this file.
					LOCTEXT("PinnenTrailtip", "Modify Pinned States"),
					MakeMenuDelegate,
					TogglePinnedAction,
					EUserInterfaceActionType::Button,
					false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Pinned")
				);
				PinnedTrails.AddEntry(Entry);
			}
		}
	}
}

TSharedRef<SWidget> CreateFramesBeforeWidget(UMotionTrailToolOptions* Settings)
{
	// clang-format off
	return
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
						.Padding(FMargin(1.0f))
						[
							SNew(SSpinBox<int32>)
								.IsEnabled_Lambda([Settings]() { return (Settings->MotionTrailRange == EMotionTrailRange::SpecifiedRange); })
								.MinValue(0)
								.MaxValue(100)
								.MinDesiredWidth(50)
								.ToolTipText_Lambda(
									[Settings]() -> FText
									{
										return FText::AsNumber(Settings->FramesBefore);
									}
								)
								.Value_Lambda(
									[Settings]() -> int32
									{
										return Settings->FramesBefore;
									}
								)
								.OnValueChanged_Lambda(
									[Settings](int32 InValue)
									{
										Settings->FramesBefore = InValue;
									}
								)
						]
				]
		];
	// clang-format on
}

FToolMenuEntry CreateFramesBefore(UMotionTrailToolOptions* Settings)
{
	return FToolMenuEntry::InitWidget("FramesBefore", CreateFramesBeforeWidget(Settings), LOCTEXT("FramesBefore", "Frames Before"));
}

TSharedRef<SWidget> CreateFramesAfterWidget(UMotionTrailToolOptions* Settings)
{
	// clang-format off
	return
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
						.Padding(FMargin(1.0f))
						[
							SNew(SSpinBox<int32>)
								.IsEnabled_Lambda([Settings]() { return (Settings->MotionTrailRange == EMotionTrailRange::SpecifiedRange); })
								.MinValue(0)
								.MaxValue(100)
								.MinDesiredWidth(50)
								.ToolTipText_Lambda(
									[Settings]() -> FText
									{
										return FText::AsNumber(Settings->FramesAfter);
									}
								)
								.Value_Lambda(
									[Settings]() -> int32
									{
										return Settings->FramesAfter;
									}
								)
								.OnValueChanged_Lambda(
									[Settings](int32 InValue)
									{
										Settings->FramesAfter = InValue;
									}
								)
						]
				]
		];
	// clang-format on
}

FToolMenuEntry CreateFramesAfter(UMotionTrailToolOptions* Settings)
{
	return FToolMenuEntry::InitWidget("FramesAfter", CreateFramesAfterWidget(Settings), LOCTEXT("FramesAfter", "Frames After"));
}

TSharedRef<SWidget> CreateTrailStyleWidget(UMotionTrailToolOptions* Settings)
{
	// clang-format off
	return
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
						.Padding(FMargin(1.0f))
						[
							SNew(SComboButton)
								.OnGetMenuContent_Lambda([Settings]()
									{
										FMenuBuilder MenuBuilder(true, NULL); //maybe todo look at settting these up with commands
										MenuBuilder.BeginSection("TrailStyles");

										TArray <TPair<FText, FText>>& TrailStyles = Settings->GetTrailStyles();
										for (int32 Index = 0; Index < TrailStyles.Num(); ++Index)
										{
											FUIAction ItemAction(FExecuteAction::CreateUObject(Settings, &UMotionTrailToolOptions::SetTrailStyle, Index));
											MenuBuilder.AddMenuEntry(TrailStyles[Index].Key, TAttribute<FText>(), FSlateIcon(), ItemAction);
										}

										MenuBuilder.EndSection();
										return MenuBuilder.MakeWidget();
									})
								.ButtonContent()
								[
									SNew(SHorizontalBox)
										+ SHorizontalBox::Slot()
										[
											SNew(STextBlock)
												.Text_Lambda([Settings]()
													{
														const int32 Index = Settings->GetTrailStyleIndex();
														const TArray <TPair<FText, FText>>& TrailStyles = Settings->GetTrailStyles();
														return TrailStyles[Index].Key;
													})
												.ToolTipText_Lambda([Settings]()
													{
														const int32 Index = Settings->GetTrailStyleIndex();
														const TArray <TPair<FText, FText>>& TrailStyles = Settings->GetTrailStyles();
														return TrailStyles[Index].Value;
													})
										]
								]
						]
				]
		];
	// clang-format on
}

FToolMenuEntry CreateTrailStyle(UMotionTrailToolOptions* Settings)
{
	return FToolMenuEntry::InitWidget("TrailStyle", CreateTrailStyleWidget(Settings), LOCTEXT("TrailStyle", "Trail Style"));
}

TSharedRef<SWidget> CreateMaxNumberPinnedWidget(UMotionTrailToolOptions* Settings)
{
	// clang-format off
	return
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
						.Padding(FMargin(1.0f))
						[
							SNew(SSpinBox<int32>)
								.MinValue(5)
								.MaxValue(100)
								.MinDesiredWidth(50)
								.ToolTipText_Lambda(
									[Settings]() -> FText
									{
										return FText::AsNumber(Settings->MaxNumberPinned);
									}
								)
								.Value_Lambda(
									[Settings]() -> int32
									{
										return Settings->MaxNumberPinned;
									}
								)
								.OnValueChanged_Lambda(
									[Settings](int32 InValue)
									{
										Settings->MaxNumberPinned = InValue;
									}
								)
						]
				]
		];
	// clang-format on
}

FToolMenuEntry CreateMaxNumberPinned(UMotionTrailToolOptions* Settings)
{
	return FToolMenuEntry::InitWidget("MaxNumberPinned", CreateMaxNumberPinnedWidget(Settings), LOCTEXT("MaxNumberPinned", "Max Number Pinned"));
}

TSharedRef<SWidget> CreateTrailColorWidget(UMotionTrailToolOptions* Settings, FName PropertyName)
{
	// clang-format off
	return
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
						.Padding(FMargin(1.0f))
						[
							SNew(SColorBlock).IsEnabled(true)
								//.Size(FVector2D(6.0, 38.0))
								.Color_Lambda([Settings, PropertyName]()
									{
										FTrackInstancePropertyBindings Binding(PropertyName, PropertyName.ToString());
										FLinearColor Color = Binding.GetCurrentValue<FLinearColor>(*Settings);
										return Color;
									}
								)
								.OnMouseButtonDown_Lambda([Settings, PropertyName](const FGeometry&, const FPointerEvent&)
									{
										FColorPickerArgs PickerArgs;
										PickerArgs.bUseAlpha = false;
										FTrackInstancePropertyBindings Binding(PropertyName, PropertyName.ToString());
										FLinearColor Color = Binding.GetCurrentValue<FLinearColor>(*Settings);
										PickerArgs.InitialColor = Color;
										PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateLambda([Settings, PropertyName](FLinearColor Color)
											{
												FTrackInstancePropertyBindings Binding(PropertyName, PropertyName.ToString());
												Binding.CallFunction<FLinearColor>(*Settings, Color);
												FPropertyChangedEvent Event(Binding.GetProperty(*Settings));
												Settings->PostEditChangeProperty(Event);
											});
										OpenColorPicker(PickerArgs);
										return FReply::Handled();
									})
						]
				]
		];
	// clang-format on
}

FToolMenuEntry CreateTrailColor(UMotionTrailToolOptions* Settings, FName PropertyName, FName Label, FName ToolTip)
{
	FText Text = FText::FromString(Label.ToString());
	FText TooltipText = FText::FromString(ToolTip.ToString());

	constexpr bool bNoIndent = false;
	constexpr bool bSearchable = true;
	constexpr bool bNoPadding = false;
	return FToolMenuEntry::InitWidget(PropertyName, CreateTrailColorWidget(Settings, PropertyName), Text, bNoIndent, bSearchable,
		bNoPadding, TooltipText);
}

// CreatePropertyWidget and CreateProperty templates are in the header.

void CreatePinnedSubMenu(UToolMenu* InSubMenu, UMotionTrailToolOptions* Settings, ETrailCategory Category)
{
	FToolMenuSection& PinnedSection = InSubMenu->AddSection("PinnedSection", LOCTEXT("PinnedSection", "Pinned"));

	FUIAction PinSelectedAction(
		FExecuteAction::CreateLambda([Settings, Category]()
			{
				if (Settings->bShowTrails == false)
				{
					Settings->bShowTrails = true;
					FPropertyChangedEvent ShowTrailEvent(UMotionTrailToolOptions::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, bShowTrails)));
					Settings->PostEditChangeProperty(ShowTrailEvent);
				}
				Settings->PinSelection(Category);
			}
		),
		FCanExecuteAction()
	);

	FToolMenuEntry PinSelected = FToolMenuEntry::InitMenuEntry(
		"PinSelected",
		LOCTEXT("PinSelected", "Pin Selected"),
		LOCTEXT("PinSelectedTrails", "Pin Selected Trails"),
		FSlateIcon(),
		PinSelectedAction,
		EUserInterfaceActionType::Button
	);
	PinSelected.InsertPosition.Name = PinSelected.Name;
	PinSelected.InsertPosition.Position = EToolMenuInsertType::First;
	PinnedSection.AddEntry(PinSelected);

	FUIAction SelectSocketAction(
		FExecuteAction::CreateLambda([Settings]()
			{
				// FIXME temp approach for selecting the parent
				FSlateApplication::Get().DismissAllMenus();

				static const FActorPickerModeModule& ActorPickerMode = FModuleManager::Get().GetModuleChecked<FActorPickerModeModule>("ActorPickerMode");

				ActorPickerMode.BeginActorPickingMode(
					FOnGetAllowedClasses(),
					FOnShouldFilterActor::CreateLambda([](const AActor* InActor)
						{
							UActorComponent* Component = InActor->GetComponentByClass(USkeletalMeshComponent::StaticClass());
							return Component != nullptr;
						}),
					FOnActorSelected::CreateLambda([Settings](AActor* InActor)
						{
							FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
							TSharedPtr< ILevelEditor > LevelEditor = LevelEditorModule.GetFirstLevelEditor();

							if (USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(InActor->GetComponentByClass(USkeletalMeshComponent::StaticClass())))
							{
								TSharedPtr<SWidget> MenuWidget =
									SNew(SSocketChooserPopup)
									.SceneComponent(Component)
									.OnSocketChosen_Lambda([Settings, Component](FName InSocketName) mutable
										{
											if (Settings->bShowTrails == false)
											{
												Settings->bShowTrails = true;
												FPropertyChangedEvent ShowTrailEvent(UMotionTrailToolOptions::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, bShowTrails)));
												Settings->PostEditChangeProperty(ShowTrailEvent);
											}
											Settings->PinComponent(Component, InSocketName);
										}
									);
								FSlateApplication::Get().PushMenu(
									LevelEditor.ToSharedRef(),
									FWidgetPath(),
									MenuWidget.ToSharedRef(),
									FSlateApplication::Get().GetCursorPos(),
									FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
								);
							}
						})
				);
			}
		),
		FCanExecuteAction::CreateLambda([Category]()
			{
				// Pin Socket only applies to transform/controlrig trails, not mixer trails
				return EnumHasAnyFlags(Category, ETrailCategory::Transform | ETrailCategory::ControlRig);
			}
		)
	);

	FToolMenuEntry SelectSocket = FToolMenuEntry::InitMenuEntry(
		"SelectSocket",
		LOCTEXT("SelectSocket", "Pin Socket"),
		LOCTEXT("SelectSocketTrails", "Pin a Skeletal Mesh Socket by selecting it"),
		FSlateIcon(),
		SelectSocketAction,
		EUserInterfaceActionType::Button
	);
	SelectSocket.InsertPosition.Name = SelectSocket.Name;
	PinnedSection.AddEntry(SelectSocket);

	FUIAction UnpinAllAction(
		FExecuteAction::CreateLambda([Settings, Category]()
			{
				Settings->DeleteAllPinned(Category);
			}
		),
		FCanExecuteAction()
	);

	FToolMenuEntry UnpinAll = FToolMenuEntry::InitMenuEntry(
		"UnpinAll",
		LOCTEXT("UnpinAll", "Unpin All"),
		LOCTEXT("UnpinAllTrails", "Unpin All Trails"),
		FSlateIcon(),
		UnpinAllAction,
		EUserInterfaceActionType::Button
	);
	UnpinAll.InsertPosition.Name = UnpinAll.Name;
	PinnedSection.AddEntry(UnpinAll);

	//add pinned items
	if (Settings->GetNumPinned() > 0)
	{
		FToolMenuSection& PinnedTrails = InSubMenu->AddSection("PinnedTrails", LOCTEXT("PinnedTrails", "Pinned Trails"));
		CreatePinnedItems(Settings, PinnedTrails, Category);
	}
}

void CreateAdvancedSubMenu(UToolMenu* InSubMenu, UMotionTrailToolOptions* Settings)
{
	FToolMenuSection& TrailSettings = InSubMenu->AddSection("TrailSettings", LOCTEXT("TrailSettings", "Trail Settings"));

	{
		FUIAction Action;
		Action.ExecuteAction = FExecuteAction::CreateLambda(
			[Settings]() -> void
			{
				Settings->bShowKeys = !Settings->bShowKeys;
				FPropertyChangedEvent Event(UMotionTrailToolOptions::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, bShowKeys)));
				Settings->PostEditChangeProperty(Event);
			}
		);
		Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
			[Settings]() -> ECheckBoxState
			{
				return Settings->bShowKeys ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		);

		FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
			"ShowKeys",
			LOCTEXT("ShowKeys", "Show Keys"),
			LOCTEXT("ShowKeysTooltip", "Show keys"),
			FSlateIcon(),
			Action,
			EUserInterfaceActionType::Check
		);
		TrailSettings.AddEntry(Entry);
	}

	{
		FUIAction Action;
		Action.ExecuteAction = FExecuteAction::CreateLambda(
			[Settings]() -> void
			{
				Settings->bShowMarks = !Settings->bShowMarks;
				FPropertyChangedEvent Event(UMotionTrailToolOptions::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, bShowMarks)));
				Settings->PostEditChangeProperty(Event);
			}
		);
		Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
			[Settings]() -> ECheckBoxState
			{
				return Settings->bShowMarks ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		);

		FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
			"ShowMarks",
			LOCTEXT("ShowMarks", "Show Marks"),
			LOCTEXT("ShowMarksTooltip", "Show Marks"),
			FSlateIcon(),
			Action,
			EUserInterfaceActionType::Check
		);
		TrailSettings.AddEntry(Entry);
	}
	{
		const FName DoubleProperty("KeySize");
		const FName Label("Key Size");
		TrailSettings.AddEntry(CreateProperty<double>(Settings, DoubleProperty, Label));
	}
	{
		const FName DoubleProperty("MarkSize");
		const FName Label("Mark Size");
		TrailSettings.AddEntry(CreateProperty<double>(Settings, DoubleProperty, Label));
	}
	{
		const FName DoubleProperty("TrailThickness");
		const FName Label("Trail Thickness");
		TrailSettings.AddEntry(CreateProperty<double>(Settings, DoubleProperty, Label));
	}

	FToolMenuSection& ColorSettings = InSubMenu->AddSection("ColorSettings", LOCTEXT("ColorSettings", "Color Settings"));
	{
		const FName ColorProperty("DefaultColor");
		const FName Label("Default Color");
		const FName Tooltip("Color for default style");
		ColorSettings.AddEntry(CreateTrailColor(Settings, ColorProperty, Label, Tooltip));
	}
	{
		const FName ColorProperty("TimePreColor");
		const FName Label("Time Previous Color");
		const FName Tooltip("Color before current time for time style");
		ColorSettings.AddEntry(CreateTrailColor(Settings, ColorProperty, Label, Tooltip));
	}
	{
		const FName ColorProperty("TimePostColor");
		const FName Label("Time Post Color");
		const FName Tooltip("Color after current time for time style");
		ColorSettings.AddEntry(CreateTrailColor(Settings, ColorProperty, Label, Tooltip));
	}
	{
		const FName ColorProperty("DashPreColor");
		const FName Label("Dash Previous Color");
		const FName Tooltip("Previous Color for dash style");
		ColorSettings.AddEntry(CreateTrailColor(Settings, ColorProperty, Label, Tooltip));
	}
	{
		const FName ColorProperty("DashPostColor");
		const FName Label("Dash Post Color");
		const FName Tooltip("Next Color for dash style");
		ColorSettings.AddEntry(CreateTrailColor(Settings, ColorProperty, Label, Tooltip));
	}
	{
		const FName ColorProperty("KeyColor");
		const FName Label("Key Color");
		const FName Tooltip("Color for keys");
		ColorSettings.AddEntry(CreateTrailColor(Settings, ColorProperty, Label, Tooltip));
	}
	{
		const FName ColorProperty("SelectedKeyColor");
		const FName Label("Selected Key Color");
		const FName Tooltip("Color for selected keys");
		ColorSettings.AddEntry(CreateTrailColor(Settings, ColorProperty, Label, Tooltip));
	}
	{
		const FName ColorProperty("MarkColor");
		const FName Label("Mark Color");
		const FName Tooltip("Color for marks");
		ColorSettings.AddEntry(CreateTrailColor(Settings, ColorProperty, Label, Tooltip));
	}
	{
		const FName ColorProperty("CurrentFrameMarkColor");
		const FName Label("Current Mark Color");
		const FName Tooltip("Color for mark at current frame");
		ColorSettings.AddEntry(CreateTrailColor(Settings, ColorProperty, Label, Tooltip));
	}

	FToolMenuSection& PinSettings = InSubMenu->AddSection("PinSettings", LOCTEXT("PinSettings", "Pin Settings"));
	{
		PinSettings.AddEntry(CreateMaxNumberPinned(Settings));
	}
}

} // namespace Private

void PopulateMotionTrailMenu(UToolMenu* InMenu, ETrailCategory Category)
{
	if (!InMenu)
	{
		return;
	}

	UMotionTrailToolOptions* Settings = GetMutableDefault<UMotionTrailToolOptions>();

	FToolMenuSection& PathModeSection = InMenu->AddSection("PathModeSection", LOCTEXT("PathMode", "Path Mode"));

	// Playback Range
	{
		FUIAction Action;
		Action.ExecuteAction = FExecuteAction::CreateLambda(
			[Settings]() -> void
			{
				Settings->MotionTrailRange = EMotionTrailRange::PlaybackRange;
				FPropertyChangedEvent Event(UMotionTrailToolOptions::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, MotionTrailRange)));
				Settings->PostEditChangeProperty(Event);
			}
		);
		Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
			[Settings]() -> ECheckBoxState
			{
				return (Settings->MotionTrailRange == EMotionTrailRange::PlaybackRange) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		);

		FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
			"PlaybackRange",
			LOCTEXT("PlaybackRangeLabel", "Playback Range"),
			LOCTEXT("PlaybackRangeTooltip", "Show full trail along playback range"),
			FSlateIcon(),
			Action,
			EUserInterfaceActionType::RadioButton
		);
		PathModeSection.AddEntry(Entry);
	}

	// Selection Range
	{
		FUIAction Action;
		Action.ExecuteAction = FExecuteAction::CreateLambda(
			[Settings]() -> void
			{
				Settings->MotionTrailRange = EMotionTrailRange::SelectionRange;
				FPropertyChangedEvent Event(UMotionTrailToolOptions::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, MotionTrailRange)));
				Settings->PostEditChangeProperty(Event);
			}
		);
		Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
			[Settings]() -> ECheckBoxState
			{
				return (Settings->MotionTrailRange == EMotionTrailRange::SelectionRange) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		);

		FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
			"SelectionRange",
			LOCTEXT("SelectionRangeLabel", "Selection Range"),
			LOCTEXT("FullTrailTooltip", "Show trail in the selection range"),
			FSlateIcon(),
			Action,
			EUserInterfaceActionType::RadioButton
		);
		PathModeSection.AddEntry(Entry);
	}

	// Set Frames
	{
		FUIAction Action;
		Action.ExecuteAction = FExecuteAction::CreateLambda(
			[Settings]() -> void
			{
				Settings->MotionTrailRange = EMotionTrailRange::SpecifiedRange;
				FPropertyChangedEvent Event(UMotionTrailToolOptions::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, MotionTrailRange)));
				Settings->PostEditChangeProperty(Event);
			}
		);
		Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
			[Settings]() -> ECheckBoxState
			{
				return Settings->MotionTrailRange == EMotionTrailRange::SpecifiedRange ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		);

		FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
			"Set Frames",
			LOCTEXT("SetFramesLabel", "Set Frames"),
			LOCTEXT("SetframesTooltip", "Specify frame range"),
			FSlateIcon(),
			Action,
			EUserInterfaceActionType::RadioButton
		);
		PathModeSection.AddEntry(Entry);
	}
	PathModeSection.AddEntry(Private::CreateFramesBefore(Settings));
	PathModeSection.AddEntry(Private::CreateFramesAfter(Settings));

	Private::CreatePinnedSubMenu(InMenu, Settings, Category);

	FToolMenuSection& PathOptionsMenu = InMenu->AddSection(TEXT("PathOptions"), LOCTEXT("PathOptions", "Path Options"));
	// Show selected trails
	{
		FUIAction Action;
		Action.ExecuteAction = FExecuteAction::CreateLambda(
			[Settings]() -> void
			{
				Settings->bShowSelectedTrails = !Settings->bShowSelectedTrails;
				FPropertyChangedEvent Event(UMotionTrailToolOptions::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, bShowSelectedTrails)));
				Settings->PostEditChangeProperty(Event);
			}
		);
		Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
			[Settings]() -> ECheckBoxState
			{
				return Settings->bShowSelectedTrails ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		);

		FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
			"ShowSelectedtrails",
			LOCTEXT("ShowSelectedtrailsLabel", "Show Trails On Selection"),
			LOCTEXT("ShowSelectedtrailsLabelTooltip", "Show trails on selected sequencer items."),
			FSlateIcon(),
			Action,
			EUserInterfaceActionType::ToggleButton
		);
		PathOptionsMenu.AddEntry(Entry);
	}
	PathOptionsMenu.AddEntry(Private::CreateTrailStyle(Settings));

	PathOptionsMenu.AddSubMenu(TEXT("Advanced"), LOCTEXT("Advanced", "Advanced"), LOCTEXT("Advanced_tooltip", "Advanced options"),
		FNewToolMenuDelegate::CreateLambda([Settings](UToolMenu* InSubMenu)
			{
				Private::CreateAdvancedSubMenu(InSubMenu, Settings);
			}));
}

FToolUIAction MakeMotionTrailToggleAction(ETrailCategory Category)
{
	UMotionTrailToolOptions* Settings = GetMutableDefault<UMotionTrailToolOptions>();

	FToolUIAction CheckboxMenuAction;
	CheckboxMenuAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda(
		[Settings, Category](const FToolMenuContext& Context) -> void
		{
			const bool bAltDown = FSlateApplication::Get().GetModifierKeys().IsAltDown();
			const bool bControlDown = FSlateApplication::Get().GetModifierKeys().IsControlDown();
			const bool bShiftDown = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
			const bool bModifierDown = bAltDown || bControlDown || bShiftDown;
			auto HandleModifier = [Settings, Category, bAltDown, bControlDown, bShiftDown]() {
				if (bAltDown)
				{
					Settings->UnPinSelection(Category);
					return;
				}
				if (bControlDown)
				{
					Settings->DeleteAllPinned(Category);
					Settings->PinSelection(Category);
					return;
				}
				if (bShiftDown)
				{
					Settings->PinSelection(Category);
					return;
				}
			};

			const bool bCurrentlyVisible = FTrailCategoryRegistry::IsCategoryVisible(Category);

			if (bCurrentlyVisible && bModifierDown)
			{
				HandleModifier();
				return;
			}

			// Toggle this category's visibility
			FTrailCategoryRegistry::SetCategoryVisible(Category, !bCurrentlyVisible);

			// Ensure bShowTrails is on if any category is now visible
			if (!bCurrentlyVisible)
			{
				if (!Settings->bShowTrails)
				{
					Settings->bShowTrails = true;
					FPropertyChangedEvent ShowTrailEvent(UMotionTrailToolOptions::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, bShowTrails)));
					Settings->PostEditChangeProperty(ShowTrailEvent);
				}
				if (Settings->GetNumPinned() == 0)
				{
					Settings->PinSelection(Category);
				}
				else if (bModifierDown)
				{
					HandleModifier();
				}
			}
			else
			{
				// If no categories are visible anymore, turn off the toggle
				if (FTrailCategoryRegistry::GetVisibleCategories() == ETrailCategory::None)
				{
					Settings->bShowTrails = false;
					FPropertyChangedEvent ShowTrailEvent(UMotionTrailToolOptions::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, bShowTrails)));
					Settings->PostEditChangeProperty(ShowTrailEvent);
				}
			}
		}
	);
	CheckboxMenuAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda(
		[Category](const FToolMenuContext& Context) -> ECheckBoxState
		{
			return FTrailCategoryRegistry::IsCategoryVisible(Category) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
	);

	return CheckboxMenuAction;
}

void RegisterMotionPathsToolbarEntry(
	const FName& InMenuName,
	const FName& OwnerName,
	const FName& EntryName,
	const FText& Label,
	const FText& Tooltip,
	ETrailCategory Category)
{
	FToolMenuOwnerScoped ScopeOwner(OwnerName);

	UToolMenu* const Menu = UToolMenus::Get()->ExtendMenu(InMenuName);
	FToolMenuSection& PreviewToolsSection = Menu->FindOrAddSection(
		"PreviewTools", LOCTEXT("PreviewToolsLabel", "Preview Tools"));

	FNewToolMenuDelegate MakeMenuDelegate = FNewToolMenuDelegate::CreateLambda(
		[Category](UToolMenu* Submenu)
		{
			PopulateMotionTrailMenu(Submenu, Category);
		}
	);

	FToolUIAction CheckboxMenuAction = MakeMotionTrailToggleAction(Category);

	FToolMenuEntry MotionPathsSubmenu = FToolMenuEntry::InitSubMenu(
		EntryName,
		Label,
		Tooltip,
		MakeMenuDelegate,
		CheckboxMenuAction,
		EUserInterfaceActionType::ToggleButton,
		false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Tracks.Animation")
	);
	MotionPathsSubmenu.SetShowInToolbarTopLevel(true);

	PreviewToolsSection.AddEntry(MotionPathsSubmenu);
}

void UnregisterMotionPathsToolbarEntry(const FName& OwnerName)
{
	UToolMenus::Get()->UnregisterOwnerByName(OwnerName);
}

} // namespace UE::Sequencer::MotionTrailMenu

#undef LOCTEXT_NAMESPACE
