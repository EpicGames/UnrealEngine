// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/SMotionTrailOptions.h"

#include "DetailsViewArgs.h"
#include "IDetailsView.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SlotBase.h"
#include "Tools/MotionTrailOptions.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "GameFramework/Actor.h"

#define LOCTEXT_NAMESPACE "MotionTrail"

void SMotionTrailOptions::Construct(const FArguments& InArgs)
{
	UMotionTrailToolOptions* Settings = GetMutableDefault<UMotionTrailToolOptions>();
	FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ViewIdentifier = "MotionTrailOptions";

	DetailsView = PropertyEditor.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(Settings);

	ChildSlot
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
			SNew(SVerticalBox)
			
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.HAlign(HAlign_Fill)
				[
					DetailsView.ToSharedRef()
				]
			]
			
		];
}

int32 UMotionTrailToolOptions::GetNumPinned() const
{
	return PinnedTrails.Num();
}

UMotionTrailToolOptions::FPinnedTrail* UMotionTrailToolOptions::GetPinnedTrail(int32 Index)  
{
	if (Index >= 0 && Index < PinnedTrails.Num())
	{
		return &(PinnedTrails[Index]);
	}
	return nullptr;
}

void UMotionTrailToolOptions::ResetPinnedItems()
{
	PinnedTrails.Reset();
}

void UMotionTrailToolOptions::AddPinned(const FPinnedTrail& InPinnedTrail)
{
	if (PinnedTrails.Num() >= MaxNumberPinned)
	{
		UE_LOGF(LogTemp, Warning, "MotionTrails: Cannot Pin trail %ls Max number reached. Please delete pinned trail if you want to add this one.", *InPinnedTrail.TrailName.ToString());
		return;
	}
	if (UMotionTrailToolOptions::FPinnedTrail* Trail = PinnedTrails.FindByKey(InPinnedTrail.TrailGuid))
	{
		return;
	}
	PinnedTrails.Add(InPinnedTrail);
	OnAddPinned.Broadcast(InPinnedTrail.TrailGuid);
}

int32 UMotionTrailToolOptions::GetIndexFromGuid(FGuid InGuid) const
{
	return PinnedTrails.IndexOfByKey(InGuid);
}

void UMotionTrailToolOptions::PinSelection(ETrailCategory Category) const
{
	OnPinSelection.Broadcast(Category);
}

void UMotionTrailToolOptions::UnPinSelection(ETrailCategory Category)
{
	OnUnPinSelection.Broadcast(Category);
}

void UMotionTrailToolOptions::PinComponent(USceneComponent* InSceneComponent, const FName& InSocketName) const
{
	OnPinComponent.Broadcast(InSceneComponent, InSocketName);
}

void UMotionTrailToolOptions::DeletePinned(int32 Index)
{
	if (Index >= 0 && Index < PinnedTrails.Num())
	{
		FGuid Guid = PinnedTrails[Index].TrailGuid;
		PinnedTrails.RemoveAt(Index);
		OnDeletePinned.Broadcast(Guid);
	}
}

void UMotionTrailToolOptions::DeleteAllPinned(ETrailCategory Category)
{
	if (Category == ETrailCategory::All)
	{
		PinnedTrails.SetNum(0);
	}
	else
	{
		for (int32 Index = PinnedTrails.Num() - 1; Index >= 0; --Index)
		{
			if (EnumHasAnyFlags(PinnedTrails[Index].Category, Category))
			{
				FGuid Guid = PinnedTrails[Index].TrailGuid;
				PinnedTrails.RemoveAt(Index);
				OnDeletePinned.Broadcast(Guid);
			}
		}
	}
	OnDeleteAllPinned.Broadcast(Category);
}

void UMotionTrailToolOptions::PutPinnnedInSpace(int32 Index, AActor* InActor, const FName& InComponentName)
{
	if (Index >= 0 && Index < PinnedTrails.Num())
	{
		FGuid Guid = PinnedTrails[Index].TrailGuid;
		if (InActor)
		{
			PinnedTrails[Index].SpaceName = FText::FromString(InActor->GetActorLabel());
		}
		else
		{
			PinnedTrails[Index].SpaceName.Reset();
		}
		OnPutPinnedInSpace.Broadcast(Guid,InActor,InComponentName);
	}
}

void UMotionTrailToolOptions::SetLinearColor(int32 Index, const FLinearColor& Color)
{
	if (Index >= 0 && Index < PinnedTrails.Num())
	{
		PinnedTrails[Index].TrailColor = Color;
		OnSetLinearColor.Broadcast(PinnedTrails[Index].TrailGuid, Color);
	}
}

void UMotionTrailToolOptions::SetHasOffset(int32 Index, bool bHasOffset)
{
	if (Index >= 0 && Index < PinnedTrails.Num())
	{
		PinnedTrails[Index].bHasOffset = bHasOffset;
		OnSetHasOffset.Broadcast(PinnedTrails[Index].TrailGuid, bHasOffset);
	}
}


TArray<TPair<FText, FText>>& UMotionTrailToolOptions::GetTrailStyles()  
{
	if (TrailStylesText.Num() == 0)
	{
		TPair<FText, FText> TextToAdd;
		TextToAdd.Key = (LOCTEXT("Default", "Default"));
		TextToAdd.Value = (LOCTEXT("DefaultTooltip", "Use specified single trail color"));
		TrailStylesText.Add(TextToAdd);

		TextToAdd.Key = (LOCTEXT("Dashed", "Dashed"));
		TextToAdd.Value = (LOCTEXT("DashedTooltip", "Alternate color every other frame"));
		TrailStylesText.Add(TextToAdd);

		TextToAdd.Key = (LOCTEXT("Time", "Time"));
		TextToAdd.Value = (LOCTEXT("TimeTooltip", "Alternate color before and after current Sequencer time"));
		TrailStylesText.Add(TextToAdd);

		TextToAdd.Key = (LOCTEXT("HeatMap", "Heat Map"));
		TextToAdd.Value = (LOCTEXT("HeatMapTooltip", "Color shows speed from fast(Red) to slow(Blue)"));
		TrailStylesText.Add(TextToAdd);

	}
	return TrailStylesText;
}

void UMotionTrailToolOptions::SetTrailStyle(int32 Index)
{
	if (Index >= 0 && Index < 4)
	{
		TrailStyle = (EMotionTrailTrailStyle)(uint8)(Index);
		FPropertyChangedEvent Event(UMotionTrailToolOptions::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, TrailStyle)));
		PostEditChangeProperty(Event);
	}
}

int32 UMotionTrailToolOptions::GetTrailStyleIndex() const
{
	return (int32)TrailStyle;
}




#undef LOCTEXT_NAMESPACE
