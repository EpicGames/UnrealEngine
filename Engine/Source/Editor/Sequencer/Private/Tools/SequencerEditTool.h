// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Framework/DelayedDrag.h"
#include "ISequencerEditTool.h"
#include "Sequencer.h"

class FSlateWindowElementList;

/**
 * Abstract base class for edit tools.
 */
class FSequencerEditTool
	: public UE::Sequencer::ISequencerEditTool
{
public:

	FSequencerEditTool(FSequencer& InSequencer)
		: Sequencer(InSequencer)
	{ }

	/** Virtual destructor. */
	~FSequencerEditTool() = default;

public:

	// ISequencerEditTool interface

	virtual FReply OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return FReply::Unhandled();
	}

	virtual FReply OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return FReply::Unhandled();
	}

	virtual FReply OnMouseButtonDoubleClick(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return FReply::Unhandled();
	}

	virtual FReply OnMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return FReply::Unhandled();
	}

	virtual FReply OnMouseWheel(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return FReply::Unhandled();
	}

	virtual void OnMouseCaptureLost() override
	{
		// do nothing
	}

	virtual void OnMouseEnter(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		// do nothing
	}

	virtual void OnMouseLeave(SWidget& OwnerWidget, const FPointerEvent& MouseEvent) override
	{
		// do nothing
	}

	virtual int32 OnPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const override
	{
		return LayerId;
	}

	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override
	{
		return FCursorReply::Unhandled();
	}

	ISequencer& GetSequencer() const
	{
		return Sequencer;
	}

protected:

	/** This edit tool's sequencer */
	FSequencer& Sequencer;
};

namespace UE
{
namespace Sequencer
{

struct FDelayedDrag_Hotspot : FDelayedDrag
{
	FDelayedDrag_Hotspot(FVector2D InInitialPosition, FKey InApplicableKey, TSharedPtr<ITrackAreaHotspot> InHotspot)
		: FDelayedDrag(InInitialPosition, InApplicableKey)
		, Hotspot(MoveTemp(InHotspot))
	{
		SetTriggerScaleFactor(0.1f);
	}

	TSharedPtr<ITrackAreaHotspot> Hotspot;
};


} // namespace Sequencer
} // namespace UE
