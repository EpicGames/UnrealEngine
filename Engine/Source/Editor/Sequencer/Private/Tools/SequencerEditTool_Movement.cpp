// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/SequencerEditTool_Movement.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "SequencerTimeDomainOverride.h"
#include "MVVM/Selection/Selection.h"
#include "Editor.h"
#include "Fonts/FontMeasure.h"
#include "MVVM/Views/STrackAreaView.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "SequencerCommonHelpers.h"
#include "SSequencer.h"
#include "MVVM/Views/ITrackAreaHotspot.h"
#include "SequencerHotspots.h"
#include "MVVM/ViewModels/VirtualTrackArea.h"
#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "SequencerSettings.h"
#include "Tools/EditToolDragOperations.h"
#include "IKeyArea.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/Timecode.h"
#include "MovieSceneTimeHelpers.h"

const FName FSequencerEditTool_Movement::Identifier = "Movement";


FSequencerEditTool_Movement::FSequencerEditTool_Movement(FSequencer& InSequencer, UE::Sequencer::STrackAreaView& InTrackArea)
	: FSequencerEditTool(InSequencer)
	, TrackArea(InTrackArea)
	, CursorDecorator(nullptr)
{ }


FReply FSequencerEditTool_Movement::OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	using namespace UE::Sequencer;

	TSharedRef<SSequencer> SequencerWidget = Sequencer.GetUnderlyingSequencerWidget();

	TSharedPtr<FTrackAreaViewModel> TrackAreaViewModel = TrackArea.GetViewModel();
	TSharedPtr<ITrackAreaHotspot> Hotspot = TrackAreaViewModel->GetHotspot();

	DelayedDrag.Reset();

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton || MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton)
	{
		const FVirtualTrackArea VirtualTrackArea = SequencerWidget->GetVirtualTrackArea(&TrackArea);

		DelayedDrag = FDelayedDrag_Hotspot(VirtualTrackArea.CachedTrackAreaGeometry().AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()), MouseEvent.GetEffectingButton(), Hotspot);

		USequencerSettings* Settings = Sequencer.GetSequencerSettings();

		const bool bSnapPlayTimeToPressedKey = 
			(Settings->GetIsSnapEnabled() && Settings->GetSnapPlayTimeToPressedKey()) || (MouseEvent.IsShiftDown() && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton);

		if (bSnapPlayTimeToPressedKey)
		{
			if (TSharedPtr<FKeyHotspot> KeyHotspot = HotspotCast<FKeyHotspot>(DelayedDrag->Hotspot))
			{
				TOptional<FFrameNumber> Time   = KeyHotspot->GetTime();
				TOptional<ETimeDomain>  Domain = KeyHotspot->GetDomain();
				if (Time && Domain)
				{
					FTimeDomainOverride DomainOverride = Sequencer.OverrideTimeDomain(Domain.GetValue());
					Sequencer.SetLocalTime(Time.GetValue());
				}
			}
		}

		UpdateCursorDecorator(MyGeometry, MouseEvent);

		return FReply::Handled().PreventThrottling();
	}

	UpdateCursorDecorator(MyGeometry, MouseEvent);

	return FReply::Unhandled();
}


FReply FSequencerEditTool_Movement::OnMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	using namespace UE::Sequencer;

	if (DelayedDrag.IsSet())
	{
		TSharedRef<SSequencer> SequencerWidget = Sequencer.GetUnderlyingSequencerWidget();
		const FVirtualTrackArea VirtualTrackArea = SequencerWidget->GetVirtualTrackArea(&TrackArea);

		FReply Reply = FReply::Handled();

		if (DelayedDrag->IsDragging())
		{
			// If we're already dragging, just update the drag op if it exists
			if (DragOperation.IsValid())
			{
				DragPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

				double CurrentTime = VirtualTrackArea.PixelToSeconds(DragPosition.X);
				Sequencer.UpdateAutoScroll(CurrentTime);

				DragOperation->OnDrag(MouseEvent, FVector2D(DragPosition), VirtualTrackArea);
			}
		}
		// Otherwise we can attempt a new drag
		else if (DelayedDrag->AttemptDragStart(MouseEvent))
		{
			DragOperation = CreateDrag(MouseEvent);

			if (DragOperation.IsValid())
			{
				DragOperation->OnBeginDrag(MouseEvent, DelayedDrag->GetInitialPosition(), VirtualTrackArea);

				// Steal the capture, as we're now the authoritative widget in charge of a mouse-drag operation
				Reply.CaptureMouse(OwnerWidget.AsShared());
			}
		}

		UpdateCursorDecorator(MyGeometry, MouseEvent);

		return Reply;
	}

	UpdateCursorDecorator(MyGeometry, MouseEvent);

	return FReply::Unhandled();
}

bool FSequencerEditTool_Movement::GetHotspotTime(FFrameTime& HotspotTime) const
{
	if (DelayedDrag->Hotspot.IsValid())
	{
		TOptional<FFrameNumber> OptionalHotspotTime = DelayedDrag->Hotspot->GetTime();
		if (OptionalHotspotTime.IsSet())
		{
			HotspotTime = OptionalHotspotTime.GetValue();
			return true;
		}
	}
	return false;
}

FFrameTime FSequencerEditTool_Movement::GetHotspotOffsetTime(FFrameTime CurrentTime) const
{
	//@todo abstract dragging offset from shift
	if (DelayedDrag->Hotspot.IsValid() && FSlateApplication::Get().GetModifierKeys().IsShiftDown())
	{
		TOptional<FFrameTime> OptionalOffsetTime = DelayedDrag->Hotspot->GetOffsetTime();
		if (OptionalOffsetTime.IsSet())
		{
			return OptionalOffsetTime.GetValue();
		}
	}
	return CurrentTime - OriginalHotspotTime;
}

TSharedPtr<UE::Sequencer::ISequencerEditToolDragOperation> FSequencerEditTool_Movement::CreateDrag(const FPointerEvent& MouseEvent)
{
	using namespace UE::Sequencer;

	FSequencerSelection& Selection = *Sequencer.GetViewModel()->GetSelection();
	FSelectionEventSuppressor SuppressEvents = Selection.SuppressEvents();

	const TSharedRef<SSequencer> SequencerWidget = Sequencer.GetUnderlyingSequencerWidget();

	GetHotspotTime(OriginalHotspotTime);

	if (DelayedDrag->Hotspot.IsValid())
	{
		// Let the hotspot start a drag first, if it wants to
		TSharedPtr<ISequencerEditToolDragOperation> HotspotDrag = DelayedDrag->Hotspot->InitiateDrag(MouseEvent);
		if (HotspotDrag.IsValid())
		{
			return HotspotDrag;
		}

		// Gather the draggable sections from all selected models
		const bool bModelsSelected = Selection.TrackArea.Num() > 0;
		const bool bKeySelected = Selection.KeySelection.Num() > 0;
		
		// @todo sequencer: Make this a customizable UI command modifier?
		const bool bIsDuplicateEvent = 
			MouseEvent.IsAltDown() || 
			(!MouseEvent.GetModifierKeys().AnyModifiersDown() && MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton);

		FSectionHotspotBase* SectionHotspot = DelayedDrag->Hotspot->CastThis<FSectionHotspotBase>();
		FKeyHotspot*         KeyHotspot     = DelayedDrag->Hotspot->CastThis<FKeyHotspot>();

		// If they have both keys and sections selected then we only support moving them right now, so we
		// check for that first before trying to figure out if they're resizing or dilating.
		if (bModelsSelected && bKeySelected && !bIsDuplicateEvent)
		{
			return MakeShareable(new FMoveKeysAndSections(Sequencer, ESequencerMoveOperationType::MoveKeys | ESequencerMoveOperationType::MoveSections));
		}
		else if (bIsDuplicateEvent)
		{
			if (KeyHotspot)
			{
				const TSet<FSequencerSelectedKey>& HoveredKeys = KeyHotspot->Keys;

				bool bUniqueDrag = false;
				for (const FSequencerSelectedKey& Key : HoveredKeys)
				{
					// If any are not selected, we'll treat this as a unique drag
					if (!Selection.KeySelection.IsSelected(Key.KeyHandle))
					{
						bUniqueDrag = true;
					}
				};

				if (bUniqueDrag)
				{
					Selection.KeySelection.Empty();
					Selection.TrackArea.Empty();
					for (const FSequencerSelectedKey& Key : HoveredKeys)
					{
						Selection.KeySelection.Select(Key.WeakChannel.Pin(), Key.KeyHandle);
					}
				}
			}
			else if (SectionHotspot)
			{
				if (!Selection.TrackArea.IsSelected(SectionHotspot->WeakSectionModel.Pin()))
				{
					Selection.KeySelection.Empty();
					Selection.TrackArea.Empty();
					Selection.TrackArea.Select(SectionHotspot->WeakSectionModel.Pin());
				}
			}

			return MakeShareable(new FDuplicateKeysAndSections(Sequencer, ESequencerMoveOperationType::MoveKeys | ESequencerMoveOperationType::MoveSections));
		}

		// Moving section(s)?
		if (SectionHotspot)
		{
			if (!Selection.TrackArea.IsSelected(SectionHotspot->WeakSectionModel.Pin()))
			{
				Selection.KeySelection.Empty();
				Selection.TrackArea.Empty();
				Selection.TrackArea.Select(SectionHotspot->WeakSectionModel.Pin());
			}

			if (MouseEvent.IsShiftDown())
			{
				const bool bDraggingByEnd = false;
				const bool bIsSlipping = true;
				return MakeShareable( new FResizeSection( Sequencer, bDraggingByEnd, bIsSlipping ) );
			}
			else
			{
				return MakeShareable( new FMoveKeysAndSections( Sequencer, ESequencerMoveOperationType::MoveSections) );
			}
		}
		// Moving key(s)?
		else if (KeyHotspot)
		{
			const TSet<FSequencerSelectedKey>& HoveredKeys = KeyHotspot->Keys;

			bool bUniqueDrag = false;
			for (const FSequencerSelectedKey& Key : HoveredKeys)
			{
				// If any are not selected, we'll treat this as a unique drag
				if (!Selection.KeySelection.IsSelected(Key.KeyHandle))
				{
					bUniqueDrag = true;
					break;
				}
			};

			if (bUniqueDrag)
			{
				Selection.KeySelection.Empty();
				Selection.TrackArea.Empty();
				for (const FSequencerSelectedKey& Key : HoveredKeys)
				{
					Selection.KeySelection.Select(Key.WeakChannel.Pin(), Key.KeyHandle);
				}
			}

			TSet<TWeakObjectPtr<UMovieSceneSection>> NoSections;
			return MakeShareable( new FMoveKeysAndSections( Sequencer, ESequencerMoveOperationType::MoveKeys) );
		}
	}
	// If we're not dragging a hotspot, sections take precedence over keys
	else if (Selection.TrackArea.Num())
	{
		return MakeShareable( new FMoveKeysAndSections( Sequencer, ESequencerMoveOperationType::MoveSections ) );
	}
	else if (Selection.KeySelection.Num())
	{
		return MakeShareable( new FMoveKeysAndSections( Sequencer, ESequencerMoveOperationType::MoveKeys) );
	}

	return nullptr;
}


FReply FSequencerEditTool_Movement::OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	using namespace UE::Sequencer;

	DelayedDrag.Reset();

	TSharedPtr<FTrackAreaViewModel> TrackAreaViewModel = TrackArea.GetViewModel();

	if (DragOperation.IsValid())
	{
		TSharedRef<SSequencer> SequencerWidget = Sequencer.GetUnderlyingSequencerWidget();

		DragOperation->OnEndDrag(MouseEvent, MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()), SequencerWidget->GetVirtualTrackArea(&TrackArea));
		DragOperation = nullptr;

		const bool bIsDuplicateEvent =
			MouseEvent.IsAltDown() ||
			(!MouseEvent.GetModifierKeys().AnyModifiersDown() && MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton);

		if (bIsDuplicateEvent)
		{
			GEditor->EndTransaction();
		}

		Sequencer.StopAutoscroll();

		// Only return handled if we actually started a drag
		return FReply::Handled().ReleaseMouseCapture();
	}

	SequencerHelpers::PerformDefaultSelection(Sequencer, MouseEvent);

	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && !Sequencer.IsReadOnly())
	{
		TSharedPtr<SWidget> MenuContent = SequencerHelpers::SummonContextMenu( Sequencer, MyGeometry, MouseEvent );
		if (MenuContent.IsValid())
		{
			FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

			TSharedPtr<IMenu> Menu = FSlateApplication::Get().PushMenu(
				OwnerWidget.AsShared(),
				WidgetPath,
				MenuContent.ToSharedRef(),
				MouseEvent.GetScreenSpacePosition(),
				FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
				);

			// Lock the hotspot while the menu is open
			TrackAreaViewModel->LockHotspot(true);

			// Unlock and reset the hotspot when the menu closes
			{
				TSharedPtr<ITrackAreaHotspot> ExistingHotspot = TrackAreaViewModel->GetHotspot();
				Menu->GetOnMenuDismissed().AddLambda(
					[TrackAreaViewModel, ExistingHotspot](TSharedRef<IMenu>)
					{
						TrackAreaViewModel->LockHotspot(false);
						if (TrackAreaViewModel->GetHotspot() == ExistingHotspot)
						{
							TrackAreaViewModel->SetHotspot(nullptr);
						}
					}
				);
			}

			UpdateCursorDecorator(MyGeometry, MouseEvent);
	
			return FReply::Handled().SetUserFocus(MenuContent.ToSharedRef(), EFocusCause::SetDirectly).ReleaseMouseCapture();
		}
	}

	UpdateCursorDecorator(MyGeometry, MouseEvent);

	return FReply::Handled();
}


void FSequencerEditTool_Movement::OnMouseCaptureLost()
{
	// Delaying nulling out until next tick because this could be invoked during OnMouseMove()
	GEditor->GetTimerManager()->SetTimerForNextTick([this]()
	{
		DelayedDrag.Reset();
		DragOperation = nullptr;
		CursorDecorator = nullptr;
	});
}


int32 FSequencerEditTool_Movement::OnPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	using namespace UE::Sequencer;

	// Let the drag operation paint first (for preview indicators)
	if (DragOperation.IsValid())
	{
		LayerId = DragOperation->OnPaint(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	}

	if (CursorDecorator)
	{
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(CursorDecorator->ImageSize, FSlateLayoutTransform(DragPosition + FVector2f(5.f, -25.f))),
			CursorDecorator
			);
	}

	if (DelayedDrag.IsSet() && DelayedDrag->IsDragging())
	{
		const TSharedPtr<ITrackAreaHotspot>& Hotspot = DelayedDrag->Hotspot;

		if (Hotspot.IsValid())
		{
			FFrameTime CurrentTime;

			if (GetHotspotTime(CurrentTime))
			{
				const TSharedRef<SSequencer> SequencerWidget = Sequencer.GetUnderlyingSequencerWidget();

				const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);
				const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
				const FLinearColor DrawColor = FAppStyle::GetSlateColor("SelectionColor").GetColor(FWidgetStyle());
				const FVector2D BoxPadding = FVector2D(4.0f, 2.0f);
				const float MousePadding = 20.0f;

				// calculate draw position
				const FVirtualTrackArea VirtualTrackArea = SequencerWidget->GetVirtualTrackArea(&TrackArea);
				const float HorizontalDelta = DragPosition.X - DelayedDrag->GetInitialPosition().X;
				const float InitialY = DelayedDrag->GetInitialPosition().Y;

				const FVector2D OldPos = FVector2D(VirtualTrackArea.FrameToPixel(OriginalHotspotTime), InitialY);
				const FVector2D NewPos = FVector2D(VirtualTrackArea.FrameToPixel(CurrentTime), InitialY);

				TArray<FVector2D> LinePoints;
				{
					LinePoints.AddUninitialized(2);
					LinePoints[0] = FVector2D(0.0f, 0.0f);
					LinePoints[1] = FVector2D(0.0f, VirtualTrackArea.GetPhysicalSize().Y);
				}

				// draw old position vertical
				FSlateDrawElement::MakeLines(
					OutDrawElements,
					LayerId + 1,
					AllottedGeometry.ToPaintGeometry(FVector2f(1.0f, 1.0f), FSlateLayoutTransform(FVector2f(OldPos.X, 0.0f))),
					LinePoints,
					ESlateDrawEffect::None,
					FLinearColor::White.CopyWithNewOpacity(0.5f),
					false
				);

				// draw new position vertical
				FSlateDrawElement::MakeLines(
					OutDrawElements,
					LayerId + 1,
					AllottedGeometry.ToPaintGeometry(FVector2f(1.0f, 1.0f), FSlateLayoutTransform(FVector2f(NewPos.X, 0.0f))),
					LinePoints,
					ESlateDrawEffect::None,
					DrawColor,
					false
				);

				// draw time string
				const FString TimeString = TimeToString(CurrentTime, false);
				const FVector2D TimeStringSize = FontMeasureService->Measure(TimeString, SmallLayoutFont);
				const FVector2D TimePos = FVector2D(NewPos.X - MousePadding - TimeStringSize.X, NewPos.Y - 0.5f * TimeStringSize.Y);

				FSlateDrawElement::MakeBox( 
					OutDrawElements,
					LayerId + 2, 
					AllottedGeometry.ToPaintGeometry(TimeStringSize + 2.0f * BoxPadding, FSlateLayoutTransform(TimePos - BoxPadding)),
					FAppStyle::GetBrush("WhiteBrush"),
					ESlateDrawEffect::None, 
					FLinearColor::Black.CopyWithNewOpacity(0.5f)
				);

				FSlateDrawElement::MakeText(
					OutDrawElements,
					LayerId + 3,
					AllottedGeometry.ToPaintGeometry(TimeStringSize, FSlateLayoutTransform(TimePos)),
					TimeString,
					SmallLayoutFont,
					ESlateDrawEffect::None,
					DrawColor
				);

				// draw offset string
				FFrameTime OffsetTime = GetHotspotOffsetTime(CurrentTime);
				const FString OffsetString = TimeToString(OffsetTime, true);
				const FVector2D OffsetStringSize = FontMeasureService->Measure(OffsetString, SmallLayoutFont);
				const FVector2D OffsetPos = FVector2D(NewPos.X + MousePadding, NewPos.Y - 0.5f * OffsetStringSize.Y);

				FSlateDrawElement::MakeBox( 
					OutDrawElements,
					LayerId + 2, 
					AllottedGeometry.ToPaintGeometry(OffsetStringSize + 2.0f * BoxPadding, FSlateLayoutTransform(OffsetPos - BoxPadding)),
					FAppStyle::GetBrush("WhiteBrush"),
					ESlateDrawEffect::None, 
					FLinearColor::Black.CopyWithNewOpacity(0.5f)
				);

				FSlateDrawElement::MakeText(
					OutDrawElements,
					LayerId + 3,
					AllottedGeometry.ToPaintGeometry(TimeStringSize, FSlateLayoutTransform(OffsetPos)),
					OffsetString,
					SmallLayoutFont,
					ESlateDrawEffect::None,
					DrawColor
				);
			}
		}
	}

	return LayerId;
}

void FSequencerEditTool_Movement::UpdateCursorDecorator(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent)
{
	using namespace UE::Sequencer;

	DragPosition = MyGeometry.AbsoluteToLocal(CursorEvent.GetScreenSpacePosition());
	TSharedPtr<FTrackAreaViewModel> TrackAreaViewModel = TrackArea.GetViewModel();
	TSharedPtr<ITrackAreaHotspot> Hotspot = DelayedDrag.IsSet()
		? DelayedDrag->Hotspot
		: TrackAreaViewModel->GetHotspot();

	if (Hotspot.IsValid())
	{
		CursorDecorator = Hotspot->GetCursorDecorator(MyGeometry, CursorEvent);
	}
}

FCursorReply FSequencerEditTool_Movement::OnCursorQuery(const SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	using namespace UE::Sequencer;

	if (DragOperation)
	{
		return DragOperation->GetCursor();
	}

	return FCursorReply::Cursor(EMouseCursor::CardinalCross);
}


FName FSequencerEditTool_Movement::GetIdentifier() const
{
	return Identifier;
}


bool FSequencerEditTool_Movement::CanDeactivate() const
{
	return !DelayedDrag.IsSet();
}


FString FSequencerEditTool_Movement::TimeToString(FFrameTime Time, bool IsDelta) const
{
	USequencerSettings* Settings = Sequencer.GetSequencerSettings();
	check(Settings);

	// We don't use the Sequencer's Numeric Type interface as we want to show a "+" only for delta movement and not the absolute time.
	EFrameNumberDisplayFormats DisplayFormat = Settings->GetTimeDisplayFormat();
	switch (DisplayFormat)
	{
		case EFrameNumberDisplayFormats::Seconds:
		{
			FFrameRate TickResolution = Sequencer.GetFocusedTickResolution();
			double TimeInSeconds = TickResolution.AsSeconds(Time);
			return IsDelta ? FString::Printf(TEXT("[%+.2fs]"), TimeInSeconds) : FString::Printf(TEXT("%.2fs"), TimeInSeconds);
		}
		case EFrameNumberDisplayFormats::Frames:
		{
			FFrameRate TickResolution = Sequencer.GetFocusedTickResolution();
			FFrameRate DisplayRate    = Sequencer.GetFocusedDisplayRate();

			// Convert from sequence resolution into display rate frames.
			FFrameTime DisplayTime = FFrameRate::TransformTime(Time, TickResolution, DisplayRate);
			FString SubframeIndicator = FMath::IsNearlyZero(DisplayTime.GetSubFrame()) ? TEXT("") : TEXT("*");
			int32 ZeroPadFrames = Settings->GetZeroPadFrames();
			return IsDelta ? FString::Printf(TEXT("[%+0*d%s]"), ZeroPadFrames, DisplayTime.GetFrame().Value, *SubframeIndicator) : FString::Printf(TEXT("%0*d%s"), ZeroPadFrames, DisplayTime.GetFrame().Value, *SubframeIndicator);
		}
		case EFrameNumberDisplayFormats::NonDropFrameTimecode:
		{
			FFrameRate SourceFrameRate = Sequencer.GetFocusedTickResolution();
			FFrameRate DestinationFrameRate = Sequencer.GetFocusedDisplayRate();

			FFrameNumber DisplayRateFrameNumber = FFrameRate::TransformTime(Time, SourceFrameRate, DestinationFrameRate).FloorToFrame();

			FTimecode AsNonDropTimecode = FTimecode::FromFrameNumber(DisplayRateFrameNumber, DestinationFrameRate, false);

			const bool bForceSignDisplay = IsDelta;
			return IsDelta ? FString::Printf(TEXT("[%s]"), *AsNonDropTimecode.ToString(bForceSignDisplay)) : FString::Printf(TEXT("%s"), *AsNonDropTimecode.ToString(bForceSignDisplay));
		}
		case EFrameNumberDisplayFormats::DropFrameTimecode:
		{
			FFrameRate SourceFrameRate = Sequencer.GetFocusedTickResolution();
			FFrameRate DestinationFrameRate = Sequencer.GetFocusedDisplayRate();

			FFrameNumber DisplayRateFrameNumber = FFrameRate::TransformTime(Time, SourceFrameRate, DestinationFrameRate).FloorToFrame();

			FTimecode AsDropTimecode = FTimecode::FromFrameNumber(DisplayRateFrameNumber, DestinationFrameRate, true);

			const bool bForceSignDisplay = IsDelta;
			return IsDelta ? FString::Printf(TEXT("[%s]"), *AsDropTimecode.ToString(bForceSignDisplay)) : FString::Printf(TEXT("%s"), *AsDropTimecode.ToString(bForceSignDisplay));
		}
	}

	return FString();
}

TSharedPtr<UE::Sequencer::ITrackAreaHotspot> FSequencerEditTool_Movement::GetDragHotspot() const
{
	return DelayedDrag.IsSet() ? DelayedDrag->Hotspot : nullptr;
}

UE::Sequencer::ISequencerEditToolDragOperation* FSequencerEditTool_Movement::GetCurrentDragOperation() const
{
	return DragOperation.Get();
}
