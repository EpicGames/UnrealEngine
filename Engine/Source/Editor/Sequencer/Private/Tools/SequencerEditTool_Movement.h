// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Tools/SequencerEditTool.h"

class SSequencer;

namespace UE
{
namespace Sequencer
{
	class STrackAreaView;
}
}

class FSequencerEditTool_Movement
	: public FSequencerEditTool
{
public:

	/** Static identifier for this edit tool */
	static const FName Identifier;

	/** Create and initialize a new instance. */
	FSequencerEditTool_Movement(FSequencer& InSequencer, UE::Sequencer::STrackAreaView& InTrackArea);

public:

	using FSequencerEditTool::OnCursorQuery;

	// ISequencerEditTool interface

	virtual FReply OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseCaptureLost() override;
	virtual int32 OnPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const override;
	virtual FCursorReply OnCursorQuery(const SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	virtual FName GetIdentifier() const override;
	virtual bool CanDeactivate() const override;
	virtual TSharedPtr<UE::Sequencer::ITrackAreaHotspot> GetDragHotspot() const override;
	virtual UE::Sequencer::ISequencerEditToolDragOperation* GetCurrentDragOperation() const override;

protected:

	FString TimeToString(FFrameTime Time, bool IsDelta) const;

private:

	TSharedPtr<UE::Sequencer::ISequencerEditToolDragOperation> CreateDrag(const FPointerEvent& MouseEvent);

	bool GetHotspotTime(FFrameTime& HotspotTime) const;
	FFrameTime GetHotspotOffsetTime(FFrameTime CurrentTime) const;
	void UpdateCursorDecorator(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Helper class responsible for handling delayed dragging */
	TOptional<UE::Sequencer::FDelayedDrag_Hotspot> DelayedDrag;

	/** Current drag operation if any */
	TSharedPtr<UE::Sequencer::ISequencerEditToolDragOperation> DragOperation;

	/** Current local position the mouse is dragged to. */
	FVector2f DragPosition;

	/** The hotspot's time before dragging started. */
	FFrameTime OriginalHotspotTime;

	/** TrackArea this object belongs to */
	UE::Sequencer::STrackAreaView& TrackArea;

	/** Software cursor decorator brush */
	const FSlateBrush* CursorDecorator;
};
