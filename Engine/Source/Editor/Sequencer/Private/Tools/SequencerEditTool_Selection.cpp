// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/SequencerEditTool_Selection.h"
#include "Application/ThrottleManager.h"
#include "Styling/AppStyle.h"
#include "SequencerCommonHelpers.h"
#include "SequencerTimeSliderController.h"
#include "SSequencer.h"
#include "ISequencerEditTool.h"
#include "ISequencerSection.h"
#include "SequencerHotspots.h"
#include "IKeyArea.h"
#include "Tools/SequencerEntityVisitor.h"
#include "SequencerCommands.h"
#include "Misc/InputState.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/Views/ISequencerTreeView.h"
#include "MVVM/Views/ITrackAreaHotspot.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/VirtualTrackArea.h"

#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/Views/STrackAreaView.h"
#include "MVVM/Views/SOutlinerView.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/IPinnableExtension.h"
#include "MVVM/Extensions/ISelectableExtension.h"


struct FSelectionPreviewVisitor final
	: ISequencerEntityVisitor
{
	FSelectionPreviewVisitor(FSequencerSelectionPreview& InSelectionPreview, UE::Sequencer::FSequencerSelection& InSelection, ESelectionPreviewState InSetStateTo, bool bInPinned)
		: SelectionPreview(InSelectionPreview)
		, ExistingSelection(InSelection.KeySelection.GetSelected())
		, SetStateTo(InSetStateTo)
		, bPinned(bInPinned)
	{
		ModifierKeys = FSlateApplication::Get().GetModifierKeys();
	}

	virtual void VisitKeys(const UE::Sequencer::TViewModelPtr<UE::Sequencer::FChannelModel>& Channel, const TRange<FFrameNumber>& VisitRangeFrames) const override
	{
		using namespace UE::Sequencer;

		TViewModelPtr<IOutlinerExtension> OutlinerItem = Channel->GetLinkedOutlinerItem();
		TSharedPtr<IPinnableExtension> PinnableItem = OutlinerItem.ImplicitCast();

		const bool bIsItemPinned = (PinnableItem && PinnableItem->IsPinned()) || (OutlinerItem && OutlinerItem->ShouldAnchorToTop());
		if (bIsItemPinned != bPinned)
		{
			return;
		}

		KeyHandlesScratch.Reset();
		Channel->GetKeyArea()->GetKeyInfo(&KeyHandlesScratch, nullptr, VisitRangeFrames);

		for (int32 Index = 0; Index < KeyHandlesScratch.Num(); ++Index)
		{
			// Under default behavior keys have priority, so if a key is changing selection state then we remove any sections from the selection. The user can bypass this
			// by holding down the control key which will allow selecting both keys and sections.
			const bool bPrioritizeLayerBars = ModifierKeys.IsControlDown() && ModifierKeys.IsAltDown();
			const bool bPrioritizeKeys = !bPrioritizeLayerBars && !ModifierKeys.IsControlDown();
			const bool bKeyIsSelected = ExistingSelection.Contains(KeyHandlesScratch[Index]);

			// Clear selected models
			if (((bKeyIsSelected && SetStateTo == ESelectionPreviewState::NotSelected) ||
				(!bKeyIsSelected && SetStateTo == ESelectionPreviewState::Selected)))
			{
				if (bPrioritizeKeys)
				{
					SelectionPreview.EmptyDefinedModelStates();
				}
				else if (bPrioritizeLayerBars)
				{
					SelectionPreview.EmptyDefinedKeyStates();
				}
			}

			SelectionPreview.SetSelectionState(Channel, KeyHandlesScratch[Index], SetStateTo);
		}
	}

	virtual void VisitDataModel(UE::Sequencer::FViewModel* DataModel) const override
	{
		using namespace UE::Sequencer;

		ISelectableExtension* Selectable = DataModel->CastThis<ISelectableExtension>();
		if (!Selectable || !EnumHasAnyFlags(Selectable->IsSelectable(), ESelectionIntent::PersistentSelection))
		{
			return;
		}

		// If key selection has priority then we check to see if there are any keys selected. If there are key selected, we don't add this section.
		// Otherwise, we bypass this check
		const bool bPrioritizeLayerBars = ModifierKeys.IsControlDown() && ModifierKeys.IsAltDown();
		const bool bPrioritizeKeys = !bPrioritizeLayerBars && !ModifierKeys.IsControlDown();
		const bool bKeyStateCheck = bPrioritizeKeys ? SelectionPreview.GetDefinedKeyStates().Num() == 0 : true;

		if (bKeyStateCheck)
		{
			SelectionPreview.SetSelectionState(DataModel->AsShared(), SetStateTo);
		}
	}

private:

	FSequencerSelectionPreview& SelectionPreview;
	const TSet<FKeyHandle>& ExistingSelection;
	mutable TArray<FKeyHandle> KeyHandlesScratch;
	ESelectionPreviewState SetStateTo;
	bool bPinned;
	FModifierKeysState ModifierKeys;
};

class FScrubTimeDragOperation
	: public UE::Sequencer::ISequencerEditToolDragOperation
{
public:

	FScrubTimeDragOperation(FSequencer& InSequencer, UE::Sequencer::STrackAreaView& InTrackArea)
		: Sequencer(InSequencer), NextPlayerStatus(EMovieScenePlayerStatus::Stopped)
	{}

public:

	// ISequencerEditToolDragOperation interface

	virtual FCursorReply GetCursor() const override
	{
		return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
	}

	virtual void OnBeginDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override
	{
		FSlateThrottleManager::Get().DisableThrottle(true);

		// Start a new marquee selection
		InitialPosition = CurrentPosition = VirtualTrackArea.PhysicalToVirtual(LocalMousePos);
		CurrentMousePos = LocalMousePos;
		NextPlayerStatus = (Sequencer.GetPlaybackStatus() == EMovieScenePlayerStatus::Playing) ? EMovieScenePlayerStatus::Playing :
			 EMovieScenePlayerStatus::Stopped;
		
		Sequencer.SetPlaybackStatus(EMovieScenePlayerStatus::Scrubbing);
		SequencerStartTime = Sequencer.GetLocalTime();
		InitialTime = VirtualTrackArea.PixelToSeconds(LocalMousePos.X);
	}

	virtual void OnDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override
	{
		CurrentTime = VirtualTrackArea.PixelToSeconds(LocalMousePos.X);
		FFrameTime FrameTime = CalculateScrubTime();

		Sequencer.SnapSequencerTime(FrameTime);
		Sequencer.SetLocalTimeDirectly(FrameTime, true);
	}

	virtual void OnEndDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override
	{
		FSlateThrottleManager::Get().DisableThrottle(false);
		if (NextPlayerStatus == EMovieScenePlayerStatus::Stopped)
		{
			//Call Pause() instead of SetPlaybackStatus(EMovieScenePlayerStatus::Stopped)
			//so the correct delegates/evals can happen so all systems like audio are probably stopped (Pause set's Stopped status).
			Sequencer.Pause();
		}
		else
		{
			Sequencer.SetPlaybackStatus(NextPlayerStatus);
		}
	}

	virtual int32 OnPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const override
	{
		return LayerId;
	}

private:

	FFrameTime CalculateScrubTime()
	{
		double Diff = CurrentTime - InitialTime;
		FFrameTime FrameTime= SequencerStartTime.Rate.AsFrameTime(Diff);
		FrameTime += SequencerStartTime.Time;
		return FrameTime;
	}

	/** The sequencer itself */
	FSequencer& Sequencer;

	FVector2D InitialPosition;
	FVector2D CurrentPosition;
	FVector2D CurrentMousePos;

	FQualifiedFrameTime SequencerStartTime;
	double InitialTime;
	double CurrentTime;

	EMovieScenePlayerStatus::Type NextPlayerStatus; //reset on end drag, make sure to keep playing if it was
};

class FMarqueeDragOperation
	: public UE::Sequencer::ISequencerEditToolDragOperation
{
public:

	FMarqueeDragOperation(FSequencer& InSequencer, UE::Sequencer::STrackAreaView& InTrackArea)
		: Sequencer(InSequencer)
		, TrackArea(InTrackArea)
		, SequencerWidget(InSequencer.GetUnderlyingSequencerWidget())
		, PreviewState(ESelectionPreviewState::Selected)
	{}

public:

	// ISequencerEditToolDragOperation interface

	virtual FCursorReply GetCursor() const override
	{
		return FCursorReply::Cursor(EMouseCursor::Crosshairs);
	}

	virtual void OnBeginDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override
	{
		// Start a new marquee selection
		InitialPosition = CurrentPosition = VirtualTrackArea.PhysicalToVirtual(LocalMousePos);
		CurrentMousePos = LocalMousePos;

		EventSuppressor = Sequencer.GetViewModel()->GetSelection()->SuppressEventsLongRunning();

		if (MouseEvent.IsShiftDown())
		{
			PreviewState = ESelectionPreviewState::Selected;
		}
		else if (!MouseEvent.IsControlDown() && MouseEvent.IsAltDown())
		{
			PreviewState = ESelectionPreviewState::NotSelected;
		}
		else
		{
			PreviewState = ESelectionPreviewState::Selected;

			// @todo: selection in transactions
			//leave selections in the tree view alone so that dragging operations act similarly to click operations which don't change treeview selection state.
			Sequencer.GetViewModel()->GetSelection()->KeySelection.Empty();
			Sequencer.GetViewModel()->GetSelection()->TrackArea.Empty();
		}
	}

	virtual void OnDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override
	{
		using namespace UE::Sequencer;

		// hange the current marquee selection
		const FVector2D MouseDelta = MouseEvent.GetCursorDelta();

		// Handle virtual scrolling when at the vertical extremes of the widget (performed before we clamp the mouse pos)
		{
			const float ScrollThresholdV = VirtualTrackArea.GetPhysicalSize().Y * 0.025f;

			TSharedPtr<SOutlinerView> TreeView = TrackArea.GetOutliner().Pin();

			float Difference = LocalMousePos.Y - ScrollThresholdV;
			if (Difference < 0 && MouseDelta.Y < 0)
			{
				TreeView->ScrollByDelta( Difference * 0.1f );
			}

			Difference = LocalMousePos.Y - (VirtualTrackArea.GetPhysicalSize().Y - ScrollThresholdV);
			if (Difference > 0 && MouseDelta.Y > 0)
			{
				TreeView->ScrollByDelta( Difference * 0.1f) ;
			}
		}

		// Clamp the vertical position to the actual bounds of the track area
		LocalMousePos.Y = FMath::Clamp(LocalMousePos.Y, 0.f, VirtualTrackArea.GetPhysicalSize().Y);
		CurrentPosition = VirtualTrackArea.PhysicalToVirtual(LocalMousePos);

		// Clamp software cursor position to bounds of the track area
		CurrentMousePos = LocalMousePos;
		CurrentMousePos.X = FMath::Clamp(CurrentMousePos.X, 0.f, VirtualTrackArea.GetPhysicalSize().X);

		TRange<double> ViewRange = Sequencer.GetViewRange();

		// Handle virtual scrolling when at the horizontal extremes of the widget
		{
			const double ScrollThresholdH = ViewRange.Size<double>() * 0.025f;

			double Difference = CurrentPosition.X - (ViewRange.GetLowerBoundValue() + ScrollThresholdH);
			if (Difference < 0 && MouseDelta.X < 0)
			{
				Sequencer.StartAutoscroll(Difference);
			}
			else
			{
				Difference = CurrentPosition.X - (ViewRange.GetUpperBoundValue() - ScrollThresholdH);
				if (Difference > 0 && MouseDelta.X > 0)
				{
					Sequencer.StartAutoscroll(Difference);
				}
				else
				{
					Sequencer.StopAutoscroll();
				}
			}
		}

		// Calculate the size of a key in virtual space
		FVector2D VirtualKeySize;
		VirtualKeySize.X = SequencerSectionConstants::KeySize.X / VirtualTrackArea.GetPhysicalSize().X * ViewRange.Size<float>();
		// Vertically, virtual units == physical units
		VirtualKeySize.Y = SequencerSectionConstants::KeySize.Y;

		// Visit everything using the preview selection primarily as well as the 
		FSequencerSelectionPreview& SelectionPreview = Sequencer.GetSelectionPreview();

		// Ensure the preview is empty before calculating the intersection
		SelectionPreview.Empty();

		// Now walk everything within the current marquee range, setting preview selection states as we go
		FSequencerEntityWalker Walker(FSequencerEntityRange(TopLeft(), BottomRight(), VirtualTrackArea.GetTickResolution()), VirtualKeySize);
		Walker.Traverse(FSelectionPreviewVisitor(SelectionPreview, *Sequencer.GetViewModel()->GetSelection(), PreviewState, TrackArea.ShowPinned()), Sequencer.GetViewModel()->GetRootModel());
	}

	virtual void OnEndDrag(const FPointerEvent& MouseEvent, FVector2D LocalMousePos, const UE::Sequencer::FVirtualTrackArea& VirtualTrackArea) override
	{
		using namespace UE::Sequencer;

		// finish dragging the marquee selection
		FSequencerSelection& Selection = *Sequencer.GetViewModel()->GetSelection();
		FSequencerSelectionPreview& SelectionPreview = Sequencer.GetSelectionPreview();

		// Patch everything from the selection preview into the actual selection
		for (const TPair<FKeyHandle, ESelectionPreviewState>& Pair : SelectionPreview.GetDefinedKeyStates())
		{
			if (Pair.Value == ESelectionPreviewState::Selected)
			{
				Selection.KeySelection.Select(SelectionPreview.GetChannelForKey(Pair.Key), Pair.Key);
			}
			else
			{
				Selection.KeySelection.Deselect(Pair.Key);
			}
		}

		for (const TPair<TWeakPtr<FViewModel>, ESelectionPreviewState>& Pair : SelectionPreview.GetDefinedModelStates())
		{
			if (TSharedPtr<FViewModel> Model = Pair.Key.Pin())
			{
				if (TViewModelPtr<IOutlinerExtension> OutlinerItem = CastViewModel<IOutlinerExtension>(Model))
				{
					Selection.Outliner.Select(OutlinerItem);
				}
				else
				{
					Selection.TrackArea.Select(Model);
				}
			}
		}

		// Broadcast selection events
		EventSuppressor = nullptr;

		// We're done with this now
		SelectionPreview.Empty();
	}

	virtual int32 OnPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const override
	{
		using namespace UE::Sequencer;

		// Both the pinned and non-pinned track areas share the same FTrackAreaViewModel and
		// thus the same edit tool, so OnPaint is invoked for both areas. Only draw the
		// marquee in the owner track area.
		//
		// SWidget::Paint sets GetPaintSpaceGeometry() (PersistentState.AllottedGeometry)
		// before calling OnPaint, so when the owner area is currently executing its OnPaint,
		// TrackArea.GetPaintSpaceGeometry() == AllottedGeometry. For the non-owner area,
		// AllottedGeometry is that area's own geometry, which differs from TrackArea's.
		const FVector2D PaintOrigin = AllottedGeometry.GetAccumulatedLayoutTransform().GetTranslation();
		const FVector2D OwnerOrigin = TrackArea.GetPaintSpaceGeometry().GetAccumulatedLayoutTransform().GetTranslation();
		if (!PaintOrigin.Equals(OwnerOrigin))
		{
			return LayerId;
		}

		// convert to physical space for rendering
		const FVirtualTrackArea VirtualTrackArea = SequencerWidget->GetVirtualTrackArea(&TrackArea);

		FVector2D SelectionTopLeft = VirtualTrackArea.VirtualToPhysical(TopLeft());
		FVector2D SelectionBottomRight = VirtualTrackArea.VirtualToPhysical(BottomRight());

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(SelectionBottomRight - SelectionTopLeft, FSlateLayoutTransform(SelectionTopLeft)),
			FAppStyle::GetBrush(TEXT("MarqueeSelection"))
			);

		return LayerId + 1;
	}

private:

	FVector2D TopLeft() const
	{
		return FVector2D(
			FMath::Min(InitialPosition.X, CurrentPosition.X),
			FMath::Min(InitialPosition.Y, CurrentPosition.Y)
		);
	}
	
	FVector2D BottomRight() const
	{
		return FVector2D(
			FMath::Max(InitialPosition.X, CurrentPosition.X),
			FMath::Max(InitialPosition.Y, CurrentPosition.Y)
		);
	}

	/** The sequencer itself */
	FSequencer& Sequencer;

	UE::Sequencer::STrackAreaView& TrackArea;

	TUniquePtr<UE::Sequencer::FSelectionEventSuppressor> EventSuppressor;

	/** Sequencer widget */
	TSharedRef<SSequencer> SequencerWidget;

	/** Whether we should select/deselect things in this marquee operation */
	ESelectionPreviewState PreviewState;

	FVector2D InitialPosition;
	FVector2D CurrentPosition;
	FVector2D CurrentMousePos;
};


const FName FSequencerEditTool_Selection::Identifier = "Selection";

FSequencerEditTool_Selection::FSequencerEditTool_Selection(FSequencer& InSequencer, UE::Sequencer::STrackAreaView& InTrackArea)
	: FSequencerEditTool(InSequencer)
	, TrackArea(InTrackArea)
	, CursorDecorator(nullptr)
{ }


FCursorReply FSequencerEditTool_Selection::OnCursorQuery(const SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	if (UE::CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport))
	{
		return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
	}

	if (DragOperation)
	{
		return DragOperation->GetCursor();
	}

	return FCursorReply::Cursor(EMouseCursor::Default);
}


int32 FSequencerEditTool_Selection::OnPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	if (DragOperation.IsValid())
	{
		LayerId = DragOperation->OnPaint(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	}

	if (CursorDecorator)
	{
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(CursorDecorator->ImageSize, FSlateLayoutTransform(MousePosition + FVector2D(5.f, 5.f))),
			CursorDecorator
			);
	}

	return LayerId;
}

static FFrameTime ComputeFrameTimeFromMouse(const FSequencer& Sequencer, const FGeometry& Geometry, const FVector2D& ScreenSpacePosition, const FSequencerTimeSliderController::FScrubRangeToScreen& RangeToScreen, bool  bCheckSnapping)
{
	FVector2D CursorPos = Geometry.AbsoluteToLocal(ScreenSpacePosition);
	double    MouseValue = RangeToScreen.LocalXToInput(CursorPos.X);

	if (bCheckSnapping && Sequencer.GetSequencerSettings()->GetForceWholeFrames())
	{
		FFrameNumber        SnappedFrameNumber = (MouseValue * Sequencer.GetFocusedDisplayRate()).FloorToFrame();
		FQualifiedFrameTime RoundedPlayFrame = FQualifiedFrameTime(SnappedFrameNumber, Sequencer.GetFocusedDisplayRate());
		return RoundedPlayFrame.ConvertTo(Sequencer.GetFocusedTickResolution());
	}
	else
	{
		return MouseValue * Sequencer.GetFocusedTickResolution();
	}
}

FReply FSequencerEditTool_Selection::OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	using namespace UE::Sequencer;

	++ThrottleSlateCounter;
	FSlateThrottleManager::Get().DisableThrottle(true);

	UpdateCursor(MyGeometry, MouseEvent);
	bIsScrubbingTime = UE::CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport);

	DelayedDrag.Reset();

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (bIsScrubbingTime && Sequencer.GetSequencerSettings()->GetScrubTimeStartFromCursor())
		{

			FSequencerTimeSliderController::FScrubRangeToScreen RangeToScreen = FSequencerTimeSliderController::FScrubRangeToScreen(Sequencer.GetViewRange(), MyGeometry.Size);
			FFrameTime FrameTime = ComputeFrameTimeFromMouse(Sequencer, MyGeometry, MouseEvent.GetScreenSpacePosition(), RangeToScreen, true);

			if (Sequencer.GetPlaybackStatus() == EMovieScenePlayerStatus::Playing)
			{
				Sequencer.SetPlaybackStatus(EMovieScenePlayerStatus::Paused);
				Sequencer.SetLocalTimeLooped(FrameTime);
				Sequencer.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);
			}
			else
			{
				Sequencer.SetLocalTimeLooped(FrameTime);
			}
		}
		TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = Sequencer.GetViewModel()->CastThisShared<FSequencerEditorViewModel>();
		TSharedPtr<ITrackAreaHotspot> Hotspot = SequencerViewModel->GetHotspot();
		DelayedDrag = FDelayedDrag_Hotspot(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()), EKeys::LeftMouseButton, Hotspot);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}


FReply FSequencerEditTool_Selection::OnMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	using namespace UE::Sequencer;

	UpdateCursor(MyGeometry, MouseEvent);

	if (DelayedDrag.IsSet())
	{
		FReply Reply = FReply::Handled();

		TSharedRef<SSequencer> SequencerWidget = Sequencer.GetUnderlyingSequencerWidget();
		// Use the track area that actually received this event so the drag operates in the
		// correct coordinate space (pinned or non-pinned) and selects the right set of keys.
		STrackAreaView& EventTrackArea = static_cast<STrackAreaView&>(OwnerWidget);
		const FVirtualTrackArea VirtualTrackArea = SequencerWidget->GetVirtualTrackArea(&EventTrackArea);

		if (DragOperation.IsValid())
		{
			FVector2D LocalPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
			DragOperation->OnDrag(MouseEvent, LocalPosition, VirtualTrackArea);
		}
		else if (DelayedDrag->AttemptDragStart(MouseEvent))
		{
			if (DelayedDrag->Hotspot.IsValid())
			{
				// We only allow resizing with the marquee selection tool enabled
				if (DelayedDrag->Hotspot->CastThis<FSectionHotspot>() == nullptr && DelayedDrag->Hotspot->CastThis<FKeyHotspot>() == nullptr)
				{
					DragOperation = DelayedDrag->Hotspot->InitiateDrag(MouseEvent);
				}
			}

			if (!DragOperation.IsValid())
			{
				if (bIsScrubbingTime)
				{
					DragOperation = MakeShareable(new FScrubTimeDragOperation(Sequencer, EventTrackArea));
				}
				else
				{
					DragOperation = MakeShareable(new FMarqueeDragOperation(Sequencer, EventTrackArea));
				}
			}

			if (DragOperation.IsValid())
			{
				DragOperation->OnBeginDrag(MouseEvent, DelayedDrag->GetInitialPosition(), VirtualTrackArea);

				// Steal the capture, as we're now the authoritative widget in charge of a mouse-drag operation
				Reply.CaptureMouse(OwnerWidget.AsShared());
			}
		}

		return Reply;
	}
	return FReply::Unhandled();
}

FReply FSequencerEditTool_Selection::OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	using namespace UE::Sequencer;

	if (ThrottleSlateCounter > 0)
	{
		--ThrottleSlateCounter;
		FSlateThrottleManager::Get().DisableThrottle(false);
	}

	UpdateCursor(MyGeometry, MouseEvent);

	DelayedDrag.Reset();
	if (DragOperation.IsValid())
	{
		const TSharedRef<SSequencer> SequencerWidget = Sequencer.GetUnderlyingSequencerWidget();
		FVector2D LocalPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		STrackAreaView& EventTrackArea = static_cast<STrackAreaView&>(OwnerWidget);
		DragOperation->OnEndDrag(MouseEvent, LocalPosition, SequencerWidget->GetVirtualTrackArea(&EventTrackArea));
		DragOperation = nullptr;

		CursorDecorator = nullptr;

		Sequencer.StopAutoscroll();
		return FReply::Handled().ReleaseMouseCapture();
	}
	else
	{
		SequencerHelpers::PerformDefaultSelection(Sequencer, MouseEvent);

		if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && !Sequencer.IsReadOnly())
		{
			TSharedPtr<SWidget> MenuContent = SequencerHelpers::SummonContextMenu( Sequencer, MyGeometry, MouseEvent );
			if (MenuContent.IsValid())
			{
				FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

				FSlateApplication::Get().PushMenu(
					OwnerWidget.AsShared(),
					WidgetPath,
					MenuContent.ToSharedRef(),
					MouseEvent.GetScreenSpacePosition(),
					FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
					);

				return FReply::Handled().SetUserFocus(MenuContent.ToSharedRef(), EFocusCause::SetDirectly).ReleaseMouseCapture();
			}
		}

		return FReply::Handled();
	}
}


void FSequencerEditTool_Selection::OnMouseLeave(SWidget& OwnerWidget, const FPointerEvent& MouseEvent)
{
	if (!DragOperation.IsValid())
	{
		CursorDecorator = nullptr;
	}
	bIsScrubbingTime = false;
}


void FSequencerEditTool_Selection::OnMouseCaptureLost()
{
	// Delaying nulling out until next tick because this could be invoked during OnMouseMove()
	GEditor->GetTimerManager()->SetTimerForNextTick([this]()
	{
		DelayedDrag.Reset();
		DragOperation = nullptr;
		CursorDecorator = nullptr;
		bIsScrubbingTime = false;
	});
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FReply FSequencerEditTool_Selection::OnKeyDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	const bool bIsScrubCommand = UE::CurveEditor::InputState::IsCommandPressed(FSequencerCommands::Get().ScrubTimeViewport, InKeyEvent);
	if (bIsScrubCommand)
	{
		UE::CurveEditor::InputState::SetScrubTimeKeyPressed(InKeyEvent.GetKey(), true);
		bIsScrubbingTime = true;
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply FSequencerEditTool_Selection::OnKeyUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	const bool bIsScrubCommand = UE::CurveEditor::InputState::IsCommandPressed(FSequencerCommands::Get().ScrubTimeViewport, InKeyEvent);
	if (bIsScrubCommand && bIsScrubbingTime)
	{
		UE::CurveEditor::InputState::SetScrubTimeKeyPressed(InKeyEvent.GetKey(), false);
		bIsScrubbingTime = false;
		//would be nice to cancle the drag but doesn't seem like we can like we can with curve editor drag handlers.
		return FReply::Handled();
	}
	return FReply::Unhandled();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FName FSequencerEditTool_Selection::GetIdentifier() const
{
	return Identifier;
}

bool FSequencerEditTool_Selection::CanDeactivate() const
{
	const bool bIsScrubCommand = UE::CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport);
	return !bIsScrubCommand && !DelayedDrag.IsSet();
}

TSharedPtr<UE::Sequencer::ITrackAreaHotspot> FSequencerEditTool_Selection::GetDragHotspot() const
{
	return DelayedDrag.IsSet() ? DelayedDrag->Hotspot : nullptr;
}

void FSequencerEditTool_Selection::UpdateCursor(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	MousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	// Don't update the brush if we have a drag operation
	if (!DragOperation.IsValid())
	{
		bIsScrubbingTime = UE::CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport);

		if (MouseEvent.IsShiftDown())
		{
			CursorDecorator = FAppStyle::Get().GetBrush(TEXT("Sequencer.CursorDecorator_MarqueeAdd"));
		}
		else if (MouseEvent.IsAltDown())
		{
			CursorDecorator = FAppStyle::Get().GetBrush(TEXT("Sequencer.CursorDecorator_MarqueeSubtract"));
		}
		else
		{
			CursorDecorator = nullptr;
		}
	}
}
