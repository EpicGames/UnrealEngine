// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Tools/SequencerEditTool.h"

class SSequencer;

namespace UE
{
namespace Sequencer
{
	class STrackAreaView;
}
}

class FSequencerEditTool_Selection
	: public FSequencerEditTool
{
public:

	/** Static identifier for this edit tool */
	static const FName Identifier;

	/** Create and initialize a new instance. */
	FSequencerEditTool_Selection(FSequencer& InSequencer, UE::Sequencer::STrackAreaView& InTrackArea);

public:

	using FSequencerEditTool::OnCursorQuery;

	// ISequencerEditTool interface	
	virtual int32 OnPaint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const override;
	virtual FReply OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(SWidget& OwnerWidget, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseCaptureLost() override;
	virtual FCursorReply OnCursorQuery(const SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	virtual FName GetIdentifier() const override;
	virtual bool CanDeactivate() const override;
	virtual TSharedPtr<UE::Sequencer::ITrackAreaHotspot> GetDragHotspot() const override;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual FReply OnKeyDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

private:

	/** Update the software cursor */
	void UpdateCursor(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Helper class responsible for handling delayed dragging */
	TOptional<UE::Sequencer::FDelayedDrag_Hotspot> DelayedDrag;
	
	/** Current drag operation if any */
	TSharedPtr<UE::Sequencer::ISequencerEditToolDragOperation> DragOperation;

	/** Cached mouse position for software cursor rendering */
	FVector2D MousePosition;

	/** TrackArea this object belongs to */
	UE::Sequencer::STrackAreaView& TrackArea;

	/** Software cursor decorator brush */
	const FSlateBrush* CursorDecorator;

	/** Whether or not we are scrubbing time*/
	bool bIsScrubbingTime = false;

	/** Throttle slate counter on mouse down*/
	int32 ThrottleSlateCounter = 0;
};
