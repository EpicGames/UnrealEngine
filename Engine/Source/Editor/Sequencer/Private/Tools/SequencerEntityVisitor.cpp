// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/SequencerEntityVisitor.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "IKeyArea.h"
#include "MVVM/Extensions/IGeometryExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Extensions/ITrackLaneExtension.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/FolderModel.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Misc/FrameTime.h"
#include "MovieSceneSection.h"
#include "SequencerCoreFwd.h"
#include "MVVM/ViewModels/LayerBarModel.h"

FSequencerEntityRange::FSequencerEntityRange(const TRange<double>& InRange, FFrameRate InTickResolution)
	: TickResolution(InTickResolution)
	, Range(InRange)
{
}

FSequencerEntityRange::FSequencerEntityRange(FVector2D TopLeft, FVector2D BottomRight, FFrameRate InTickResolution)
	: TickResolution(InTickResolution)
	, Range(TopLeft.X, BottomRight.X)
	, VerticalTop(TopLeft.Y), VerticalBottom(BottomRight.Y)
{
}

bool FSequencerEntityRange::IntersectSection(const UMovieSceneSection* InSection) const
{
	// Test horizontal bounds
	return (InSection->GetRange() / TickResolution).Overlaps(Range);
}

bool FSequencerEntityRange::IntersectKeyArea(TSharedPtr<UE::Sequencer::FViewModel> InNode, float VirtualKeyHeight) const
{
	using namespace UE::Sequencer;

	IGeometryExtension* GeometryExtension = InNode->CastThis<IGeometryExtension>();
	if (VerticalTop.IsSet() && GeometryExtension)
	{
		const FVirtualGeometry VirtualGeometry = GeometryExtension->GetVirtualGeometry();
		const float NodeCenter = VirtualGeometry.GetTop() + VirtualGeometry.GetHeight()/2;
		return NodeCenter + VirtualKeyHeight/2 > VerticalTop.GetValue() && NodeCenter - VirtualKeyHeight/2 < VerticalBottom.GetValue();
	}
	return true;
}

bool FSequencerEntityRange::IntersectVertical(float Top, float Bottom) const
{
	if (VerticalTop.IsSet())
	{
		return Top <= VerticalBottom.GetValue() && Bottom >= VerticalTop.GetValue();
	}
	return true;
}

FSequencerEntityWalker::FSequencerEntityWalker(const FSequencerEntityRange& InRange, FVector2D InVirtualKeySize)
	: Range(InRange), VirtualKeySize(InVirtualKeySize)
{}

void FSequencerEntityWalker::Traverse(const ISequencerEntityVisitor& Visitor, TSharedPtr<UE::Sequencer::FViewModel> Item)
{
	using namespace UE::Sequencer;

	IOutlinerExtension* OutlinerItem = Item->CastThis<IOutlinerExtension>();
	if (!OutlinerItem || !OutlinerItem->IsFilteredOut())
	{
		ConditionallyIntersectModel(Visitor, Item);
	}
}

void FSequencerEntityWalker::ConditionallyIntersectModel(const ISequencerEntityVisitor& Visitor, const TSharedPtr<UE::Sequencer::FViewModel>& DataModel)
{
	using namespace UE::Sequencer;

	const ITrackAreaExtension* TrackArea         = DataModel->CastThis<ITrackAreaExtension>();
	const IGeometryExtension*  GeometryExtension = DataModel->CastThis<IGeometryExtension>();
	if (GeometryExtension && TrackArea)
	{
		const FTrackAreaParameters TrackAreaParameters = TrackArea->GetTrackAreaParameters();
		const FVirtualGeometry     VirtualGeometry     = GeometryExtension->GetVirtualGeometry();

		const float Height = TrackAreaParameters.LaneType == ETrackAreaLaneType::Nested
			? VirtualGeometry.NestedBottom - VirtualGeometry.Top
			: VirtualGeometry.Height;

		auto VisitIfOverlaps = [this, &Visitor, &VirtualGeometry, Height]
			(const TViewModelPtr<ITrackLaneExtension>& InTrackLaneExtension)
		{
			const FTrackLaneVirtualAlignment Alignment = InTrackLaneExtension->ArrangeVirtualTrackLaneView();
			if (!Alignment.IsVisible())
			{
				return;
			}

			const FTrackLaneVerticalArrangement VerticalArrange = Alignment.VerticalAlignment.ArrangeWithin(Height);

			if (Range.IntersectVertical(
				VirtualGeometry.Top + VerticalArrange.Offset,
				VirtualGeometry.Top + VerticalArrange.Offset + VerticalArrange.Height))
			{
				if (Range.Range.Overlaps(Alignment.Range / Range.TickResolution))
				{
					Visitor.VisitDataModel(InTrackLaneExtension.AsModel().Get());

					if (TViewModelPtr<FChannelModel> ChannelModel = InTrackLaneExtension.ImplicitCast())
					{
						VisitChannel(Visitor, ChannelModel);
					}
				}
			}
		};

		if (Range.IntersectVertical(VirtualGeometry.Top, VirtualGeometry.Top + Height))
		{
			for (const FViewModelPtr& TopLevelChild : TrackArea->GetTopLevelChildTrackAreaModels())
			{
				if (const TViewModelPtr<FLayerBarModel> LayerBarModel = TopLevelChild.ImplicitCast())
				{
					VisitIfOverlaps(LayerBarModel);
				}
			}

			for (const TViewModelPtr<ITrackLaneExtension>& TrackLane
				: DataModel->GetChildrenOfType<ITrackLaneExtension>(EViewModelListType::TrackArea))
			{
				VisitIfOverlaps(TrackLane);
			}

			if (Range.IntersectKeyArea(DataModel, VirtualKeySize.Y))
			{
				const IOutlinerExtension* OutlinerItem = DataModel->CastThis<IOutlinerExtension>();
				const bool bIsExpanded = !OutlinerItem || OutlinerItem->IsExpanded();
				VisitAnyChannels(Visitor, DataModel.ToSharedRef(), !bIsExpanded);
			}
		}
	}

	// Iterate into expanded nodes
	const IOutlinerExtension* OutlinerExtension = DataModel->CastThis<IOutlinerExtension>();
	if (OutlinerExtension == nullptr || OutlinerExtension->IsExpanded())
	{
		for (TSharedPtr<UE::Sequencer::FViewModel> Child : DataModel->GetChildren(EViewModelListType::Outliner))
		{
			// Do not visit nodes that are currently filtered out
			const IOutlinerExtension* OutlinerItem = DataModel->CastThis<IOutlinerExtension>();
			if (!OutlinerItem || !OutlinerItem->IsFilteredOut())
			{
				ConditionallyIntersectModel(Visitor, Child);
			}
		}
	}
}

void FSequencerEntityWalker::VisitAnyChannels(const ISequencerEntityVisitor& Visitor, const TSharedRef<UE::Sequencer::FViewModel>& InNode, bool bAnyParentCollapsed )
{
	using namespace UE::Sequencer;

	if (!Visitor.CheckEntityMask(ESequencerEntity::Key))
	{
		return;
	}

	// If this node has or is a key area, visit all the keys on the track
	FChannelGroupModel* ChannelGroup = nullptr;
	if (FChannelGroupModel* ChannelGroupNode = InNode->CastThis<FChannelGroupModel>())
	{
		ChannelGroup = ChannelGroupNode;
	}
	if (ITopLevelChannelHolderExtension* TopLevelChannelHolder = InNode->CastThis<ITopLevelChannelHolderExtension>())
	{
		// TODO: Top-level channel group -- there used to be only one at most but this assumption isn't necessarily true anymore
		if (auto It = TopLevelChannelHolder->GetTopLevelChannels().IterateSubList<FChannelGroupModel>(); It)
		{
			const TViewModelPtr<FChannelGroupModel>& TopLevelChannel = *It;
			ChannelGroup = TopLevelChannel.Get();
		}
	}

	if (ChannelGroup)
	{
		const IOutlinerExtension* OutlinerItem = ChannelGroup->CastThis<IOutlinerExtension>();
		if (!OutlinerItem || OutlinerItem->IsFilteredOut() == false)
		{
			for (const TWeakViewModelPtr<FChannelModel>& WeakChannel : ChannelGroup->GetChannels())
			{
				if (TViewModelPtr<FChannelModel> Channel = WeakChannel.Pin())
				{
					VisitChannel(Visitor, Channel);
				}
			}
		}
	}
	// Otherwise it might be a collapsed node that contains key areas as children. If so we visit them as if they were a part of this track so that key groupings are visited properly.
	else if (bAnyParentCollapsed)
	{
		const IOutlinerExtension* OutlinerItem = InNode->CastThis<IOutlinerExtension>();
		const bool bIsExpanded = !OutlinerItem || OutlinerItem->IsExpanded();

		if ((InNode->IsA<FFolderModel>() || InNode->IsA<FObjectBindingModel>()) && !bIsExpanded)
		{
			return;
		}

		for (TSharedPtr<FViewModel> ChildNode : InNode->GetChildren(EViewModelListType::Outliner))
		{
			VisitAnyChannels(Visitor, ChildNode.ToSharedRef(), bAnyParentCollapsed || !bIsExpanded);
		}
	}

}

void FSequencerEntityWalker::VisitChannel(const ISequencerEntityVisitor& Visitor, const UE::Sequencer::TViewModelPtr<UE::Sequencer::FChannelModel>& Channel)
{
	const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
	const bool bLayerBarsOnly = ModifierKeys.IsControlDown() && ModifierKeys.IsAltDown();
	if (bLayerBarsOnly)
	{
		return;
	}

	const UMovieSceneSection* const Section = Channel.IsValid() ? Channel->GetSection() : nullptr;
	if (!Section)
	{
		return;
	}

	const FFrameTime HalfKeySizeFrames = (VirtualKeySize.X*.5f) * Range.TickResolution;
	const FFrameTime RangeStartFrame   = Range.Range.GetLowerBoundValue() * Range.TickResolution;
	const FFrameTime RangeEndFrame     = Range.Range.GetUpperBoundValue() * Range.TickResolution;

	TRange<FFrameNumber> VisitRangeFrames( (RangeStartFrame-HalfKeySizeFrames).CeilToFrame(), (RangeEndFrame+HalfKeySizeFrames).FloorToFrame() );
	VisitRangeFrames = TRange<FFrameNumber>::Intersection(Section->GetRange(), VisitRangeFrames);
	Visitor.VisitKeys(Channel, VisitRangeFrames);
}
