// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Curves/KeyHandle.h"
#include "HAL/PlatformCrt.h"
#include "MVVM/ViewModelPtr.h"
#include "Math/Range.h"
#include "Math/Vector2D.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"

class UMovieSceneSection;
namespace UE::Sequencer { class FViewModel; }

namespace UE
{
namespace Sequencer
{
	class FChannelModel;
}
}

/** Enum of different types of entities that are available in the sequencer */
namespace ESequencerEntity
{
	enum Type
	{
		Key			= 1<<0,
		Section		= 1<<1,
	};

	static const uint32 Everything = (uint32)-1;
}

/** Visitor class used to handle specific sequencer entities */
struct ISequencerEntityVisitor
{
	ISequencerEntityVisitor(uint32 InEntityMask = ESequencerEntity::Everything) : EntityMask(InEntityMask) {}

	virtual void VisitKeys(const UE::Sequencer::TViewModelPtr<UE::Sequencer::FChannelModel>& Channel, const TRange<FFrameNumber>& VisitRangeFrames) const { }

	virtual void VisitDataModel(UE::Sequencer::FViewModel* Item) const { }

	/** Check if the specified type of entity is applicable to this visitor */
	bool CheckEntityMask(ESequencerEntity::Type Type) const { return (EntityMask & Type) != 0; }

protected:
	virtual ~ISequencerEntityVisitor() = default;

	/** Bitmask of allowable entities */
	uint32 EntityMask;
};

/** A range specifying time (and possibly vertical) bounds in the sequencer */
struct FSequencerEntityRange
{
	FSequencerEntityRange(const TRange<double>& InRange, FFrameRate InTickResolution);
	FSequencerEntityRange(FVector2D TopLeft, FVector2D BottomRight, FFrameRate InTickResolution);

	/** Check whether the specified section intersects the horizontal range */
	bool IntersectSection(const UMovieSceneSection* InSection) const;

	/** Check whether the specified node's key area intersects this range */
	bool IntersectKeyArea(TSharedPtr<UE::Sequencer::FViewModel> InNode, float VirtualKeyHeight) const;

	/** Check whether the specified vertical range intersects this range */
	bool IntersectVertical(float Top, float Bottom) const;

	/** tick resolution of the current time-base */
	FFrameRate TickResolution;

	/** Start/end times */
	TRange<double> Range;

	/** Optional vertical bounds */
	TOptional<float> VerticalTop, VerticalBottom;
};

/** Struct used to iterate a two dimensional *visible* range with a user-supplied visitor */
struct FSequencerEntityWalker
{
	/** Construction from the range itself, and an optional virtual key size, where key bounds must be taken into consideration */
	FSequencerEntityWalker(const FSequencerEntityRange& InRange, FVector2D InVirtualKeySize);

	/** Visit the specified node (recursively) with this range and a user-supplied visitor */
	void Traverse(const ISequencerEntityVisitor& Visitor, TSharedPtr<UE::Sequencer::FViewModel> Item);

private:

	/** Check whether the specified node intersects the range's vertical space, and visit any keys within it if so */
	void ConditionallyIntersectModel(const ISequencerEntityVisitor& Visitor, const TSharedPtr<UE::Sequencer::FViewModel>& DataModel);
	/** Visit any keys within any key area nodes that belong to the specified node that overlap the range's horizontal space */
	void VisitAnyChannels(const ISequencerEntityVisitor& Visitor, const TSharedRef<UE::Sequencer::FViewModel>& InNode, bool bAnyParentCollapsed);
	/** Visit any keys within the specified key area that overlap the range's horizontal space */
	void VisitChannel(const ISequencerEntityVisitor& Visitor, const UE::Sequencer::TViewModelPtr<UE::Sequencer::FChannelModel>& Channel);

	/** The bounds of the range */
	FSequencerEntityRange Range;

	/** Key size in virtual space */
	FVector2D VirtualKeySize;
};
