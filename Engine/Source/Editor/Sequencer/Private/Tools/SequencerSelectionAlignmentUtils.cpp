// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/SequencerSelectionAlignmentUtils.h"
#include "Containers/Set.h"
#include "Curves/KeyHandle.h"
#include "IKeyArea.h"
#include "ISequencer.h"
#include "Misc/FrameNumber.h"
#include "Misc/Optional.h"
#include "MovieSceneSection.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/LayerBarModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "ScopedTransaction.h"
#include "Sequencer.h"

namespace UE::Sequencer
{

TSet<TViewModelPtr<FLayerBarModel>> GatherAllSelectedLayerBars(const TSharedRef<FSequencerSelection>& InSequencerSelection, TOptional<FFrameNumber>& OutEarliestTime)
{
	TSet<TViewModelPtr<FLayerBarModel>> LayerBarModels;

	const int32 OutlinerCount = InSequencerSelection->Outliner.Num();
	const int32 TrackAreaCount = InSequencerSelection->TrackArea.Num();

	LayerBarModels.Reserve(OutlinerCount + TrackAreaCount);

	for (const FViewModelPtr& ViewModel : InSequencerSelection->Outliner)
	{
		const TViewModelPtr<ITrackAreaExtension> Track = ViewModel.ImplicitCast();
		if (!Track.IsValid())
		{
			continue;
		}

		for (const FViewModelPtr& TrackAreaModel : Track->GetTopLevelChildTrackAreaModels())
		{
			if (const TViewModelPtr<FLayerBarModel> LayerBarModel = TrackAreaModel.ImplicitCast())
			{
				LayerBarModels.Add(LayerBarModel);
			}
		}

		for (const TViewModelPtr<FLayerBarModel>& LayerBarModel : Track->GetTrackAreaModelListAs<FLayerBarModel>())
		{
			LayerBarModels.Add(LayerBarModel);
		}
	}

	for (const FViewModelPtr& ViewModel : InSequencerSelection->TrackArea)
	{
		if (const TViewModelPtr<FLayerBarModel> LayerBarModel = ViewModel.ImplicitCast())
		{
			LayerBarModels.Add(LayerBarModel);
		}
	}

	// Disallow operations on all layer bars that have a descendant that is selected
	for (const TViewModelPtr<FLayerBarModel>& InLayerBarModel : LayerBarModels)
	{
		const TViewModelPtr<IOutlinerExtension> OutlinerExtension = InLayerBarModel->GetLinkedOutlinerItem();
		if (!OutlinerExtension.IsValid())
		{
			continue;
		}

		for (const TViewModelPtr<IOutlinerExtension>& ChildOutlinerExtension : OutlinerExtension.AsModel()->GetDescendantsOfType<IOutlinerExtension>())
		{
			if (InSequencerSelection->Outliner.IsSelected(ChildOutlinerExtension))
			{
				return {};
			}
		}

		const FFrameNumber ThisTime = InLayerBarModel->ComputeRange().GetLowerBoundValue();
		if (!OutEarliestTime.IsSet() || ThisTime < OutEarliestTime.GetValue())
		{
			OutEarliestTime = ThisTime;
		}
	}

	return MoveTemp(LayerBarModels);
}

TSet<TViewModelPtr<FSectionModel>> GatherAllSelectedSections(const TSharedRef<FSequencerSelection>& InSequencerSelection, TOptional<FFrameNumber>& OutEarliestTime)
{
	TSet<TViewModelPtr<FSectionModel>> SectionModels;

	SectionModels.Reserve(InSequencerSelection->TrackArea.Num());

	for (const FViewModelPtr& ViewModel : InSequencerSelection->TrackArea)
	{
		if (const TViewModelPtr<FSectionModel> SectionModel = ViewModel.ImplicitCast())
		{
			SectionModels.Add(SectionModel);

			const FFrameNumber ThisTime = SectionModel->GetLayerBarRange().GetLowerBoundValue();
			if (!OutEarliestTime.IsSet() || ThisTime < OutEarliestTime.GetValue())
			{
				OutEarliestTime = ThisTime;
			}
		}
	}

	return MoveTemp(SectionModels);
}

TSet<FKeyHandle> GatherAllSelectedKeyframes(const TSharedRef<FSequencerSelection>& InSequencerSelection, TOptional<FFrameNumber>& OutEarliestTime)
{
	TSet<FKeyHandle> OutKeyframes;

	for (const FKeyHandle KeyHandle : InSequencerSelection->KeySelection)
	{
		if (const TSharedPtr<FChannelModel> ChannelModel = InSequencerSelection->KeySelection.GetModelForKey(KeyHandle))
		{
			if (const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea())
			{
				UMovieSceneSection* const Section = ChannelModel->GetSection();
				if (IsValid(Section) && !Section->IsReadOnly())
				{
					OutKeyframes.Add(KeyHandle);

					const FFrameNumber ThisTime = KeyArea->GetKeyTime(KeyHandle);
					if (!OutEarliestTime.IsSet() || ThisTime < OutEarliestTime.GetValue())
					{
						OutEarliestTime = ThisTime;
					}
				}
			}
		}
	}

	return MoveTemp(OutKeyframes);
}

} // namespace UE::Sequencer

using namespace UE::Sequencer;

bool FSequencerSelectionAlignmentUtils::CanAlignSelection(const ISequencer& InSequencer)
{
	const TSharedPtr<FSequencerEditorViewModel> ViewModel = InSequencer.GetViewModel();
	if (!ViewModel.IsValid())
	{
		return false;
	}

	const TSharedPtr<FSequencerSelection> SequencerSelection = ViewModel->GetSelection();
	if (!SequencerSelection.IsValid())
	{
		return false;
	}

	const TSharedRef<FSequencerSelection> SequencerSelectionRef = SequencerSelection.ToSharedRef();

	TOptional<FFrameNumber> EarliestLayerBarTime;
	const TSet<TViewModelPtr<FLayerBarModel>> LayerBarModels = GatherAllSelectedLayerBars(SequencerSelectionRef, EarliestLayerBarTime);

	TOptional<FFrameNumber> EarliestSectionTime;
	const TSet<TViewModelPtr<FSectionModel>> SectionModels = GatherAllSelectedSections(SequencerSelectionRef, EarliestSectionTime);

	TOptional<FFrameNumber> EarliestKeyframeTime;
	const TSet<FKeyHandle> KeyframeModels = GatherAllSelectedKeyframes(SequencerSelectionRef, EarliestKeyframeTime);

	const bool bIsLayerBarOrSection = EarliestLayerBarTime.IsSet() || EarliestSectionTime.IsSet();
	const bool bNoLayerBarOrSection = !EarliestLayerBarTime.IsSet() && !EarliestSectionTime.IsSet();
	const int32 KeyFrameCount = KeyframeModels.Num();

	// To avoid having to do a bunch of extra processing to support aligning layers bars AND keys at the same time,
	// we will only allow layer bar OR key selection alignments since layer bars are effected by keys.
	return (bIsLayerBarOrSection && KeyFrameCount == 0)
		|| (bNoLayerBarOrSection && KeyFrameCount > 0);
}

void FSequencerSelectionAlignmentUtils::AlignSelectionToTime(const ISequencer& InSequencer, const FQualifiedFrameTime& InFrameTime, const bool bInTransact)
{
	const TSharedPtr<FSequencerEditorViewModel> ViewModel = InSequencer.GetViewModel();
	if (!ViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencerSelection> SequencerSelection = ViewModel->GetSelection();
	if (!SequencerSelection.IsValid())
	{
		return;
	}

	const TSharedRef<FSequencerSelection> SequencerSelectionRef = SequencerSelection.ToSharedRef();

	TOptional<FFrameNumber> EarliestLayerBarTime;
	const TSet<TViewModelPtr<FLayerBarModel>> LayerBarModels = GatherAllSelectedLayerBars(SequencerSelectionRef, EarliestLayerBarTime);

	TOptional<FFrameNumber> EarliestSectionTime;
	const TSet<TViewModelPtr<FSectionModel>> SectionModels = GatherAllSelectedSections(SequencerSelectionRef, EarliestSectionTime);

	const FFrameNumber FrameNumber = InFrameTime.Time.FrameNumber;
	const bool bIsLayerBarOrSection = EarliestLayerBarTime.IsSet() || EarliestSectionTime.IsSet();
	const bool bNoLayerBarOrSection = !EarliestLayerBarTime.IsSet() && !EarliestSectionTime.IsSet();
	const int32 KeyFrameCount = SequencerSelection->KeySelection.Num();

	const FScopedTransaction Transaction(NSLOCTEXT("Sequencer", "AlignToPlayhead", "Align to Playhead"), bInTransact);

	// Layer Bars and Sections
	// Set all to the time, regardless of the relative offsets of each selection
	if (bIsLayerBarOrSection && KeyFrameCount == 0)
	{
		for (const TViewModelPtr<FLayerBarModel>& LayerBarModel : LayerBarModels)
		{
			const FFrameNumber RelativeDistance = FrameNumber - LayerBarModel->ComputeRange().GetLowerBoundValue();
			LayerBarModel->Offset(RelativeDistance);
		}

		for (const TViewModelPtr<FSectionModel>& SectionModel : SectionModels)
        {
			const FFrameNumber RelativeDistance = FrameNumber - SectionModel->GetLayerBarRange().GetLowerBoundValue();
            SectionModel->OffsetLayerBar(RelativeDistance);
        }
	}
	// Keyframes
	// Find the earliest time in the selection and move the selection, while maintaining relative offsets between each selection
	else if (bNoLayerBarOrSection && KeyFrameCount > 0)
	{
		TOptional<FFrameNumber> EarliestKeyframeTime;
		const TSet<FKeyHandle> Keyframes = GatherAllSelectedKeyframes(SequencerSelectionRef, EarliestKeyframeTime);
		if (EarliestKeyframeTime.IsSet())
		{
			const FFrameNumber RelativeDistance = FrameNumber - EarliestKeyframeTime.GetValue();

			for (const FKeyHandle& Key : Keyframes)
			{
				if (const TSharedPtr<FChannelModel> ChannelModel = SequencerSelection->KeySelection.GetModelForKey(Key))
				{
					if (const TSharedPtr<IKeyArea> KeyArea = ChannelModel->GetKeyArea())
					{
						UMovieSceneSection* const Section = ChannelModel->GetSection();
						if (IsValid(Section) && !Section->IsLocked() && Section->TryModify())
						{
							const FFrameNumber NewTime = RelativeDistance + KeyArea->GetKeyTime(Key);
							KeyArea->SetKeyTime(Key, NewTime);
							Section->ExpandToFrame(NewTime);
						}
					}
				}
			}
		}
	}
}

void FSequencerSelectionAlignmentUtils::AlignSelectionToPlayhead(const ISequencer& InSequencer, const bool bInTransact)
{
	AlignSelectionToTime(InSequencer, InSequencer.GetGlobalTime(), bInTransact);
}
