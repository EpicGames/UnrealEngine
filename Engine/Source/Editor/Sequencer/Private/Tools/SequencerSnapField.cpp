// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/SequencerSnapField.h"
#include "Containers/PagedArray.h"
#include "MovieScene.h"
#include "SSequencer.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneSequence.h"
#include "ISequencerSection.h"
#include "SequencerSettings.h"
#include "IKeyArea.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/Views/SOutlinerView.h"
#include "MVVM/Extensions/ISnappableExtension.h"
#include "MVVM/Extensions/IClockExtension.h"
#include "IKeyArea.h"

struct FSnapGridVisitor : ISequencerEntityVisitor, UE::Sequencer::ISnapField
{
	FSnapGridVisitor(UE::Sequencer::ISnapCandidate& InCandidate, uint32 EntityMask)
		: ISequencerEntityVisitor(EntityMask)
		, Candidate(InCandidate)
	{
		InCandidate.GetCapabilities(Applicability, Capabilities);
	}

	virtual void VisitKeys(const UE::Sequencer::TViewModelPtr<UE::Sequencer::FChannelModel>& Channel, const TRange<FFrameNumber>& VisitRangeFrames) const override
	{
		using namespace UE::Sequencer;

		const bool bCallFunction = EnumHasAnyFlags(Capabilities, ISnapCandidate::ESnapCapabilities::IsKeyApplicable);
		const bool bDefaultState = EnumHasAnyFlags(Applicability, ISnapCandidate::EDefaultApplicability::Keys);

		if (!bCallFunction)
		{
			if (bDefaultState)
			{
				// Faster implementation if we need to add all the keys.
				// Only need to allocate space for the times in this case
				TimesScratch.Reset();
				Channel->GetKeyArea()->GetKeyInfo(nullptr, &TimesScratch, VisitRangeFrames);

				const int32 StartNum = Snaps.Num();
				const int32 NumKeys = TimesScratch.Num();
				Snaps.Reserve(StartNum + NumKeys);

				for (int32 Index = 0; Index < NumKeys; ++Index)
				{
					Snaps.Emplace(FSnapPoint::Key, TimesScratch[Index]);
				}
			}
		}
		else
		{
			// Call the function for each key individually. Much slower.
			// Need to allocate handles as well
			HandlesScratch.Reset();
			TimesScratch.Reset();
			Channel->GetKeyArea()->GetKeyInfo(&HandlesScratch, &TimesScratch, VisitRangeFrames);

			for (int32 Index = 0; Index < HandlesScratch.Num(); ++Index)
			{
				FKeyHandle KeyHandle = HandlesScratch[Index];

				if (Candidate.IsKeyApplicable(KeyHandle, Channel))
				{
					Snaps.Add(FSnapPoint{ FSnapPoint::Key, TimesScratch[Index] });
				}
			}
		}
	}

	virtual void VisitDataModel(UE::Sequencer::FViewModel* DataModel) const
	{
		using namespace UE::Sequencer;

		if (ISnappableExtension* Snappable = DataModel->CastThis<ISnappableExtension>())
		{
			Snappable->AddToSnapField(Candidate, *const_cast<FSnapGridVisitor*>(this));
		}
	}

	virtual void AddSnapPoint(const UE::Sequencer::FSnapPoint& SnapPoint) override
	{
		Snaps.Emplace(SnapPoint);
	}

	UE::Sequencer::ISnapCandidate& Candidate;
	/** utilize a chunked array to reduce the number of resize/grow memcpies for very large data sets (ie, > 1million keys) */
	mutable TPagedArray<UE::Sequencer::FSnapPoint> Snaps;
	mutable TArray<FKeyHandle> HandlesScratch;
	mutable TArray<FFrameNumber> TimesScratch;

	UE::Sequencer::ISnapCandidate::ESnapCapabilities Capabilities;
	UE::Sequencer::ISnapCandidate::EDefaultApplicability Applicability;
};

FSequencerSnapField::FSequencerSnapField(const FSequencer& InSequencer, UE::Sequencer::ISnapCandidate& Candidate, uint32 EntityMask)
{
	Initialize(InSequencer, Candidate, EntityMask);
	Finalize();
}

void FSequencerSnapField::AddExplicitSnap(UE::Sequencer::FSnapPoint InSnap)
{
	if (InSnap.Weighting == 1.f && InSnap.Type != UE::Sequencer::FSnapPoint::Key)
	{
		InSnap.Weighting = 10.f;
	}
	SortedSnaps.Add(InSnap);
}

void FSequencerSnapField::Initialize(const FSequencer& InSequencer, UE::Sequencer::ISnapCandidate& Candidate, uint32 EntityMask)
{
	using namespace UE::Sequencer;

	TViewModelPtr<IClockExtension> Extension = InSequencer.GetViewModel()->GetRootSequenceModel().ImplicitCast();
	if (Extension.IsValid() && Extension->SupportsSnapping())
	{
		ClockExtension = Extension;
	}

	TSharedPtr<SOutlinerView> TreeView = InSequencer.GetUnderlyingSequencerWidget()->GetTreeView();

	TArray<TViewModelPtr<IOutlinerExtension>> VisibleItems;
	TreeView->GetVisibleItems(VisibleItems);

	TRange<double> ViewRange = InSequencer.GetViewRange();
	FSequencerEntityWalker Walker(
		FSequencerEntityRange(ViewRange, InSequencer.GetFocusedTickResolution()),
		FVector2D(SequencerSectionConstants::KeySize));

	// Traverse the visible space, collecting snapping times as we go
	FSnapGridVisitor Visitor(Candidate, EntityMask);
	for (const TViewModelPtr<IOutlinerExtension>& Item : VisibleItems)
	{
		Walker.Traverse(Visitor, Item);
	}

	// Add the playback range start/end bounds as potential snap candidates
	TRange<FFrameNumber> PlaybackRange = InSequencer.GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();
	if(UE::MovieScene::DiscreteSize(PlaybackRange) > 0)
	{
		Visitor.Snaps.Emplace(FSnapPoint{ FSnapPoint::PlaybackRange, UE::MovieScene::DiscreteInclusiveLower(PlaybackRange)});
		Visitor.Snaps.Emplace(FSnapPoint{ FSnapPoint::PlaybackRange, UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange)});
	}

	// Add the current time as a potential snap candidate
	Visitor.Snaps.Emplace(FSnapPoint{ FSnapPoint::CurrentTime, InSequencer.GetLocalTime().Time.FrameNumber });

	// Add the selection range bounds as a potential snap candidate
	TRange<FFrameNumber> SelectionRange = InSequencer.GetFocusedMovieSceneSequence()->GetMovieScene()->GetSelectionRange();
	if (UE::MovieScene::DiscreteSize(SelectionRange) > 0)
	{
		Visitor.Snaps.Emplace(FSnapPoint{ FSnapPoint::InOutRange, UE::MovieScene::DiscreteInclusiveLower(SelectionRange)});
		Visitor.Snaps.Emplace(FSnapPoint{ FSnapPoint::InOutRange, UE::MovieScene::DiscreteExclusiveUpper(SelectionRange) - 1});
	}

	if (InSequencer.GetSequencerSettings()->GetShowMarkedFrames())
	{
		// Add in the marked frames
		for (const FMovieSceneMarkedFrame& MarkedFrame : InSequencer.GetFocusedMovieSceneSequence()->GetMovieScene()->GetMarkedFrames())
		{
			Visitor.Snaps.Emplace( FSnapPoint{ FSnapPoint::Mark, MarkedFrame.FrameNumber } );
		}

		// Add in the global marked frames
		for (const FMovieSceneMarkedFrame& MarkedFrame : InSequencer.GetGlobalMarkedFrames())
		{
			Visitor.Snaps.Emplace(FSnapPoint{ FSnapPoint::Mark, MarkedFrame.FrameNumber });
		}
	}

	// Copy the paged array to our linear array ready for final sorting
	Visitor.Snaps.ToArray(SortedSnaps);

	TickResolution = InSequencer.GetFocusedTickResolution();
	DisplayRate = InSequencer.GetFocusedDisplayRate();
	ScrubStyle = InSequencer.GetScrubStyle();
}

void FSequencerSnapField::Finalize()
{
	using namespace UE::Sequencer;

	// Sort
	SortedSnaps.Sort([](const FSnapPoint& A, const FSnapPoint& B){
		return A.Time < B.Time;
	});


	const int32 NumSnaps = SortedSnaps.Num();

	TArray<FSnapPoint> FinalSnaps;
	FinalSnaps.Reserve(NumSnaps);

	const FSnapPoint* const RESTRICT Snaps = SortedSnaps.GetData();

	// Remove duplicates
	for (int32 Index = 0; Index < NumSnaps; /* incremented inside inner loop */)
	{
		const FFrameNumber CurrentTime = SortedSnaps[Index].Time;

		FSnapPoint FinalSnap = Snaps[Index];
		FinalSnap.Weighting = 0.f;

		// Add up all weights of the same time
		for ( ; Index < NumSnaps && Snaps[Index].Time == CurrentTime; ++Index)
		{
			FinalSnap.Weighting += Snaps[Index].Weighting;
		}

		FinalSnaps.Add(FinalSnap);
	}

	FinalSnaps.Shrink();
	Swap(FinalSnaps, SortedSnaps);
}

TOptional<FSequencerSnapField::FSnapResult> FSequencerSnapField::Snap(const FFrameTime& InTime, const FFrameTime& Threshold) const
{
	int32 Min = 0;
	int32 Max = SortedSnaps.Num();

	// Binary search, then linearly search a range
	for ( ; Min != Max ; )
	{
		int32 SearchIndex = Min + (Max - Min) / 2;

		UE::Sequencer::FSnapPoint ProspectiveSnap = SortedSnaps[SearchIndex];
		if (ProspectiveSnap.Time > InTime + Threshold)
		{
			Max = SearchIndex;
		}
		else if (ProspectiveSnap.Time < InTime - Threshold)
		{
			Min = SearchIndex + 1;
		}
		else
		{
			// Linearly search forwards and backwards to find the closest or heaviest snap

			float SnapWeight = 0.f;
			FFrameTime SnapDelta = ProspectiveSnap.Time - InTime;

			// Search forwards while we're in the threshold
			for (int32 FwdIndex = SearchIndex; FwdIndex < Max && SortedSnaps[FwdIndex].Time < InTime + Threshold; ++FwdIndex)
			{
				FFrameTime ThisSnapDelta = InTime - SortedSnaps[FwdIndex].Time;
				float ThisSnapWeight = SortedSnaps[FwdIndex].Weighting;
				if (ThisSnapWeight > SnapWeight || (ThisSnapWeight == SnapWeight && FMath::Abs(ThisSnapDelta) < FMath::Abs(SnapDelta)))
				{
					SnapDelta = ThisSnapDelta;
					SnapWeight = ThisSnapWeight;
					ProspectiveSnap = SortedSnaps[FwdIndex];
				}
			}

			// Search backwards while we're in the threshold
			for (int32 BckIndex = SearchIndex-1; BckIndex >= Min && SortedSnaps[BckIndex].Time > InTime - Threshold; --BckIndex)
			{
				FFrameTime ThisSnapDelta = InTime - SortedSnaps[BckIndex].Time;
				float ThisSnapWeight = SortedSnaps[BckIndex].Weighting;
				if (ThisSnapWeight > SnapWeight || (ThisSnapWeight == SnapWeight && FMath::Abs(ThisSnapDelta) < FMath::Abs(SnapDelta)))
				{
					SnapDelta = ThisSnapDelta;
					SnapWeight = ThisSnapWeight;
					ProspectiveSnap = SortedSnaps[BckIndex];
				}
			}

			if (SnapWeight != 0.f)
			{
				FSnapResult Result = { InTime, ProspectiveSnap.Time, ProspectiveSnap.Weighting };
				return Result;
			}

			break;
		}
	}

	return TOptional<FSequencerSnapField::FSnapResult>();
}

TOptional<FSequencerSnapField::FSnapResult> FSequencerSnapField::Snap(const TArray<FFrameTime>& InTimes, const FFrameTime& Threshold) const
{
	TOptional<FSnapResult> ProspectiveSnap;

	const FFrameTime IntervalSnapThreshold = FFrameTime::FromDecimal((TickResolution / DisplayRate).AsDecimal());
	int32 NumSnaps = 0;
	float MaxSnapWeight = 0.f;
	for (FFrameTime Time : InTimes)
	{
		TOptional<FSnapResult> ThisSnap;
		
		if (bSnapToLikeTypes)
		{
			ThisSnap = Snap(Time, FMath::Max(IntervalSnapThreshold, Threshold));
		}

		if (ClockExtension.IsValid() && ClockExtension->ShouldSnapFrameTime())
		{
			FFrameNumber ClockSnapTime = ClockExtension->SnapFrameTime(Time).FrameNumber;
			if (ClockSnapTime != Time)
			{
				if (ThisSnap.IsSet())
				{
					ThisSnap->SnappedTime = ClockSnapTime;
				}
				else
				{
					ThisSnap = { Time, ClockSnapTime, 1.f };
				}
			}
		}

		const bool bShouldTrySnappingToInterval = !bSnapToLikeTypes || IntervalSnapThreshold > Threshold;
		if (bSnapToInterval || bShouldTrySnappingToInterval)
		{
			FFrameTime ThisTime = ThisSnap.IsSet() ? ThisSnap->SnappedTime : Time;
			FFrameTime IntervalTime = FFrameRate::TransformTime(ThisTime, TickResolution, DisplayRate);
			FFrameNumber PlayIntervalTime = IntervalTime.RoundToFrame();
			FFrameNumber IntervalSnap = FFrameRate::TransformTime(PlayIntervalTime, DisplayRate, TickResolution).FloorToFrame();

			FFrameTime ThisSnapAmount = IntervalSnap - ThisTime;
			bool bSnapToLikeTypesHasPriority = ThisSnap.IsSet() && FMath::Abs(ThisTime - Time) <= FMath::Abs(IntervalSnap - Time);
			if (!bSnapToLikeTypesHasPriority && FMath::Abs(ThisSnapAmount) <= IntervalSnapThreshold)
			{
				ThisSnap = { Time, IntervalSnap, 1.f };
			}
		}

		if (!ThisSnap.IsSet())
		{
			continue;
		}

		if (!ProspectiveSnap.IsSet() || ThisSnap->SnappedWeight > MaxSnapWeight)
		{
			ProspectiveSnap = ThisSnap;
			MaxSnapWeight = ThisSnap->SnappedWeight;
		}
	}

	return ProspectiveSnap;
}