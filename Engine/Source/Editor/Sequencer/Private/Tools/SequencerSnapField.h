// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Tools/SequencerEntityVisitor.h"
#include "MVVM/Extensions/ISnappableExtension.h"

class IKeyArea;
class FSequencer;

namespace UE::Sequencer
{
	class IClockExtension;
}

enum class ESequencerScrubberStyle : uint8;

/** A snapping field that provides efficient snapping calculations on a range of values */
class FSequencerSnapField
{
public:

	/** A snap result denoting the time that was snapped, and the resulting snapped time */
	struct FSnapResult
	{
		/** The time before it was snapped */
		FFrameTime OriginalTime;
		/** The time after it was snapped */
		FFrameNumber SnappedTime;
		/** The total weight of the time that was snapped to */
		float SnappedWeight = 0.f;
	};

	FSequencerSnapField(){}

	/** Construction from a sequencer and a snap canidate implementation. Optionally provide an entity mask to completely ignore some entity types */
	FSequencerSnapField(const FSequencer& InSequencer, UE::Sequencer::ISnapCandidate& Candidate, uint32 EntityMask = ESequencerEntity::Everything);

	void Initialize(const FSequencer& InSequencer, UE::Sequencer::ISnapCandidate& Candidate, uint32 EntityMask = ESequencerEntity::Everything);

	void AddExplicitSnap(UE::Sequencer::FSnapPoint InSnap);

	void Finalize();

	void SetSnapToInterval(bool bInSnapToInterval) { bSnapToInterval = bInSnapToInterval; }

	void SetSnapToLikeTypes(bool bInSnapToLikeTypes) { bSnapToLikeTypes = bInSnapToLikeTypes; }

	/** Snap the specified time to this field with the given threshold */
	TOptional<FSnapResult> Snap(const FFrameTime& InTime, const FFrameTime& Threshold) const;

	/** Snap the specified times to this field with the given threshold. Will return the closest snap value of the entire intersection. */
	TOptional<FSnapResult> Snap(const TArray<FFrameTime>& InTimes, const FFrameTime& Threshold) const;

private:
	/** Array of snap points, approximately grouped, and sorted in ascending order by time */
	TArray<UE::Sequencer::FSnapPoint> SortedSnaps;

	FFrameRate TickResolution;
	FFrameRate DisplayRate;
	ESequencerScrubberStyle ScrubStyle;

	UE::Sequencer::TViewModelPtr<UE::Sequencer::IClockExtension> ClockExtension;

	bool bSnapToInterval = false;
	bool bSnapToLikeTypes = false;
};
