// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/TrailCategory.h"

ETrailCategory FTrailCategoryRegistry::VisibleCategories = ETrailCategory::None;

void FTrailCategoryRegistry::SetCategoryVisible(ETrailCategory Category, bool bVisible)
{
	if (bVisible)
	{
		VisibleCategories |= Category;
	}
	else
	{
		VisibleCategories &= ~Category;
	}
}

bool FTrailCategoryRegistry::IsCategoryVisible(ETrailCategory Category)
{
	return EnumHasAnyFlags(VisibleCategories, Category);
}

ETrailCategory FTrailCategoryRegistry::GetVisibleCategories()
{
	return VisibleCategories;
}
