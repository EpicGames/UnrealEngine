// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoveSequencerTimeSliderViewportInteraction.h"

#include "AnimatedRange.h"
#include "ISequencer.h"
#include "Misc/InputState.h"
#include "SequencerCommands.h"
#include "SequencerEdMode.h"
#include "EditorViewportClient.h"

UMoveSequencerTimeSliderViewportInteraction::UMoveSequencerTimeSliderViewportInteraction()
{
	InteractionName = TEXT("SequencerTimeSliderInteraction");
	
	if (ensure(ViewportClickDragBehavior.IsValid()))
	{
		using namespace UE::Editor::ViewportInteractions;
		if (FSequencerCommands::IsRegistered())
		{
			ViewportClickDragBehavior->SetBindings({
				FButtonBinding(EKeys::LeftMouseButton).Required(false).TriggersStart(),
				FButtonBinding(EKeys::RightMouseButton).Required(false).TriggersStart(),
				FButtonBinding(EKeys::MiddleMouseButton).Required(false).TriggersStart(),
				FButtonBinding(FSequencerCommands::Get().ScrubTimeViewport)
			});
			
			// Once the gizmo plays by the same binding-based priority rules, this can be removed
			ViewportClickDragBehavior->SetDefaultPriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY);
			ViewportClickDragBehavior->SetDragConfirmedPriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY);
		}
	}
}


bool UMoveSequencerTimeSliderViewportInteraction::CanBeActivated() const
{
	return IsEnabled() && EdMode;
}

void UMoveSequencerTimeSliderViewportInteraction::OnDragStart(const FInputDeviceRay& InDragStartPos)
{
	Super::OnDragStart(InDragStartPos);
	
	if (!EdMode)
	{
		return;
	}

	const FEditorViewportClient* const Client = GetEditorViewportClient();
	if (Client && Client->Viewport && EdMode->IsPressingMoveTimeSlider(Client->Viewport))
	{
		UE::CurveEditor::InputState::SetCommandPressed(FSequencerCommands::Get().ScrubTimeViewport, Client->Viewport, /*bInPressed=*/true);

		SetMouseCursorOverride(EMouseCursor::ResizeLeftRight);
	}

	if (TSharedPtr<ISequencer> Sequencer = EdMode->GetFirstActiveSequencer())
	{
		NextPlayerStatus = Sequencer->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing
			? EMovieScenePlayerStatus::Playing
			: EMovieScenePlayerStatus::Stopped;
	
		Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Scrubbing);
		Sequencer->OnBeginScrubbingEvent().Broadcast();
		
		AccumulatedChange = 0.0f;
		StartingFrame = Sequencer->GetLocalTime().Time;
	}
}

void UMoveSequencerTimeSliderViewportInteraction::OnDragDelta(float InMouseDeltaX, float InMouseDeltaY)
{
	FEditorViewportClient* Client = GetEditorViewportClient();
	if (!Client)
	{
		return;
	}
	
	if (!EdMode)
	{
		return;
	}

	const TSharedPtr<ISequencer> ActiveSequencer = EdMode->GetFirstActiveSequencer();
	if (!ActiveSequencer)
	{
		return;
	}
	
	FIntPoint Origin, Size;
	Client->GetViewportDimensions(Origin, Size);
	if (Size.X == 0.0f)
	{
		return;
	}
	
	const float ViewPortSize = static_cast<float>(Size.X);
	
	AccumulatedChange += InMouseDeltaX;
	
	const float FloatViewDiff = AccumulatedChange / ViewPortSize;

	const FFrameRate TickResolution = ActiveSequencer->GetFocusedTickResolution();
	const FAnimatedRange Range = ActiveSequencer->GetViewRange();
	const TRange<FFrameNumber> ViewRange(TickResolution.AsFrameNumber(Range.GetLowerBoundValue()), TickResolution.AsFrameNumber(Range.GetUpperBoundValue()));
	FFrameNumber FrameDiff = ViewRange.Size<FFrameNumber>() * FloatViewDiff;
	
	FFrameTime SnappedTime = StartingFrame + FrameDiff;
	ActiveSequencer->SnapSequencerTime(SnappedTime);
	ActiveSequencer->SetLocalTime(SnappedTime);
}

void UMoveSequencerTimeSliderViewportInteraction::OnDragEnd(const FInputDeviceRay& InDragEndPos)
{
	Super::OnDragEnd(InDragEndPos);
	
	if (!EdMode)
	{
		return;
	}

	if (const TSharedPtr<ISequencer> ActiveSequencer = EdMode->GetFirstActiveSequencer())
	{
		if (NextPlayerStatus == EMovieScenePlayerStatus::Stopped)
		{
			//Call Pause() instead of SetPlaybackStatus(EMovieScenePlayerStatus::Stopped)
			//so the correct delegates/evals can happen so all systems like audio are probably stopped (Pause set's Stopped status).
			ActiveSequencer->Pause();
		}
		else
		{
			ActiveSequencer->SetPlaybackStatus(NextPlayerStatus);
		}
		ActiveSequencer->OnEndScrubbingEvent().Broadcast();
	}
}
