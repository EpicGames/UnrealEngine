// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "ViewportInteractions/ViewportDragInteraction.h"
#include "MovieSceneFwd.h"
#include "MoveSequencerTimeSliderViewportInteraction.generated.h"

class FSequencerEdMode;
/**
 * 
 */
UCLASS()
class SEQUENCER_API UMoveSequencerTimeSliderViewportInteraction : public UViewportDragInteraction
{
	GENERATED_BODY()
	
public:
	UMoveSequencerTimeSliderViewportInteraction();
	
	virtual bool CanBeActivated() const override;
	virtual void OnDragStart(const FInputDeviceRay& InDragStartPos) override;
	virtual void OnDragDelta(float InMouseDeltaX, float InMouseDeltaY) override;
	virtual void OnDragEnd(const FInputDeviceRay& InDragEndPos) override;
	
	FSequencerEdMode* EdMode = nullptr;
	
private:
	float AccumulatedChange = 0.0f;
	FFrameTime StartingFrame;
	EMovieScenePlayerStatus::Type NextPlayerStatus = EMovieScenePlayerStatus::Stopped;
};
