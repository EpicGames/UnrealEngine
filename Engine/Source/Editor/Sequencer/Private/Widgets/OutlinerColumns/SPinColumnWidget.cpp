// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerColumns/SPinColumnWidget.h"

#include "MVVM/Extensions/IPinnableExtension.h"
#include "MVVM/PinEditorExtension.h"
#include "MVVM/ViewModels/EditorViewModel.h"

namespace UE::Sequencer
{

#define LOCTEXT_NAMESPACE "SPinColumnWidget"

void SPinColumnWidget::OnToggleOperationComplete()
{
	// refresh the sequencer tree after operation is complete
	RefreshSequencerTree();
}

void SPinColumnWidget::Construct(const FArguments& InArgs, const TWeakPtr<IOutlinerColumn> InWeakOutlinerColumn, const FCreateOutlinerColumnParams& InParams)
{
	SetToolTipText(LOCTEXT("PinTooltip", "Pin this track and force it to always stay at the top of the list.\n\n"
		"Saved with the asset."));

	SColumnToggleWidget::Construct(
		SColumnToggleWidget::FArguments(),
		InWeakOutlinerColumn,
		InParams
	);
}

bool SPinColumnWidget::IsActive() const
{
	TSharedPtr<FEditorViewModel> Editor = WeakEditor.Pin();
	if (Editor)
	{
		FPinEditorExtension* PinEditorExtension = Editor->CastDynamic<FPinEditorExtension>();
		if (PinEditorExtension)
		{
			return PinEditorExtension->IsNodePinned(WeakOutlinerExtension);
		}
	}
	
	return false;
}

void SPinColumnWidget::SetIsActive(const bool bInIsActive)
{
	TSharedPtr<FEditorViewModel> Editor = WeakEditor.Pin();
	if (Editor)
	{
		FPinEditorExtension* PinEditorExtension = Editor->CastDynamic<FPinEditorExtension>();
		if (PinEditorExtension)
		{
			PinEditorExtension->SetNodePinned(WeakOutlinerExtension, bInIsActive);
		}
	}
}

bool SPinColumnWidget::IsChildActive() const
{
	// children of a pin widget cannot have a different pinned state from their parent
	return false;
}

const FSlateBrush* SPinColumnWidget::GetActiveBrush() const
{
	static const FName NAME_PinnedBrush = TEXT("Sequencer.Column.Unpinned");
	return FAppStyle::Get().GetBrush(NAME_PinnedBrush);
}

#undef LOCTEXT_NAMESPACE

} // namespace UE::Sequencer
