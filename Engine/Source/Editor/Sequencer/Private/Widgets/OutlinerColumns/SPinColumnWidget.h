// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Views/OutlinerColumns/SColumnToggleWidget.h"

namespace UE::Sequencer
{

class IPinnableExtension;

/**
 * A widget that shows and controls the pinned state of outliner items.
 */
class SPinColumnWidget
	: public SColumnToggleWidget
{
public:
	SLATE_BEGIN_ARGS(SPinColumnWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<IOutlinerColumn> InWeakOutlinerColumn, const FCreateOutlinerColumnParams& InParams);

public:

	/** Refreshes the Sequencer Tree to update pinned state of items. */
	virtual void OnToggleOperationComplete();

protected:

	/** Returns whether or not this outliner item is pinned. */
	virtual bool IsActive() const override;

	/** Sets this item as pinned or unpinned. If selected, applies to all selected items. */
	virtual void SetIsActive(const bool bInIsActive) override;

	/** Returns whether or not a child of this item is pinned. */
	virtual bool IsChildActive() const override;

	/** Returns the brush to display when this item is pinned. */
	virtual const FSlateBrush* GetActiveBrush() const override;

};

} // namespace UE::Sequencer