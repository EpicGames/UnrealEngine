// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerIndicators/SConditionIndicatorWidget.h"

#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/OutlinerColumns/IOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerIndicators/IOutlinerIndicatorBuilder.h"
#include "MVVM/ViewModels/OutlinerIndicators/ConditionOutlinerIndicatorBuilder.h"
#include "MVVM/Extensions/IConditionableExtension.h"
#include "MVVM/Selection/SequencerCoreSelection.h"
#include "MVVM/Selection/SequencerOutlinerSelection.h"

namespace UE::Sequencer
{

void SConditionIndicatorWidget::Construct(const FArguments& InArgs, const TWeakPtr<IOutlinerColumn> InWeakOutlinerColumn, const FCreateOutlinerColumnParams& InParams)
{
	SColumnToggleWidget::Construct(
		SColumnToggleWidget::FArguments(),
		InWeakOutlinerColumn,
		InParams
	);

	WeakConditionStateCacheExtension = CastViewModel<FConditionStateCacheExtension>(InParams.OutlinerExtension.AsModel()->GetSharedData());
}

bool SConditionIndicatorWidget::IsActive() const
{
	if (TViewModelPtr<FConditionStateCacheExtension> StateCache = WeakConditionStateCacheExtension.Pin())
	{
		return EnumHasAnyFlags(StateCache->GetCachedFlags(ModelID), ECachedConditionState::HasCondition);
	}

	return false;
}

const FSlateBrush* SConditionIndicatorWidget::GetActiveBrush() const
{
	static const FName NAME_ConditionBrush = TEXT("Sequencer.Indicator.Condition");
	return FAppStyle::Get().GetBrush(NAME_ConditionBrush);
}

FSlateColor SConditionIndicatorWidget::GetImageColorAndOpacity() const
{
	FLinearColor OutColor = FLinearColor::Black;
	TSharedPtr<FEditorViewModel> Editor = WeakEditor.Pin();
	TViewModelPtr<IOutlinerExtension> OutlinerItem = WeakOutlinerExtension.Pin();

	bool bActive = IsActive();
	bool bEvaluatingTrue = false;
	bool bChildHasCondition = false;
	bool bSectionHasCondition = false;
	if (TViewModelPtr<FConditionStateCacheExtension> StateCache = WeakConditionStateCacheExtension.Pin())
	{
		bEvaluatingTrue = EnumHasAnyFlags(StateCache->GetCachedFlags(ModelID), ECachedConditionState::ConditionEvaluatingTrue);
		bChildHasCondition = EnumHasAnyFlags(StateCache->GetCachedFlags(ModelID), ECachedConditionState::ChildHasCondition);
		bSectionHasCondition = EnumHasAnyFlags(StateCache->GetCachedFlags(ModelID), ECachedConditionState::SectionHasCondition);
	}

	if (!Editor || !OutlinerItem)
	{
		return OutColor;
	}

	float Opacity = 1.0f;

	if (IsActive() || (bChildHasCondition && !OutlinerItem->IsExpanded()) || bSectionHasCondition)
	{
		Opacity = 1.0f;
	}
	else
	{
		// Not active, invisible
		Opacity = 0.0f;
	}
	OutColor.A = Opacity; // The background should be at the specified opacity and the icon should stay 100% black.
	
	return OutColor;
}

FSlateColor SConditionIndicatorWidget::GetIndicatorBackgroundColorAndOpacity() const
{
	static const FLinearColor ConditionColor = FLinearColor::FromSRGBColor(FColor(92, 220, 205));
	TSharedPtr<FEditorViewModel> Editor = WeakEditor.Pin();
	TViewModelPtr<IOutlinerExtension> OutlinerItem = WeakOutlinerExtension.Pin();

	bool bActive = IsActive();
	bool bEvaluatingTrue = false;
	bool bChildHasCondition = false;
	bool bSectionHasCondition = false;
	if (TViewModelPtr<FConditionStateCacheExtension> StateCache = WeakConditionStateCacheExtension.Pin())
	{
		bEvaluatingTrue = EnumHasAnyFlags(StateCache->GetCachedFlags(ModelID), ECachedConditionState::ConditionEvaluatingTrue);
		bChildHasCondition = EnumHasAnyFlags(StateCache->GetCachedFlags(ModelID), ECachedConditionState::ChildHasCondition);
		bSectionHasCondition = EnumHasAnyFlags(StateCache->GetCachedFlags(ModelID), ECachedConditionState::SectionHasCondition);
	}

	FLinearColor OutColor = ConditionColor;

	if (!Editor || !OutlinerItem)
	{
		return OutColor;
	}

	float Opacity = 1.0f;

	if (IsActive())
	{
		if (bEvaluatingTrue)
		{
			Opacity = 1.0f;
		}
		else
		{
			Opacity = 0.5f;
		}
	}
	else if ((bChildHasCondition && !OutlinerItem->IsExpanded()) || bSectionHasCondition)
	{
		Opacity = 0.25f;
	}
	else
	{
		// Not active, invisible
		Opacity = 0.0f;
	}
	OutColor.A = Opacity; // The background should be at the specified opacity and the icon should stay 100% black.

	return OutColor;
}

} // namespace UE::Sequencer
