// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerIndicators/STimeWarpIndicatorWidget.h"

#include "MVVM/SharedViewModelData.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "MVVM/ViewModels/OutlinerColumns/IOutlinerColumn.h"
#include "MVVM/ViewModels/OutlinerIndicators/IOutlinerIndicatorBuilder.h"
#include "MVVM/ViewModels/OutlinerIndicators/TimeWarpOutlinerIndicatorBuilder.h"
#include "MVVM/Selection/SequencerCoreSelection.h"
#include "MVVM/Selection/SequencerOutlinerSelection.h"

namespace UE::Sequencer
{

void STimeWarpIndicatorWidget::Construct(const FArguments& InArgs, const TWeakPtr<IOutlinerColumn> InWeakOutlinerColumn, const FCreateOutlinerColumnParams& InParams)
{
	SColumnToggleWidget::Construct(
		SColumnToggleWidget::FArguments(),
		InWeakOutlinerColumn,
		InParams
	);
}

bool STimeWarpIndicatorWidget::IsActive() const
{
	return true;
}

const FSlateBrush* STimeWarpIndicatorWidget::GetActiveBrush() const
{
	static const FName NAME_TimeWarpBrush = TEXT("Sequencer.Indicator.TimeWarp");
	return FAppStyle::Get().GetBrush(NAME_TimeWarpBrush);
}

FSlateColor STimeWarpIndicatorWidget::GetImageColorAndOpacity() const
{
	return FLinearColor::Black;
}

} // namespace UE::Sequencer
