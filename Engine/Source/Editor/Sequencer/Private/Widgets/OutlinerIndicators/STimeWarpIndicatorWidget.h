// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Views/OutlinerColumns/SColumnToggleWidget.h"

namespace UE::Sequencer
{	

class IOutlinerColumn;
class IOutlinerIndicator;
class FTimeWarpOutlinerIndicatorBuilder;

/**
 * A widget that shows an indicator based on the presence of time warp on a row
 */
class STimeWarpIndicatorWidget
	: public SColumnToggleWidget
{
public:
	SLATE_BEGIN_ARGS(STimeWarpIndicatorWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<IOutlinerColumn> InWeakOutlinerColumn, const FCreateOutlinerColumnParams& InParams);

	/** Get the color and opacity of the column toggle widget. */
	virtual FSlateColor GetImageColorAndOpacity() const override;

protected:

	/** Returns whether or not the item is directly active or not. */
	virtual bool IsActive() const override;

	/** Sets the active state of this item to the input value. */
	virtual void SetIsActive(const bool bInIsActive) override { /* no-op for now, no toggle*/ }

	/** Returns true if a child of this item is active. */
	virtual bool IsChildActive() const override { return false; }

	/** Returns true if this item is implicitly active, but not directly active */
	virtual bool IsImplicitlyActive() const override { return false; }

	/** Returns the brush to be used to represent a widget is active. */
	virtual const FSlateBrush* GetActiveBrush() const override;

protected:

	/** Get the image this widget displays. */
	virtual const FSlateBrush* GetBrush() const override { return GetActiveBrush(); }

protected:

	/** Handles left click-type inputs and potentially begins drag and drop operation. */
	FReply HandleClick() { return FReply::Handled(); }

private:

	TWeakPtr<FTimeWarpOutlinerIndicatorBuilder> WeakOutlinerIndicator;
};

} // namespace UE::Sequencer