// Copyright Epic Games, Inc. All Rights Reserved.

#include "STopSliderAreaOverlay.h"

#include "MVVM/Extensions/ITopTimeSliderOverlayExtension.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

namespace UE::Sequencer
{
void STopSliderAreaOverlay::Construct(const FArguments& InArgs, TWeakViewModelPtr<FViewModel> InRootModel)
{
	WeakRootModel = MoveTemp(InRootModel);
	SetVisibility(EVisibility::HitTestInvisible);
}

int32 STopSliderAreaOverlay::OnPaint(
	const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled
	) const
{
	const TViewModelPtr<FViewModel> RootModelPin = WeakRootModel.Pin();
	if (!RootModelPin)
	{
		return LayerId;
	}

	for (TViewModelPtr<ITopTimeSliderOverlayExtension> Child : RootModelPin->GetDescendantsOfType<ITopTimeSliderOverlayExtension>())
	{
		LayerId = Child->OnPaintTimeSliderOverlay(
			Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle
			);
	}
	return LayerId;
}
}
