// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::Sequencer
{
/** Widget that overlay the track area. It overlays descendant FViewModels implementing ITopTimeSliderOverlayExtension to draw itself. */
class STopSliderAreaOverlay : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(STopSliderAreaOverlay){}
	SLATE_END_ARGS()

	/** @param InRootModel We search for models implementing the TModelInterface under InRootModel's hierarchy.  */
	void Construct(const FArguments& InArgs, TWeakViewModelPtr<FViewModel> InRootModel);

	//~ Begin SWidget Interface
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	//~ End SWidget Interface

private:

	/** On this model, we search for IOverlayExtension sub-interfaces. */
	TWeakViewModelPtr<FViewModel> WeakRootModel;
};
}