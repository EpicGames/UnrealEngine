// Copyright Epic Games, Inc. All Rights Reserved.

#include "STrackAreaOverlay.h"

#include "MVVM/Extensions/ITrackAreaOverlayExtension.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

namespace UE::Sequencer
{
void STrackAreaOverlay::Construct(const FArguments& InArgs, TWeakViewModelPtr<FViewModel> InRootModel)
{
	WeakRootModel = MoveTemp(InRootModel);
	SetVisibility(EVisibility::HitTestInvisible);
}

int32 STrackAreaOverlay::OnPaint(
	const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled
	) const
{
	const TViewModelPtr<FViewModel> RootModelPin = WeakRootModel.Pin();
	if (!RootModelPin)
	{
		return LayerId;
	}

	for (TViewModelPtr<ITrackAreaOverlayExtension> Child : RootModelPin->GetDescendantsOfType<ITrackAreaOverlayExtension>())
	{
		LayerId = Child->OnPaintTrackAreaOverlay(
			Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle
			);
	}
	return LayerId;
}
}
