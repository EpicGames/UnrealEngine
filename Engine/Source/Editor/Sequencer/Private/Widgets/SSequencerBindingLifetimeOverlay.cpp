// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SSequencerBindingLifetimeOverlay.h"

#include "MVVM/ObjectBindingModelStorageExtension.h"
#include "MVVM/ViewModels/BindingLifetimeOverlayModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/Views/STrackAreaView.h"

#include "SequencerCommonHelpers.h"
#include "MovieSceneTimeHelpers.h"
#include "Sequencer.h"
#include "Rendering/DrawElementTypes.h"

#define LOCTEXT_NAMESPACE "SSequencerBindingLifetimeOverlay"

namespace UE
{
	namespace Sequencer
	{
		void SSequencerBindingLifetimeOverlay::Construct(const FArguments& InArgs, TWeakPtr<STrackAreaView> InWeakTrackArea, TWeakPtr<FSequencerEditorViewModel> InWeakEditor, TWeakPtr<FBindingLifetimeOverlayModel> InWeakBindingLifetimeOverlayModel)
		{
			WeakBindingLifetimeOverlayModel = InWeakBindingLifetimeOverlayModel;
			WeakEditor = InWeakEditor;
			WeakTrackArea = InWeakTrackArea; 
			TimeToPixel = InWeakTrackArea.Pin()->GetTimeToPixel();
		}

		TSharedPtr<FBindingLifetimeOverlayModel> SSequencerBindingLifetimeOverlay::GetBindingLifetimeOverlayModel() const
		{
			return WeakBindingLifetimeOverlayModel.Pin();
		}

		TSharedPtr<FSequencer> SSequencerBindingLifetimeOverlay::GetSequencer() const
		{
			if (TSharedPtr<FSequencerEditorViewModel> Editor = WeakEditor.Pin())
			{
				return Editor->GetSequencerImpl();
			}
			return nullptr;
		}

		int32 SSequencerBindingLifetimeOverlay::GetOverlapPriority() const
		{
			return 1000;
		}

		TSharedRef<const SWidget> SSequencerBindingLifetimeOverlay::AsWidget() const
		{
			return AsShared();
		}

		FTrackLaneScreenAlignment SSequencerBindingLifetimeOverlay::GetAlignment(const ITrackLaneWidgetSpace& InScreenSpace, const FGeometry& InParentGeometry) const
		{
			if (TSharedPtr<FBindingLifetimeOverlayModel> BindingLifetimeOverlayModel = GetBindingLifetimeOverlayModel())
			{
				return BindingLifetimeOverlayModel->ArrangeVirtualTrackLaneView().ToScreen(InScreenSpace.GetScreenSpace(), InParentGeometry);
			}
			return FTrackLaneScreenAlignment{};
		}


		int32 SSequencerBindingLifetimeOverlay::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
		{
			const ESlateDrawEffect DrawEffects = bParentEnabled
				? ESlateDrawEffect::None
				: ESlateDrawEffect::DisabledEffect;
			static const FSlateBrush* InactiveLifetimeOverlay = FAppStyle::GetBrush("Sequencer.Section.SequencerDeactivatedOverlay");

			if (TSharedPtr<FBindingLifetimeOverlayModel> BindingLifetimeOverlayModel = GetBindingLifetimeOverlayModel())
			{
				// If applicable, paint an overlay over the range that the object binding is deactivated
				const TArray<FFrameNumberRange>& InverseLifetimeRange = BindingLifetimeOverlayModel->GetInverseLifetimeRange();
				if (!InverseLifetimeRange.IsEmpty())
				{
					for (const FFrameNumberRange& ExcludedRange : InverseLifetimeRange)
					{
						const FFrameNumber LowerFrame = ExcludedRange.HasLowerBound() ? ExcludedRange.GetLowerBoundValue() : TimeToPixel->PixelToFrame(AllottedGeometry.Position.X).FloorToFrame();
						const FFrameNumber UpperFrame = ExcludedRange.HasUpperBound() ? ExcludedRange.GetUpperBoundValue() : TimeToPixel->PixelToFrame(AllottedGeometry.Position.X + AllottedGeometry.Size.X).CeilToFrame();

						const float ExcludedRangeL = TimeToPixel->FrameToPixel(LowerFrame);
						const float ExcludedRangeR = TimeToPixel->FrameToPixel(UpperFrame);

						// Black line on left
						FSlateDrawElement::MakeBox(
							OutDrawElements,
							LayerId + 50,
							AllottedGeometry.ToPaintGeometry(FVector2f(1.0f, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(ExcludedRangeL, 0.f))),
							FAppStyle::GetBrush("WhiteBrush"),
							ESlateDrawEffect::None,
							FLinearColor::Black
						);

						// Black tint for excluded regions
						// Artificially increase layer id to ensure we paint over any possible section elements
						FSlateDrawElement::MakeBox(
							OutDrawElements,
							LayerId + 50,
							AllottedGeometry.ToPaintGeometry(FVector2f(ExcludedRangeR - ExcludedRangeL, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(ExcludedRangeL, 0.0f))),
							InactiveLifetimeOverlay,
							ESlateDrawEffect::None
						);


						// Black line on right
						FSlateDrawElement::MakeBox(
							OutDrawElements,
							LayerId + 50,
							AllottedGeometry.ToPaintGeometry(FVector2f(1.0f, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(ExcludedRangeR, 0.f))),
							FAppStyle::GetBrush("WhiteBrush"),
							ESlateDrawEffect::None,
							FLinearColor::Black
						);

					}
				}
			}
			return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId + 1, InWidgetStyle, bParentEnabled);
		}

	} // namespace Sequencer
} // namespace UE

#undef LOCTEXT_NAMESPACE
