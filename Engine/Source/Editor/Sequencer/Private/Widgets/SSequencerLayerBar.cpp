// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SSequencerLayerBar.h"

#include "Tools/EditToolDragOperations.h"
#include "Tools/DragOperation_Stretch.h"
#include "Tools/SequencerEditTool_Movement.h"

#include "MVVM/ObjectBindingModelStorageExtension.h"
#include "MVVM/ViewModels/LayerBarModel.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/Views/STrackAreaView.h"

#include "Misc/InputState.h"
#include "SequencerCommands.h"
#include "SequencerCommonHelpers.h"
#include "MovieSceneTimeHelpers.h"
#include "Sequencer.h"
#include "ISequencerSection.h"

#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SSequencerLayerBar"

namespace UE
{
namespace Sequencer
{

/** A hotspot representing a section */
struct FLayerBarHotspot
	: ITrackAreaHotspot
	, IMouseHandlerHotspot
{
	UE_SEQUENCER_DECLARE_CASTABLE(FLayerBarHotspot, ITrackAreaHotspot, IMouseHandlerHotspot);

	FLayerBarHotspot(TSharedPtr<FLayerBarModel> InLayerBar, TSharedPtr<FSequencer> InSequencer)
		: LayerBar(InLayerBar)
		, WeakSequencer(InSequencer)
	{}

	void UpdateOnHover(FTrackAreaViewModel& InTrackArea) const override
	{
		InTrackArea.AttemptToActivateTool(FSequencerEditTool_Movement::Identifier);
	}
	TOptional<FFrameNumber> GetTime() const override
	{
		return LayerBar->ComputeRange().GetLowerBoundValue();

	}
	TOptional<FFrameTime> GetOffsetTime() const override
	{
		return TOptional<FFrameTime>();
	}
	TSharedPtr<ISequencerEditToolDragOperation> InitiateDrag(const FPointerEvent& MouseEvent) override
	{
		TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();

		FSequencerSelection& Selection = Sequencer->GetSelection();

		if (!Selection.TrackArea.IsSelected(LayerBar))
		{
			Selection.KeySelection.Empty();
			Selection.TrackArea.Empty();
			Selection.TrackArea.Select(LayerBar);
		}

		if (!MouseEvent.IsShiftDown())
		{
			return MakeShareable( new FMoveKeysAndSections( *Sequencer, ESequencerMoveOperationType::MoveSections | ESequencerMoveOperationType::MoveKeys) );
		}

		return nullptr;
	}
	bool PopulateContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, FFrameTime MouseDownTime) override
	{
		return true;
	}
	void HandleMouseSelection(FHotspotSelectionManager& SelectionManager) override
	{
		if (SelectionManager.MouseEvent->GetEffectingButton() == EKeys::LeftMouseButton)
		{
			SelectionManager.DefaultModelSelection(LayerBar);
		}
	}

	TSharedPtr<FLayerBarModel> LayerBar;
	TWeakPtr<FSequencer> WeakSequencer;
};

/** A hotspot representing a section */
struct FLayerBarStretchHotspot
	: ITrackAreaHotspot
	, IMouseHandlerHotspot
{
	UE_SEQUENCER_DECLARE_CASTABLE(FLayerBarStretchHotspot, ITrackAreaHotspot, IMouseHandlerHotspot);

	FLayerBarStretchHotspot(TSharedPtr<FLayerBarModel> InLayerBar, EStretchConstraint InStretchConstraint, TSharedPtr<FSequencer> InSequencer)
		: LayerBar(InLayerBar)
		, WeakSequencer(InSequencer)
		, StretchConstraint(InStretchConstraint)
	{}

	void UpdateOnHover(FTrackAreaViewModel& InTrackArea) const override
	{
		InTrackArea.AttemptToActivateTool(FSequencerEditTool_Movement::Identifier);
	}
	TOptional<FFrameNumber> GetTime() const override
	{
		TRange<FFrameNumber> Range = LayerBar->ComputeRange();
		return StretchConstraint == EStretchConstraint::AnchorToStart ? Range.GetUpperBoundValue() : Range.GetLowerBoundValue();

	}
	TOptional<FFrameTime> GetOffsetTime() const override
	{
		return TOptional<FFrameTime>();
	}
	TSharedPtr<ISequencerEditToolDragOperation> InitiateDrag(const FPointerEvent& MouseEvent) override
	{
		TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
		FSequencerSelection& Selection = Sequencer->GetSelection();

		if (!Selection.TrackArea.IsSelected(LayerBar))
		{
			Selection.KeySelection.Empty();
			Selection.TrackArea.Empty();
			Selection.TrackArea.Select(LayerBar);
		}

		return MakeShared<FEditToolDragOperation_Stretch>(Sequencer.Get(), StretchConstraint, GetTime().GetValue());
	}
	bool PopulateContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, FFrameTime MouseDownTime) override
	{
		return true;
	}
	void HandleMouseSelection(FHotspotSelectionManager& SelectionManager) override
	{
		if (SelectionManager.MouseEvent->GetEffectingButton() == EKeys::LeftMouseButton)
		{
			SelectionManager.SelectModelExclusive(LayerBar);
		}
	}
	FCursorReply GetCursor() const override
	{
		return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
	}
	int32 Priority() const override
	{
		return 10;
	}

	TSharedPtr<FLayerBarModel> LayerBar;
	TWeakPtr<FSequencer> WeakSequencer;
	EStretchConstraint StretchConstraint;
};

UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FLayerBarHotspot);
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(FLayerBarStretchHotspot);

class SSequencerLayerBarHandle : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSequencerLayerBarHandle) {}
		SLATE_ATTRIBUTE(const FSlateBrush*, BackgroundBrush)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FTrackAreaViewModel> InWeakTrackArea, TWeakPtr<SSequencerLayerBar> InWeakParent, EStretchConstraint InHandleType)
	{
		WeakTrackArea = InWeakTrackArea;
		WeakParent = InWeakParent;
		HandleType = InHandleType;

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(InArgs._BackgroundBrush)
		];
	}

	void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport))
		{
			SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
			return;
		}

		TSharedPtr<SSequencerLayerBar> LayerBarWidget = WeakParent.Pin();
		TSharedPtr<FSequencer> Sequencer = LayerBarWidget->GetSequencer();
		if (LayerBarWidget)
		{
			TSharedPtr<FLayerBarModel> LayerBar  = LayerBarWidget->GetLayerBarModel();
			if (LayerBar)
			{
				TSharedPtr<FTrackAreaViewModel> TrackArea = WeakTrackArea.Pin();
				TrackArea->AddHotspot(MakeShared<FLayerBarStretchHotspot>(LayerBar, HandleType, Sequencer));
			}
		}

		SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
	}

	void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		TSharedPtr<SSequencerLayerBar> LayerBarWidget = WeakParent.Pin();
		TSharedPtr<FLayerBarModel> LayerBar  = LayerBarWidget->GetLayerBarModel();
		if (LayerBarWidget)
		{
			TSharedPtr<FTrackAreaViewModel> TrackArea = WeakTrackArea.Pin();
			TrackArea->RemoveHotspot(FLayerBarStretchHotspot::ID);
		}

		SCompoundWidget::OnMouseLeave(MouseEvent);
	}

	TWeakPtr<FTrackAreaViewModel> WeakTrackArea;
	TWeakPtr<SSequencerLayerBar> WeakParent;
	EStretchConstraint HandleType;
};

void SSequencerLayerBar::Construct(const FArguments& InArgs, TWeakPtr<STrackAreaView> InWeakTrackArea, TWeakPtr<FSequencerEditorViewModel> InWeakEditor, TWeakPtr<FLayerBarModel> InWeakLayerBar)
{
	WeakLayerBar   = InWeakLayerBar;
	WeakEditor     = InWeakEditor;
	WeakTrackArea  = InWeakTrackArea;
	OverlayText    = InArgs._OverlayText;

	TSharedPtr<STrackAreaView> TrackArea = WeakTrackArea.Pin();

	ChildSlot
	[
		SNew(SOverlay)

		// Overlay text (always present, visible when text is non-empty)
		+ SOverlay::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(6.f, 0.f)
		[
			SNew(STextBlock)
			.Text(OverlayText)
			.Font(FAppStyle::GetFontStyle("NormalFont"))
			.ColorAndOpacity(FLinearColor::White)
			.ShadowOffset(FVector2D(1.f, 1.f))
			.ShadowColorAndOpacity(FLinearColor(0.f, 0.f, 0.f, 0.5f))
		]

		// Stretch handles (visible on hover)
		+ SOverlay::Slot()
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SSequencerLayerBar::GetHandleVisibility)

			+ SHorizontalBox::Slot()
			.Padding(FMargin(1.f, 2.f))
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(8.f)
				[
					SNew(SSequencerLayerBarHandle, TrackArea->GetViewModel(), SharedThis(this), EStretchConstraint::AnchorToEnd)
					.BackgroundBrush(FAppStyle::Get().GetBrush("Sequencer.LayerBar.HandleLeft"))
				]
			]

			+ SHorizontalBox::Slot()
			[
				SNew(SSpacer)
			]

			+ SHorizontalBox::Slot()
			.Padding(FMargin(1.f, 2.f))
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(8.f)
				[
					SNew(SSequencerLayerBarHandle, TrackArea->GetViewModel(), SharedThis(this), EStretchConstraint::AnchorToStart)
					.BackgroundBrush(FAppStyle::Get().GetBrush("Sequencer.LayerBar.HandleRight"))
				]
			]
		]
	];
}

TSharedPtr<FLayerBarModel> SSequencerLayerBar::GetLayerBarModel() const
{
	return WeakLayerBar.Pin();
}

TSharedPtr<FSequencer> SSequencerLayerBar::GetSequencer() const
{
	if (TSharedPtr<FSequencerEditorViewModel> Editor = WeakEditor.Pin())
	{
		return Editor->GetSequencerImpl();
	}
	return nullptr;
}

TSharedRef<const SWidget> SSequencerLayerBar::AsWidget() const
{
	return AsShared();
}

FTrackLaneScreenAlignment SSequencerLayerBar::GetAlignment(const ITrackLaneWidgetSpace& InScreenSpace, const FGeometry& InParentGeometry) const
{
	if (TSharedPtr<FLayerBarModel> LayerBar = WeakLayerBar.Pin())
	{
		return LayerBar->ArrangeVirtualTrackLaneView().ToScreen(InScreenSpace.GetScreenSpace(), InParentGeometry);
	}
	return FTrackLaneScreenAlignment{};
}

EVisibility SSequencerLayerBar::GetHandleVisibility() const
{
	return !CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport) && IsHovered()
		? EVisibility::Visible : EVisibility::Collapsed;
}

int32 SSequencerLayerBar::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const ESlateDrawEffect DrawEffects = bParentEnabled
		? ESlateDrawEffect::None
		: ESlateDrawEffect::DisabledEffect;

	static const FSlateBrush* LayerBarBackgroundBrush = FAppStyle::Get().GetBrush("Sequencer.LayerBar.Background");
	static const FSlateBrush* SelectedSectionOverlay  = FAppStyle::Get().GetBrush("Sequencer.Section.CollapsedSelectedSectionOverlay");

	TViewModelPtr<FLinkedOutlinerExtension> LayerBar = WeakLayerBar.Pin();
	TViewModelPtr<IOutlinerExtension>  OutlinerItem = LayerBar ? LayerBar->GetLinkedOutlinerItem() : nullptr;
	float TotalNodeHeight = OutlinerItem ? OutlinerItem->GetOutlinerSizing().GetTotalHeight() : 0.0f;
	// Draw the background
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(
			FVector2f(AllottedGeometry.GetLocalSize().X, TotalNodeHeight),
			FSlateLayoutTransform()
		),
		LayerBarBackgroundBrush,
		DrawEffects
	);
	++LayerId;

	TSharedPtr<FSequencer>		Sequencer		 = GetSequencer();
	FSequencerSelection&        Selection        = Sequencer->GetSelection();
	FSequencerSelectionPreview& SelectionPreview = Sequencer->GetSelectionPreview();

	ESelectionPreviewState SelectionPreviewState = SelectionPreview.GetSelectionState(WeakLayerBar);

	const bool bIsPreviewUnselected = SelectionPreviewState == ESelectionPreviewState::NotSelected;
	const bool bIsUnSelected        = SelectionPreviewState == ESelectionPreviewState::Undefined && !Selection.TrackArea.IsSelected(WeakLayerBar);

	if (!bIsPreviewUnselected && !bIsUnSelected)
	{
		FLinearColor SelectionColor = FAppStyle::GetSlateColor(SequencerSectionConstants::SelectionColorName).GetColor(FWidgetStyle());

		// Use a muted selection color for selection previews
		if (SelectionPreviewState == ESelectionPreviewState::Selected)
		{
			SelectionColor = SelectionColor.LinearRGBToHSV();
			SelectionColor.R += 0.1f; // +10% hue
			SelectionColor.G = 0.6f; // 60% saturation

			SelectionColor = SelectionColor.HSVToLinearRGB();
		}

		// Draw a selection highlight
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(FVector2f(AllottedGeometry.GetLocalSize().X - 2.f, TotalNodeHeight-3.f), FSlateLayoutTransform(FVector2f(1.f, 1.f))),
			SelectedSectionOverlay,
			DrawEffects,
			SelectionColor.CopyWithNewOpacity(0.8f)
		);
		++LayerId;
	}

	return SCompoundWidget::OnPaint(Args, AllottedGeometry.MakeChild(FVector2f(AllottedGeometry.GetLocalSize().X, TotalNodeHeight), FSlateLayoutTransform()), MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

FReply SSequencerLayerBar::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Unhandled();
}

void SSequencerLayerBar::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (CurveEditor::InputState::IsScrubTimeCommandPressed(FSequencerCommands::Get().ScrubTimeViewport))
	{
		SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
		return;
	}

	TSharedPtr<FLayerBarModel> LayerBar = WeakLayerBar.Pin();
	if (LayerBar)
	{
		TSharedPtr<FSequencer> Sequencer = GetSequencer();
		TSharedPtr<FTrackAreaViewModel> TrackArea = WeakTrackArea.Pin()->GetViewModel();
		TrackArea->AddHotspot(MakeShared<FLayerBarHotspot>(LayerBar, Sequencer));
	}

	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
}

FReply SSequencerLayerBar::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Unhandled();
}

void SSequencerLayerBar::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	TSharedPtr<FLayerBarModel> LayerBar = WeakLayerBar.Pin();
	if (LayerBar)
	{
		TSharedPtr<FTrackAreaViewModel> TrackArea = WeakTrackArea.Pin()->GetViewModel();
		TrackArea->RemoveHotspot(FLayerBarHotspot::ID);
	}

	SCompoundWidget::OnMouseLeave(MouseEvent);
}

} // namespace Sequencer
} // namespace UE

#undef LOCTEXT_NAMESPACE
