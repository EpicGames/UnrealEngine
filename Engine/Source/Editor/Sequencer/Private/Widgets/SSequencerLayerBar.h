// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Range.h"
#include "Misc/FrameNumber.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#include "MVVM/Extensions/ITrackLaneExtension.h"
#include "MVVM/Extensions/ViewModelExtensionCollection.h"

class ISequencer;
class FSequencer;
class FScopedTransaction;
class FSequencerDisplayNode;
struct FTimeToPixel;

namespace UE
{
namespace Sequencer
{

class FLayerBarModel;
class FSequencerEditorViewModel;
class FViewModel;
class ILayerBarExtension;
class STrackAreaView;

class SSequencerLayerBar
	: public SCompoundWidget
	, public ITrackLaneWidget
{
public:
	SLATE_BEGIN_ARGS(SSequencerLayerBar) {}
		SLATE_ATTRIBUTE(FText, OverlayText)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<STrackAreaView> InWeakTrackArea, TWeakPtr<FSequencerEditorViewModel> InWeakEditor, TWeakPtr<FLayerBarModel> InWeakLayerBar);

	TSharedPtr<FLayerBarModel> GetLayerBarModel() const;
	TSharedPtr<FSequencer> GetSequencer() const;

private:

	/*~ ITrackLaneWidget interface */
	TSharedRef<const SWidget> AsWidget() const override;
	FTrackLaneScreenAlignment GetAlignment(const ITrackLaneWidgetSpace& ScreenSpace, const FGeometry& InParentGeometry) const override;

	/*~ SCompoundWidget interface */
	int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;

	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	void OnMouseLeave(const FPointerEvent& MouseEvent) override;

	EVisibility GetHandleVisibility() const;

private:

	TWeakPtr<FLayerBarModel> WeakLayerBar;
	TWeakPtr<FSequencerEditorViewModel> WeakEditor;
	TWeakPtr<STrackAreaView> WeakTrackArea;
	TAttribute<FText> OverlayText;
};

} // namespace Sequencer
} // namespace UE

