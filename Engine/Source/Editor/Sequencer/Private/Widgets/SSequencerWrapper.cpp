// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerWrapper.h"

#include "SSequencer.h"
#include "Widgets/SNullWidget.h"

namespace UE::Sequencer
{
void SSequencerWrapperWidget::Construct(const FArguments& InArgs, const TSharedRef<SSequencer>& InSequencerWidget)
{
	SequencerWidget = InSequencerWidget;
	ChildSlot
	[
		SequencerWidget.ToSharedRef()
	];
}

void SSequencerWrapperWidget::DestroySequencerWidget()
{
	ChildSlot
	[
		SNullWidget::NullWidget
	];
	SequencerWidget.Reset();
}

void SSequencerWrapperWidget::EnablePendingFocusOnHovering(const bool InEnabled)
{
	if (SequencerWidget)
	{
		SequencerWidget->EnablePendingFocusOnHovering(InEnabled);
	}
}
}
