// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerWidget.h"
#include "SSequencer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SSequencer;

namespace UE::Sequencer
{
/** 
 * Wraps the SSequencer widget to manage its lifetime to the public API.
 * 
 * This allows the widget to be returned by ISequencer::GetSequenceWidget but allow SSequencer to be destroyed by ~FSequencer.
 * ~FSequencer will allow SSequencer to be destroyed even though API callers are still referencing SSequencerWrapperWidget.
 */
class SSequencerWrapperWidget : public SCompoundWidget, public ISequencerWidget
{
public:
	
	SLATE_BEGIN_ARGS(SSequencerWrapperWidget){}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedRef<SSequencer>& InSequencerWidget);
	
	/** Destroys the SSequencer widget */
	void DestroySequencerWidget();
	
	//~ Begin ISequencerWidget Interface
	virtual void EnablePendingFocusOnHovering(const bool InEnabled) override;
	//~ End ISequencerWidget Interface

private:
	
	/** The Sequencer widget. */
	TSharedPtr<SSequencer> SequencerWidget;
};
}

