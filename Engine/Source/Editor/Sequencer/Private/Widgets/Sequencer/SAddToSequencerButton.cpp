// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAddToSequencerButton.h"

#include "Sequencer.h"
#include "SequencerCommands.h"
#include "SPositiveActionButton.h"
#include "Filters/Menus/SequencerFilterMenuContext.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "MVVM/ViewModels/SequencerOutlinerViewModel.h"

#define LOCTEXT_NAMESPACE "SAddToSequencerButton"

class FSequencer;

namespace UE::Sequencer
{
void SAddToSequencerButton::Construct(const FArguments& InArgs)
{
	WeakSequencer = InArgs._Sequencer;
	ExtenderForAddMenuAttr = InArgs._ExtenderForAddMenu;
	
	ChildSlot
	[
		SNew(SPositiveActionButton)
		.OnGetMenuContent(this, &SAddToSequencerButton::MakeAddMenu)
		.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
		.Text(LOCTEXT("Add", "Add"))
		.IsEnabled_Lambda([this]()
		{
			const TSharedPtr<FSequencer> SequencerPin = WeakSequencer.Pin();
			return SequencerPin && !SequencerPin->IsReadOnly();
		})
	];
}

TSharedRef<SWidget> SAddToSequencerButton::MakeAddMenu()
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencer> SequencerPin = WeakSequencer.Pin();
	if (!SequencerPin)
	{
		return SNullWidget::NullWidget;
	}

	FMenuBuilder MenuBuilder(true, SequencerPin->GetCommandBindings(), ExtenderForAddMenuAttr.Get());

	if (SequencerPin->GetHostCapabilities().bSupportsAddFromContentBrowser)
	{
		MenuBuilder.AddMenuEntry(
			FSequencerCommands::Get().AddSelectionFromContentBrowser,
			NAME_None, TAttribute<FText>(), TAttribute<FText>(), 
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Use")
			);
	}

	FSequencerOutlinerViewModel* OutlinerViewModel = SequencerPin->GetViewModel()->GetOutliner()->CastThisChecked<FSequencerOutlinerViewModel>();
	OutlinerViewModel->BuildContextMenu(MenuBuilder);

	return MenuBuilder.MakeWidget();
}
} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE