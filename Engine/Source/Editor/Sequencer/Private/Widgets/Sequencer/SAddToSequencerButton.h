// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FExtender;
class FSequencer;

namespace UE::Sequencer
{
/** Implements the add button for adding tracks, etc. to Sequencer. */
class SAddToSequencerButton : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SAddToSequencerButton){}
		/** Used to add items to the sequencer instance, detecting whether it is read-only, etc. */
		SLATE_ARGUMENT(TWeakPtr<FSequencer>, Sequencer)
		/** External extenders for the add button menu. */
		SLATE_ATTRIBUTE(TSharedPtr<FExtender>, ExtenderForAddMenu)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	
private:
	
	/** Used to generate the menu content */
	TWeakPtr<FSequencer> WeakSequencer;
	
	/** External extenders for the add button menu. */
	TAttribute<TSharedPtr<FExtender>> ExtenderForAddMenuAttr;
	
	/** @return Add combo button content */
	TSharedRef<SWidget> MakeAddMenu();
};
} // namespace UE::Sequencer

