// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerSearchAndFilterRow.h"

#include "Filters/Menus/SequencerViewOptionsMenu.h"
#include "Filters/SequencerFilterBar.h"
#include "Filters/Utils/HideIsolateViewModelUtils.h"
#include "Filters/Utils/LinkedFilteringViewUtils.h"
#include "Filters/ViewModels/HideIsolateShowViewModel.h"
#include "Filters/Linking/Mode/ILinkedFilterViewModel.h"
#include "Filters/Linking/Mode/LinkedFilterViewModel.h"
#include "Filters/Widgets/Linking/SFilterLinkStateSwitcher.h"
#include "Filters/Widgets/Linking/SLinkedFilteringToggleButton.h"
#include "Filters/Widgets/SSearchAndFilterWidget.h"
#include "Filters/Widgets/SSequencerSearchBox.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/Views/SOutlinerView.h"
#include "SSequencer.h"
#include "Filters/Menus/SequencerTrackFilterMenu.h"
#include "Widgets/Sequencer/SAddToSequencerButton.h"
#include "Widgets/Sequencer/SSequencerViewOptionsButton.h"

#define LOCTEXT_NAMESPACE "SSearchAndFilterRow"

namespace UE::Sequencer
{
namespace FilterRowDetail
{
using FOutlinerExtensionsAttr = TAttribute<TSet<TWeakViewModelPtr<IOutlinerExtension>>>; 

static FOutlinerExtensionsAttr MakeSelectedTracksAttr(const TSharedRef<FSequencer>& InSequencer)
{
	return FOutlinerExtensionsAttr::CreateLambda([WeakSequencer = InSequencer.ToWeakPtr()]() -> TSet<TWeakViewModelPtr<IOutlinerExtension>>
	{
		const TSharedPtr<FSequencer> SequencerPin = WeakSequencer.Pin();
		if (!SequencerPin)
		{
			return {};
		}
			
		const TSharedPtr<FSequencerSelection> Selection = SequencerPin->GetViewModel()->GetSelection();
		if (!Selection)
		{
			return {};
		}
			
		return Selection->Outliner.GetSelected();
	});
}

static FOutlinerExtensionsAttr MakeAllTracksAttr(const TSharedRef<FSequencer>& InSequencer)
{
	return FOutlinerExtensionsAttr::CreateLambda([WeakSequencer = InSequencer.ToWeakPtr()]()
	{
		const TSharedPtr<FSequencer> SequencerPin = WeakSequencer.Pin();
		return SequencerPin ? GetAllTracks(*SequencerPin->GetViewModel()) : TSet<TWeakViewModelPtr<IOutlinerExtension>>();
	});
}

static FOnGetContent MakeDelegate_AddFilterMenuContent(const TSharedRef<FSequencerFilterBar>& InFilterBar)
{
	return FOnGetContent::CreateLambda([Menu = MakeShared<FSequencerTrackFilterMenu>(), WeakFilterBar = InFilterBar.ToWeakPtr()]
	{
		const TSharedPtr<FSequencerFilterBar> FilterBar = WeakFilterBar.Pin();
		return ensure(FilterBar) ? Menu->CreateMenu(FilterBar.ToSharedRef()) : SNullWidget::NullWidget;
	});
}
}
void SSequencerSearchAndFilterRow::Construct(const FArguments& InArgs)
{
	WeakSequencer = InArgs._Sequencer;
	SequencerFilterModel = InArgs._FilterViewModel;
	check(SequencerFilterModel);
	
	ChildSlot
	[
		SNew(SHorizontalBox)

		// Add Button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SAddToSequencerButton)
			.Sequencer(InArgs._Sequencer)
			.ExtenderForAddMenu(InArgs._ExtenderForAddMenu)
		]

		// Advanced Search Filter Combo Button
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		[
			MakeFilterContent(InArgs)
		]
		
		// View Options Combo Button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.f, 0.f, 0.f, 0.f)
		[
			SNew(SSequencerViewOptionsButton)
			.OnGetMenuContent_Lambda([this, ViewOptions = MakeShared<FSequencerViewOptionsMenu>()]
			{
				const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
				return Sequencer 
					? ViewOptions->CreateMenuForSequencer(WeakSequencer, Sequencer->GetCommandBindings().ToSharedRef()) 
					: SNullWidget::NullWidget;
			})
		]
	];
}

TSharedPtr<SSequencerSearchBox> SSequencerSearchAndFilterRow::GetActiveSearchBox() const
{
	static_assert(static_cast<int32>(ELinkedFilterMode::Count) == 2, "Update this switch");
	switch (SequencerFilterModel->GetFilterMode())
	{
	case ELinkedFilterMode::Linked: return LinkedFilterWidgets->GetSearchBox();
	case ELinkedFilterMode::Instanced: return InstancedFilterWidgets->GetSearchBox();
	default: checkNoEntry(); return nullptr;
	}
}

TSharedRef<SWidget> SSequencerSearchAndFilterRow::MakeFilterContent(const FArguments& InArgs)
{
	const TSharedPtr<FLinkedFilterViewModel> LinkedFilterViewModel = InArgs._FilterViewModel;
	const auto MakeWidget = [this, &LinkedFilterViewModel, &InArgs](const TSharedRef<FSequencerFilterBar>& InFilterBar)
	{
		const TSharedRef<FSequencer> SequencerPin = WeakSequencer.Pin().ToSharedRef();
		const TSharedRef<FHideIsolateShowViewModel> IsolateViewModel = MakeShared<FHideIsolateShowViewModel>(
			InFilterBar, InFilterBar->GetHideIsolateFilter(),
			FilterRowDetail::MakeSelectedTracksAttr(SequencerPin), FilterRowDetail::MakeAllTracksAttr(SequencerPin)
			);
		IsolateViewModel->OnRequestScrollToView().AddSP(this, &SSequencerSearchAndFilterRow::ScrollIntoView);

		const TSharedPtr<ISequencerTrackFilters> TrackFilters = StaticCastSharedPtr<ISequencerTrackFilters>(InFilterBar.ToSharedPtr());
		return SNew(SSearchAndFilterWidget, InFilterBar, IsolateViewModel)
			.OnMakeAddFilterMenuContent(FilterRowDetail::MakeDelegate_AddFilterMenuContent(InFilterBar))
			.OnSearchTextChanged_Lambda([InFilterBar](const FText& InSearchText)
			{
				InFilterBar->SetTextFilterString(InSearchText.ToString());
			})
			.BeforeSearchBox()
			[
				SNew(SLinkedFilteringToggleButton, LinkedFilterViewModel.ToSharedRef())
				.Visibility(InArgs._FilterToggleButtonVisibility)
			];
	};
	
	LinkedFilterWidgets = MakeWidget(LinkedFilterViewModel->GetLinkedFilterBarImpl());
	InstancedFilterWidgets = MakeWidget(LinkedFilterViewModel->GetInstancedFilterBarImpl());
	return SNew(SFilterLinkStateSwitcher)
		.LinkedFilterViewModel(InArgs._FilterViewModel)
		.LinkedContent() 
		[
			LinkedFilterWidgets.ToSharedRef()
		]
		.InstancedContent() 
		[
			InstancedFilterWidgets.ToSharedRef()
		];
}

void SSequencerSearchAndFilterRow::ScrollIntoView(const TWeakViewModelPtr<IOutlinerExtension>& InItem)
{
	if (const TSharedPtr<FSequencer> SequencerPin = WeakSequencer.Pin())
	{
		const TSharedRef<SSequencer> SequencerWidget = SequencerPin->GetUnderlyingSequencerWidget();
		SequencerWidget->GetTreeView()->RequestScrollIntoView(InItem);
	}
}
} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE