// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FExtender;
class FSequencer;
class FSequencerFilterBar;
class ISequencerTrackFilters;
class SSequencerSearchBox;

namespace UE::Sequencer
{
class FLinkedFilterViewModel;
class IOutlinerExtension;
class SSearchAndFilterWidget;

/** The row of widgets displayed above the Sequence tree view. */
class SSequencerSearchAndFilterRow : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SSequencerSearchAndFilterRow){}
		
		/** Used to generate the subwidgets */
		SLATE_ARGUMENT(TWeakPtr<FSequencer>, Sequencer)

		/** 
		 * Determines the filtering widgets displayed. 
		 * Some widgets are stateful, e.g. the search box keeps a search history that should be different between the modes.
		 */
		SLATE_ARGUMENT(TSharedPtr<FLinkedFilterViewModel>, FilterViewModel)

		/** The visibility of the toggle button that switches between linked and instanced modes.. */
		SLATE_ATTRIBUTE(EVisibility, FilterToggleButtonVisibility)
		
		/** External extenders for the add button menu. */
		SLATE_ATTRIBUTE(TSharedPtr<FExtender>, ExtenderForAddMenu)
		
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	
	/** @return The search box that is currently being displayed. */
	TSharedPtr<SSequencerSearchBox> GetActiveSearchBox() const;
	
private:
	
	/** Used to update Sequencer's filters. */
	TWeakPtr<FSequencer> WeakSequencer;

	/** Filter model for Sequencer. */
	TSharedPtr<FLinkedFilterViewModel> SequencerFilterModel;
	/** Content displayed when ELinkedFilterMode::Linked */
	TSharedPtr<SSearchAndFilterWidget> LinkedFilterWidgets;
	/** Content displayed when ELinkedFilterMode::Instanced */
	TSharedPtr<SSearchAndFilterWidget> InstancedFilterWidgets;
	
	/** @return The filtering widget. */
	TSharedRef<SWidget> MakeFilterContent(const FArguments& InArgs);
	
	/** Handles request to scroll the item into view. */
	void ScrollIntoView(const TWeakViewModelPtr<IOutlinerExtension>& InItem);
};
} // namespace UE::Sequencer

