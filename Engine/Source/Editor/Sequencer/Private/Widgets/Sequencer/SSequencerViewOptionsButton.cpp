// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequencerViewOptionsButton.h"

#include "Filters/Menus/SequencerViewOptionsMenu.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "SSequencerViewOptionsButton"

class FSequencer;

namespace UE::Sequencer
{
void SSequencerViewOptionsButton::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SComboButton)
		.ContentPadding(2.f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.ToolTipText(LOCTEXT("ViewOptionsToolTip", "View Options"))
		.ComboButtonStyle(FAppStyle::Get(), TEXT("SimpleComboButtonWithIcon"))
		.OnGetMenuContent(InArgs._OnGetMenuContent)
		.HasDownArrow(false)
		.ButtonContent()
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::Get().GetBrush(TEXT("Icons.Settings")))
		]
	];
}
} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE