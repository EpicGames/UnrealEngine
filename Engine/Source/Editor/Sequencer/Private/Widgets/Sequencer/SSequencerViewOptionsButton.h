// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::Sequencer
{
/** Implements view options for Sequencer. */
class SSequencerViewOptionsButton : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SSequencerViewOptionsButton){}
		/** Gets the content of the view options. */
		SLATE_EVENT(FOnGetContent, OnGetMenuContent)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
};
} // namespace UE::Sequencer

