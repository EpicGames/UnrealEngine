// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FrameNumberDetailsCustomization.h"
#include "IStructureDetailsView.h"
#include "Misc/NotifyHook.h"
#include "MovieScene.h"
#include "MovieSceneMarkedFrame.h"
#include "PropertyEditorModule.h"
#include "Sequencer.h"
#include "Widgets/SCompoundWidget.h"

class SMarkedFrameDetails : public SCompoundWidget, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SMarkedFrameDetails)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const int32 InMarkedFrameIndex, const TWeakPtr<FSequencer>& InWeakSequencer)
	{
		WeakSequencer = InWeakSequencer;

		const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
		if (!Sequencer.IsValid())
		{
			return;
		}

		const UMovieSceneSequence* const FocusedMovieSceneSequence = Sequencer->GetFocusedMovieSceneSequence();
		if (!IsValid(FocusedMovieSceneSequence))
		{
			return;
		}

		UMovieScene* const FocusedMovieScene = FocusedMovieSceneSequence->GetMovieScene();
		if (!IsValid(FocusedMovieScene))
		{
			return;
		}

		if (FocusedMovieScene->GetMarkedFrames().Num() == 0)
		{
			return;
		}

		WeakMovieSceneToModify = FocusedMovieScene;

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bShowScrollBar = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.NotifyHook = this;

		FStructureDetailsViewArgs StructureDetailsViewArgs;
		StructureDetailsViewArgs.bShowObjects = true;
		StructureDetailsViewArgs.bShowAssets = true;
		StructureDetailsViewArgs.bShowClasses = true;
		StructureDetailsViewArgs.bShowInterfaces = true;
		
		const TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(FMovieSceneMarkedFrame::StaticStruct()
			, (uint8*)&FocusedMovieScene->GetMarkedFrames()[InMarkedFrameIndex]);

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

		DetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureDetailsViewArgs, nullptr);

		DetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout(TEXT("FrameNumber"),
			FOnGetPropertyTypeCustomizationInstance::CreateSP(Sequencer.ToSharedRef(), &FSequencer::MakeFrameNumberDetailsCustomization));

		DetailsView->SetStructureData(StructOnScope);

		ChildSlot
		[
			DetailsView->GetWidget().ToSharedRef()
		];

		SetEnabled(!AreMarkedFramesLocked());
	}

	//~ Begin FNotifyHook

	virtual void NotifyPreChange(FProperty* InPropertyAboutToChange) override
	{
		if (WeakMovieSceneToModify.IsValid())
		{
			WeakMovieSceneToModify->Modify();
		}
	}

	virtual void NotifyPreChange(FEditPropertyChain* InPropertyAboutToChange) override
	{
		if (WeakMovieSceneToModify.IsValid())
		{
			WeakMovieSceneToModify->Modify();
		}
	}

	//~ End FNotifyHook

	bool AreMarkedFramesLocked() const
	{
		const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
		if (!Sequencer.IsValid())
		{
			return false;
		}

		if (Sequencer->IsReadOnly())
		{
			return true;
		}

		const UMovieSceneSequence* const FocusedMovieSceneSequence = Sequencer->GetFocusedMovieSceneSequence();
		if (IsValid(FocusedMovieSceneSequence))
		{
			const UMovieScene* const MovieScene = FocusedMovieSceneSequence->GetMovieScene();
			if (IsValid(MovieScene))
			{
				return MovieScene->IsReadOnly() ? true : MovieScene->AreMarkedFramesLocked();
			}
		}

		return false;
	};

private:
	TWeakObjectPtr<UMovieScene> WeakMovieSceneToModify;
	TWeakPtr<FSequencer> WeakSequencer;

	TSharedPtr<IStructureDetailsView> DetailsView;
};
