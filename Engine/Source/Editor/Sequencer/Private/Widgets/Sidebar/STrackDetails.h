// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailsView.h"
#include "Misc/NotifyHook.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "PropertyPermissionList.h"
#include "Sequencer.h"
#include "Widgets/SCompoundWidget.h"

struct FTrackDetailsWidgetCustomizations
{
	TMap<UStruct*, FOnGetDetailCustomizationInstance> DetailCustomizationInstances;
	TMap<FName, FOnGetPropertyTypeCustomizationInstance> PropertyTypeCustomizationInstances;
};

class STrackDetails : public SCompoundWidget, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(STrackDetails)
	{}
		SLATE_ARGUMENT(TOptional<FDetailsViewArgs>, ViewArgs)
		SLATE_ARGUMENT(FTrackDetailsWidgetCustomizations, Customizations)
		SLATE_ARGUMENT(bool, NotifyMovieSceneDataChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TArray<TWeakObjectPtr<>>& InWeakObjects, const TWeakPtr<FSequencer>& InWeakSequencer)
	{
		WeakObjectsToModify = InWeakObjects;
		WeakSequencer = InWeakSequencer;

		bNotifyMovieSceneDataChanged = InArgs._NotifyMovieSceneDataChanged;

		const TSharedPtr<FSequencer> Sequencer = InWeakSequencer.Pin();
		check(Sequencer.IsValid());

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

		FDetailsViewArgs DetailsViewArgs;
		if (InArgs._ViewArgs.IsSet())
		{
			DetailsViewArgs = InArgs._ViewArgs.GetValue();
		}
		else
		{
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.bCustomFilterAreaLocation = true;
			DetailsViewArgs.bCustomNameAreaLocation = true;
			DetailsViewArgs.bHideSelectionTip = true;
			DetailsViewArgs.bLockable = false;
			DetailsViewArgs.bSearchInitialKeyFocus = true;
			DetailsViewArgs.bUpdatesFromSelection = false;
			DetailsViewArgs.bShowOptions = false;
			DetailsViewArgs.bShowModifiedPropertiesOption = false;
			DetailsViewArgs.bShowScrollBar = false;
			DetailsViewArgs.NotifyHook = this;
			DetailsViewArgs.ColumnWidth = 0.45f;
		}

		DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

		for (const TPair<UStruct*, FOnGetDetailCustomizationInstance>& Instance : InArgs._Customizations.DetailCustomizationInstances)
		{
			DetailsView->RegisterInstancedCustomPropertyLayout(Instance.Key, Instance.Value);
		}
		for (const TPair<FName, FOnGetPropertyTypeCustomizationInstance>& Instance : InArgs._Customizations.PropertyTypeCustomizationInstances)
		{
			DetailsView->RegisterInstancedCustomPropertyTypeLayout(Instance.Key, Instance.Value);
		}

		DetailsView->SetIsPropertyVisibleDelegate(
			FIsPropertyVisible::CreateLambda([](const FPropertyAndParent& PropertyAndParent)
			{			
				return FPropertyEditorPermissionList::Get().DoesPropertyPassFilter(PropertyAndParent.Property.GetOwnerStruct(), PropertyAndParent.Property.GetFName());
			}));

		Sequencer->OnInitializeDetailsPanel().Broadcast(DetailsView.ToSharedRef(), Sequencer.ToSharedRef());
		DetailsView->SetObjects(InWeakObjects);

		ChildSlot
		[
			DetailsView.ToSharedRef()
		];

		SetEnabled(!Sequencer->IsReadOnly());
	}

	//~ Begin FNotifyHook

	virtual void NotifyPreChange(FProperty* const InPropertyAboutToChange) override
	{
		ModifyObjects();
	}

	virtual void NotifyPreChange(FEditPropertyChain* const InPropertyAboutToChange) override
	{
		ModifyObjects();
	}

	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* const InPropertyThatChanged) override
	{
		if (!bNotifyMovieSceneDataChanged)
		{
			return;
		}

		if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
		{
			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		}
	}

	//~ End FNotifyHook

	TSharedPtr<IDetailsView> GetDetailsView() const
	{
		return DetailsView;
	}

protected:
	void ModifyObjects()
	{
		for (const TWeakObjectPtr<>& WeakObject : WeakObjectsToModify)
		{
			if (WeakObject.IsValid())
			{
				WeakObject->Modify();
			}
		}
	}

	TArray<TWeakObjectPtr<>> WeakObjectsToModify;
	TWeakPtr<FSequencer> WeakSequencer;

	bool bNotifyMovieSceneDataChanged = false;

	TSharedPtr<IDetailsView> DetailsView;
};
