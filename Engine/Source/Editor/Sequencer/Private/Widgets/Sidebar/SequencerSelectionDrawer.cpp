// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerSelectionDrawer.h"
#include "Channels/MovieSceneChannelEditorData.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IKeyArea.h"
#include "ISequencerSection.h"
#include "Menus/CurveChannelSectionSidebarExtension.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneFolder.h"
#include "MVVM/Extensions/ISectionOwnerExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/ViewModels/DecorationOutlinerModel.h"
#include "MVVM/ViewModels/FolderModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SectionOutlinerModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "Sequencer.h"
#include "SequencerCommonHelpers.h"
#include "SequencerUtilities.h"
#include "SKeyEditInterface.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Sidebar/SMarkedFrameDetails.h"
#include "Widgets/Sidebar/STrackDetails.h"

#define LOCTEXT_NAMESPACE "SequencerSelectionDrawer"

namespace UE::Sequencer::Private
{
	FKeyEditData GetKeyEditData(const FKeySelection& InKeySelection)
	{
		if (InKeySelection.Num() == 1)
		{
			for (const FKeyHandle Key : InKeySelection)
			{
				if (const TSharedPtr<FChannelModel> Channel = InKeySelection.GetModelForKey(Key))
				{
					FKeyEditData KeyEditData;
					KeyEditData.KeyStruct     = Channel->GetKeyArea()->GetKeyStruct(Key);
					KeyEditData.OwningSection = Channel->GetSection();
					if (const FMovieSceneChannelMetaData* MetaData = Channel->GetKeyArea()->GetChannel().GetMetaData())
					{
						KeyEditData.OwningObject = MetaData->WeakOwningObject;
					}
					return KeyEditData;
				}
			}
		}
		else
		{
			TArray<FKeyHandle> KeyHandles;
			UMovieSceneSection* CommonSection = nullptr;
			for (FKeyHandle Key : InKeySelection)
			{
				TSharedPtr<FChannelModel> Channel = InKeySelection.GetModelForKey(Key);
				if (Channel.IsValid())
				{
					KeyHandles.Add(Key);
					if (!CommonSection)
					{
						CommonSection = Channel->GetSection();
					}
					else if (CommonSection != Channel->GetSection())
					{
						CommonSection = nullptr;
						break;
					}
				}
			}

			if (CommonSection)
			{
				FKeyEditData KeyEditData;
				KeyEditData.KeyStruct     = CommonSection->GetKeyStruct(KeyHandles);
				KeyEditData.OwningSection = CommonSection;
				return KeyEditData;
			}
		}

		return FKeyEditData();
	}

	TSharedPtr<FSequencerSelection> GetSelection(const ISequencer& InSequencer)
	{
		const TSharedPtr<FSequencerEditorViewModel> ViewModel = InSequencer.GetViewModel();
		if (!ViewModel.IsValid())
		{
			return nullptr;
		}

		return ViewModel->GetSelection();
	}
}

const FName FSequencerSelectionDrawer::UniqueId = TEXT("SequencerSelectionDrawer");

FSequencerSelectionDrawer::FSequencerSelectionDrawer(const TWeakPtr<FSequencer>& InWeakSequencer)
	: WeakSequencer(InWeakSequencer)
{
}

FSequencerSelectionDrawer::~FSequencerSelectionDrawer()
{
	UnbindEvents();
}

FName FSequencerSelectionDrawer::GetUniqueId() const
{
	return UniqueId;
}

FName FSequencerSelectionDrawer::GetSectionId() const
{
	return TEXT("Selection");
}

FText FSequencerSelectionDrawer::GetSectionDisplayText() const
{
	return LOCTEXT("SelectionDisplayText", "Selection");
}

TSharedRef<SWidget> FSequencerSelectionDrawer::CreateContentWidget()
{
	BindEvents();

	const TSharedRef<SScrollBox> OutWidget = SNew(SScrollBox)
		+ SScrollBox::Slot()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("NoBorder")))
			.Padding(0.f)
			[
				SAssignNew(ContentBox, SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(1.f)
				[
					CreateNoSelectionHintText()
				]
			]
		];

	ContentBox->RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &FSequencerSelectionDrawer::TickContentBox));

	RequestSelectionUpdate();

	return OutWidget;
}

void FSequencerSelectionDrawer::BindEvents()
{
	using namespace UE::Sequencer;

	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UnbindEvents();

	ActorAddedToSequencerDelegate = Sequencer->OnActorAddedToSequencer().AddLambda([this](AActor* InActor, const FGuid InGuid)
		{
			RequestSelectionUpdate();
		});

	MovieSceneDataChangedDelegate = Sequencer->OnMovieSceneDataChanged().AddLambda([this](const EMovieSceneDataChangeType InChangeType)
		{
			// Update when a channel is overriden with a curve extension
			if (InChangeType == EMovieSceneDataChangeType::MovieSceneStructureItemsChanged)
			{
				RequestSelectionUpdate();
			}
		});

	if (const TSharedPtr<FSequencerSelection> SequencerSelection = UE::Sequencer::Private::GetSelection(*Sequencer.Get()))
	{
		SequencerSelection->OnChanged.AddSP(this, &FSequencerSelectionDrawer::RequestSelectionUpdate);
	}
}

void FSequencerSelectionDrawer::UnbindEvents()
{
	using namespace UE::Sequencer;

	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	if (ActorAddedToSequencerDelegate.IsValid())
	{
		Sequencer->OnActorAddedToSequencer().Remove(ActorAddedToSequencerDelegate);
		ActorAddedToSequencerDelegate.Reset();
	}
	if (MovieSceneDataChangedDelegate.IsValid())
	{
		Sequencer->OnMovieSceneDataChanged().Remove(MovieSceneDataChangedDelegate);
		MovieSceneDataChangedDelegate.Reset();
	}

	if (const TSharedPtr<FSequencerSelection> SequencerSelection = UE::Sequencer::Private::GetSelection(*Sequencer.Get()))
	{
		SequencerSelection->OnChanged.RemoveAll(this);
	}
}

void FSequencerSelectionDrawer::ResetContent()
{
	if (ContentBox.IsValid())
	{
		ContentBox->ClearChildren();
	}

	CurveChannelExtension.Reset();

	ChannelExtensions.Reset();
}

EActiveTimerReturnType FSequencerSelectionDrawer::TickContentBox(const double InCurrentTime, const float InDeltaTime)
{
	if (ContentBox.IsValid() && bRequestSelectionUpdate)
	{
		bRequestSelectionUpdate = false;
		UpdateFromSelectionNextFrame();
	}

	return EActiveTimerReturnType::Continue;
}

void FSequencerSelectionDrawer::RequestSelectionUpdate()
{
	bRequestSelectionUpdate = true;
}

void FSequencerSelectionDrawer::UpdateFromSelectionNextFrame()
{
	using namespace UE::Sequencer;

	ResetContent();

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencerSelection> SequencerSelection = UE::Sequencer::Private::GetSelection(*Sequencer.Get());
	if (!SequencerSelection.IsValid())
	{
		return;
	}

	const TSharedRef<FSequencerSelection> SelectionRef = SequencerSelection.ToSharedRef();

	auto AddToContent = [this](const TSharedRef<SWidget>& InWidget)
		{
			if (ContentBox.IsValid())
			{
				ContentBox->AddSlot()
					.AutoHeight()
					[
						InWidget
					];
			}
		};

	if (SelectionRef->Outliner.Num() > 0
		|| SelectionRef->TrackArea.Num() > 0
		|| SelectionRef->KeySelection.Num() > 0
		|| SelectionRef->MarkedFrames.Num() > 0)
	{
		const ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>(TEXT("Sequencer"));
		const TSharedPtr<FExtensibilityManager> SidebarExtensibilityManager = SequencerModule.GetSidebarExtensibilityManager();

		FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/false
			, Sequencer->GetCommandBindings()
			, SidebarExtensibilityManager->GetAllExtenders()
			, /*bInCloseSelfOnly=*/true, &FCoreStyle::Get(), /*bInSearchable=*/true, TEXT("Sequencer.Sidebar"));

		/**
		 * Selection details display order preference:
		 *  1) Key items
		 *  2) Track area items (if no key selected)
		 *  3) Outliner items (if no key or track area selected)
		 *  4) Marked frames
		 */

		// 1) Key items
		BuildKeySelectionDetails(SelectionRef, MenuBuilder);

		// Early out for key selections
		const bool bIsKeySelected = SequencerSelection->KeySelection.Num() > 0;
		if (bIsKeySelected)
		{
			AddToContent(MenuBuilder.MakeWidget());
			return;
		}

		// 2) Track area items
		BuildTrackAreaDetails(SelectionRef, MenuBuilder);

		// 3) Outliner items
		const bool bIsTrackAreaSelected = SequencerSelection->TrackArea.Num() > 0;
		if (!bIsTrackAreaSelected)
		{
			BuildOutlinerDetails(SelectionRef, MenuBuilder);
		}

		// 4) Marked frames
		BuildMarkedFrameDetails(SelectionRef, MenuBuilder);

		AddToContent(MenuBuilder.MakeWidget());
	}
	else
	{
		AddToContent(CreateNoSelectionHintText());
	}
}

void FSequencerSelectionDrawer::BuildKeySelectionDetails(const TSharedRef<UE::Sequencer::FSequencerSelection>& InSelection, FMenuBuilder& MenuBuilder)
{
	if (InSelection->KeySelection.Num() == 0)
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	MenuBuilder.BeginSection(TEXT("KeyEdit"), LOCTEXT("KeyEditMenuSection", "Key Edit"));
	{
		MenuBuilder.AddWidget(CreateKeyFrameDetails(InSelection).ToSharedRef(), FText::GetEmpty(), /*bInNoIndent=*/true);
	}
	MenuBuilder.EndSection();
}

void FSequencerSelectionDrawer::BuildTrackAreaDetails(const TSharedRef<UE::Sequencer::FSequencerSelection>& InSelection, FMenuBuilder& MenuBuilder)
{
	using namespace UE::Sequencer;

	TArray<TWeakObjectPtr<>> AllSectionObjects; 

	for (const FViewModelPtr TrackAreaItem : InSelection->TrackArea)
	{
		if (const TViewModelPtr<FSectionModel> SectionModel = TrackAreaItem.ImplicitCast())
		{
			UMovieSceneSection* const Section = SectionModel->GetSection();
			if (IsValid(Section))
			{
				AllSectionObjects.Add(SectionModel->GetSection());
			}
		}
	}

	if (!AllSectionObjects.IsEmpty())
	{
		SequencerHelpers::BuildEditSectionMenu(WeakSequencer, AllSectionObjects, MenuBuilder, false);
	}
}

void FSequencerSelectionDrawer::BuildOutlinerDetails(const TSharedRef<UE::Sequencer::FSequencerSelection>& InSelection, FMenuBuilder& MenuBuilder)
{
	using namespace UE::Sequencer;

	if (InSelection->Outliner.Num() == 0)
	{
		return;
	}

	const TSharedPtr<FSequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	TArray<TWeakObjectPtr<>> WeakFolderObjects;
	TArray<TWeakObjectPtr<>> WeakSectionObjects;
	TArray<TWeakObjectPtr<>> WeakTrackObjects;
	TArray<TWeakObjectPtr<>> WeakDecorationObjects;
	TSet<TViewModelPtr<FObjectBindingModel>> ObjectBindings;
	TSet<TViewModelPtr<FChannelGroupOutlinerModel>> ChannelGroups;
	TArray<TPair<TWeakObjectPtr<UMovieSceneTrack>, int32>> SelectedTrackRows;

	bool bIsSectionOutliner = false;
	for (const FViewModelPtr OutlinerItem : InSelection->Outliner)
	{
		if (const TViewModelPtr<ISectionOwnerExtension> SectionOwner = OutlinerItem.ImplicitCast())
		{
			WeakSectionObjects.Append(SectionOwner->GetSections());
		}

		if (const TViewModelPtr<ITrackExtension> TrackExtension = OutlinerItem.ImplicitCast())
		{
			UMovieSceneTrack* Track = TrackExtension->GetTrack();
			if (IsValid(Track))
			{
				WeakTrackObjects.Add(Track);
				// Only add a 'track row' as selected if we have a track selected and there's only a single track row, and the track allows multiple rows.
				if (Track->SupportsMultipleRows() && Track->GetMaxRowIndex() == 0)
				{
					SelectedTrackRows.Add(TPair<TWeakObjectPtr<UMovieSceneTrack>, int32>(Track, 0));
				}
			}
		}
		else if (const TViewModelPtr<ITrackRowExtension> TrackRowExtension = OutlinerItem.ImplicitCast())
		{
			UMovieSceneTrack* Track = TrackRowExtension->GetParentTrack();
			if (IsValid(Track))
			{
				WeakTrackObjects.Add(Track);
				SelectedTrackRows.Add(TPair<TWeakObjectPtr<UMovieSceneTrack>, int32>(Track, TrackRowExtension->GetRowIndex()));
			}
		}
		else if (const TViewModelPtr<FObjectBindingModel> ObjectBindingModel = OutlinerItem.ImplicitCast())
		{
			ObjectBindings.Add(ObjectBindingModel);
			SequencerHelpers::BuildObjectBindingMenu(WeakSequencer, ObjectBindingModel, MenuBuilder);
		}
		else if (const TViewModelPtr<FFolderModel> FolderModel = OutlinerItem.ImplicitCast())
		{
			WeakFolderObjects.Add(FolderModel->GetFolder());
		}
		// Ex. "Location.X", "Rotation.Roll", "Color.R", etc.
        else if (const TViewModelPtr<FChannelGroupOutlinerModel> ChannelGroupOutlinerModel = OutlinerItem.ImplicitCast())
        {
        	ChannelGroups.Add(ChannelGroupOutlinerModel);
        }
		// Handle per-decoration outliner row - check before FSectionOutlinerModel
		else if (const TViewModelPtr<FDecorationOutlinerModel> DecorationOutlinerModel = OutlinerItem.ImplicitCast())
		{
			if (UObject* Decoration = DecorationOutlinerModel->GetDecoration())
			{
				WeakDecorationObjects.AddUnique(Decoration);
			}
		}
		else if (const TViewModelPtr<FSectionOutlinerModel> SectionOutlinerModel = OutlinerItem.ImplicitCast())
		{
			bIsSectionOutliner = true;
		}
	}

	if (!WeakFolderObjects.IsEmpty())
	{
		const TSharedRef<STrackDetails> TrackDetails = SNew(STrackDetails, WeakFolderObjects, Sequencer.ToSharedRef())
			.NotifyMovieSceneDataChanged(true);
		MenuBuilder.AddWidget(TrackDetails, FText::GetEmpty(), true);
	}

	if (!ObjectBindings.IsEmpty())
	{
		MenuBuilder.BeginSection(TEXT("Possessable"));
		MenuBuilder.EndSection();

		// Shows duplicate information as above?
		//MenuBuilder.BeginSection(TEXT("CustomBinding"));
		//MenuBuilder.EndSection();
	}

	if (!ChannelGroups.IsEmpty())
	{
		BuildExtensionDetails(ChannelGroups, MenuBuilder);
	}
	

	if (!SelectedTrackRows.IsEmpty() && !Algo::AnyOf(SelectedTrackRows, [](const TPair<TWeakObjectPtr<UMovieSceneTrack>, int32> TrackRow) {
		return TrackRow.Key.IsValid() && !TrackRow.Key->SupportsMultipleRows();
		}))
	{
		MenuBuilder.BeginSection(TEXT("TrackRowMetadata"));
		{
			// Empty here, will be implemented by extension.
		}
		MenuBuilder.EndSection();
	}

	if (!WeakTrackObjects.IsEmpty())
	{
		SequencerHelpers::BuildEditTrackMenu(WeakSequencer, WeakTrackObjects, MenuBuilder, false);
	}

	// Show decoration properties before section properties so decoration class headers appear above their section content
	if (!WeakDecorationObjects.IsEmpty())
	{
		SequencerHelpers::BuildEditDecorationMenu(WeakSequencer, WeakDecorationObjects, MenuBuilder);
	}

	// Only show section details for a selected track/trackrow if a single infinite section on the row or a section outliner
	if (WeakSectionObjects.Num() == 1 && WeakSectionObjects[0].IsValid())
	{
		if (UMovieSceneSection* Section = Cast<UMovieSceneSection>(WeakSectionObjects[0].Get()))
		{
			if (bIsSectionOutliner || (!Section->GetRange().HasLowerBound() && !Section->GetRange().HasUpperBound()))
			{
				SequencerHelpers::BuildEditSectionMenu(WeakSequencer, WeakSectionObjects, MenuBuilder, false);
			}
		}
	}
}

void FSequencerSelectionDrawer::BuildMarkedFrameDetails(const TSharedRef<UE::Sequencer::FSequencerSelection>& InSelection, FMenuBuilder& MenuBuilder)
{
	if (InSelection->MarkedFrames.Num() == 0)
	{
		return;
	}

	MenuBuilder.BeginSection(TEXT("MarkedFrames"), LOCTEXT("MarkedFramesMenuSection", "Marked Frames"));

	for (const int32 MarkIndex : InSelection->MarkedFrames)
	{
		const TSharedRef<SMarkedFrameDetails> MarkedFrameDetails = SNew(SMarkedFrameDetails, MarkIndex, WeakSequencer);
		MenuBuilder.AddWidget(MarkedFrameDetails, FText::GetEmpty(), /*bInNoIndent=*/true);
	}

	MenuBuilder.EndSection();
}

void FSequencerSelectionDrawer::BuildExtensionDetails(const TSet<UE::Sequencer::TViewModelPtr<UE::Sequencer::FChannelGroupOutlinerModel>>& InChannelGroups, FMenuBuilder& MenuBuilder)
{
	using namespace UE::Sequencer;

	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>(TEXT("Sequencer"));
	const TSharedPtr<FExtensibilityManager> SidebarExtensibilityManager = SequencerModule.GetSidebarExtensibilityManager();
	const TSharedPtr<FExtender> Extender = SidebarExtensibilityManager->GetAllExtenders();

	TArray<FName> ChannelTypeNames;
	TArray<ISequencerChannelInterface*> ChannelInterfaces;
	TArray<FMovieSceneChannelHandle> ChannelHandles;
	TArray<TWeakObjectPtr<UMovieSceneSection>> WeakSceneSections;

	for (const TViewModelPtr<FChannelGroupOutlinerModel>& ChannelModel : InChannelGroups)
	{
		for (const TSharedRef<IKeyArea>& KeyArea : ChannelModel->GetAllKeyAreas())
		{
			if (ISequencerChannelInterface* const SequencerChannelIterface = KeyArea->FindChannelEditorInterface())
			{
				const FMovieSceneChannelHandle& Channel = KeyArea->GetChannel();

				ChannelTypeNames.Add(Channel.GetChannelTypeName());
				ChannelInterfaces.Add(SequencerChannelIterface);
				ChannelHandles.Add(Channel);
				WeakSceneSections.Add(KeyArea->GetOwningSection());
			}
		}
	}

	// Need to make sure all channels are the same type to allow editing of multiple channels as one
	bool bAllChannelNamesEqual = AreAllSameNames(ChannelTypeNames);

	// Channel Interface Extensions (Perlin Noise, Easing, Wave)
	if (ChannelInterfaces.Num() > 0)
	{
		if (bAllChannelNamesEqual)
		{
			if (const TSharedPtr<ISidebarChannelExtension> ChannelExtension = ChannelInterfaces[0]->ExtendSidebarMenu_Raw(MenuBuilder, Extender, ChannelHandles, WeakSceneSections, WeakSequencer))
			{
				ChannelExtensions.Add(ChannelExtension);
			}
		}
		else
		{
			// Display different channels separately and don't allow to edit "all-in-one"
			for (int32 Index = 0; Index < ChannelInterfaces.Num(); ++Index)
			{
				if (const TSharedPtr<ISidebarChannelExtension> ChannelExtension = ChannelInterfaces[Index]->ExtendSidebarMenu_Raw(MenuBuilder, Extender, { ChannelHandles[Index] }, { WeakSceneSections[Index] }, WeakSequencer))
				{
					ChannelExtensions.Add(ChannelExtension);
				}
			}
		}
	}

	// Curve Channel Options (Pre-Finity, Post-Finity, etc.)
	CurveChannelExtension = MakeShared<FCurveChannelSectionSidebarExtension>(Sequencer);
	CurveChannelExtension->AddSections(WeakSceneSections);
	CurveChannelExtension->ExtendMenu(MenuBuilder, false);
}

TSharedRef<SWidget> FSequencerSelectionDrawer::CreateHintText(const FText& InMessage)
{
	return SNew(SBox)
		.HAlign(HAlign_Center)
		.Padding(2.f, 12.f, 2.f, 12.f)
		[
			SNew(STextBlock)
			.Text(InMessage)
			.TextStyle(FAppStyle::Get(), "HintText")
		];
}

TSharedRef<SWidget> FSequencerSelectionDrawer::CreateNoSelectionHintText()
{
	return CreateHintText(LOCTEXT("NoSelection", "Select an object to view details."));
}

FKeyEditData FSequencerSelectionDrawer::GetKeyEditData() const
{
	using namespace UE::Sequencer;

	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return FKeyEditData();
	}

	const TSharedPtr<FSequencerSelection> SequencerSelection = UE::Sequencer::Private::GetSelection(*Sequencer.Get());
	if (!SequencerSelection.IsValid())
	{
		return FKeyEditData();
	}

	return UE::Sequencer::Private::GetKeyEditData(SequencerSelection->KeySelection);
}

TSharedPtr<SWidget> FSequencerSelectionDrawer::CreateKeyFrameDetails(const TSharedRef<UE::Sequencer::FSequencerSelection>& InSequencerSelection)
{
	using namespace UE::Sequencer;

	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return nullptr;
	}

	const FKeyEditData KeyEditData = UE::Sequencer::Private::GetKeyEditData(InSequencerSelection->KeySelection);
	if (KeyEditData.KeyStruct.IsValid())
	{
		return SNew(SKeyEditInterface, Sequencer.ToSharedRef())
			.EditData(this, &FSequencerSelectionDrawer::GetKeyEditData);
	}

	return CreateHintText(LOCTEXT("InvalidKeyCombination", "Selected keys must belong to the same section."));
}

bool FSequencerSelectionDrawer::AreAllSameNames(const TArray<FName>& InNames) const
{
	for (int32 Index = 0; Index < InNames.Num(); ++Index)
	{
		if (Index > 0 && InNames[Index] != InNames[0])
		{
			return false;
		}
	}
	return true;
}

#undef LOCTEXT_NAMESPACE
