// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelPtr.h"
#include "Sidebar/ISidebarDrawerContent.h"
#include "Templates/SharedPointerFwd.h"

class FMenuBuilder;
class FName;
class FSequencer;
class FText;
class ISidebarChannelExtension;
class SVerticalBox;
class SWidget;
enum class EActiveTimerReturnType : uint8;
struct FCurveChannelSectionSidebarExtension;
struct FKeyEditData;

namespace UE::Sequencer
{
	class FChannelGroupOutlinerModel;
	class FSequencerSelection;
}

class FSequencerSelectionDrawer : public ISidebarDrawerContent
{
public:
	static const FName UniqueId;

	FSequencerSelectionDrawer(const TWeakPtr<FSequencer>& InWeakSequencer);
	virtual ~FSequencerSelectionDrawer() override;

	//~ Begin ISidebarDrawerContent
	virtual FName GetUniqueId() const override;
	virtual FName GetSectionId() const override;
	virtual FText GetSectionDisplayText() const override;
	virtual TSharedRef<SWidget> CreateContentWidget() override;
	//~ End ISidebarDrawerContent

protected:
	void BindEvents();
	void UnbindEvents();

	EActiveTimerReturnType TickContentBox(const double InCurrentTime, const float InDeltaTime);

	void RequestSelectionUpdate();
	void UpdateFromSelectionNextFrame();

	void BuildKeySelectionDetails(const TSharedRef<UE::Sequencer::FSequencerSelection>& InSelection, FMenuBuilder& MenuBuilder);
	void BuildTrackAreaDetails(const TSharedRef<UE::Sequencer::FSequencerSelection>& InSelection, FMenuBuilder& MenuBuilder);
	void BuildOutlinerDetails(const TSharedRef<UE::Sequencer::FSequencerSelection>& InSelection, FMenuBuilder& MenuBuilder);
	void BuildMarkedFrameDetails(const TSharedRef<UE::Sequencer::FSequencerSelection>& InSelection, FMenuBuilder& MenuBuilder);
	void BuildExtensionDetails(const TSet<UE::Sequencer::TViewModelPtr<UE::Sequencer::FChannelGroupOutlinerModel>>& InChannelGroups, FMenuBuilder& MenuBuilder);

	TSharedRef<SWidget> CreateHintText(const FText& InMessage);
	TSharedRef<SWidget> CreateNoSelectionHintText();

	FKeyEditData GetKeyEditData() const;
	
	TSharedPtr<SWidget> CreateKeyFrameDetails(const TSharedRef<UE::Sequencer::FSequencerSelection>& InSequencerSelection);

	void ResetContent();

	bool AreAllSameNames(const TArray<FName>& InNames) const;

	TWeakPtr<FSequencer> WeakSequencer;

	TSharedPtr<SVerticalBox> ContentBox;

	TSharedPtr<FCurveChannelSectionSidebarExtension> CurveChannelExtension;

	TArray<TSharedPtr<ISidebarChannelExtension>> ChannelExtensions;

	bool bRequestSelectionUpdate = false;

	FDelegateHandle ActorAddedToSequencerDelegate;
	FDelegateHandle MovieSceneDataChangedDelegate;
};
