// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


/** Helper class that automates releasing of acquired resources */
class FAcquiredResources
{
public:

	/** Default construction */
	FAcquiredResources() = default;

	/** Copy construction is disabled */
	FAcquiredResources(const FAcquiredResources&) = delete;
	FAcquiredResources& operator=(const FAcquiredResources&) = delete;

	/** Move construction/assignment implies a transfer of ownership of the acquired resources */
	FAcquiredResources(FAcquiredResources&&) = default;
	FAcquiredResources& operator=(FAcquiredResources&&) = default;

	/**
	 * Destructor that releases any acquired resources
	 */
	~FAcquiredResources()
	{
		Release();
	}

	/**
	 * Add an acquired resource to this container by providing its releaser function
	 *
	 * @param InReleaser 	A releaser function of the signature void() that defines how to release the resource
	 */
	template<typename T>
	void Add(T&& InReleaser)
	{
		Releasers.Add(FReleaser{ MoveTemp(InReleaser), NAME_None });
	}

	/**
	 * Add a named resource to this container by providing its releaser function
	 *
	 * @param InIdentifier 	Identifier for this resource
	 * @param InReleaser 	A releaser function of the signature void() that defines how to release the resource
	 */
	template<typename T>
	void Add(FName InIdentifier, T&& InReleaser)
	{
		Releasers.Add(FReleaser{ MoveTemp(InReleaser), InIdentifier });
	}

	/**
	 * Release all acquired resources in reverse order
	 */
	void Release()
	{
		for (int32 Index = Releasers.Num() - 1; Index >= 0; --Index)
		{
			Releasers[Index].Callback();
		}

		Releasers.Empty();
	}

	/**
	 * Release the resource(s) that correspond to the specified identifier
	 *
	 * @param InIdentifier 	The identifier of the resource(s) to release
	 */
	void Release(FName InIdentifier)
	{
		check(InIdentifier != NAME_None);
		for (int32 Index = Releasers.Num() - 1; Index >= 0; --Index)
		{
			if (Releasers[Index].Identifier == InIdentifier)
			{
				Releasers[Index].Callback();
				Releasers.RemoveAtSwap(Index, EAllowShrinking::No);
			}
		}
	}

private:

	/** Private array of releaser data */
	struct FReleaser
	{
		TFunction<void()> Callback;
		FName Identifier;
	};
	TArray<FReleaser> Releasers;
};
