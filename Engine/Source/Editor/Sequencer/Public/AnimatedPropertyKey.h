// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/UnrealType.h"
#include "Templates/SubclassOfField.h"

struct FAnimatedPropertyKey
{
	/**
	 * The name of the type of property that can be animated (i.e. FBoolProperty)
	 */
	FName PropertyTypeName;

	/**
	 * The name of the type of object that can be animated inside the property (i.e. the name of the struct or object for FStructProperty or FObjectProperty). NAME_None for any properties.
	 */
	FName ObjectTypeName;

	friend uint32 GetTypeHash(FAnimatedPropertyKey InKey)
	{
		return GetTypeHash(InKey.PropertyTypeName) ^ GetTypeHash(InKey.ObjectTypeName);
	}

	friend bool operator==(FAnimatedPropertyKey A, FAnimatedPropertyKey B)
	{
		return A.PropertyTypeName == B.PropertyTypeName && A.ObjectTypeName == B.ObjectTypeName;
	}

	static FAnimatedPropertyKey FromProperty(const FProperty* Property)
	{
		FAnimatedPropertyKey Definition;
		Definition.PropertyTypeName = Property->GetClass()->GetFName();

		if (const FStructProperty* StructProperty = CastField<const FStructProperty>(Property))
		{
			Definition.ObjectTypeName = StructProperty->Struct->GetFName();
		}
		else if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property))
		{
			if (ObjectProperty->PropertyClass)
			{
				Definition.ObjectTypeName = ObjectProperty->PropertyClass->GetFName();
			}
		}
		else if(const FArrayProperty* ArrayProperty = CastField<const FArrayProperty>(Property))
		{
			Definition.PropertyTypeName = ArrayProperty->Inner->GetClass()->GetFName();
			if (const FStructProperty* InnerStructProperty = CastField<const FStructProperty>(ArrayProperty->Inner))
			{
				Definition.ObjectTypeName = InnerStructProperty->Struct->GetFName();
			}
		}
		return Definition;
	}

	static FAnimatedPropertyKey FromObjectType(const UClass* Class)
	{
		FAnimatedPropertyKey Definition;
		Definition.PropertyTypeName = NAME_ObjectProperty;
		Definition.ObjectTypeName = Class->GetFName();
		return Definition;
	}

	static FAnimatedPropertyKey FromStructType(const UStruct* Struct)
	{
		check(Struct);
		return FromStructType(Struct->GetFName());
	}

	static FAnimatedPropertyKey FromStructType(FName StructName)
	{
		FAnimatedPropertyKey Definition;
		Definition.PropertyTypeName = NAME_StructProperty;
		Definition.ObjectTypeName = StructName;
		return Definition;
	}

	static FAnimatedPropertyKey FromPropertyTypeName(FName PropertyTypeName)
	{
		FAnimatedPropertyKey Definition;
		Definition.PropertyTypeName = PropertyTypeName;
		return Definition;
	}

	static FAnimatedPropertyKey FromPropertyType(TSubclassOfField<FProperty> PropertyType)
	{
		FAnimatedPropertyKey Definition;
		Definition.PropertyTypeName = PropertyType->GetFName();
		return Definition;
	}

private:

	FAnimatedPropertyKey()
		: PropertyTypeName(NAME_None)
		, ObjectTypeName(NAME_None)
	{}
};
