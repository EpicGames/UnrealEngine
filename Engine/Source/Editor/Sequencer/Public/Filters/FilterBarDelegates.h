// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "MVVM/ViewModelPtr.h"

namespace UE::Sequencer
{
class IOutlinerExtension;

DECLARE_MULTICAST_DELEGATE_TwoParams(FChangeOutlinerExtensionFilterState, const TViewModelPtr<IOutlinerExtension>&, bool bIsFilteredOut);
}