// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"

namespace UE::Sequencer
{
/** Describes what a filter wants to happen to an item. */
enum class EItemFilterState : uint8
{
	/** The item should be displayed to the user. */
	Include,
	/** The item should not be displayed to the user. */
	Exclude,
	/** The filter does not care or is not applicable. Let other filters decide. */
	DoNotCare
};

/** @return Exclude if either is set to exclude, Include if both are set to Include, and DoNotCare otherwise. */
EItemFilterState CombineConjunctive(EItemFilterState InLeft, EItemFilterState InRight);
/** @return Include if either is set to Include, Exclude if both are set to Exclude, and DoNotCare otherwise. */
EItemFilterState CombineDisjunctive(EItemFilterState InLeft, EItemFilterState InRight);

/** @return The result of combining any number of EItemFilterState values, folding left using the 2-param Combine. */
template<typename... TStates>
EItemFilterState CombineConjunctive(EItemFilterState InFirst, TStates... InRest);
/** @return The result of combining any number of EItemFilterState values, folding left using the 2-param Combine. */
template<typename... TStates>
EItemFilterState CombineDisjunctive(EItemFilterState InFirst, TStates... InRest);

/** @return Whether an item should be displayed */
bool PassesFilterState(EItemFilterState InState);

/** Additional info filters may want to return after having evaluated an item. */
enum class EFilterResultFlags : uint8
{
	None,
	
	/** 
	 * Apply EItemFilterState to all children. 
	 * Example: 
	 * - Suppose EItemFilterState::Include is returned on "Cube.Transform" 
	 * - Also assume EItemFilterState::Include for "Cube.Transform.Location", "Cube.Transform.Location.X", etc. 
	 * This allows filter optimization to optimize needless evaluation.
	 */
	ApplyResultToChildren = 1 << 0
};
ENUM_CLASS_FLAGS(EFilterResultFlags);

/** The result of evaluating a filter. */
struct FFilterResult
{
	/** What should happen to the item state */
	EItemFilterState ItemState;
	
	/** Additional information the filtered returned, e.g. that the filter need not be invoked for children. */
	EFilterResultFlags Flags;

	FFilterResult(EItemFilterState ItemState, EFilterResultFlags Flags = EFilterResultFlags::None)
		: ItemState(ItemState)
		, Flags(Flags)
	{}
};
}

namespace UE::Sequencer
{
inline EItemFilterState CombineConjunctive(EItemFilterState InLeft, EItemFilterState InRight)
{
	switch (InLeft)
	{
	case EItemFilterState::Include: return InRight == EItemFilterState::Exclude ? EItemFilterState::Exclude : InLeft;
	case EItemFilterState::Exclude: return EItemFilterState::Exclude;
	case EItemFilterState::DoNotCare: return InRight;
	default: checkNoEntry(); return EItemFilterState::DoNotCare;
	}
}

inline EItemFilterState CombineDisjunctive(EItemFilterState InLeft, EItemFilterState InRight)
{
	switch (InLeft)
	{
	case EItemFilterState::Include: return EItemFilterState::Include;
	case EItemFilterState::Exclude: return InRight == EItemFilterState::Include ? EItemFilterState::Include : InLeft;
	case EItemFilterState::DoNotCare: return InRight;
	default: checkNoEntry(); return EItemFilterState::DoNotCare;
	}
}

template<typename... TStates>
EItemFilterState CombineConjunctive(EItemFilterState InFirst, TStates... InRest)
{
	return CombineConjunctive(InFirst, CombineConjunctive(InRest...));
}

template<typename... TStates>
EItemFilterState CombineDisjunctive(EItemFilterState InFirst, TStates... InRest)
{
	return CombineDisjunctive(InFirst, CombineDisjunctive(InRest...));
}

inline bool PassesFilterState(EItemFilterState InState)
{
	// We'll treat DoNotCare as Include. 
	return InState != EItemFilterState::Exclude;
}
}