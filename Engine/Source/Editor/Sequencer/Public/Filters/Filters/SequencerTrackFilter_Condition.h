// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerTrackFilterBase.h"

class FSequencerTrackFilter_Condition : public FSequencerTrackFilter
{
public:
	static FString StaticName() { return TEXT("Condition"); }

	FSequencerTrackFilter_Condition(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin FSequencerTrackFilter
	virtual bool ShouldUpdateOnTrackValueChanged() const override;
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	//~ End FSequencerTrackFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin IFilter
	virtual FString GetName() const override;
	virtual bool PassesFilter(FSequencerTrackFilterType InItem) const override;
	//~ End IFilter
};
