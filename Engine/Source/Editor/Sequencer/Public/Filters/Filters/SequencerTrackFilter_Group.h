// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerTrackFilterBase.h"

class ISequencer;
class UMovieSceneNodeGroup;

class FSequencerTrackFilter_Group : public FSequencerTrackFilter
{
public:
	static FString StaticName() { return TEXT("Group"); }

	static void ForEachMovieSceneNodeGroup(UMovieScene* const InMovieScene
		, FSequencerTrackFilterType InItem
		, const TFunctionRef<bool(const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InParent, UMovieSceneNodeGroup*)>& InFunction);

	FSequencerTrackFilter_Group(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);
	virtual ~FSequencerTrackFilter_Group() override;

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual void OnPreFilter() override;
	//~ End FSequencerTrackFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin IFilter
	virtual FString GetName() const override;
	virtual bool PassesFilter(FSequencerTrackFilterType InItem) const override;
	//~ End IFilter

	void UpdateMovieScene(UMovieScene* const InMovieScene);

	bool HasActiveGroupFilter() const;

private:
	void HandleGroupsChanged();

	TWeakObjectPtr<UMovieScene> MovieSceneWeak;
};
