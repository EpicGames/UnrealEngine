// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerTrackFilterBase.h"

/** Persistent hide/isolate identity for a Sequencer outliner node. */
struct FSequencerFilterPersistentPath
{
	FString StableRootPath;
	FString SectionDisambiguator;
	FString RelativeSuffix;

	FSequencerFilterPersistentPath() = default;
	FSequencerFilterPersistentPath(FString InStableRootPath, FString InSectionDisambiguator, FString InRelativeSuffix)
		: StableRootPath(MoveTemp(InStableRootPath))
		, SectionDisambiguator(MoveTemp(InSectionDisambiguator))
		, RelativeSuffix(MoveTemp(InRelativeSuffix))
	{}

	friend bool operator==(const FSequencerFilterPersistentPath& InA, const FSequencerFilterPersistentPath& InB)
	{
		return InA.StableRootPath == InB.StableRootPath
			&& InA.SectionDisambiguator == InB.SectionDisambiguator
			&& InA.RelativeSuffix == InB.RelativeSuffix;
	}

	explicit operator bool() const
	{
		return IsValid();
	}

	friend uint32 GetTypeHash(const FSequencerFilterPersistentPath& InPath)
	{
		return HashCombine(
			HashCombine(GetTypeHash(InPath.StableRootPath), GetTypeHash(InPath.SectionDisambiguator)),
			GetTypeHash(InPath.RelativeSuffix)
		);
	}

	bool IsValid() const
	{
		return !StableRootPath.IsEmpty() || !SectionDisambiguator.IsEmpty() || !RelativeSuffix.IsEmpty();
	}
};

class FSequencerTrackFilter_HideIsolate : public FSequencerTrackFilter
{
public:
	static FString StaticName() { return TEXT("HideIsolate"); }

	FSequencerTrackFilter_HideIsolate(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	void ResetFilter();

	UE_DEPRECATED(5.8, "GetHiddenTracks() is deprecated. Use GetHiddenTrackPaths() instead")
	const TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>>& GetHiddenTracks() const;
	UE_DEPRECATED(5.8, "GetIsolatedTracks() is deprecated. Use GetIsolatedTrackPaths() instead")
	const TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>>& GetIsolatedTracks() const;

	const TSet<FSequencerFilterPersistentPath>& GetHiddenTrackPaths() const;
	const TSet<FSequencerFilterPersistentPath>& GetIsolatedTrackPaths() const;

	void HideTracks(const TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>>& InTracks, const bool bInAddToExisting);
	void UnhideTracks(const TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>>& InTracks);

	void IsolateTracks(const TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>>& InTracks, const bool bInAddToExisting);
	void UnisolateTracks(const TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>>& InTracks);

	void IsolateCategoryGroupTracks(const TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>>& InTracks
		, const TSet<FName>& InCategoryNames
		, const bool bInAddToExisting);

	void ShowAllTracks();

	bool HasHiddenTracks() const;
	bool HasIsolatedTracks() const;
	bool HasHiddenOrIsolatedTracks() const;

	bool IsTrackHidden(const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InTrack) const;
	bool IsTrackIsolated(const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InTrack) const;

	void EmptyHiddenTracks(const bool bInBroadcastChange = true);
	void EmptyIsolatedTracks(const bool bInBroadcastChange = true);

	//~ Begin FSequencerTrackFilter
	virtual void BindCommands() override;
	//~ End FSequencerTrackFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTipText() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin IFilter
	virtual FString GetName() const override;
	virtual bool PassesFilter(FSequencerTrackFilterType InItem) const override;
	//~ End IFilter

	bool IsActive() const;

private:
	bool HasIsolatedDescendant(const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InTrack) const;

	void RebuildHiddenTracksCache() const;
	void RebuildIsolatedTracksCache() const;

	/** Track outliner paths that are being hidden from view */
	TSet<FSequencerFilterPersistentPath> HiddenTracks;

	/** Track outliner paths that are being isolated in the view */
	TSet<FSequencerFilterPersistentPath> IsolatedTracks;

	/** Old deprecated storage of weak pointers that may become stale. */
	mutable TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>> CachedHiddenTracks;
	mutable TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>> CachedIsolatedTracks;

	mutable bool bHiddenTracksCacheDirty = true;
	mutable bool bIsolatedTracksCacheDirty = true;
};
