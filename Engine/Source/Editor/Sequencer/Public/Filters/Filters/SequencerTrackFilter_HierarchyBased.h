// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerTrackFilterBase.h"

namespace UE::Sequencer
{
/** Base class for filters that intend to implement FSequencerTrackFilter::Evaluate. */
class FSequencerTrackFilter_HierarchyBased : public FSequencerTrackFilter
{
public:
	
	FSequencerTrackFilter_HierarchyBased(ISequencerTrackFilters& InOutFilterInterface, TSharedPtr<FFilterCategory>&& InCategory)
		: FSequencerTrackFilter(InOutFilterInterface, MoveTemp(InCategory))
	{}

	// The new algorithm does not invoke PassesFilter. This is implement simply for completeness.
	virtual bool PassesFilter(TViewModelPtr<FViewModel> InItem) const override final
	{
		return PassesFilterState(Evaluate(InItem).ItemState);
	}

	// Subclasses must implement Evaluate
	virtual FFilterResult Evaluate(FSequencerTrackFilterType InItem) const override = 0;
};
}
