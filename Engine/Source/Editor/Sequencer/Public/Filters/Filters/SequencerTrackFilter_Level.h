// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerTrackFilterBase.h"

class FSequencerTrackFilter_Level : public FSequencerTrackFilter
{
public:
	static FString StaticName() { return TEXT("Level"); }

	FSequencerTrackFilter_Level(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);
	virtual ~FSequencerTrackFilter_Level() override;

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual void OnPreFilter() override;
	//~ End FSequencerTrackFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTipText() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin IFilter
	virtual FString GetName() const override;
	virtual bool PassesFilter(FSequencerTrackFilterType InItem) const override;
	//~ End IFilter

	bool IsActive() const;

	void UpdateWorld(UWorld* const InWorld);
	void ResetFilter();

	const TSet<FString>& GetAllWorldLevels() const;

	bool HasHiddenLevels() const;
	bool HasAllLevelsHidden() const;
	const TSet<FString>& GetHiddenLevels() const;
	bool IsLevelHidden(const FString& InLevelName) const;
	void HideLevel(const FString& InLevelName);
	void UnhideLevel(const FString& InLevelName);

	void HideAllLevels(const bool bInHide);
	bool CanHideAllLevels(const bool bInHide) const;

private:
	void HandleLevelsChanged();

	TSet<FString> HiddenLevels;

	TWeakObjectPtr<UWorld> CachedWorld;
	TSet<FString> AllWorldLevels;
};
