// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerTrackFilter_HierarchyBased.h"
#include "Templates/SharedPointer.h"

class USequencerTrackFilterSelectedExtension;

namespace UE::Sequencer
{
class FSequencerEditorViewModel;
}

class FSequencerTrackFilter_Selected : public UE::Sequencer::FSequencerTrackFilter_HierarchyBased
{
public:
	static FString StaticName() { return TEXT("Selected"); }

	FSequencerTrackFilter_Selected(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);
	virtual ~FSequencerTrackFilter_Selected() override;

	//~ Begin FSequencerTrackFilter
	virtual bool ShouldUpdateOnTrackValueChanged() const override;
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual void ActiveStateChanged(const bool bInActive) override;
	//~ End FSequencerTrackFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	virtual void OnPreFilter() override;

	//~ Begin IFilter
	virtual FString GetName() const override;
	virtual UE::Sequencer::FFilterResult Evaluate(FSequencerTrackFilterType InItem) const override;
	//~ End IFilter

protected:
	
	void BindSelectionChanged();
	void UnbindSelectionChanged();

	FDelegateHandle OnSelectionChangedHandle;
	
private:
	
	/** Objects extending this filter. Cached at construction to speed up filtering. */
	const TArray<TWeakObjectPtr<USequencerTrackFilterSelectedExtension>> CachedWeakExtensions;
	
	/** Whether IncludedViewModels needs to be rebuilt. */
	bool bNeedsPrefilterRebuild = true;
	/** Contains the selected items. */
	TMap<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::FViewModel>, UE::Sequencer::EFilterResultFlags> FilterCache;
	
	void BindEvents();
	void UnbindEvents();

	void OnSequencerSelectionChanged();
	void OnViewportSelectionChanged(UObject* const InObject);

	/** Cached weak reference to the view model captured at construction. */
	TWeakPtr<UE::Sequencer::FSequencerEditorViewModel> WeakViewModel;
};
