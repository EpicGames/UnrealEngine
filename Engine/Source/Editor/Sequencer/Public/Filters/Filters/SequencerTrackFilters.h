// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraComponent.h"
#include "Components/LightComponentBase.h"
#include "Components/SkeletalMeshComponent.h"
#include "Filters/SequencerTrackFilterBase.h"
#include "MVVM/ViewModels/FolderModel.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Tracks/MovieSceneDataLayerTrack.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "Tracks/MovieSceneFadeTrack.h"
#include "Tracks/MovieSceneLevelVisibilityTrack.h"
#include "Tracks/MovieSceneParticleTrack.h"
#include "Tracks/MovieSceneSlomoTrack.h"
#include "Tracks/MovieSceneSubTrack.h"

class FSequencerTrackFilter_Audio : public FSequencerTrackFilter_ClassType<UMovieSceneAudioTrack>
{
public:
	static FString StaticName() { return TEXT("Audio"); }

	FSequencerTrackFilter_Audio(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin IFilter
	virtual FString GetName() const override { return StaticName(); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;
	//~ End FSequencerTrackFilter
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_Event : public FSequencerTrackFilter_ClassType<UMovieSceneEventTrack>
{
public:
	static FString StaticName() { return TEXT("Event"); }

	FSequencerTrackFilter_Event(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin IFilter
	virtual FString GetName() const override { return StaticName(); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;
	//~ End FSequencerTrackFilter
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_LevelVisibility : public FSequencerTrackFilter_ClassType<UMovieSceneLevelVisibilityTrack>
{
public:
	static FString StaticName() { return TEXT("LevelVisibility"); }

	FSequencerTrackFilter_LevelVisibility(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin IFilter
	virtual FString GetName() const override { return StaticName(); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;
	//~ End FSequencerTrackFilter
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_Particle : public FSequencerTrackFilter_ClassType<UMovieSceneParticleTrack>
{
public:
	static FString StaticName() { return TEXT("Particle"); }

	FSequencerTrackFilter_Particle(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin IFilter
	virtual FString GetName() const override { return StaticName(); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;
	//~ End FSequencerTrackFilter
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_CinematicShot : public FSequencerTrackFilter_ClassType<UMovieSceneCinematicShotTrack>
{
public:
	static FString StaticName() { return TEXT("CinematicShot"); }

	FSequencerTrackFilter_CinematicShot(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin IFilter
	virtual FString GetName() const override { return StaticName(); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;
	//~ End FSequencerTrackFilter
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_Subsequence : public FSequencerTrackFilter_ClassType<UMovieSceneSubTrack>
{
public:
	static FString StaticName() { return TEXT("SubSequence"); }

	FSequencerTrackFilter_Subsequence(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin IFilter
	virtual FString GetName() const override { return StaticName(); }
	virtual bool PassesFilter(FSequencerTrackFilterType InItem) const override;
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;
	//~ End FSequencerTrackFilter
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_SkeletalMesh : public FSequencerTrackFilter_ComponentType<USkeletalMeshComponent>
{
public:
	static FString StaticName() { return TEXT("SkeletalMesh"); }

	FSequencerTrackFilter_SkeletalMesh(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin IFilter
	virtual FString GetName() const override { return StaticName(); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;
	//~ End FSequencerTrackFilter
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_Camera : public FSequencerTrackFilter_ComponentType<UCameraComponent>
{
public:
	static FString StaticName() { return TEXT("Camera"); }

	FSequencerTrackFilter_Camera(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin IFilter
	virtual FString GetName() const override { return StaticName(); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;
	//~ End FSequencerTrackFilter
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_Light : public FSequencerTrackFilter_ComponentType<ULightComponentBase>
{
public:
	static FString StaticName() { return TEXT("Light"); }

	FSequencerTrackFilter_Light(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin IFilter
	virtual FString GetName() const override { return StaticName(); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;
	//~ End FSequencerTrackFilter
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_CameraCut : public FSequencerTrackFilter_ClassType<UMovieSceneCameraCutTrack>
{
public:
	static FString StaticName() { return TEXT("CameraCut"); }

	FSequencerTrackFilter_CameraCut(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin IFilter
	virtual FString GetName() const override { return StaticName(); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;
	//~ End FSequencerTrackFilter
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_Fade : public FSequencerTrackFilter_ClassType<UMovieSceneFadeTrack>
{
public:
	static FString StaticName() { return TEXT("Fade"); }

	FSequencerTrackFilter_Fade(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin IFilter
	virtual FString GetName() const override { return StaticName(); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;
	//~ End FSequencerTrackFilter
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_DataLayer : public FSequencerTrackFilter_ClassType<UMovieSceneDataLayerTrack>
{
public:
	static FString StaticName() { return TEXT("DataLayer"); }

	FSequencerTrackFilter_DataLayer(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin IFilter
	virtual FString GetName() const override { return StaticName(); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;
	//~ End FSequencerTrackFilter
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_TimeDilation : public FSequencerTrackFilter_ClassType<UMovieSceneSlomoTrack>
{
public:
	static FString StaticName() { return TEXT("TimeDilation"); }

	FSequencerTrackFilter_TimeDilation(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin IFilter
	virtual FString GetName() const override { return StaticName(); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;
	//~ End FSequencerTrackFilter
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_Folder : public FSequencerTrackFilter_ModelType<UE::Sequencer::FFolderModel>
{
public:
	static FString StaticName() { return TEXT("Folder"); }

	FSequencerTrackFilter_Folder(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin IFilter
	virtual FString GetName() const override { return StaticName(); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;
	//~ End FSequencerTrackFilter
};
