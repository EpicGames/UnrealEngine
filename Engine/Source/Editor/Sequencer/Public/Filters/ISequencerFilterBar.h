// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SBasicFilterBar.h"
#include "Framework/Commands/UICommandList.h"
#include "MVVM/ViewModelPtr.h"

template<typename InFilterType> class FSequencerFilterBase;
class FSequencerTrackFilter;
class FString;
class FText;
class FTextFilterExpressionEvaluator;
class ISequencer;
class ISequencerTextFilterExpressionContext;
struct FSequencerFilterSuggestion;

/** Generic interface for all Sequencer filter bar implementations. */
class ISequencerFilterBar : public TSharedFromThis<ISequencerFilterBar>
{
public:
	
	// This type is deprecated. However, DECLARE_EVENT_TwoParams does not support UE_DEPRECATED macro in Linux.
	DECLARE_EVENT_TwoParams(ISequencerFilterBar, FOnFilterBarStateChanged, const bool /*InIsVisible*/, const EFilterBarLayout /*InNewLayout*/);
	DECLARE_MULTICAST_DELEGATE_OneParam(FMuteStateChangedDelegate, bool /* bNewIsMuted */);
	
	virtual ~ISequencerFilterBar() = default;

	virtual FName GetIdentifier() const = 0;

	virtual ISequencer& GetSequencer() const = 0;

	virtual TSharedPtr<FUICommandList> GetCommandList() const = 0;

	virtual FString GetTextFilterString() const = 0;
	virtual void SetTextFilterString(const FString& InText) = 0;

	/** Returns true if the current filter bar text filter string contains the specified text expression.
	 * The text expression must have key, operator, and value tokens. */
	virtual bool DoesTextFilterStringContainExpressionPair(const ISequencerTextFilterExpressionContext& InExpression) const = 0;

	virtual bool AreFiltersMuted() const = 0;
	virtual void MuteFilters(const bool bInMute) = 0;

	virtual bool CanResetFilters() const = 0;
	virtual void ResetFilters() = 0;

	virtual bool HasAnyFilterActive(const bool bCheckTextFilter = true
		, const bool bInCheckHideIsolateFilter = true
		, const bool bInCheckCommonFilters = true
		, const bool bInCheckInternalFilters = true
		, const bool bInCheckCustomTextFilters = true) const = 0;

	virtual bool HasAnyFilterEnabled() const = 0;

	virtual void RequestFilterUpdate() = 0;

	virtual void EnableAllFilters(const bool bInEnable, const TArray<FString>& InExceptionFilterNames) = 0;

	template<typename InFilterClass>
	TArray<TSharedRef<FSequencerFilterBase<InFilterClass>>> GetCommonFilters() const;

	virtual void ActivateCommonFilters(const bool bInActivate, const TArray<FString>& InExceptionFilterNames) = 0;

	virtual bool AreAllEnabledFiltersActive(const bool bInActive, const TArray<FString> InExceptionFilterNames) const = 0;
	virtual void ActivateAllEnabledFilters(const bool bInActivate, const TArray<FString> InExceptionFilterNames) = 0;

	virtual bool IsFilterActiveByDisplayName(const FString& InFilterName) const = 0;
	virtual bool IsFilterEnabledByDisplayName(const FString& InFilterName) const = 0;
	virtual bool SetFilterActiveByDisplayName(const FString& InFilterName, const bool bInActive, const bool bInRequestFilterUpdate = true) = 0;
	virtual bool SetFilterEnabledByDisplayName(const FString& InFilterName, const bool bInEnabled, const bool bInRequestFilterUpdate = true) = 0;

	virtual TArray<FText> GetFilterDisplayNames() const = 0;
	virtual TArray<FText> GetCustomTextFilterNames() const = 0;

	virtual int32 GetTotalDisplayNodeCount() const = 0;
	virtual int32 GetFilteredDisplayNodeCount() const  = 0;

	virtual const FTextFilterExpressionEvaluator& GetTextFilterExpressionEvaluator() const = 0;
	virtual TArray<TSharedRef<ISequencerTextFilterExpressionContext>> GetTextFilterExpressionContexts() const = 0;
	/** 
	 * @return The current filter state exported as string, such that it can be saved in a config and reapplied again.
	 * @see FSequencerFilterBarConfig::GetCustomTextFilters, FSequencerTrackFilter_CustomText 
	 */
	virtual FString GenerateTextFilterStringFromEnabledFilters() const = 0;

	virtual void OpenTextExpressionHelp() = 0;
	virtual void SaveCurrentFilterSetAsCustomTextFilter() = 0;
	virtual void CreateNewTextFilter() = 0;

	/** Event called when the visibility or layout state has changed */
	UE_DEPRECATED(5.8, "Use OnVisibilityChanged instead")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual FOnFilterBarStateChanged& OnStateChanged() { return OnFilterBarStateChanged_DEPRECATED; }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Event called to request that the subscribe update its view since the filters state has changed */
	virtual FSimpleMulticastDelegate& OnRequestUpdate() = 0;
	
	/** Event called when the search text has changed. */
	virtual FSimpleMulticastDelegate& OnTextFilterTextChanged() = 0;
	
	/** Event called when the mute state of the filters has changed. */
	virtual FMuteStateChangedDelegate& OnMuteFiltersChanged() = 0;

	UE_DEPRECATED(5.8, "This function was removed")
	virtual bool ShouldShowFilterBarWidget() const { return false; }
	UE_DEPRECATED(5.8, "This function was removed")
	virtual bool IsFilterBarVisible() const { return false; }
	UE_DEPRECATED(5.8, "This function was removed")
	virtual void ToggleFilterBarVisibility() {}

	UE_DEPRECATED(5.8, "This function was removed")
	virtual bool IsFilterBarLayout(const EFilterBarLayout InLayout) const { return false; }
	UE_DEPRECATED(5.8, "This function was removed")
	virtual void SetToVerticalLayout() {}
	UE_DEPRECATED(5.8, "This function was removed")
	virtual void SetToHorizontalLayout() {}
	UE_DEPRECATED(5.8, "This function was removed")
	virtual void ToggleFilterBarLayout() {}

	/** @return True if a common filter, specified by InFilterName, passes its filter. Used to support default text
	 * expressions for common filters without needing to create an actual text expression class for each common filter. */
	virtual bool DoesCommonFilterPass(const UE::Sequencer::TViewModelPtr<UE::Sequencer::FViewModel>& InViewModel, const FString& InFilterName) const = 0;

	/**
	 * Populates a list of suggested filter keys for the common filters used in the Sequencer filter bar.
	 *
	 * This function allows users to query and retrieve possible key suggestions for building advanced
	 * filter expressions. The resulting suggestions are typically used by UI elements like search boxes
	 * to provide auto-completion or hints.
	 *
	 * @param OutSuggestions An array to populate with filter key suggestions
	 */
	virtual void GetCommonFilterKeySuggestions(TArray<FSequencerFilterSuggestion>& OutSuggestions) const = 0;

	/**
	 * Populates a list of suggested filter values for the specified filter in the Sequencer filter bar.
	 *
	 * This method is useful for supporting auto-completion and providing context-aware suggestions
	 * when users interact with filtering interfaces in the Sequencer UI. It queries possible values
	 * for a given filter key and populates the output array with relevant suggestions.
	 *
	 * @param InFilterName The name of the filter to query for value suggestions
	 * @param OutSuggestions An array to populate with filter value suggestions
	 */
	virtual void GetCommonFilterValueSuggestions(const FString& InFilterName, TArray<FSequencerFilterSuggestion>& OutSuggestions) const = 0;

	/** @return Whether there is a common filter with InFilterName. */
	virtual bool IsCommonFilterName(const FString& InFilterName) const = 0;

protected:
	/** Internal implementation to bypass template limitations in virtual functions */
	virtual void GetCommonFiltersImpl(TArray<TSharedPtr<void>>& OutTypeErasedFilters) const = 0;
	
private:
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FOnFilterBarStateChanged OnFilterBarStateChanged_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

#include "Filters/ISequencerFilterBar.inl"
