// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerFilterBase.h"

template<typename InFilterClass>
TArray<TSharedRef<FSequencerFilterBase<InFilterClass>>> ISequencerFilterBar::GetCommonFilters() const
{
	TArray<TSharedRef<FSequencerFilterBase<InFilterClass>>> OutFilters;
	GetCommonFiltersImpl(*reinterpret_cast<TArray<TSharedPtr<void>>*>(&OutFilters));
	return OutFilters;
}
