// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/TextFilterExpressionEvaluator.h"
#include "Misc/TextFilterUtils.h"

enum class SEQUENCER_API ESequencerTextFilterValueType : uint8
{
	String,
	Boolean,
	Integer
};

struct FSequencerTextFilterKeyword
{
	FString Keyword;
	FText Description;
};

/** Extends the ITextFilterExpressionContext interface to add support for suggestions. */
class ISequencerTextFilterExpressionContext : public ITextFilterExpressionContext
{
public:
	virtual TSet<FName> GetKeys() const = 0;

	virtual ESequencerTextFilterValueType GetValueType() const = 0;
	virtual TArray<FSequencerTextFilterKeyword> GetValueKeywords() const { return {}; }

	virtual FText GetDescription() const = 0;
	virtual FText GetCategory() const { return FText::GetEmpty(); }

	static bool CompareFStringForExactBool(const FTextFilterString& InValue, const bool bInPassedFilter)
	{
		if (InValue.CompareFString(TEXT("TRUE"), ETextFilterTextComparisonMode::Exact))
		{
			return bInPassedFilter;
		}
		if (InValue.CompareFString(TEXT("FALSE"), ETextFilterTextComparisonMode::Exact))
		{
			return !bInPassedFilter;
		}
		return true;
	}

	static bool CompareFStringForExactBool(const FTextFilterString& InValue
		, const ETextFilterComparisonOperation InComparisonOperation
		, const bool bInPassedFilter)
	{
		switch (InComparisonOperation)
		{
		case ETextFilterComparisonOperation::Equal:
			return CompareFStringForExactBool(InValue, bInPassedFilter);
		case ETextFilterComparisonOperation::NotEqual:
			return CompareFStringForExactBool(InValue, !bInPassedFilter);
		default:
			return true;
		}
	}
};
