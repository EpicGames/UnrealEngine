// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerFilterBar.h"

class FSequencerTrackFilter_HideIsolate;
class FSequencerTrackFilter;
class FSequencerTrackFilter_CustomText;
struct FSequencerFilterData;

class ISequencerTrackFilters : public ISequencerFilterBar
{
public:
	virtual ~ISequencerTrackFilters() override = default;

	virtual bool IsFilterEnabled(TSharedRef<FSequencerTrackFilter> InFilter) const = 0;
	virtual bool SetFilterEnabled(const TSharedRef<FSequencerTrackFilter> InFilter, const bool bInEnabled, const bool bInRequestFilterUpdate = true) = 0;

	virtual bool IsFilterActive(const TSharedRef<FSequencerTrackFilter> InFilter) const = 0;
	virtual bool SetFilterActive(const TSharedRef<FSequencerTrackFilter>& InFilter, const bool bInActive, const bool bInRequestFilterUpdate = true) = 0;

	virtual bool AddCustomTextFilter(const TSharedRef<FSequencerTrackFilter_CustomText>& InFilter, const bool bInAddToConfig) = 0;
	virtual bool RemoveCustomTextFilter(const TSharedRef<FSequencerTrackFilter_CustomText>& InFilter, const bool bInAddToConfig) = 0;
	
	virtual TSharedRef<FSequencerTrackFilter_HideIsolate> GetHideIsolateFilter() const = 0;
	virtual void HideSelectedTracks() = 0;
	virtual void IsolateSelectedTracks() = 0;

	virtual void ShowOnlyLocationCategoryGroups() = 0;
	virtual void ShowOnlyRotationCategoryGroups() = 0;
	virtual void ShowOnlyScaleCategoryGroups() = 0;

	virtual bool HasSelectedTracks() const = 0;

	virtual FSequencerFilterData& GetFilterData() = 0;
};
