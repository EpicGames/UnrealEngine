// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerFilterAreaConfig.generated.h"

#define UE_API SEQUENCER_API

/** 
 * Config data that applies the entire filter area, regardless whether the filter bar displayed is in instanced or linked mode.
 * @see FSequencerFilterBarConfig
 */
USTRUCT()
struct FSequencerFilterAreaConfig
{
	GENERATED_BODY()
	
	UE_API void SetIsFilterBarVisible(bool bNewIsVisible);
	UE_API bool IsFilterBarVisible() const;
	
	UE_API void SetPreserveFiltersOnUnlink(bool bValue);
	UE_API bool GetPreserveFiltersOnUnlink() const;
	
	UE_API void SetIsLinkedFiltering(bool bValue);
	UE_API bool IsLinkedFiltering() const;
	
private:
	
	/** Whether the filter bar should be visible. */
	UPROPERTY()
	bool bIsFilterBarVisible = true;
	
	/** Whether when you unlink your filters, you want the filters in your linked filter bar to be copied into the unlinked filter bar. */
	UPROPERTY()
	bool bPreserveFiltersOnUnlink = true;
	
	/** Whether the UI is in linked filtering mode. True: linked filter. False: Instanced filtering.  */
	UPROPERTY()
	bool bIsLinkedFiltering = true;
};

#undef UE_API
