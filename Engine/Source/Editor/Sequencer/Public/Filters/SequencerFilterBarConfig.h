// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SBasicFilterBar.h"
#include "SequencerFilterBarConfig.generated.h"

USTRUCT()
struct FSequencerFilterSet
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString Label;

	/** Enabled and active states of common filters. Enabled if in the map. Active if the value of the key is true. */
	UPROPERTY()
	TMap<FString, bool> EnabledStates;

	UPROPERTY()
	FString TextFilterString;
};

/** 
 * Config data for a single filter bar instance.
 * @see FSequencerFilterAreaConfig
 */
USTRUCT()
struct FSequencerFilterBarConfig
{
	GENERATED_BODY()

public:
	/** Common Filters */
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FSequencerFilterBarConfig() = default;
	FSequencerFilterBarConfig(const FSequencerFilterBarConfig&) = default;
	FSequencerFilterBarConfig(FSequencerFilterBarConfig&& Other) = default;
	FSequencerFilterBarConfig& operator=(const FSequencerFilterBarConfig& Other) = default;
	FSequencerFilterBarConfig& operator=(FSequencerFilterBarConfig&& Other) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	SEQUENCER_API bool IsFilterEnabled(const FString& InFilterName) const;
	SEQUENCER_API bool SetFilterEnabled(const FString& InFilterName, const bool bInActive);

	SEQUENCER_API bool IsFilterActive(const FString& InFilterName) const;
	SEQUENCER_API bool SetFilterActive(const FString& InFilterName, const bool bInActive);

	const FSequencerFilterSet& GetCommonActiveSet() const;

	/** Custom Text Filters */

	SEQUENCER_API TArray<FCustomTextFilterData>& GetCustomTextFilters();
	SEQUENCER_API bool HasCustomTextFilter(const FString& InFilterName) const;
	SEQUENCER_API FCustomTextFilterData* FindCustomTextFilter(const FString& InFilterName);
	SEQUENCER_API bool AddCustomTextFilter(FCustomTextFilterData InFilterData);
	SEQUENCER_API bool RemoveCustomTextFilter(const FString& InFilterName);

	/** Filter Bar Layout */
	UE_DEPRECATED(5.8, "Sequencer now uses horizontal filter layout.")
	SEQUENCER_API EFilterBarLayout GetFilterBarLayout() const;
	UE_DEPRECATED(5.8, "Sequencer now uses horizontal filter layout.")
	SEQUENCER_API void SetFilterBarLayout(const EFilterBarLayout InLayout);

protected:
	/** The currently active set of common and custom text filters that should be restored on editor load */
	UPROPERTY()
	FSequencerFilterSet ActiveFilters;

	/** User created custom text filters */
	UPROPERTY()
	TArray<FCustomTextFilterData> CustomTextFilters;

	/** The layout style for the filter bar widget */
	UE_DEPRECATED(5.8, "Sequencer now uses horizontal filter layout.")
	UPROPERTY()
	EFilterBarLayout FilterBarLayout_DEPRECATED = EFilterBarLayout::Vertical;
};
