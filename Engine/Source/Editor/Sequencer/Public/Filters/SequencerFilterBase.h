// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/FilterBase.h"
#include "Filters/ISequencerFilterBar.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/InputChord.h"
#include "Templates/SharedPointer.h"
#include "MovieSceneSequence.h"

class ISequencer;

/** Sequencer base filter for all Sequencer filters (Sequencer and Sequence Navigator) */
template<typename InFilterType>
class FSequencerFilterBase
	: public FFilterBase<InFilterType>
	, public TSharedFromThis<FSequencerFilterBase<InFilterType>>
{
public:
	static bool SupportsLevelSequence(UMovieSceneSequence* const InSequence)
	{
		static UClass* const LevelSequenceClass = FindObject<UClass>(nullptr, TEXT("/Script/LevelSequence.LevelSequence"), EFindObjectFlags::ExactClass);
		return InSequence
			&& LevelSequenceClass
			&& InSequence->GetClass()->IsChildOf(LevelSequenceClass);
	}

	static bool SupportsUMGSequence(UMovieSceneSequence* const InSequence)
	{
		static UClass* const WidgetAnimationClass = FindObject<UClass>(nullptr, TEXT("/Script/UMG.WidgetAnimation"), EFindObjectFlags::ExactClass);
		return InSequence
			&& WidgetAnimationClass
			&& InSequence->GetClass()->IsChildOf(WidgetAnimationClass);
	}

	static bool SupportsDaySequence(UMovieSceneSequence* const InSequence)
	{
		static UClass* const DaySequenceClass = FindObject<UClass>(nullptr, TEXT("/Script/DaySequence.DaySequence"), EFindObjectFlags::ExactClass);
		return InSequence
			&& DaySequenceClass
			&& InSequence->GetClass()->IsChildOf(DaySequenceClass);
	}

	static FText BuildTooltipTextForCommand(const FText& InBaseText, const TSharedPtr<FUICommandInfo>& InCommand)
	{
		const TSharedRef<const FInputChord> FirstValidChord = InCommand->GetFirstValidChord();
		if (FirstValidChord->IsValidChord())
		{
			return FText::Format(NSLOCTEXT("Sequencer", "TrackFilterTooltipText", "{0} ({1})")
				, InBaseText, FirstValidChord->GetInputText());
		}
		return InBaseText;
	}

	FSequencerFilterBase(ISequencerFilterBar& InOutFilterInterface, TSharedPtr<FFilterCategory>&& InCategory = nullptr)
		: FFilterBase<InFilterType>(InCategory)
		, FilterInterface(InOutFilterInterface)
	{}

	//~ Begin IFilter
	virtual bool PassesFilter(InFilterType InItem) const override { return true; };
	//~ End IFilter

	//~ Begin FFilterBase

	virtual FText GetDisplayName() const override { return FText::GetEmpty(); }

	virtual FText GetToolTipText() const override
	{
		if (const TSharedPtr<FUICommandInfo> ToggleCommand = GetToggleCommand())
		{
			return BuildTooltipTextForCommand(GetDefaultToolTipText(), ToggleCommand);
		}
		return GetDefaultToolTipText();
	}

	virtual FLinearColor GetColor() const override { return FLinearColor::White; }

	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override {}

	virtual void SaveSettings(const FString& InIniFilename, const FString& InIniSection, const FString& InSettingsString) const override {}
	virtual void LoadSettings(const FString& InIniFilename, const FString& InIniSection, const FString& InSettingsString) override {}

	virtual bool IsInverseFilter() const override { return false; }

	virtual void ActiveStateChanged(const bool bInActive) override {}

	//~ End FFilterBase

	ISequencer& GetSequencer() const { return FilterInterface.GetSequencer(); }

	virtual ISequencerFilterBar& GetFilterInterface() const { return FilterInterface; }

	virtual FText GetDefaultToolTipText() const { return FText(); }
	virtual FSlateIcon GetIcon() const { return FSlateIcon(); }
	virtual bool IsCustomTextFilter() const { return false; }

	virtual void BindCommands()
	{
		if (const TSharedPtr<FUICommandInfo> ToggleCommand = GetToggleCommand())
		{
			MapToggleAction(ToggleCommand);
		}
	}

	virtual void SetActive(const bool bInActive)
	{
		FFilterBase<InFilterType>::SetActive(bInActive);
		ActiveStateChanged(bInActive);
	}

	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const { return nullptr; }

	bool CanToggleFilter() const
	{
		const FString FilterName = GetDisplayName().ToString();
		return FilterInterface.IsFilterActiveByDisplayName(FilterName);
	}

	void ToggleFilter()
	{
		const FString FilterName = GetDisplayName().ToString();
		const bool bNewState = !FilterInterface.IsFilterActiveByDisplayName(FilterName);
		FilterInterface.SetFilterActiveByDisplayName(FilterName, bNewState);
	}

	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const { return true; }

	/** Event called before nodes are filtered to give the filter a chance to pre-process */
	virtual void OnPreFilter() {}

protected:
	void MapToggleAction(const TSharedPtr<FUICommandInfo>& InCommand)
	{
		FilterInterface.GetCommandList()->MapAction(
			InCommand,
			FExecuteAction::CreateSP(this, &FSequencerFilterBase::ToggleFilter),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FSequencerFilterBase::CanToggleFilter));
	}

	ISequencerFilterBar& FilterInterface;

private:
	//~ Begin IFilter
	/** Hide behind private to force use of GetIcon() instead */
	virtual FName GetIconName() const override { return FName(); }
	//~ End IFilter
};
