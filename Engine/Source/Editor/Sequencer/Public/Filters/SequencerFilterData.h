// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FilterBarDelegates.h"
#include "MVVM/ViewModelPtr.h"
#include "Misc/EnumClassFlags.h"
#include "MovieSceneTrack.h"

#define UE_API SEQUENCER_API

class ISequencer;

namespace UE::Sequencer
{
	class IObjectBindingExtension;
	class IOutlinerExtension;
	class ITrackExtension;
}

using FSequencerTrackFilterType = UE::Sequencer::FViewModelPtr;

namespace UE::Sequencer
{
/** Utility argument for FSequencerFilterData. */
enum class ENodeVisitFlags : uint8
{
	None,
	
	IncludeSelf = 1 << 0,
	IncludeParents = 1 << 1,
	IncludeChildren = 1 << 2,
	
	IncludeAll = IncludeSelf | IncludeParents | IncludeChildren,
};
ENUM_CLASS_FLAGS(ENodeVisitFlags);
}

/** Represents a cache between nodes for a filter operation. */
struct FSequencerFilterData
{
	UE_API FSequencerFilterData(const FString& InRawFilterText);

	UE_API bool operator==(const FSequencerFilterData& InRhs) const;
	UE_API bool operator!=(const FSequencerFilterData& InRhs) const;

	UE_API void Reset();

	UE_API FString GetRawFilterText() const;

	UE_API uint32 GetDisplayNodeCount() const;
	UE_API uint32 GetTotalNodeCount() const;

	UE_API uint32 GetFilterInCount() const;
	UE_API uint32 GetFilterOutCount() const;

	UE_API void IncrementTotalNodeCount();
	UE_API void AddTotalNodeCount(int32 InNodeCount);

	UE_API void FilterInNode(UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension> InNodeWeak);
	UE_API void FilterOutNode(UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension> InNodeWeak);

	UE_DEPRECATED(5.8, "Use the version with EFilterInFlags instead")
	UE_API void FilterInParentChildNodes(const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InNode, const bool bInIncludeSelf, const bool bInIncludeParents, const bool bInIncludeChildren = false);
	
	/** @return Number of nodes visited */
	UE_API int32 FilterInParentChildNodes(
		const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InNode, UE::Sequencer::ENodeVisitFlags InFlags
		);
	/** @return Number of nodes visited */
	UE_API int32 FilterOutParentChildNodes(
		const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InNode, 
		UE::Sequencer::ENodeVisitFlags InFlags
		);

	UE_API void FilterInNodeWithAncestors(const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InNode);

	UE_API bool ContainsFilterInNodes(const FSequencerFilterData& InOtherData) const;

	UE_API bool IsFilteredOut(const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InNode) const;
	
	UE_API UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::ITrackExtension> ResolveTrack(FSequencerTrackFilterType InNode);
	UE_API UMovieSceneTrack* ResolveMovieSceneTrackObject(FSequencerTrackFilterType InNode);
	UE_API UObject* ResolveTrackBoundObject(ISequencer& InSequencer, FSequencerTrackFilterType InNode);

	TMap<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>, UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::ITrackExtension>> ResolvedTracks;
	TMap<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>, TWeakObjectPtr<UMovieSceneTrack>> ResolvedTrackObjects;
	TMap<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>, TWeakObjectPtr<>> ResolvedObjects;
	
	/** Broadcasts when the filter state of IOutlinerExtension is changed. */
	UE::Sequencer::FChangeOutlinerExtensionFilterState& OnChangeFilterState() { return ChangeFilterStateEvent; }

protected:
	FString RawFilterText;

	uint32 TotalNodeCount = 0;

	/** Nodes to be displayed in the UI */
	TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>> FilterInNodes;
	/**
	 * Nodes that have been processed, i.e. either filtered in or out. 
	 * This is used to detect when new IOutlinerExtensions are created after filters have run.
	 */
	TSet<UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::IOutlinerExtension>> ProcessedNodes;
	
	/** Broadcasts when the filter state of IOutlinerExtension is changed. */
	UE::Sequencer::FChangeOutlinerExtensionFilterState ChangeFilterStateEvent;
	
	/** @return Number of nodes visited */
	int32 ApplyToNodes(
		const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& InNode, 
		TFunctionRef<void(const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>&)>&& InCallback,
		UE::Sequencer::ENodeVisitFlags InFlags
		);
};

#undef UE_API
