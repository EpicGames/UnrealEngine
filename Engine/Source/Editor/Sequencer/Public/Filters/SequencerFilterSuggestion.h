// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"

#define LOCTEXT_NAMESPACE "SequencerFilterSuggestion"

struct FSequencerFilterSuggestion
{
	static FSequencerFilterSuggestion MakeSimpleSuggestion(FString InSuggestionString)
	{
		FSequencerFilterSuggestion Suggestion;
		Suggestion.Suggestion = MoveTemp(InSuggestionString);
		Suggestion.DisplayName = FText::FromString(Suggestion.Suggestion);
		return MoveTemp(Suggestion);
	}

	static void MakeBooleanSuggestions(TArray<FSequencerFilterSuggestion>& OutSuggestions
		, const FText& InOptionalTrueDescription = FText()
		, const FText& InOptionalFalseDescription = FText())
	{
		FSequencerFilterSuggestion TrueSuggestion;
		TrueSuggestion.Suggestion = TEXT("True");
		TrueSuggestion.DisplayName = LOCTEXT("BooleanTrueSuggestion", "True");
		TrueSuggestion.Description = InOptionalTrueDescription;
		OutSuggestions.AddUnique(MoveTemp(TrueSuggestion));

		FSequencerFilterSuggestion FalseSuggestion;
		FalseSuggestion.Suggestion = TEXT("False");
		FalseSuggestion.DisplayName = LOCTEXT("BooleanFalseSuggestion", "False");
		FalseSuggestion.Description = InOptionalFalseDescription;
		OutSuggestions.AddUnique(MoveTemp(FalseSuggestion));
	}

	bool operator==(const FSequencerFilterSuggestion& InOther) const
	{
		return Suggestion.Equals(InOther.Suggestion, ESearchCase::IgnoreCase);
	}

	/** The raw suggestion string that should be used with the search box */
	FString Suggestion;

	/** The user-facing display name of this suggestion */
	FText DisplayName;

	/** The user-facing category name of this suggestion (if any) */
	FText CategoryName;

	/** Describes what this suggestion will do */
	FText Description;
};

#undef LOCTEXT_NAMESPACE
