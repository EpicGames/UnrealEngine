// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/ISequencerTextFilterExpressionContext.h"
#include "Filters/SequencerTrackFilterBase.h"
#include "Misc/TextFilterExpressionEvaluator.h"

class UMovieScene;
class UMovieSceneSequence;
class UMovieSceneTrack;

/** Text expression context to test the given asset data against the current text filter */
class FSequencerTextFilterExpressionContext : public ISequencerTextFilterExpressionContext
{ 
public:
	SEQUENCER_API FSequencerTextFilterExpressionContext(ISequencerTrackFilters& InFilterInterface);

	void SetFilterItem(FSequencerTrackFilterType InFilterItem);

	//~ Begin ITextFilterExpressionContext

	SEQUENCER_API virtual bool TestBasicStringExpression(const FTextFilterString& InValue
		, const ETextFilterTextComparisonMode InTextComparisonMode) const override;

	SEQUENCER_API virtual bool TestComplexExpression(const FName& InKey
		, const FTextFilterString& InValue
		, const ETextFilterComparisonOperation InComparisonOperation
		, const ETextFilterTextComparisonMode InTextComparisonMode) const override;

	//~ End ITextFilterExpressionContext

protected:
	SEQUENCER_API UMovieSceneSequence* GetFocusedMovieSceneSequence() const;
	SEQUENCER_API UMovieScene* GetFocusedGetMovieScene() const;

	SEQUENCER_API UMovieSceneTrack* GetMovieSceneTrack() const;
	SEQUENCER_API UObject* GetBoundObject() const;

	ISequencerTrackFilters& FilterInterface;

	FSequencerTrackFilterType FilterItem;
};
