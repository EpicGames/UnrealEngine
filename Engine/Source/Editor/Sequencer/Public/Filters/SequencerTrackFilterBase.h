// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FilterResult.h"
#include "Filters/FilterBase.h"
#include "Filters/ISequencerTrackFilters.h"
#include "GameFramework/Actor.h"
#include "Misc/IFilter.h"
#include "MovieSceneTrack.h"
#include "MovieSceneTrackEditor.h"
#include "SequencerFilterBase.h"
#include "SequencerFilterData.h"
#include "MVVM/ViewModelPtr.h"
#include "Textures/SlateIcon.h"

class ISequencer;
class UMovieScene;
class UMovieSceneSequence;
struct FSequencerFilterData;

namespace UE::Sequencer
{
	class IOutlinerExtension;

	namespace ExtensionHooks
	{
		static FName Hierarchy(TEXT("Hierarchy"));
		static FName Show(TEXT("Show"));
	}
}

using FSequencerTrackFilterType = UE::Sequencer::FViewModelPtr;

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter : public FSequencerFilterBase<FSequencerTrackFilterType>
{
public:
	template<typename InTrackClassType>
	static bool IsSequenceTrackSupported(UMovieSceneSequence* const InSequence)
	{
		const ETrackSupport Support = InSequence
			? InSequence->IsTrackSupported(InTrackClassType::StaticClass())
			: ETrackSupport::NotSupported;
		return Support == ETrackSupport::Supported;
	}

	SEQUENCER_API FSequencerTrackFilter(ISequencerTrackFilters& InOutFilterInterface, TSharedPtr<FFilterCategory>&& InCategory = nullptr);

	//~ Begin FSequencerFilterBase
	SEQUENCER_API virtual ISequencerTrackFilters& GetFilterInterface() const override;
	//~ End FSequencerFilterBase

	/** Evaluates the filter on an item. It allows returning more information to optimize evaluation. */
	SEQUENCER_API virtual UE::Sequencer::FFilterResult Evaluate(FSequencerTrackFilterType InItem) const;
	
	/** Returns whether this filter needs reevaluating any time track values have been modified, not just tree changes */
	virtual bool ShouldUpdateOnTrackValueChanged() const { return false; }

	/** Returns whether the filter supports the sequence type */
	SEQUENCER_API virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;

	virtual TSubclassOf<UMovieSceneTrack> GetTrackClass() const { return nullptr; }

protected:
	UMovieSceneSequence* GetFocusedMovieSceneSequence() const;
	UMovieScene* GetFocusedGetMovieScene() const;
};

//////////////////////////////////////////////////////////////////////////
//

/** Base filter for filtering Sequencer tracks based track model */
template<typename InModelType>
class FSequencerTrackFilter_ModelType : public FSequencerTrackFilter
{
public:
	FSequencerTrackFilter_ModelType(ISequencerTrackFilters& InOutFilterInterface, TSharedPtr<FFilterCategory> InCategory)
		: FSequencerTrackFilter(InOutFilterInterface, MoveTemp(InCategory))
	{}

	//~ Begin IFilter
	virtual bool PassesFilter(FSequencerTrackFilterType InItem) const override
	{
		const UE::Sequencer::TViewModelPtr<InModelType> Model = InItem->FindAncestorOfType<InModelType>();
		return Model.IsValid(); // show child tracks
	}
	//~ End IFilter
};

//////////////////////////////////////////////////////////////////////////
//

/** Base filter for filtering Sequencer tracks based object class type */
template<typename InClassType>
class FSequencerTrackFilter_ClassType : public FSequencerTrackFilter
{
public:
	FSequencerTrackFilter_ClassType(ISequencerTrackFilters& InOutFilterInterface, TSharedPtr<FFilterCategory> InCategory)
		: FSequencerTrackFilter(InOutFilterInterface, MoveTemp(InCategory))
	{}

	//~ Begin IFilter
	virtual bool PassesFilter(FSequencerTrackFilterType InItem) const override
	{
		FSequencerFilterData& FilterData = GetFilterInterface().GetFilterData();
		const UMovieSceneTrack* const TrackObject = FilterData.ResolveMovieSceneTrackObject(InItem);
		return TrackObject && TrackObject->IsA(InClassType::StaticClass());
	}
	//~ End IFilter

	//~ Begin FSequencerTrackFilter
	virtual TSubclassOf<UMovieSceneTrack> GetTrackClass() const override
	{
		return InClassType::StaticClass();
	}
	//~ End FSequencerTrackFilter
};

//////////////////////////////////////////////////////////////////////////
//

/** Base filter for filtering Sequencer tracks based object component type */
template<typename InComponentType>
class FSequencerTrackFilter_ComponentType : public FSequencerTrackFilter
{
public:
	FSequencerTrackFilter_ComponentType(ISequencerTrackFilters& InOutFilterInterface, TSharedPtr<FFilterCategory> InCategory)
		: FSequencerTrackFilter(InOutFilterInterface, MoveTemp(InCategory))
	{}

	//~ Begin IFilter
	virtual bool PassesFilter(FSequencerTrackFilterType InItem) const override
	{
		FSequencerFilterData& FilterData = GetFilterInterface().GetFilterData();

		UObject* const BoundObject = FilterData.ResolveTrackBoundObject(GetSequencer(), InItem);
		if (!BoundObject)
		{
			return false;
		}

		if (BoundObject->IsA(InComponentType::StaticClass()))
		{
			return true;
		}

		if (BoundObject->IsA(AActor::StaticClass()))
		{
			const AActor* const Actor = CastChecked<const AActor>(BoundObject);
			if (Actor->FindComponentByClass(InComponentType::StaticClass()))
			{
				return true;
			}
		}

		return false;
	}
	//~ End IFilter
};
