// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerTrackFilterExtension.generated.h"

class FFilterCategory;
class FSequencerTrackFilter;
class ISequencerTrackFilters;

/**
 * Derive from this class to make additional track filters available in Sequencer.
 */
UCLASS(MinimalAPI, Abstract)
class USequencerTrackFilterExtension : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Override to add additional Sequencer track filters.
	 *
	 * @param InOutFilterInterface The filter interface to extend
	 * @param InPreferredCategory  The preferred filter bar category to place any newly added filters to, but can be any category
	 * @param InOutFilterList      Filter list to add additional filters to
	 */
	virtual void AddTrackFilterExtensions(ISequencerTrackFilters& InOutFilterInterface
		, const TSharedRef<FFilterCategory>& InPreferredCategory
		, TArray<TSharedRef<FSequencerTrackFilter>>& InOutFilterList) const
	{}
};
