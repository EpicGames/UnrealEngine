// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerCoreFwd.h"
#include "Templates/FunctionFwd.h"
#include "UObject/Object.h"
#include "SequencerTrackFilterSelectedExtension.generated.h"

class ISequencer;

/** Derive from this class to customize FSequencerTrackFilter_Selected. */
UCLASS(MinimalAPI, Abstract)
class USequencerTrackFilterSelectedExtension : public UObject
{
	GENERATED_BODY()
public:
	
	enum class EBreakBehavior : uint8 { Break, Continue };
	
	/** Enumerates all view models that the FSequencerTrackFilter_Selected should treat as selected on top of what is selected in Sequencer. */
	virtual void EnumerateViewModelsConsideredAsSelected(
		const ISequencer& InSequencer, TFunctionRef<EBreakBehavior(const UE::Sequencer::FViewModelPtr&)> InCallback
		)
	{}
};
