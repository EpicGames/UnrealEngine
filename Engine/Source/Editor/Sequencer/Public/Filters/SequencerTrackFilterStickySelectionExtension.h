// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerTrackFilterStickySelectionExtension.generated.h"

class ISequencer;

/** Derive from this class to customize FSequencerTrackFilter_StickSelection. */
UCLASS(MinimalAPI, Abstract)
class USequencerTrackFilterStickySelectionExtension : public UObject
{
	GENERATED_BODY()
public:
	
	/** @return Whether this system is currenty changing selection and it should cause sticky selection to reset. */
	virtual bool IsPerformingSelectionThatShouldClearStickySelection(const ISequencer& InSequencer) { return false; }
};
