// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerTrackFilterTextExpressionExtension.generated.h"

class FSequencerTextFilterExpressionContext;
class ISequencerTrackFilters;

/**
 * Derive from this class to make additional track filter text expressions available in Sequencer.
 */
UCLASS(MinimalAPI, Abstract)
class USequencerTrackFilterTextExpressionExtension : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Override to add additional Sequencer track filter text expressions.
	 *
	 * @param InOutFilterInterface The filter interface to extend
	 * @param InOutExpressionList  Expression list to add additional text expressions to
	 */
	virtual void AddTrackFilterTextExpressionExtensions(ISequencerTrackFilters& InOutFilterInterface
		, TArray<TSharedRef<FSequencerTextFilterExpressionContext>>& InOutExpressionList) const
	{}
};
