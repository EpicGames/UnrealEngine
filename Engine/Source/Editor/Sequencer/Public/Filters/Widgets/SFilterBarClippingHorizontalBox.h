// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SBoxPanel.h"

#define UE_API SEQUENCER_API

class SComboButton;

/**
 * Specialized widget for handling horizontal clipping of filter widgets that go beyond the filter bar widget width.
 */
class SFilterBarClippingHorizontalBox : public SHorizontalBox
{
public:
	static UE_API TSharedRef<SWidget> WrapVerticalListWithHeading(const TSharedRef<SWidget>& InWidget, const FPointerEventHandler InMouseButtonUpEvent);

	SLATE_BEGIN_ARGS(SFilterBarClippingHorizontalBox) 
		: _IsFocusable(true)
	{}
		SLATE_ARGUMENT(FOnGetContent, OnWrapButtonClicked)
		SLATE_ARGUMENT(bool, IsFocusable)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	UE_API TSharedRef<SComboButton> CreateWrapButton();

	/** Returns to index of the first clipped child */
	int32 GetClippedIndex() const { return ClippedIndex; }

protected:
	//~ Begin SPanel
	UE_API virtual void OnArrangeChildren(const FGeometry& InAllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	//~ End SPanel

	/** The button that is displayed when a toolbar or menubar is clipped */
	TWeakPtr<SComboButton> WeakWrapButton;

	/** Callback for when the wrap button is clicked */
	FOnGetContent OnWrapButtonClicked;

	/** Index of the first clipped child/block */
	mutable int32 ClippedIndex = INDEX_NONE;
	/** Index of the last clipped child/block */
	mutable int32 LastClippedIndex = INDEX_NONE;

	/** Number of clipped children not including the wrap button */
	mutable int32 NumClippedChildren = 0;

	TSharedPtr<FActiveTimerHandle> WrapButtonOpenTimer;

	bool bIsFocusable = true;
};

#undef UE_API
