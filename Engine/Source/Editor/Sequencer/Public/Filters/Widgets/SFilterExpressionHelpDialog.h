// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/ISequencerFilterBar.h"
#include "Filters/ISequencerTextFilterExpressionContext.h"
#include "Filters/SequencerFilterBase.h"
#include "Widgets/SWindow.h"

#define UE_API SEQUENCER_API

class ISequencerFilterBar;
enum class ESequencerTextFilterValueType : uint8;

struct FFilterExpressionHelpEntry
{
	FFilterExpressionHelpEntry(const TSet<FName>& InKeys, const ESequencerTextFilterValueType InValueType, const FText& InDescription)
	{
		Keys = InKeys;
		ValueType = InValueType;
		Description = InDescription;
	}

	FFilterExpressionHelpEntry(const TSharedRef<ISequencerTextFilterExpressionContext>& InExpressionContext)
	{
		Keys = InExpressionContext->GetKeys();
		ValueType = InExpressionContext->GetValueType();
		Description = InExpressionContext->GetDescription();
	}

	TSet<FName> Keys;

	ESequencerTextFilterValueType ValueType;

	FText Description;
};

struct FFilterExpressionHelpDialogConfig
{
	static constexpr float DefaultMaxDesiredWidth = 460.f;
	static constexpr float DefaultMaxDesiredHeight = 560.f;

	UE_API FFilterExpressionHelpDialogConfig();

	// Explicitly define special members so the deprecated field is not referenced implicitly
	FFilterExpressionHelpDialogConfig(const FFilterExpressionHelpDialogConfig&) = default;
	FFilterExpressionHelpDialogConfig(FFilterExpressionHelpDialogConfig&& InOther) noexcept
		: IdentifierName(MoveTemp(InOther.IdentifierName))
		, DialogTitle(MoveTemp(InOther.DialogTitle))
		, DocumentationLink(MoveTemp(InOther.DocumentationLink))
		, MaxDesiredWidth(InOther.MaxDesiredWidth)
		, MaxDesiredHeight(InOther.MaxDesiredHeight)
		, ExpressionHelpEntries(MoveTemp(InOther.ExpressionHelpEntries))
	{}

	FFilterExpressionHelpDialogConfig& operator=(const FFilterExpressionHelpDialogConfig&) = default;
	FFilterExpressionHelpDialogConfig& operator=(FFilterExpressionHelpDialogConfig&& InOther) noexcept
	{
		if (this != &InOther)
		{
			IdentifierName = MoveTemp(InOther.IdentifierName);
			DialogTitle = MoveTemp(InOther.DialogTitle);
			DocumentationLink = MoveTemp(InOther.DocumentationLink);
			MaxDesiredWidth = InOther.MaxDesiredWidth;
			MaxDesiredHeight = InOther.MaxDesiredHeight;
			ExpressionHelpEntries = MoveTemp(InOther.ExpressionHelpEntries);
		}
		return *this;
	}

	template<typename InFilterClass>
	void AddCommonFilterHelpEntries(const TSharedRef<ISequencerFilterBar>& InFilterBar)
	{
		TSet<FName> EntryNames;

		for (const TSharedRef<ISequencerTextFilterExpressionContext>& ExpressionContext : InFilterBar->GetTextFilterExpressionContexts())
		{
			if (!ExpressionContext->GetKeys().IsEmpty())
			{
				ExpressionHelpEntries.Add(ExpressionContext);
				EntryNames.Append(ExpressionContext->GetKeys());
			}
		}

		for (const TSharedRef<FSequencerFilterBase<InFilterClass>> CommonFilter : InFilterBar->template GetCommonFilters<InFilterClass>())
		{
			// Avoid adding duplicate entries for common filters if they've already been added through a text expression
			const FName CommonFilterName = *CommonFilter->GetName();
			if (!EntryNames.Contains(CommonFilterName))
			{
				ExpressionHelpEntries.Add(FFilterExpressionHelpEntry(
					{ CommonFilterName }, ESequencerTextFilterValueType::Boolean, CommonFilter->GetDefaultToolTipText()));
			}
		}
	}

	FName IdentifierName;

	FText DialogTitle;
	FString DocumentationLink;

	float MaxDesiredWidth = DefaultMaxDesiredWidth;
	float MaxDesiredHeight = DefaultMaxDesiredHeight;

	TArray<FFilterExpressionHelpEntry> ExpressionHelpEntries;

	UE_DEPRECATED(5.8, "Use ExpressionHelpEntries instead")
	TArray<TSharedRef<ISequencerTextFilterExpressionContext>> TextFilterExpressionContexts_DEPRECATED;
};

class SFilterExpressionHelpDialog : public SWindow
{
public:
	SEQUENCER_API static void Open(FFilterExpressionHelpDialogConfig&& InConfig);

	static bool IsOpen(const FName InName);

	static void CloseWindow(const FName InName);

	void Construct(const FArguments& InArgs, FFilterExpressionHelpDialogConfig&& InConfig);

protected:
	static const FSlateColor KeyColor;
	static const FSlateColor ValueColor;

	static TMap<FName, TSharedPtr<SFilterExpressionHelpDialog>> DialogInstance;

	TSharedRef<SWidget> ConstructDialogHeader();

	TSharedRef<SWidget> ConstructExpressionWidgetList();
	TSharedRef<SWidget> ConstructExpressionWidget(const FFilterExpressionHelpEntry& InHelpEntry);

	TSharedRef<SWidget> ConstructKeysWidget(const TSet<FName>& InKeys);
	TSharedRef<SWidget> ConstructValueWidget(const ESequencerTextFilterValueType InValueType);

	void OpenDocumentationLink() const;

	FFilterExpressionHelpDialogConfig Config;
};

#undef UE_API
