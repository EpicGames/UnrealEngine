// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SBasicFilterBar.h"

class SSequencerFilterCheckBox;

/**
 * Generic Sequencer filter class used by Sequencer and Sequence Navigator filter bars.
 */
class SSequencerFilter : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal(bool, FOnIsFilterActive);
	DECLARE_DELEGATE_OneParam(FOnFilterToggle, const ECheckBoxState /*InNewState*/);

	DECLARE_DELEGATE_RetVal(TSharedRef<SWidget>, FOnGetMenuContent);

	SLATE_BEGIN_ARGS(SSequencerFilter) {}

		/** Determines how each individual filter pill looks like */
		SLATE_ARGUMENT(EFilterPillStyle, FilterPillStyle)

		SLATE_EVENT(FOnIsFilterActive, OnIsFilterActive)
		SLATE_EVENT(FOnFilterToggle, OnFilterToggle)
		SLATE_EVENT(FSimpleDelegate, OnCtrlClick)
		SLATE_EVENT(FSimpleDelegate, OnAltClick)
		SLATE_EVENT(FSimpleDelegate, OnMiddleClick)
		SLATE_EVENT(FSimpleDelegate, OnDoubleClick)

		SLATE_ATTRIBUTE(FText, DisplayName)
		SLATE_ATTRIBUTE(FText, ToolTipText)
		SLATE_ATTRIBUTE(FSlateColor, BlockColor)

		SLATE_EVENT(FOnGetMenuContent, OnGetMenuContent)

	SLATE_END_ARGS()

	SEQUENCER_API void Construct(const FArguments& InArgs);

protected:
	TSharedRef<SWidget> ConstructBasicFilterWidget();
	TSharedRef<SWidget> ConstructDefaultFilterWidget();

	bool IsActive() const;

	void OnFilterToggled(const ECheckBoxState NewState);

	FReply OnFilterCtrlClick();
	FReply OnFilterAltClick();
	FReply OnFilterMiddleButtonClick();
	FReply OnFilterDoubleClick();

	TSharedRef<SWidget> GetRightClickMenuContent();

	ECheckBoxState IsChecked() const;

	FSlateColor GetFilterImageColorAndOpacity() const;

	EVisibility GetFilterOverlayVisibility() const;

	FMargin GetFilterNamePadding() const;

	bool IsButtonEnabled() const;

	FOnIsFilterActive IsFilterActiveDelegate;

	FOnFilterToggle FilterToggleDelegate;
	FSimpleDelegate CtrlClickDelegate;
	FSimpleDelegate AltClickDelegate;
	FSimpleDelegate MiddleClickDelegate;
	FSimpleDelegate DoubleClickDelegate;

	TAttribute<FText> DisplayName;
	TAttribute<FText> ToolTipText;
	TAttribute<FSlateColor> BlockColor;

	FOnGetMenuContent GetMenuContentDelegate;

	TSharedPtr<SSequencerFilterCheckBox> ToggleButtonPtr;
};
