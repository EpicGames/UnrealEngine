// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerFilterSuggestion.h"
#include "Filters/SFilterSearchBox.h"
#include "Framework/Text/TextLayout.h"
#include "Misc/ExpressionParserTypesFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API SEQUENCER_API

class FSequencerFilterSuggestionListEntryBase;
class FTextFilterExpressionEvaluator;
class ISequencerFilterBar;
class ITableRow;
class SMenuAnchor;
class STableViewBase;
struct FFocusEvent;
struct FGeometry;
struct FKeyEvent;

/** A delegate for a callback to filter the given suggestion list, to allow custom filtering behavior */
DECLARE_DELEGATE_ThreeParams(FOnSequencerSearchBoxSuggestionFilter, const FText& /*InSearchText*/, TArray<FSequencerFilterSuggestion>& /*InPossibleSuggestions*/, FText& /*InSuggestionHighlightText*/);

/** A delegate for a callback when a suggestion entry is chosen during a search, to allow custom compositing behavior of the suggestion into the search text */
DECLARE_DELEGATE_RetVal_TwoParams(FText, FOnSequencerSearchBoxSuggestionChosen, const FText& /*InSearchText*/, const FString& /*InSuggestion*/);

/**
 * A wrapper widget around SFilterSearchBox to provide filter text expression suggestions in a dropdown menu.
 */
class SSequencerSearchBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSequencerSearchBox)
		: _SuggestionListPlacement(MenuPlacement_BelowAnchor)
		, _PossibleSuggestions(TArray<FSequencerFilterSuggestion>())
		, _DelayChangeNotificationsWhileTyping(true)
		, _MustMatchPossibleSuggestions(false)
		, _ShowSearchHistory(true)
	{}

		/** Where to place the suggestion list */
		SLATE_ARGUMENT(EMenuPlacement, SuggestionListPlacement)

		/** Invoked whenever the text changes */
		SLATE_EVENT(FOnTextChanged, OnTextChanged)

		/** Invoked whenever the text is committed (e.g. user presses enter) */
		SLATE_EVENT(FOnTextCommitted, OnTextCommitted)

		/** Initial text to display for the search text */
		SLATE_ATTRIBUTE(FText, InitialText)

		/** Hint text to display for the search text when there is no value */
		SLATE_ATTRIBUTE(FText, HintText)

		/** All possible suggestions for the search text */
		SLATE_ATTRIBUTE(TArray<FSequencerFilterSuggestion>, PossibleSuggestions)

		/** Whether the SearchBox should delay notifying listeners of text changed events until the user is done typing */
		SLATE_ATTRIBUTE(bool, DelayChangeNotificationsWhileTyping)

		/** Whether the SearchBox allows entries that don't match the possible suggestions */
		SLATE_ATTRIBUTE(bool, MustMatchPossibleSuggestions)

		/** Callback to filter the given suggestion list, to allow custom filtering behavior */
		SLATE_EVENT(FOnSequencerSearchBoxSuggestionFilter, OnSuggestionFilter)

		/** Callback when a suggestion entry is chosen during an asset search, to allow custom compositing behavior of the suggestion into the search text */
		SLATE_EVENT(FOnSequencerSearchBoxSuggestionChosen, OnSuggestionChosen)

		/** Callback delegate to have first chance handling of the OnKeyDown event */
		SLATE_EVENT(FOnKeyDown, OnKeyDownHandler)
	
		/** Whether we should show a dropdown containing the last few searches */
		SLATE_ATTRIBUTE(bool, ShowSearchHistory)

		/** Handler for when the + Button next to a search is clicked */
		SLATE_EVENT(SFilterSearchBox::FOnSaveSearchClicked, OnSaveSearchClicked)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs, const TWeakPtr<ISequencerFilterBar>& InWeakFilterBar);

	/** Get the text string currently being edited */
	UE_API FText GetText() const;

	/** Sets the text string currently being edited */
	UE_API void SetText(const TAttribute<FText>& InNewText);

	/** Set or clear the current error reporting information for this search box */
	UE_API void SetError(const FText& InError);
	UE_API void SetError(const FString& InError);

	/** Show a + button next to the current search and set the handler for when that is clicked */
	UE_API void SetOnSaveSearchHandler(SFilterSearchBox::FOnSaveSearchClicked InOnSaveSearchHandler);

	//~ Begin SWidget
	UE_API virtual FReply OnPreviewKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	UE_API virtual bool SupportsKeyboardFocus() const override;
	UE_API virtual bool HasKeyboardFocus() const override;
	UE_API virtual FReply OnFocusReceived(const FGeometry& InGeometry, const FFocusEvent& InFocusEvent) override;
	//~ End SWidget

private:
	struct FParsedExpression
	{
		TOptional<FString> Key;
		TOptional<FString> Value;
		int32 SuggestionIndex = 0;
	};

	UE_API void ExtractSearchFilterTerms(const FText& InSearchText, FParsedExpression& OutParsedPair) const;

	UE_API TSharedRef<SWidget> GetSuggestionListMenuContent();

	/** First chance handler for key down events to the editable text widget */
	UE_API FReply HandleKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent);

	/** Handler for when text in the editable text box changed */
	UE_API void HandleTextChanged(const FText& InNewText);

	/** Handler for when text in the editable text box changed */
	UE_API void HandleTextCommitted(const FText& InNewText, const ETextCommit::Type InCommitType);

	/** Called by SListView when the selection changes in the suggestion list */
	UE_API void OnSelectionChanged(TSharedPtr<FSequencerFilterSuggestionListEntryBase> InNewValue, const ESelectInfo::Type InSelectInfo);

	/** Makes the widget for a suggestion message in the list view */
	UE_API TSharedRef<ITableRow> MakeSuggestionListItemWidget(const TSharedPtr<FSequencerFilterSuggestionListEntryBase> InSuggestion, const TSharedRef<STableViewBase>& InOwnerTable);

	/** Gets the text to highlight in the suggestion list */
	UE_API FText GetHighlightText() const;

	/** Updates and shows or hides the suggestion list */
	UE_API void UpdateSuggestionList();

	/** Sets the focus to the InputText box */
	UE_API void FocusEditBox();

	/** Returns the currently selected suggestion */
	UE_API TSharedPtr<FSequencerFilterSuggestionListEntryBase> GetSelectedSuggestion() const;

	/** Default implementation of OnSuggestionFilter, if no external implementation is provided */
	UE_API void DefaultSuggestionFilterImpl(const FText& InSearchText, TArray<FSequencerFilterSuggestion>& OutPossibleSuggestions, FText& OutSuggestionHighlightText);

	/** Default implementation of OnSuggestionChosen, if no external implementation is provided */
	UE_API FText DefaultSuggestionChosenImpl(const FText& InSearchText, const FString& InSuggestion);

	/** Provides default key suggestions, if no suggestions are provided */
	UE_API void DefaultKeySuggestions(const FString& InKeyValue, TArray<FSequencerFilterSuggestion>& OutPossibleSuggestions);

	/** Provides default value suggestions, if no suggestions are provided */
	UE_API void DefaultValueSuggestions(const FString& InKeyValue, TArray<FSequencerFilterSuggestion>& OutPossibleSuggestions);
	
	/** Updates the widget search text when the filter changes (@see ISequencerFilterBar::OnTextFilterTextChanged). */
	void OnFilterTextChanged();
	/** Calls SetEnabled when the mute state changes (@see ISequencerFilterBar::OnMuteStateChanged). */
	void OnMuteStateChanged(bool bIsMuted);

	static UE_API bool IsOperatorToken(const TExpressionToken<TCHAR>& InToken);
	static UE_API bool IsLogicalOperatorToken(const TExpressionToken<TCHAR>& InToken);

	static UE_API int32 FindTokenIndex(const TArray<TExpressionToken<TCHAR>>& ExpressionTokens, const int32 InIndexToFind);

	UE_API const FTextFilterExpressionEvaluator* GetTextFilterExpressionEvaluator() const;

	TWeakPtr<ISequencerFilterBar> WeakFilterBar;

	TSharedPtr<SFilterSearchBox> SearchBox;

	TSharedPtr<SListView<TSharedPtr<FSequencerFilterSuggestionListEntryBase>>> SuggestionListView;

	TSharedPtr<SMenuAnchor> SuggestionMenuAnchor;

	TArray<TSharedPtr<FSequencerFilterSuggestionListEntryBase>> SuggestionItems;

	/** The state of the text prior to being committed */
	FText PreCommittedText;

	/** The highlight text to use for the suggestions list */
	FText SuggestionHighlightText;

	FOnTextChanged OnTextChanged;
	FOnTextCommitted OnTextCommitted;

	/** Delegate to filter the given suggestion list, to allow custom filtering behavior */
	FOnSequencerSearchBoxSuggestionFilter OnSuggestionFilter;

	/** Delegate when a suggestion entry is chosen during a search, to allow custom compositing behavior of the suggestion into the search text */
	FOnSequencerSearchBoxSuggestionChosen OnSuggestionChosen;

	/** Delegate for first chance handling for key down events */
	FOnKeyDown OnKeyDownHandler;

	/** All possible suggestions for the search text */
	TAttribute<TArray<FSequencerFilterSuggestion>> PossibleSuggestions;

	/** Determines whether the committed text should match a suggestion */
	bool bMustMatchPossibleSuggestions = false;

	bool bDisableOpeningSuggestions = true;

	FTextLocation LastCursorLocation = FTextLocation(0, 0);

	int32 LastChosenCursorOffset = INDEX_NONE;
};

#undef UE_API
