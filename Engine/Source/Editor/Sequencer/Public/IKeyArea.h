// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Color.h"
#include "UObject/ObjectKey.h"
#include "UObject/NameTypes.h"
#include "Templates/PimplPtr.h"
#include "Templates/SharedPointer.h"
#include "Containers/ArrayView.h"
#include "Curves/KeyHandle.h"
#include "MovieSceneCommonHelpers.h"
#include "Channels/MovieSceneChannelHandle.h"

#define UE_API SEQUENCER_API

struct FCurveModelID;
struct FRichCurve;
struct FKeyDrawParams;
struct FMovieSceneChannel;
struct FMovieSceneChannelProxy;
struct FSequencerChannelPaintArgs;
struct FSequencerPasteEnvironment;
struct ISequencerChannelInterface;
struct FMovieSceneChannelMetaData;
struct FMovieSceneClipboardEnvironment;
struct FGeometry;

class SWidget;
class ISequencer;
class FCurveModel;
class FStructOnScope;
class ISequencerSection;
class UMovieSceneSection;
class FMovieSceneClipboardBuilder;
class FMovieSceneClipboardKeyTrack;
class FSequencerSectionPainter;

/**
 * Interface that should be implemented for the UI portion of a key area within a section
 */
class IKeyArea : public TSharedFromThis<IKeyArea>
{
public:

	/** Serial number of the node tree from the last time this key area was seen */
	uint32 TreeSerialNumber;

	/**
	 * Constructor
	 *
	 * @param InSection The section that owns the channel that this key area represents
	 * @param InChannel Handle to the channel this key area represents
	 */
	UE_API IKeyArea(TWeakPtr<ISequencerSection> InSection, FMovieSceneChannelHandle InChannel);

	/**
	 * Reinitialize this key area with a new channel
	 *
	 * @param InSection The section that owns the channel that this key area represents
	 * @param InChannel Handle to the channel this key area represents
	 */
	UE_API void Reinitialize(TWeakPtr<ISequencerSection> InSection, FMovieSceneChannelHandle InChannel);

public:

	/**
	 * Locate the sequencer channel interface for this key area's channel
	 * @note: Channel interfaces are registered via ISequencerModule::RegisterChannelInterface
	 *
	 * @return The channel interface, or nullptr if one was not found
	 */
	UE_API ISequencerChannelInterface* FindChannelEditorInterface() const;

	/**
	 * Access the channel type identifier for the channel that this key area wraps
	 *
	 * @return The name of the channel type
	 */
	FName GetChannelTypeName() const
	{
		return ChannelHandle.GetChannelTypeName();
	}

	/**
	 * Access the channel handle that this key area represents
	 *
	 * @return The (potentially invalid) channel handle
	 */
	const FMovieSceneChannelHandle& GetChannel() const
	{
		return ChannelHandle;
	}

	/**
	 * Resolve this key area's channel handle
	 *
	 * @return A (potentially invalid) interface to this key area's channel
	 */
	UE_API FMovieSceneChannel* ResolveChannel() const;

	/**
	 * Get this key area's name
	 *
	 * @return This key area's name
	 */
	UE_API FName GetName() const;

	/**
	 * Set this key area's name
	 *
	 * @param InName This key area's name
	 */
	UE_API void SetName(FName InName);

	/**
	 * Get the color of this channel that should be drawn underneath its keys
	 *
	 * @return (Optional) This channel's color
	 */
	TOptional<FLinearColor> GetColor() const
	{
		return Color;
	}

	/**
	 * Access section that owns the channel this key area represents
	 *
	 * @return The owning section, or nullptr if it has been destroyed
	 */
	UE_API UMovieSceneSection* GetOwningSection() const;


	/**
	 * Access object that owns the channel this key area represents.
	 * @note: this may return the same as GetOwningSection (the common case)
	 *
	 * @return The owning object, or nullptr if it has been destroyed
	 */
	UE_API UObject* GetOwningObject() const;

	/**
	 * Access section editor interface
	 *
	 * @return The owning section interface, or nullptr if it has been destroyed
	 */
	UE_API TSharedPtr<ISequencerSection> GetSectionInterface() const;

	/**
	 * Check whether we should show this key area's curve on its channel UI or not
	 *
	 * @return True to show the curve, false otherwise
	 */
	UE_API bool ShouldShowCurve() const;

public:

	/**
	 * Add a key at the specified time with the current value of the channel, updating an existing key if possible
	 *
	 * @param Time            The time at which a key should be added (or updated)
	 * @param ObjectBindingID The object binding ID this key area's track is bound to 
	 * @param InSequencer     The currently active sequencer
	 * @return A handle to the key that was added or updated
	 */
	UE_API FKeyHandle AddOrUpdateKey(FFrameNumber Time, const FGuid& ObjectBindingID, ISequencer& InSequencer);

	/**
	 * Delete the specified keys. If all keys are removed, the current value at that time will be set as the default value for the channel
	 * 
	 * @param Channel               The channel to remove the keys from
	 * @param InHandles             The key handles to delete
	 * @param InTime                The time at which to evaluate for the default value if there are no keys remaining
	 */
	UE_API void DeleteKeys(TArrayView<const FKeyHandle> InHandles, FFrameNumber InTime);

	/**
	 * Duplicate the key represented by the specified handle
	 *
	 * @param InKeyHandle     A handle to the key to duplicate
	 * @return A handle to the new duplicated key
	 */
	UE_API FKeyHandle DuplicateKey(FKeyHandle InKeyHandle) const;

	/**
	 * Get the time of the key represented by the specified handle
	 *
	 * @param InKeyHandle     A handle to the key to query for
	 * @return The key's time, or TNumericLimits<int32>::Lowest() if the handle was not valid
	 */
	FORCEINLINE FFrameNumber GetKeyTime(FKeyHandle InKeyHandle) const
	{
		FFrameNumber Time = TNumericLimits<int32>::Lowest();
		GetKeyTimes(TArrayView<const FKeyHandle>(&InKeyHandle, 1), TArrayView<FFrameNumber>(&Time, 1));
		return Time;
	}

	/**
	 * Get the times of every key represented by the specified handles
	 *
	 * @param InKeyHandles     A handle to the key to query for
	 * @param OutTimes         A pre-sized array view to populate with key times
	 */
	UE_API void GetKeyTimes(TArrayView<const FKeyHandle> InKeyHandles, TArrayView<FFrameNumber> OutTimes) const;

	/**
	 * Get all key handles that exist within the given time range
	 *
	 * @param OutHandles       An array to populate with key handles, any key handles will be appended to this array
	 * @param WithinRange      (Optional) A predicate range to search within
	 */
	FORCEINLINE void GetKeyHandles(TArray<FKeyHandle>& OutHandles, const TRange<FFrameNumber>& WithinRange = TRange<FFrameNumber>::All()) const
	{
		GetKeyInfo(&OutHandles, nullptr, WithinRange);
	}

	/**
	 * Get all key times that exist within the given time range
	 *
	 * @param OutTimes         An array to populate with key times, any key times will be appended to this array
	 * @param WithinRange      (Optional) A predicate range to search within
	 */
	FORCEINLINE void GetKeyTimes(TArray<FFrameNumber>& OutTimes, const TRange<FFrameNumber>& WithinRange = TRange<FFrameNumber>::All()) const
	{
		GetKeyInfo(nullptr, &OutTimes, WithinRange);
	}

	/**
	 * Populate the specified handle and/or time arrays with information pertaining to keys that exist within the given range
	 *
	 * @param OutHandles       (Optional) An array to populate with key handles, any key handles will be appended to this array
	 * @param OutTimes         (Optional) An array to populate with key times, any key times will be appended to this array
	 * @param WithinRange      (Optional) A predicate range to search within
	 */
	UE_API void GetKeyInfo(TArray<FKeyHandle>* OutHandles, TArray<FFrameNumber>* OutTimes, const TRange<FFrameNumber>& WithinRange = TRange<FFrameNumber>::All()) const;

	/**
	 * Set the time of the key with the specified handle
	 *
	 * @param InKeyHandle      The key handle to set
	 * @param InKeyTime        The time to set the key to
	 */
	void SetKeyTime(FKeyHandle InKeyHandle, FFrameNumber InKeyTime) const
	{
		SetKeyTimes(TArrayView<const FKeyHandle>(&InKeyHandle, 1), TArrayView<const FFrameNumber>(&InKeyTime, 1));
	}

	/**
	 * Set the times of the each key with the specified handles
	 *
	 * @param InKeyHandles     An array of handles that should have their time set to times in the corresponding InKeyTimes array
	 * @param InKeyTimes       Array of times to set to, one per key handle. Must match the size of InKeyHandles
	 */
	UE_API void SetKeyTimes(TArrayView<const FKeyHandle> InKeyHandles, TArrayView<const FFrameNumber> InKeyTimes) const;

public:

	/**
	 * Gather key drawing information for the specified key handles
	 *
	 * @param InKeyHandles     An array of handles to draw
	 * @param OutKeyDrawParams Pre-sized array view of parameters to populate with each key
	 */
	UE_API void DrawKeys(TArrayView<const FKeyHandle> InKeyHandles, TArrayView<FKeyDrawParams> OutKeyDrawParams);

	/**
	 * Copy all the keys specified in KeyMask to the specified clipboard
	 *
	 * @param ClipboardBuilder The structure responsible for building clipboard information for each key
	 * @param KeyMask          A specific set of keys to copy
	 */
	UE_API void CopyKeys(FMovieSceneClipboardBuilder& ClipboardBuilder, TArrayView<const FKeyHandle> KeyMask) const;

	/**
	 * Draw additional content in addition to keys for a particular IKeyArea
	 *
	 * @param PaintArgs	       Paint arguments containing the draw element list, time-to-pixel converter and other structures
	 * @param LayerId          The slate layer to paint onto
	 * @return The new slate layer ID for subsequent elements to paint onto
	 */
	UE_API int32 DrawExtra(const FSequencerChannelPaintArgs& PaintArgs, int32 LayerId) const;

	/**
	 * Paste the specified key track into this key area
	 *
	 * @param KeyTrack         The source clipboard data to paste
	 * @param SrcEnvironment   The environment the source data was copied from
	 * @param DstEnvironment   The environment we're pasting into
	 */
	UE_API TArray<FKeyHandle> PasteKeys(const FMovieSceneClipboardKeyTrack& KeyTrack, const FMovieSceneClipboardEnvironment& SrcEnvironment, const FSequencerPasteEnvironment& DstEnvironment);


	/**
	 * Create a new model for this key area that can be used on the curve editor interface
	 *
	 * @return (Optional) A new model to be added to a curve editor
	 */
	UE_API TUniquePtr<FCurveModel> CreateCurveEditorModel(TSharedRef<ISequencer> InSequencer) const;

	/**
	 * Retrieve the interpolation mode for this key area's channel for the specified time
	 */
	UE_API EMovieSceneKeyInterpolation GetInterpolationMode(FFrameNumber InTime, EMovieSceneKeyInterpolation DefaultMode) const;

public:

	FRichCurve* GetRichCurve() const { return nullptr; }

	/**
	 * Get a key structure for editing a value on this channel
	 *
	 * @return The key structure for this channel, or nullptr
	 */
	UE_API TSharedPtr<FStructOnScope> GetKeyStruct(FKeyHandle KeyHandle) const;

	/**
	 * Check whether this key area can create an editor on the sequencer node tree
	 */
	UE_API bool CanCreateKeyEditor() const;

	/**
	 * Create an editor on the sequencer node tree
	 *
	 * @param Sequencer        The currently active sequencer
	 * @param ObjectBindingID  The ID of the object this key area's track is bound to
	 * @return The editor widget to display on the key area's node
	 */
	UE_API TSharedRef<SWidget> CreateKeyEditor(TWeakPtr<ISequencer> Sequencer, const FGuid& ObjectBindingID);

private:

	/** A weak pointer back to the originating section interface that owns this channel */
	TWeakPtr<ISequencerSection> WeakSection;

	/** Handle to the channel itself */
	FMovieSceneChannelHandle ChannelHandle;

	/** Optional property bindings class where the section resides inside a UMovieScenePropertyTrack */
	TOptional<FTrackInstancePropertyBindings> PropertyBindings;

	/** The color of this channel that should be drawn underneath its keys */
	TOptional<FLinearColor> Color;

	/** The name of this channel. */
	FName ChannelName;

	/** The display text of this channel. */
	FText DisplayText;
};

#undef UE_API
