// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UMovieSceneCinematicShotSection;
class UMovieSceneSequence;

class IMovieRendererInterface
{
public:
	virtual ~IMovieRendererInterface()= default;

	/* Render the given movie scene sequence */
	virtual void RenderMovie(UMovieSceneSequence*, const TArray<UMovieSceneCinematicShotSection*>& InSections) = 0;

	/* The display name */
	virtual FString GetDisplayName() const = 0;
};
