// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/Function.h"
#include "Channels/MovieSceneChannelEditorData.h"

class IKeyArea;

struct FMovieSceneChannelHandle;

namespace UE::Sequencer
{
	class FCategoryModel;
	class FChannelModel;
	class FSectionModel;
}

/**
 * Builds an inner layout for a section
 */
class ISectionLayoutBuilder
{
public:
	virtual ~ISectionLayoutBuilder() = default;

	/** 
	 * Pushes a new category onto the layout.  If there is a current category, this category will appear as a child of the current category
	 *
	 * @param CategoryName	The name of the category
	 * @param DisplayLabel	The localized display label for the category
	 */
	virtual void PushCategory( FName CategoryName, const FText& DisplayLabel, FGetMovieSceneTooltipText GetGroupTooltipTextDelegate, TFunction<TSharedPtr<UE::Sequencer::FCategoryModel>(FName, const FText&)> OptionalFactory) = 0;

	/**
	 * Sets the section as a key area itself
	 * @param Channel		The channel that is to be assigned as the top level channel for this section
	 */
	virtual void SetTopLevelChannel( const FMovieSceneChannelHandle& Channel, TFunction<TSharedPtr<UE::Sequencer::FChannelModel>(FName, const UE::Sequencer::FSectionModel&, const FMovieSceneChannelHandle&)> OptionalFactory ) = 0;

	/**
	 * Adds a channel onto the layout. If a category is pushed, the key area will appear as a child of the current category
	 *
	 * @param Channel		A handle to the channel to be added to the layout
	 */
	virtual void AddChannel( const FMovieSceneChannelHandle& Channel, TFunction<TSharedPtr<UE::Sequencer::FChannelModel>(FName, const UE::Sequencer::FSectionModel&, const FMovieSceneChannelHandle&)> OptionalFactory ) = 0;

	/**
	 * Pops a category off the stack
	 */
	virtual void PopCategory() = 0;


};

