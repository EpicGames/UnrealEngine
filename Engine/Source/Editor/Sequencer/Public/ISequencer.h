// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "ViewRangeInterpolation.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "Filters/ISequencerTrackFilters.h"
#include "HAL/Platform.h"
#include "IMovieScenePlayer.h"
#include "IMovieScenePlayer.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "KeyParams.h"
#include "KeyPropertyParams.h"
#include "Math/Range.h"
#include "Misc/FrameRate.h"
#include "Misc/Guid.h"
#include "Misc/QualifiedFrameTime.h"
#include "MovieSceneBinding.h"
#include "MovieSceneSequenceID.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/SWidget.h"
#include "ISequencerNumericTypeInterface.h"

#include "Bindings/MovieSceneCustomBinding.h"

#include "ISequencer.generated.h"

class AActor;
class ACameraActor;
class FSequencerKeyCollection;
class FSequencerSelectionPreview;
class FUICommandList;
class IDetailsView;
class IKeyArea;
class ISidebarDrawerContent;
class ISequencerDecorationEditor;
class ISequencerTrackEditor;
class SWidget;
struct FMovieSceneChannelHandle;
class UActorFactory;
class UCameraComponent;
class UMovieSceneCinematicShotSection;
class UMovieSceneFolder;
class UMovieSceneSection;
class UMovieSceneSequence;
class UMovieSceneSubSection;
class UMovieSceneTrack;
class UObject;
class USequencerSettings;
struct FCanKeyPropertyParams;
struct FFrameNumber;
struct FFrameTime;
struct FKeyPropertyParams;
struct FMovieSceneBinding;
struct FMovieSceneChannelHandle;
struct FMovieSceneMarkedFrame;
struct FQualifiedFrameTime;
struct FSequencerHostCapabilities;
struct FSidebarDrawerConfig;
template <typename NumericType> struct INumericTypeInterface;
class IPropertyTypeCustomization;

enum class EMapChangeType : uint8;
enum class EPropertyKeyedStatus : uint8;
enum class ENearestKeyOption : uint8;
class FCurveEditor;
class FCurveModel;
class IToolkitHost;
struct FAnimatedRange;
struct FMovieSceneChannelMetaData;
struct FMovieSceneSequencePlaybackParams;

namespace UE
{
namespace Sequencer
{
class ISequencerWidget;

enum class ETimeDomain : uint8;
class FSequencerEditorViewModel;
class SOutlinerView;
struct FCreateBindingParams;
struct FTimeDomainOverride;
struct FSequencerNumericTypeInterface;

} // namespace Sequencer
} // namespace UE

/**
 * Defines auto change modes.
 */
UENUM()
enum class EAutoChangeMode : uint8
{
	/** Create a key when a property changes. */
	AutoKey,

	/** Create a track when a property changes. */
	AutoTrack,

	/** Create a key and a track when a property changes. */
	All,

	/** Do nothing */
	None
};


/**
 * Defines allow edits mode.
 */
UENUM()
enum class EAllowEditsMode : uint8
{
	/** Allow all edits. */
	AllEdits,

	/** Allow edits to go to sequencer only. */
	AllowSequencerEditsOnly,

	/** Allow edits to go to level only */
	AllowLevelEditsOnly
};

/**
* Defines set key gruops mode.
*/
UENUM()
enum class EKeyGroupMode : uint8
{
	/** Key just changed channel */
	KeyChanged,

	/** Key just one, the parent translation, rotation or scale, when one changes */
	KeyGroup,

	/** Key All (translation, rotation, scale) when one changes */
	KeyAll
};


/**
 * Enumerates types of UI Command bindings.
 */
enum class ESequencerCommandBindings
{
	/** Bindings that are used by Sequencer widgets only. */
	Sequencer,

	/** Bindings that are shared between Sequencer and non-Sequencer widgets (subset of Sequencer commands). */
	Shared,

	/** Bindings that are available in the Curve Editor. */
	CurveEditor
};


/*
 * Allowable snapping modes when setting global time
 */
enum ESnapTimeMode
{
	/** No snapping */
	STM_None = 0x00000000,

	/** Snap to the time interval. */
	STM_Interval = 0x00000001,

	/** Snap to keys. */
	STM_Keys = 0x00000002,

	/** All snapping */
	STM_All = STM_Interval | STM_Keys
};


/**
 * Defines different types of movie scene data changes. 
 */
enum class EMovieSceneDataChangeType
{
	/** Data owned by a track has been modified such as adding or removing keys, or changing their values. */
	TrackValueChanged,
	/** Data owned by a track has been modified such as adding or removing keys, or changing their values. Refresh immediately. */
	TrackValueChangedRefreshImmediately,
	/** The structure of the movie scene has changed by adding folders, object bindings, tracks, or sections. */
	MovieSceneStructureItemAdded,
	/** The structure of the movie scene has changed by removing folders, object bindings, tracks, or sections. */
	MovieSceneStructureItemRemoved,
	/** The structure of the movie scene has changed by adding and removing folders, object bindings, tracks, or sections. */
	MovieSceneStructureItemsChanged,
	/** The active movie scene has been changed to a new movie scene. */
	ActiveMovieSceneChanged,
	/** Rebuild and evaluate everything immediately. */
	RefreshAllImmediately,
	/** It's not known what data has changed. */
	Unknown,
	/** Refresh Tree on Next Tick */
	RefreshTree
};

/**
 * Interface for sequencers.
 */
class ISequencer
	: public IMovieScenePlayer
	, public TSharedFromThis<ISequencer>
{
public:
	
	DECLARE_MULTICAST_DELEGATE(FOnGlobalTimeChanged);
	DECLARE_MULTICAST_DELEGATE(FOnPlayEvent);
	DECLARE_MULTICAST_DELEGATE(FOnStopEvent);
	DECLARE_MULTICAST_DELEGATE(FOnRecordEvent);
	DECLARE_MULTICAST_DELEGATE(FOnBeginScrubbingEvent);
	DECLARE_MULTICAST_DELEGATE(FOnEndScrubbingEvent);
	DECLARE_DELEGATE_RetVal(TArray<float>, FOnGetPlaybackSpeeds);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMovieSceneDataChanged, EMovieSceneDataChangeType);
	UE_DEPRECATED(5.8, "FOnChannelChanged is now deprecated. Use FMovieSceneOnChannelChangedWithTime")
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnChannelChanged, const FMovieSceneChannelMetaData* MetaData, UMovieSceneSection*)
	
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FMovieSceneOnChannelChangedWithTime, const TArray<FFrameNumber>&, const FMovieSceneChannelMetaData* MetaData, UMovieSceneSection*)
	DECLARE_MULTICAST_DELEGATE(FOnMovieSceneBindingsChanged);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMovieSceneBindingsPasted, const TArray<FMovieSceneBinding>&);
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSelectionChangedObjectGuids, TArray<FGuid> /*Object*/);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSelectionChangedTracks, TArray<UMovieSceneTrack*> /*Tracks*/);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSelectionChangedSections, TArray<UMovieSceneSection*> /*Sections*/);

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnCurveDisplayChanged, FCurveModel* , bool /*displayed*/,const FCurveEditor*);


	DECLARE_MULTICAST_DELEGATE_OneParam(FOnCloseEvent, TSharedRef<ISequencer>);

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnActorAddedToSequencer, AActor*, const FGuid);

	DECLARE_MULTICAST_DELEGATE(FOnTreeViewChanged);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnViewportSelectionLimitedChanged, const bool /*bInEnabled*/);

public:

	/** Close the sequencer. */
	virtual void Close() = 0;

	/** @return a multicast delegate which is executed when sequencer closes. */
	virtual FOnCloseEvent& OnCloseEvent() = 0;

	/** @return Widget used to display the sequencer */
	virtual TSharedRef<SWidget> GetSequencerWidget() const = 0;
	
	/** @return Gets the interface with which you can interact with the widget used to display Sequencer. */
	virtual UE::Sequencer::ISequencerWidget& GetSequencerWidgetInterface() const = 0;

	/** @return The root movie scene being used */
	virtual UMovieSceneSequence* GetRootMovieSceneSequence() const = 0;

	/** @return Returns the MovieScene that is currently focused for editing by the sequencer.  This can change at any time. */
	virtual UMovieSceneSequence* GetFocusedMovieSceneSequence() const = 0;

	/**@return Returns the time transform from the focused sequence back to the root, this includes global time warp*/
	virtual FMovieSceneSequenceTransform GetFocusedMovieSceneSequenceTransform() const = 0;

	/**@return Returns the time transform from the focused sequence back to the root, without timewarp, to be used for example when baking and we need to pass time for evaluation*/
	virtual FMovieSceneSequenceTransform GetFocusedMovieSceneSequenceTransformUnwarped() const = 0;

	/**@return Returns the timewrap transform that local time to warped-local-time */
	virtual FMovieSceneSequenceTransform GetLocalTimeWarpTransform() const = 0;

	/**@return Returns the timewrap transform that applies to global playback */
	virtual FMovieSceneSequenceTransform GetGlobalPlaybackWarpTransform() const = 0;

	/** @return The root movie scene being used */
	virtual FMovieSceneSequenceIDRef GetRootTemplateID() const = 0;
	virtual FMovieSceneSequenceIDRef GetFocusedTemplateID() const = 0;
	virtual const TArray<FMovieSceneSequenceID>& GetSubSequenceHierarchy() const = 0;

	/** Attempt to locate the sub section that relates to the specified sequence ID. */
	virtual UMovieSceneSubSection* FindSubSection(FMovieSceneSequenceID SequenceID) const = 0;

	TArrayView<TWeakObjectPtr<>> FindObjectsInCurrentSequence(const FGuid& InObjectBinding)
	{
		return FindBoundObjects(InObjectBinding, GetFocusedTemplateID());
	}

	/** Resets sequencer with a new animation */
	virtual void ResetToNewRootSequence(UMovieSceneSequence& NewAnimation) = 0;

	/**
	 * Focuses a sub-movie scene (MovieScene within a MovieScene) in the sequencer.
	 * 
	 * @param Section The sub-movie scene section containing the sequence instance to get.
	 */
	virtual void FocusSequenceInstance(UMovieSceneSubSection& Section) = 0;

	/**
	 * Pops the current focused movie scene from the stack.  The parent of this movie scene will be come the focused one
	 */
	virtual void PopToSequenceInstance(FMovieSceneSequenceIDRef SequenceID) = 0;

	/**
	 * Get the currently viewed sub sequence range
	 *
	 * @return The sub sequence range, or an empty optional if we're viewing the root.
	 */
	virtual TOptional<TRange<FFrameNumber>> GetSubSequenceRange() const
	{
		TOptional<TRange<FFrameNumber>> Optional;
		return Optional;
	}

	/**
	 * Retrieve the top level view model for this sequence
	 */
	virtual TSharedPtr<UE::Sequencer::FSequencerEditorViewModel> GetViewModel() const = 0;

	/**
	 * Suppresses automatic evaluation the specified sequence and signature are the only difference that would prompt a re-evaluation
	 * 
	 * @param Sequence        The sequence that the signature relates to
	 * @param InSignature     The signature to suppress
	 */
	virtual void SuppressAutoEvaluation(UMovieSceneSequence* Sequence, const FGuid& InSignature) = 0;

	/**
	* Create a new binding for the specified object
	*/
	virtual FGuid CreateBinding(UObject& InObject, const UE::Sequencer::FCreateBindingParams& InParams) = 0;

	// Override from IMovieScenePlayer
	FGuid CreateBinding(UMovieSceneSequence* InSequence, UObject* InObject) override;

	/**
	 * Attempts to add a new spawnable to the MovieScene for the specified object (asset, class or actor instance)
	 *
	 * @param	Object	The asset, class, or actor to add a spawnable for
	 * @param	ActorFactory	Optional actor factory to use to create spawnable type
	 * @param   bSetupDefaults Setup default tracks for this spawnable
	 * @return	The spawnable guid for the spawnable, or an invalid Guid if we were not able to create a spawnable
	 */
	virtual FGuid MakeNewSpawnable(UObject& SourceObject, UActorFactory* ActorFactory = nullptr, bool bSetupDefaults = true) = 0;

	/**
	 * Add actors as possessable objects to sequencer.
	 * 
	 * @param InActors The actors to add to sequencer.
	 * @param bSelectActors Select the newly added possessable objects in sequencer.
	 * @return The posssessable guids for the newly added actors.
	 */
	virtual TArray<FGuid> AddActors(const TArray<TWeakObjectPtr<AActor> >& InActors, bool bSelectActors = true) = 0;

	/**
	* Add a new empty binding to Sequencer which can be then connected to an object or actor afterwards in the binding properties menu.
	*/
	virtual FGuid AddEmptyBinding() = 0;

	/**
	 * Should be called after adding a binding to the MovieScene.
	 */
	virtual void OnAddBinding(const FGuid& ObjectBinding, UMovieScene* MovieScene) = 0;

	/**
	 * Should be called after adding a track to the MovieScene. This will set the specified track as your current selection
	 * cause it to throb, notify the sequence to rebuild any data required. The track will be added to the selected folder
	 * unless ObjectBinding points to a valid FGuid.
	 */
	virtual void OnAddTrack(const TWeakObjectPtr<UMovieSceneTrack>& InTrack, const FGuid& ObjectBinding) = 0;

	/**
	* Convert the Possessable to a Spawnable. Returns an array of Spawnable Id's
	* @param Guid The Possessable Guid.
	* @return Array of Spawnable Guids
	*/
	virtual TArray<FGuid> ConvertToSpawnable(FGuid Guid) = 0;

	/**
	 * Adds a movie scene as a section inside the current movie scene
	 * 
	 * @param Sequence The sequence to add.
	 */
	virtual void AddSubSequence(UMovieSceneSequence* Sequence) = 0;

	/** @return Returns the current auto-change mode. */
	virtual EAutoChangeMode GetAutoChangeMode() const = 0;

	/** SSets the current auto-change mode. */
	virtual void SetAutoChangeMode(EAutoChangeMode AutoChangeMode) = 0;

	/** @return Returns where edits are allowed. */
	virtual EAllowEditsMode GetAllowEditsMode() const = 0;

	/** Sets where edits are allowed */
	virtual void SetAllowEditsMode(EAllowEditsMode AllowEditsMode) = 0;

	/** @returns Returns what channels will get keyed when one channel changes */
	virtual EKeyGroupMode GetKeyGroupMode() const = 0;

	/** Sets which channels are keyed when a channel is keyed 
	* @param Mode Specifies which channels to key, all (EKeyGroupMode::KeyAll), 
	* just changed (EKeyGroup::KeyChanged), 
	* just the group,e.g. Rotation X,Y and Z if any of those are changed (EKeyGroup::KeyGroup)
	*/
	virtual void SetKeyGroupMode(EKeyGroupMode Mode) = 0;

	/** @return Returns initial key interpolation */
	virtual EMovieSceneKeyInterpolation GetKeyInterpolation() const = 0;

	/** Set initial key interpolation */
	virtual void SetKeyInterpolation(EMovieSceneKeyInterpolation) = 0;

	/** @return Returns whether key sections are infinite by default when created */
	virtual bool GetInfiniteKeyAreas() const = 0;

	/** Set infinite key area default */
	virtual void SetInfiniteKeyAreas(bool bInfiniteKeyAreas) = 0;

	/** Gets whether or not property track defaults will be automatically set when adding tracks. */
	virtual bool GetAutoSetTrackDefaults() const = 0;

	/** @return Returns whether sequencer will respond to changes and possibly create a key or track */
	virtual bool IsAllowedToChange() const 
	{
		if (IsReadOnly() || GetAllowEditsMode() == EAllowEditsMode::AllowLevelEditsOnly)
		{
			return false;
		}

		return GetAllowEditsMode() != EAllowEditsMode::AllowLevelEditsOnly || GetAutoChangeMode() != EAutoChangeMode::None;
	}

	/** Returns the Toolkit hosting the sequencer instance, if any */	
	virtual TSharedPtr<IToolkitHost> GetToolkitHost() const = 0;

	/** @return The capabilities of this sequencer host */
	virtual const FSequencerHostCapabilities& GetHostCapabilities() const = 0;

	/**
	 * Gets the current time of the time slider relative to the currently focused movie scene
	 */
	virtual FQualifiedFrameTime GetLocalTime() const = 0;

	/**
	 * Gets the global time.
	 *
	 * @return Global time in seconds.
	 * @see SetGlobalTime
	 */
	virtual FQualifiedFrameTime GetGlobalTime() const = 0;

	/**
	 * Retrieve the current local time in unwarped space. This is usually only required for time-warp operations. Prefer GetLocalTime.
	 */
	virtual FQualifiedFrameTime GetUnwarpedLocalTime() const = 0;

	/**
	 * Temporarily override the behavior of a call to one of the SetLocalTime or OnScrubPositionChanged functions to operate in a specific time-domain
	 */
	[[nodiscard]] virtual UE::Sequencer::FTimeDomainOverride OverrideTimeDomain(UE::Sequencer::ETimeDomain NewDomain) = 0;

	virtual TOptional<int32> GetLocalLoopIndex() const = 0;

	/**
	 * Sets the cursor position relative to the currently focused sequence
	 *
	 * @param Time The local time to set.
	 * @param SnapTimeMode The type of time snapping allowed.
	 * @param bEvaluate If True also evaluate
	 */
	virtual void SetLocalTime(FFrameTime Time, ESnapTimeMode SnapTimeMode = ESnapTimeMode::STM_None, bool bEvaluate = true) = 0;

	/** Set the current local time directly, with no other snapping, scrolling or manipulation */
	virtual void SetLocalTimeDirectly(FFrameTime NewTime, bool bEvaluate = true) = 0;

	/** Set the global time directly, without performing any auto-scroll, snapping or other adjustments to the supplied time  */
	virtual void SetGlobalTime(FFrameTime Time, bool bEvaluate = true) = 0;

	/** Get the last evaluated time, which may be different form the current local time*/
	virtual FFrameTime GetLastEvaluatedLocalTime() const = 0;

	/** Play from the current time to the requested time */
	virtual void PlayTo(FMovieSceneSequencePlaybackParams PlaybackParams) = 0;

	/*Modify the Sequencer time by any snap settings */
	virtual void SnapSequencerTime(FFrameTime& InOutScrubTime) = 0;

	/** Invalidate cached data so that it will be reevaluated on the next frame */
	virtual void RequestInvalidateCachedData() = 0;

	/** Forcefully reevaluate the sequence on the next frame */
	virtual void RequestEvaluate() = 0;

	/** Forcefully reevaluate the sequence immediately */
	virtual void ForceEvaluate() = 0;
	
	/** @return The camera cut that was last used by a camera cut; it is the view that the sequence should have at the current time. */
	SEQUENCER_API TWeakObjectPtr<UCameraComponent> GetLastEvaluatedCameraCut() const;

	/** Reset the timing manager to the clock source specified by the root movie scene */
	virtual void ResetTimeController() = 0;

	/** @return The current view range */
	virtual FAnimatedRange GetViewRange() const;

	/**
	 * Set the view range, growing the working range to accomodate, if necessary
	 * @param NewViewRange The new view range. Must be a finite range
	 * @param Interpolation How to interpolate to the new view range
	 */
	virtual void SetViewRange(TRange<double> NewViewRange, EViewRangeInterpolation Interpolation = EViewRangeInterpolation::Animated) = 0;

	/**
	 * Set the clamp range
	 * @param NewClampRange The new clamp range. Must be a finite range
	 */
	virtual void SetClampRange(TRange<double> NewClampRange) = 0;

	/**
	 * Sets whether perspective viewport hijacking is enabled.
	 *
	 * @param bEnabled true if the viewport should be enabled, false if it should be disabled.
	 */
	virtual void SetPerspectiveViewportPossessionEnabled(bool bEnabled) = 0;

	/*
	 * Gets whether perspective viewport hijacking is enabled.
	 */ 
	virtual bool IsPerspectiveViewportPossessionEnabled() const { return true; }

	/**
	 * Sets whether perspective viewport camera cutting is enabled.
	 *
	 * @param bEnabled true if the viewport should be enabled, false if it should be disabled.
	 */
	virtual void SetPerspectiveViewportCameraCutEnabled(bool bEnabled) = 0;

	/*
	 * Gets whether perspective viewport hijacking is enabled.
	 */ 
	virtual bool IsPerspectiveViewportCameraCutEnabled() const { return true; }

	/**
	 * Gets the list of bindings for camera objects.
	 *
	 * @param OutBindingIDs  The list of binding IDs for cameras
	 */
	virtual void GetCameraObjectBindings(TArray<FGuid>& OutBindingIDs) {}

	/*
	 * Render movie for a section.
	 * 
	 * @param InSections The given sections to render.
	 */
	virtual void RenderMovie(const TArray<UMovieSceneCinematicShotSection*>& InSections) const = 0;

	/*
	* Recreate any associated Curve Editor 
	*/
	virtual void RecreateCurveEditor() {};

	/** Whether to show the curve editor or not */
	virtual void SetShowCurveEditor(bool bInShowCurveEditor) {}
	/** @return If the curve editor is currently visible. */
	virtual bool GetCurveEditorIsVisible() const { return false; }

	/*
	 * Puts sequencer in a silent state (whereby it will not redraw viewports, or attempt to update external state besides the sequence itself)
	 */
	virtual void EnterSilentMode() = 0;

	/*
	 * Leaves a silent state (see above)
	 */
	virtual void ExitSilentMode() = 0;

	/*
	 * Checks whether we're in silent mode or not
	 */
	virtual bool IsInSilentMode() const = 0;

	/** Saves the sequence content to the asset registry. */
	virtual void Save() = 0;

	virtual FOnActorAddedToSequencer& OnActorAddedToSequencer() = 0;

	DECLARE_EVENT_TwoParams(ISequencer, FOnCameraCut, UObject*, bool)
	virtual FOnCameraCut& OnCameraCut() = 0;

	DECLARE_EVENT_OneParam(ISequencer, FOnPreSave, ISequencer&)
	virtual FOnPreSave& OnPreSave() = 0;

	DECLARE_EVENT_OneParam(ISequencer, FOnPostSave, ISequencer&)
	virtual FOnPostSave& OnPostSave() = 0;

	DECLARE_EVENT(ISequencer, FOnPreNavigateToSequence)
	virtual FOnPreNavigateToSequence& OnPreNavigateToSequence() = 0;

	DECLARE_EVENT_OneParam(ISequencer, FOnActivateSequence, FMovieSceneSequenceIDRef)
	virtual FOnActivateSequence& OnActivateSequence() = 0;

	DECLARE_EVENT_TwoParams(ISequencer, FOnInitializeDetailsPanel, TSharedRef<IDetailsView>, TSharedRef<ISequencer>)
	FOnInitializeDetailsPanel& OnInitializeDetailsPanel() { return InitializeDetailsPanelEvent; }

	/** A delegate which will can be used in response to a camera being added to the sequence. If true, the default behavior of locking the camera to the viewport and adding a camera cut will be executed */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnCameraAddedToSequencer, ACameraActor*, FGuid)
	FOnCameraAddedToSequencer& OnCameraAddedToSequencer() { return CameraAddedToSequencer; }

	/** A delegate which will determine whether a binding should be visible in the tree. */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnGetIsBindingVisible, const FMovieSceneBinding&)
	FOnGetIsBindingVisible& OnGetIsBindingVisible() { return GetIsBindingVisible; }

	/** A delegate which will determine whether a track should be visible in the tree. */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnGetIsTrackVisible, const UMovieSceneTrack*)
	FOnGetIsTrackVisible& OnGetIsTrackVisible() { return GetIsTrackVisible; }

	/** A delegate which will determine whether a recording is possible */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnGetCanRecord, FText& OutInfoText)
	FOnGetCanRecord& OnGetCanRecord() { return GetCanRecord; }

	/** A delegate which will determine whether there is a recording in progress */
	DECLARE_DELEGATE_RetVal(bool, FOnGetIsRecording)
	FOnGetIsRecording& OnGetIsRecording() { return GetIsRecording; }

	/**
	 * Gets a handle to runtime information about the object being manipulated by a movie scene
	 * 
	 * @param Object The object to get a handle for.
	 * @param bCreateHandleIfMissing Create a handle if it doesn't exist.
	 * @param CreatedFolderName - The name of the folder to place the created objects in (if bCreateHandleIfMissing is true).
	 * @return The handle to the object.
	 */
	virtual FGuid GetHandleToObject(UObject* Object, bool bCreateHandleIfMissing = true, const FName& CreatedFolderName = NAME_None) = 0;

	/**
	 * @return Returns the object change listener for sequencer instance
	 */
	virtual class ISequencerObjectChangeListener& GetObjectChangeListener() = 0;

	/**
	 * @return Returns the property keyed status handler for this sequencer instance
	 */
	virtual class ISequencerPropertyKeyedStatusHandler& GetPropertyKeyedStatusHandler() = 0;

	virtual bool CanKeyProperty(FCanKeyPropertyParams CanKeyPropertyParams) const = 0;

	virtual void KeyProperty(FKeyPropertyParams KeyPropertyParams) = 0;

	virtual EPropertyKeyedStatus GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const = 0;

	/** Refresh the sequencer tree view */
	virtual void RefreshTree() = 0;
protected:
	virtual void NotifyMovieSceneDataChangedInternal() = 0;

public:
	virtual void NotifyMovieSceneDataChanged( EMovieSceneDataChangeType DataChangeType ) = 0;

	/** Get the playback range. */
	virtual TRange<FFrameNumber> GetPlaybackRange() const = 0;

	virtual void UpdatePlaybackRange() = 0;

	virtual void SetPlaybackSpeed(float InPlaybackSpeed) = 0;
	virtual float GetPlaybackSpeed() const = 0;

	/** Restores the speed to 1. */
	virtual void RestorePlaybackSpeed() = 0;
	/** Snaps to the closest available speed to the current one.
	 * Useful when external systems update the available speeds so the current speed is no longer valid. */
	virtual void SnapToClosestPlaybackSpeed() = 0;

	/** Get all the keys for the current sequencer selection */
	virtual void GetKeysFromSelection(TUniquePtr<FSequencerKeyCollection>& KeyCollection, float DuplicateThresoldTime) = 0;
	virtual FSequencerKeyCollection* GetKeyCollection() = 0;

	virtual FFrameNumber OnGetNearestKey(FFrameTime InTime, ENearestKeyOption NearestKeyOption) = 0;

	virtual TArray<FMovieSceneMarkedFrame> GetMarkedFrames() const = 0;

	virtual void InvalidateGlobalMarkedFramesCache() = 0;

	/** Gets the currently selected tracks. */
	virtual void GetSelectedTracks(TArray<UMovieSceneTrack*>& OutSelectedTracks) = 0;

	/** Gets the currently selected track rows */
	virtual void GetSelectedTrackRows(TArray<TPair<UMovieSceneTrack*, int32>>& OutSelectedTrackRows) = 0;

	/** Gets the currently selected sections. */
	virtual void GetSelectedSections(TArray<UMovieSceneSection*>& OutSelectedSections) = 0;

	/** Gets the currently selected folders. */
	virtual void GetSelectedFolders(TArray<UMovieSceneFolder*>& OutSelectedFolders) = 0;

	/** Gets the currently selected key areas. If bIncludeSelectedKeys is true it will include key areas for selected keys, if not will only include key areas for selected display nodes */
	virtual void GetSelectedKeyAreas(TArray<const IKeyArea*>& OutSelectedKeyAreas, bool bIncludeSelectedKeys = true) = 0;

	/** Gets the currently selected Object Guids*/
	virtual void GetSelectedObjects(TArray<FGuid>& OutSelectedObjects) = 0;

	/** Selects an object by GUID */
	virtual void SelectObject(FGuid ObjectBinding) = 0;

	/** Deselects an object by GUID */
	virtual void DeselectObject(FGuid ObjectBinding) = 0;

	/** Selects a track */
	virtual void SelectTrack(UMovieSceneTrack* Track) = 0;

	/** Selects a section */
	virtual void SelectSection(UMovieSceneSection* Section) = 0;

	/** Selects a folder */
	virtual void SelectFolder(UMovieSceneFolder* Folder) = 0;

	/** Selects property tracks by property path */
	virtual void SelectByPropertyPaths(const TArray<FString>& InPropertyPaths) = 0;

	/** Selects the nodes that relate to the specified channels */
	virtual void SelectByChannels(UMovieSceneSection* Section, TArrayView<const FMovieSceneChannelHandle> InChannels, bool bSelectParentInstead, bool bSelect) = 0;

	/** Selects the nodes that relate to the specified channels */
	virtual void SelectByChannels(UMovieSceneSection* Section, const TArray<FName>& InChannelNames, bool bSelectParentInstead, bool bSelect) = 0;

	/** Selects nodes by the nth category node under a section */
	virtual void SelectByNthCategoryNode(UMovieSceneSection* Section, int Index, bool bSelect) = 0;

	/** Empties the current selection. */
	virtual void EmptySelection() = 0;

	/** Throb key or section selection */
	virtual void ThrobKeySelection() = 0;
	virtual void ThrobSectionSelection() = 0;

	virtual void OnScrubPositionChanged(FFrameTime NewScrubPosition, bool bScrubbing, bool bEvaluate) = 0;
	virtual void OnBeginScrubbing() = 0;
	virtual void OnEndScrubbing() = 0;

	/** Gets a multicast delegate which is executed whenever the global time changes. */
	virtual FOnGlobalTimeChanged& OnGlobalTimeChanged() = 0;

	/** Find the decoration editor that handles a given decoration class on this sequencer. */
	virtual ISequencerDecorationEditor* FindDecorationEditor(UClass* DecorationClass) const { return nullptr; }

	/** For a channel handle that may be owned by a decoration on its section, return the decoration editor and the owning decoration. */
	virtual TTuple<ISequencerDecorationEditor*, UObject*> FindDecorationOwner(
		const FMovieSceneChannelHandle& ChannelHandle,
		UMovieSceneSection* Section) const { return MakeTuple(nullptr, static_cast<UObject*>(nullptr)); }

	/** Gets a multicast delegate which is executed whenever the user begins playing the sequence. */
	virtual FOnPlayEvent& OnPlayEvent() = 0;

	/** Gets a multicast delegate which is executed whenever the user stops playing the sequence. */
	virtual FOnStopEvent& OnStopEvent() = 0;

	/** Gets a multicast delegate which is executed whenever the user toggles recording. */
	virtual FOnRecordEvent& OnRecordEvent() = 0;

	/** Gets a multicast delegate which is executed whenever the user begins scrubbing. */
	virtual FOnBeginScrubbingEvent& OnBeginScrubbingEvent() = 0;

	/** Gets a multicast delegate which is executed whenever the user stops scrubbing. */
	virtual FOnEndScrubbingEvent& OnEndScrubbingEvent() = 0;

	/** Gets a multicast delegate which is executed whenever the sequencer tree view changes, like when an object is added, or filtered from the view*/
	virtual FOnTreeViewChanged& OnTreeViewChanged() = 0;

	/** Gets a multicast delegate which is executed whenever the movie scene data is changed. */
	virtual FOnMovieSceneDataChanged& OnMovieSceneDataChanged() = 0;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "FOnChannelChanged is now deprecated. Use FMovieSceneOnChannelChangedWithTime")
	virtual FOnChannelChanged& OnChannelChanged() = 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Gets a multicast delegate which is executed whenever a channel is changed by Sequencer. */
	virtual FMovieSceneOnChannelChangedWithTime& OnChannelChangedWithTime() = 0;

	/** Gets a multicast delegate which is executed whenever the movie scene bindings are changed. */
	virtual FOnMovieSceneBindingsChanged& OnMovieSceneBindingsChanged() = 0;

	/** Gets a multicast delegate which is executed whenever bindings are pasted. */
	virtual FOnMovieSceneBindingsPasted& OnMovieSceneBindingsPasted() = 0;

	/** Gets a multicast delegate with an array of FGuid of bound objects which is called when the outliner node selection changes. */
	virtual FOnSelectionChangedObjectGuids& GetSelectionChangedObjectGuids() = 0;

	/** Gets a multicast delegate with an array of UMovieSceneTracks which is called when the outliner node selection changes. */
	virtual FOnSelectionChangedTracks& GetSelectionChangedTracks() = 0;

	/** Gets a multicast delegate with an array of UMovieSceneSections which is called when the outliner node selection changes. */
	virtual FOnSelectionChangedSections& GetSelectionChangedSections() = 0;

	/** Gets a multicast delegate when the curve edtior associated with this sequencer has it's selection change. */
	virtual FOnCurveDisplayChanged& GetCurveDisplayChanged() = 0;

	virtual void AddNumericTypeInterface(TSharedRef<UE::Sequencer::FSequencerNumericTypeInterface> InNumericTypeInterface) = 0;

	virtual void RemoveNumericTypeInterface(TSharedRef<UE::Sequencer::FSequencerNumericTypeInterface> InNumericTypeInterface) = 0;

	virtual TArrayView<const TSharedRef<UE::Sequencer::FSequencerNumericTypeInterface>> GetNumericTypeInterfaces() const = 0;

	/** @return a numeric type interface that will parse and display numbers as frame / time / bar.beat positions correctly */
	virtual TSharedPtr<INumericTypeInterface<double>> GetNumericTypeInterface(UE::Sequencer::ENumericIntent Intent = UE::Sequencer::ENumericIntent::Position) const = 0;

	virtual TSharedRef<IPropertyTypeCustomization> MakeFrameNumberDetailsCustomization() = 0;

	/** @return the command bindings for this sequencer */
	virtual TSharedPtr<FUICommandList> GetCommandBindings(ESequencerCommandBindings Type = ESequencerCommandBindings::Sequencer) const = 0;

	/** @return Returns a widget containing the sequencer's playback controls */
	virtual TSharedRef<SWidget> MakeTransportControls(bool bExtended) = 0;

	/** Play or toggle playback at the specified play rate */
	virtual FReply OnPlay(bool bTogglePlay = true) = 0;

	/** Pause playback */
	virtual void Pause() = 0;

	/** Getter for sequencer settings */
	virtual USequencerSettings* GetSequencerSettings() const = 0;

	/** Setter for sequencer settings */
	virtual void SetSequencerSettings(USequencerSettings*) = 0;

	/** Attempt to find a spawned object in the currently focused movie scene, or the template object for the specified binding ID, if possible */
	virtual UObject* FindSpawnedObjectOrTemplate(const FGuid& BindingId) = 0;

	/** Called when the external selection has changed in such a way that sequencer should re-synchronize its selection states */
	virtual void ExternalSelectionHasChanged() = 0;

	/** Whether the sequence is read-only */
	virtual bool IsReadOnly() const = 0;

	/** @return Whether or not this sequencer is used in the level editor */
	virtual bool IsLevelEditorSequencer() const = 0;

	/**
	 * Create a widget containing the spinboxes for setting the working and playback range
	 * 
	 * @param InnerContent		Inner content to be inserted to the middle of the widget (inbetween the playback range spinboxes)
	 * @return the widget
	 */
	virtual TSharedRef<SWidget> MakeTimeRange(const TSharedRef<SWidget>& InnerContent, bool bShowWorkingRange, bool bShowViewRange, bool bShowPlaybackRange) = 0;

	/**
	 * Get the top time slider from the main widget.
	 *
	 * @return the widget
	 */
	virtual TSharedPtr<class ITimeSlider> GetTopTimeSliderWidget() const = 0;

	/**
	* Set the selection range's end position to the requested time.
	*
	* @see GetSelectionRange, SetSelectionRange, SetSelectionRangeStart
	*/
	virtual void SetSelectionRangeEnd(FFrameTime EndFrame) = 0;

	/**
	* Set the selection range's start position to the requested time.
	*
	* @see GetSelectionRange, SetSelectionRange, SetSelectionRangeEnd
	*/
	virtual void SetSelectionRangeStart(FFrameTime StartFrame) = 0;

	/**
	* Get the selection range.
	*
	* @return The selection range.
	* @see SetSelectionRange, SetSelectionRangeEnd, SetSelectionRangeStart
	*/
	virtual TRange<FFrameNumber> GetSelectionRange() const = 0;

	/**
	* Retrieve or create a track editor for the specified track
	*/
	virtual TSharedPtr<ISequencerTrackEditor> GetTrackEditor(UMovieSceneTrack* InTrack) = 0;

public:

	/**
	* Specify that an object was implicitly added. We will notify the track editors that it was
	@InObject Object that was added to be part of a track/binding but not the real binding
	*/
	virtual void ObjectImplicitlyAdded(UObject* InObject) const = 0;

	/**
	* Specify that an object was implicitly removed. We will notify the track editors that it was
	@InObject Object that was removed that was part of a track/binding but not the real binding
	*/
	virtual void ObjectImplicitlyRemoved(UObject* InObject) const = 0;

public:

	/** Sets the specified track filter to be on or off */
	UE_DEPRECATED(5.5, "SetTrackFilterEnabled() has been deprecated. Use GetFilterInterface()->SetFilterActiveByDisplayName() instead.")
	virtual void SetTrackFilterEnabled(const FText& InTrackFilterName, bool bEnabled) = 0;

	/** Gets whether the specified track filter is on/off */
	UE_DEPRECATED(5.5, "IsTrackFilterEnabled() has been deprecated. Use GetFilterInterface()->IsFilterActiveByDisplayName() instead.")
	virtual bool IsTrackFilterEnabled(const FText& InTrackFilterName) const = 0;

	/** Gets all the available track filter names */
	UE_DEPRECATED(5.5, "GetTrackFilterNames() has been deprecated. Use GetFilterInterface()->GetFilterDisplayNames() instead.")
	virtual TArray<FText> GetTrackFilterNames() const = 0;

	/** Gets the Sequencer filter interface used to manage filters */
	virtual TSharedRef<ISequencerTrackFilters> GetFilterInterface() const = 0;

public:

	/**
	 * Get the tick resolution of the currently root sequence
	 */
	SEQUENCER_API FFrameRate GetRootTickResolution() const;

	/**
	 * Get the display rate of the currently root sequence
	 */
	SEQUENCER_API FFrameRate GetRootDisplayRate() const;

	/**
	 * Get the tick resolution of the currently focused sequence
	 */
	SEQUENCER_API FFrameRate GetFocusedTickResolution() const;

	/**
	 * Get the display rate of the currently focused sequence
	 */
	SEQUENCER_API FFrameRate GetFocusedDisplayRate() const;

	/**
	 * Create a new binding for the specified object, lightly deprecated and no longer virtual in favor of the overload with binding parameters
	 */
	SEQUENCER_API FGuid CreateBinding(UObject& InObject, const FString& InName);


	/**
	* Get the Display Name of the Object Binding Track.
	* @param InBinding the Binding of the Object
	* @return The name of the object binding track.
	*/
	virtual FText GetDisplayName(FGuid InBinding) = 0;

	/**
	* Set the Display Name of the Object Binding Track.
	* @param InBinding the Binding of the Object
	* @param InDisplayName The new name of the object binding track.
	*/
	virtual void SetDisplayName(FGuid InBinding, const FText& InDisplayName) = 0;

	/*
	* Returns priority-sorted list of custom binding types supported by this Sequencer. 
	*/
	virtual TArrayView<const TSubclassOf<UMovieSceneCustomBinding>> GetSupportedCustomBindingTypes() const { static TArray<TSubclassOf<UMovieSceneCustomBinding>> EmptyArray; return EmptyArray; }

	virtual void RefreshSupportedCustomBindingTypes() {}

	/** @return True if the Sequencer is currently limiting viewport selection to only Sequencer objects. */
	virtual bool IsViewportSelectionLimited() const = 0;

	/** Turns on or off Sequencer selection limiting. */
	virtual void SetViewportSelectionLimited(const bool bInSelectionLimited) = 0;

	/** @return True if the specified object is selectable in the viewport and not made unselectable by the Sequencer selection limiting. */
	virtual bool IsObjectSelectableInViewport(UObject* const InObject) = 0;

	/** @return Delegate executed when Sequencer selection limiting is enabled or disabled. */
	virtual FOnViewportSelectionLimitedChanged& OnViewportSelectionLimitedChanged() = 0;

	/**
	 * Registers and displays a new drawer in the sidebar.
	 * 
	 * @param InDrawerConfig Configuration info for the new drawer
	 * 
	 * @return True if the new drawer registration was successful.
	 */
	virtual bool RegisterDrawer(FSidebarDrawerConfig&& InDrawerConfig) = 0;

	/**
	 * Unregisters and removes a drawer from the sidebar.
	 *
	 * @param InDrawerId Unique drawer Id to unregister
	 * 
	 * @return True if the drawer removal was successful.
	 */
	virtual bool UnregisterDrawer(const FName InDrawerId) = 0;

	/**
	 * Registers and displays a new drawer section in the sidebar.
	 * 
	 * @param InDrawerId Unique drawer Id to register
	 * @param InSection Drawer content interface for the section
	 * 
	 * @return True if the new drawer section registration was successful.
	 */
	virtual bool RegisterDrawerSection(const FName InDrawerId, const TSharedPtr<ISidebarDrawerContent>& InSection) = 0;

	/**
	 * Unregisters and removes a drawer section from the sidebar.
	 * 
	 * @param InDrawerId Unique drawer Id that contains the section to unregister
	 * @param InSectionId Unique drawer section Id to unregister
	 * 
	 * @return True if the drawer removal was successful.
	 */
	virtual bool UnregisterDrawerSection(const FName InDrawerId, const FName InSectionId) = 0;

	virtual bool TrackSupportsConditions(const UMovieSceneTrack* Track) const = 0;

	virtual TSharedPtr<UE::Sequencer::SOutlinerView> GetOutlinerViewWidget() const = 0;

	/** @return True if the Sequencer is in the simplified view */
	virtual bool IsSimpleView() const = 0;

	/** Enables or disables the Sequencer simplified view */
	virtual void EnableSimpleView(const bool bInEnable) = 0;

protected:
	FOnInitializeDetailsPanel InitializeDetailsPanelEvent;
	FOnCameraAddedToSequencer CameraAddedToSequencer;
	FOnGetIsBindingVisible GetIsBindingVisible;
	FOnGetIsTrackVisible GetIsTrackVisible;
	FOnGetPlaybackSpeeds GetPlaybackSpeeds;
	FOnGetIsRecording GetIsRecording;
	FOnGetCanRecord GetCanRecord;
};

