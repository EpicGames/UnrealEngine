// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Misc/FrameNumber.h"

struct FGeometry;
struct FKeyDrawParams;
struct FKeyHandle;
struct FSequencerChannelPaintArgs;
struct FSequencerPasteEnvironment;
struct FMovieSceneClipboardEnvironment;

class SWidget;
class ISequencer;
class FCurveModel;
class FExtender;
class FMenuBuilder;
class FStructOnScope;
class UMovieSceneSection;
class ISectionLayoutBuilder;
class FMovieSceneClipboardBuilder;
class FMovieSceneClipboardKeyTrack;
class FTrackInstancePropertyBindings;

namespace UE::Sequencer
{
	class FChannelModel;
	class FSectionModel;
	class STrackAreaLaneView;
	struct FCreateTrackLaneViewParams;

	struct FCreateCurveEditorModelParams
	{
		UMovieSceneSection* OwningSection;
		UObject* OwningObject;
		TSharedRef<ISequencer> Sequencer;
	};

	struct FCreateKeyEditorParams
	{
		UMovieSceneSection* OwningSection;
		UObject* OwningObject;
		TSharedRef<ISequencer> Sequencer;
		FGuid ObjectBindingID;
		TWeakPtr<FTrackInstancePropertyBindings> PropertyBindings;
	};

} // namespace UE::Sequencer

/** Utility struct representing a number of selected keys on a single channel */
struct FExtendKeyMenuParams
{
	/** The section on which the channel resides */
	TWeakObjectPtr<UMovieSceneSection> Section;

	/** The owning object. Probably the same as Section, but can be different. */
	TWeakObjectPtr<UObject> WeakOwner;

	/** The channel on which the keys reside */
	FMovieSceneChannelHandle Channel;

	/** An array of key handles to operante on */
	TArray<FKeyHandle> Handles;
};

class ISidebarChannelExtension
{
	virtual TSharedPtr<ISidebarChannelExtension> ExtendMenu(FMenuBuilder& MenuBuilder, const bool bInSubMenu) = 0;
};

/**
 * Abstract interface that defines all sequencer interactions for any channel type
 * Channels are stored internally as FMovieSceneChannel*, with this interface providing a common set of operations for all channels through a safe cast from the FMovieSceneChannel*.
 * Implementations are found in TSequencerChanelInterface which calls overloaded free functions for each channel.
 */
struct ISequencerChannelInterface
{
	virtual ~ISequencerChannelInterface() = default;

	/**
	 * Add (or update) a key to the specified channel using it's current value at that time, or some external value specified by the extended editor data
	 *
	 * @param Channel               The channel to add a key to
	 * @param SectionToKey          The SectionToKey
	 * @param ExtendedEditorData    A pointer to the extended editor data for this channel of type TMovieSceneChannelTraits<>::ExtendedEditorDataType
	 * @param InTime                The time at which to add a key
	 * @param InSequencer           The currently active sequencer
	 * @param ObjectBindingID       The object binding ID for the track that this channel resides within
	 * @param PropertyBindings      (Optional) Property bindings where this channel exists on a property track
	 * @return A handle to the new or updated key
	 */
	virtual FKeyHandle AddOrUpdateKey_Raw(FMovieSceneChannel* Channel, UMovieSceneSection* SectionToKey, const void* ExtendedEditorData, FFrameNumber InTime, ISequencer& InSequencer, const FGuid& ObjectBindingID, FTrackInstancePropertyBindings* PropertyBindings) const = 0;

	/**
	 * Delete the specified keys. If all keys are removed, the current value at that time will be set as the default value for the channel
	 * 
	 * @param Channel               The channel to remove the keys from
	 * @param InHandles             The key handles to delete
	 * @param InTime                The time at which to evaluate for the default value if there are no keys remaining
	 */ 
	virtual void DeleteKeys_Raw(FMovieSceneChannel* Channel, TArrayView<const FKeyHandle> InHandles, FFrameNumber InTime) const = 0;

	/**
	 * Copy all the keys specified in KeyMask to the specified clipboard
	 *
	 * @param Channel               The channel to copy from
	 * @param Section               The section that owns the channel
	 * @param KeyAreaName           The name of the key area
	 * @param ClipboardBuilder      The structure responsible for building clipboard information for each key
	 * @param KeyMask               A specific set of keys to copy
	 */
	virtual void CopyKeys_Raw(FMovieSceneChannel* Channel, const UMovieSceneSection* Section, FName KeyAreaName, FMovieSceneClipboardBuilder& ClipboardBuilder, TArrayView<const FKeyHandle> KeyMask) const = 0;

	/**
	 * Paste the specified key track into the specified channel
	 *
	 * @param Channel               The channel to copy from
	 * @param Section               The section that owns the channel
	 * @param KeyTrack              The source clipboard data to paste
	 * @param SrcEnvironment        The environment the source data was copied from
	 * @param DstEnvironment        The environment we're pasting into
	 * @param OutPastedKeys         Array to receive key handles for any pasted keys
	 */
	virtual void PasteKeys_Raw(FMovieSceneChannel* Channel, UMovieSceneSection* Section, const FMovieSceneClipboardKeyTrack& KeyTrack, const FMovieSceneClipboardEnvironment& SrcEnvironment, const FSequencerPasteEnvironment& DstEnvironment, TArray<FKeyHandle>& OutPastedKeys) const = 0;

	/**
	 * Get an editable key struct for the specified key
	 *
	 * @param Channel               The channel on which the key resides
	 * @param KeyHandle             Handle of the key to get
	 * @return A shared editable key struct
	 */
	virtual TSharedPtr<FStructOnScope> GetKeyStruct_Raw(FMovieSceneChannelHandle Channel, FKeyHandle KeyHandle) const = 0;

	/**
	 * Check whether an editor on the sequencer node tree can be created for the specified channel
	 *
	 * @param Channel               The channel to check
	 * @return true if a key editor should be constructed, false otherwise
	 */
	virtual bool CanCreateKeyEditor_Raw(const FMovieSceneChannel* Channel) const = 0;

	/**
	 * Create an editor on the sequencer node tree
	 *
	 * @param Channel               The channel handle to create a key editor for
	 * @param Params                Creation parameters containing all the necessary structures for creating the key editor
	 * @return The editor widget to display on the node tree
	 */
	virtual TSharedRef<SWidget> CreateKeyEditor_Raw(const FMovieSceneChannelHandle& Channel, const UE::Sequencer::FCreateKeyEditorParams& Params) const = 0;

	/**
	 * Extend the key context menu
	 *
	 * @param MenuBuilder           The menu builder used to create this context menu
	 * @param Channels              Array of channels and handles that are being shown in the context menu
	 * @param InSequencer           The currently active sequencer
	 */
	virtual void ExtendKeyMenu_Raw(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, TArrayView<const FExtendKeyMenuParams> Parameters, TWeakPtr<ISequencer> InSequencer) const = 0;

	/**
	 * Extend the section context menu
	 *
	 * @param MenuBuilder           The menu builder used to create this context menu
	 * @param MenuExtender          The menu extender to use
	 * @param Channels              Array of type specific channels that exist in the selected sections
	 * @param Sections              Array of sections being shown on the context menu
	 * @param InWeakSequencer       The currently active sequencer
	 */
	virtual void ExtendSectionMenu_Raw(FMenuBuilder& MenuBuilder
		, TSharedPtr<FExtender> MenuExtender
		, TArrayView<const FMovieSceneChannelHandle> InChannels
		, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections
		, TWeakPtr<ISequencer> InWeakSequencer) const = 0;

	/**
	 * Extend the section sidebar menu
	 *
	 * @param MenuBuilder           The menu builder used to create this context menu
	 * @param InMenuExtender        The menu extender to use
	 * @param InChannels            Array of type specific channels that exist in the selected sections
	 * @param InWeakSections        Array of sections being shown on the context menu
	 * @param InWeakSequencer       The currently active sequencer
	 */
	virtual TSharedPtr<ISidebarChannelExtension> ExtendSidebarMenu_Raw(FMenuBuilder& MenuBuilder
		, TSharedPtr<FExtender> InMenuExtender
		, TArrayView<const FMovieSceneChannelHandle> InChannels
		, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections
		, TWeakPtr<ISequencer> InWeakSequencer) const = 0;

	/**
	 * Gather information on how to draw the specified keys
	 *
	 * @param Channel               The channel to query
	 * @param InKeyHandles          Array of handles to duplicate
	 * @param InOwner               The section that owns the channel
	 * @param OutKeyDrawParams      Pre-sized array to receive key draw parameters. Invalid key handles will not be assigned to this array. Must match size of InKeyHandles.
	 */
	virtual void DrawKeys_Raw(FMovieSceneChannel* Channel, TArrayView<const FKeyHandle> InKeyHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams) const = 0;

	/**
	 * Whether this channel should draw a curve on its editor UI
	 *
	 * @param Channel               The channel to query
	 * @param InSection             The section that owns the channel
	 * @return true to show the curve on the UI, false otherwise
	 */
	virtual bool ShouldShowCurve_Raw(const FMovieSceneChannel* Channel, UMovieSceneSection* InSection) const = 0;

	/**
	 * Whether this channel supports curve models
	 */
	virtual bool SupportsCurveEditorModels_Raw(const FMovieSceneChannelHandle& InChannel) const = 0;

	/**
	 * Create a new model for this channel that can be used on the curve editor interface
	 *
	 * @return (Optional) A new model to be added to a curve editor
	 */
	virtual TUniquePtr<FCurveModel> CreateCurveEditorModel_Raw(const FMovieSceneChannelHandle& Channel, const UE::Sequencer::FCreateCurveEditorModelParams& Params) const = 0;

	/**
	 * Create a new channel model for this type of channel
	 *
	 * @param InChannelHandle    The channel handle to create a model for
	 * @param InSectionModel     The section that owns this channel model
	 * @param InChannelName      The identifying name of this channel
	 * @return (Optional) A new model to be added to a curve editor
	 */
	virtual TSharedPtr<UE::Sequencer::FChannelModel> CreateChannelModel_Raw(const FMovieSceneChannelHandle& InChannelHandle, const UE::Sequencer::FSectionModel& InSection, FName InChannelName) const = 0;

	/**
	 * Create a new channel view for this type of channel
	 *
	 * @param InChannelHandle    The channel handle to create a model for
	 * @param InWeakModel        The model that is creating the view. Should not be Pinned persistently.
	 * @param Parameters         View construction parameters
	 * @return (Optional) A new model to be added to a curve editor
	 */
	virtual TSharedPtr<UE::Sequencer::STrackAreaLaneView> CreateChannelView_Raw(const FMovieSceneChannelHandle& InChannelHandle, TWeakPtr<UE::Sequencer::FChannelModel> InWeakModel, const UE::Sequencer::FCreateTrackLaneViewParams& Parameters) const = 0;

	/**
	 * Draw additional content in addition to keys for a particular channel
	 *
	 * @param InChannel          The channel to draw extra display information for
	 * @param InOwner            The owning movie scene section for this channel
	 * @param PaintArgs          Paint arguments containing the draw element list, time-to-pixel converter and other structures
	 * @param LayerId            The slate layer to paint onto
	 * @return The new slate layer ID for subsequent elements to paint onto
	 */
	virtual int32 DrawExtra_Raw(FMovieSceneChannel* InChannel, const UMovieSceneSection* InOwner, const FSequencerChannelPaintArgs& PaintArgs, int32 LayerId) const = 0;

	/**
	 * Retrieve the interpolation mode for this curve at the current time
	 *
	 * @param InChannel          The channel to query
	 * @param InTime             The time at which to compute the interpolation mode
	 * @param InDefaultMode      Default interp mode to use
	 * @return This curve's desired interpolation mode
	 */
	virtual EMovieSceneKeyInterpolation GetInterpolationMode_Raw(FMovieSceneChannel* InChannel, FFrameNumber InTime, EMovieSceneKeyInterpolation InDefaultMode) const
	{
		return InDefaultMode;
	}
};
