// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Curves/KeyHandle.h"
#include "Textures/SlateIcon.h"

class FMenuBuilder;
class ISequencer;
class ISequencerSection;
class SWidget;
class UMovieSceneSection;
class UMovieSceneTrack;
struct FSlateBrush;

namespace UE::Sequencer { class FViewModel; }

/**
 * Editor-side counterpart for section decorations. Created per-Sequencer via
 * FOnCreateDecorationEditor factories registered through ISequencerModule, so
 * each FSequencer instance owns its own decoration editors and ticks them
 * directly (mirroring ISequencerTrackEditor).
 *
 * Decoration editors create section interfaces for decorations that provide their own
 * sections (via IMovieSceneSectionProviderDecoration), add entries to section context menus,
 * and provide custom key editor and keyframe widgets for decoration channels.
 *
 * Example registration in an editor module's StartupModule():
 *
 * class FMyDecorationEditor : public ISequencerDecorationEditor
 * {
 *     virtual UClass* GetDecorationClass() const override { return UMyDecoration::StaticClass(); }
 *     virtual TSharedPtr<ISequencerSection> MakeSectionInterface(...) override { return MakeShared<FMySectionInterface>(...); }
 * };
 *
 * ISequencerModule& Module = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
 * Module.RegisterDecorationEditor(FOnCreateDecorationEditor::CreateLambda(
 *     [](TSharedRef<ISequencer> InSequencer)
 *     {
 *         return MakeShared<FMyDecorationEditor>(InSequencer);
 *     }));
 */
class ISequencerDecorationEditor
{
public:
	virtual ~ISequencerDecorationEditor() = default;

	/**
	 * Get the decoration class this editor handles.
	 * Used to match decorations to their editors.
	 */
	virtual UClass* GetDecorationClass() const = 0;

	/**
	 * Create a section interface for a section provided by this decoration.
	 * The section interface is used for displaying/editing the section in Sequencer.
	 */
	virtual TSharedPtr<ISequencerSection> MakeSectionInterface(
		UMovieSceneSection& Section,
		UMovieSceneTrack& OwningTrack,
		FGuid ObjectBinding) { return nullptr; }

	/**
	 * Add entries to the section's right-click context menu on behalf of this decoration.
	 * Called for each decoration on the section that has a registered editor.
	 */
	virtual void BuildSectionContextMenu(
		FMenuBuilder& MenuBuilder,
		UMovieSceneSection& Section,
		UObject& Decoration,
		const FGuid& ObjectBinding,
		TWeakPtr<ISequencer> Sequencer) {}

	/**
	 * Optionally create a custom key editor widget for one of this decoration's channels.
	 * Return nullptr to use the standard channel editor.
	 */
	virtual TSharedPtr<SWidget> CreateKeyEditor(
		UObject& Decoration,
		UMovieSceneSection& Section,
		FMovieSceneChannelHandle ChannelHandle,
		TWeakPtr<ISequencer> Sequencer,
		const FGuid& ObjectBindingID) { return nullptr; }

	/**
	 * Called when a section model is first created for a section that has this decoration.
	 * Override to set up per-section subscriptions (e.g., channel key events).
	 */
	virtual void OnSectionInterfaceCreated(
		UMovieSceneSection& Section,
		UObject& Decoration,
		TWeakPtr<ISequencer> Sequencer) {}

	/**
	 * Called once per Sequencer tick on every registered decoration editor.
	 * Override to drain deferred work (e.g. coalesced channel-edit propagation).
	 * Default implementation is a no-op.
	 */
	virtual void Tick(float DeltaTime) {}

	/**
	 * Optionally create a custom widget for the KeyFrame outliner column
	 * (the "add key" button) for one of this decoration's channels.
	 * The ChannelGroupModel is the outliner view model for the channel group -
	 * pass it to SKeyNavigationButtons to get correct row hover behavior.
	 * Return nullptr to use the standard add-key button.
	 */
	virtual TSharedPtr<SWidget> CreateKeyFrameWidget(
		UObject& Decoration,
		UMovieSceneSection& Section,
		FMovieSceneChannelHandle ChannelHandle,
		TWeakPtr<ISequencer> Sequencer,
		const FGuid& ObjectBindingID,
		TSharedPtr<UE::Sequencer::FViewModel> ChannelGroupModel) { return nullptr; }

	/**
	 * @return An icon brush for this decoration's outliner row, or nullptr for no icon
	 */
	virtual const FSlateBrush* GetIconBrush() const { return nullptr; }

	/**
	 * @return An icon for this decoration's entry in add-modifier menus. Defaults to empty.
	 */
	virtual FSlateIcon GetMenuIcon() const { return FSlateIcon(); }

	/**
	 * Create a custom widget for the decoration's outliner row (e.g. an asset picker dropdown).
	 * Return nullptr to use the default outliner display.
	 */
	virtual TSharedPtr<SWidget> CreateOutlinerWidget(
		UObject& Decoration,
		TWeakPtr<ISequencer> Sequencer) { return nullptr; }

	/**
	 * Add or update a key on a decoration-owned channel. Called when the standard
	 * keying path detects a channel owned by a decoration rather than a track.
	 * The default implementation keys the channel directly without track assumptions.
	 */
	SEQUENCER_API virtual FKeyHandle AddOrUpdateKey(
		UObject& Decoration,
		UMovieSceneSection& Section,
		FMovieSceneChannelHandle ChannelHandle,
		FFrameNumber Time,
		const FGuid& ObjectBindingID,
		ISequencer& InSequencer);
};
