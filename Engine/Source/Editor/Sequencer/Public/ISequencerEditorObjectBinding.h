// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FMenuBuilder;
class UMovieSceneSequence;

/**
 * Interface for sequencer object bindings
 */
class ISequencerEditorObjectBinding
{
public:

	/** The display name text to be used for UX */
	virtual FText GetDisplayName() const = 0;

	/**
	 * Builds up the sequencer's "Add" menu.
	 *
	 * @param MenuBuilder The menu builder to change.
	 */
	virtual void BuildSequencerAddMenu(FMenuBuilder& MenuBuilder) = 0;

	/**
	 * Returns whether a sequence is supported by this tool.
	 *
	 * @param InSequence The sequence that could be supported.
	 * @return true if the type is supported.
	 */
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const = 0;

public:

	/** Virtual destructor. */
	virtual ~ISequencerEditorObjectBinding() = default;
};
