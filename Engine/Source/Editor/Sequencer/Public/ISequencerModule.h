// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "ToolMenuContext.h"
#include "ISequencer.h"
#include "Modules/ModuleInterface.h"
#include "AnimatedPropertyKey.h"
#include "ISequencerChannelInterface.h"
#include "IMovieRendererInterface.h"
#include "MovieSceneSequenceEditor.h"

class IKeyArea;
class FExtender;
class FStructOnScope;
class FExtensibilityManager;
class FMenuBuilder;
class FSequencerCustomizationManager;
class ISequencerDecorationEditor;
class ISequencerTrackEditor;
class ISequencerEditorObjectBinding;
class IToolkitHost;
class UMovieSceneSequence;
struct FSequencerInitParams;

namespace UE::Sequencer
{
	class FTrackModel;
	class IObjectSchema;
	class IOutlinerColumn;
	class IOutlinerIndicatorBuilder;
} // namespace UE::Sequencer

enum class ECurveEditorTreeFilterType : uint32;

/** Forward declaration for the default templated channel interface. Include SequencerChannelInterface.h for full definition. */
template<typename> struct TSequencerChannelInterface;

namespace SequencerMenuExtensionPoints
{
	static const FName AddTrackMenu_PropertiesSection("AddTrackMenu_PropertiesSection");
}

/** Enum representing supported scrubber styles */
enum class ESequencerScrubberStyle : uint8
{
	/** Scrubber is represented as a single thin line for the current time, with a constant-sized thumb. */
	Vanilla,

	/** Scrubber thumb occupies a full 'display rate' frame, with a single thin line for the current time. Tailored to frame-accuracy scenarios. */
	FrameBlock,
};

/** A delegate which will create an auto-key handler. */
DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<ISequencerTrackEditor>, FOnCreateTrackEditor, TSharedRef<ISequencer>);

/** A delegate which will create a decoration editor for a particular sequencer instance. */
DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<ISequencerDecorationEditor>, FOnCreateDecorationEditor, TSharedRef<ISequencer>);

/** A delegate which will create an object binding handler. */
DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<ISequencerEditorObjectBinding>, FOnCreateEditorObjectBinding, TSharedRef<ISequencer>);

/** A delegate which will create a track model. */
DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<UE::Sequencer::FTrackModel>, FOnCreateTrackModel, UMovieSceneTrack*);

/** A delegate which will create an outliner column */
DECLARE_DELEGATE_RetVal(TSharedRef<UE::Sequencer::IOutlinerColumn>, FOnCreateOutlinerColumn);

/** A delegate which will create an outliner indicator item */
DECLARE_DELEGATE_RetVal(TSharedRef<UE::Sequencer::IOutlinerIndicatorBuilder>, FOnCreateOutlinerIndicator);

/** A delegate that is executed when adding menu content. */
DECLARE_DELEGATE_OneParam(FOnGetContextMenuContent, FMenuBuilder& /*MenuBuilder*/);

/** A delegate that is executed when adding menu content. */
DECLARE_DELEGATE_TwoParams(FOnGetAddMenuContent, FMenuBuilder& /*MenuBuilder*/, TSharedRef<ISequencer>);

/** A delegate that is executed when initializing tool menu context. */
DECLARE_DELEGATE_OneParam(FOnInitToolMenuContext, FToolMenuContext&);

/** A delegate that is executed when menu object is clicked. Unlike FExtender delegates we pass in the FGuid which exists even for deleted objects. */
DECLARE_DELEGATE_TwoParams(FOnBuildCustomContextMenuForGuid, FMenuBuilder&, FGuid);

/** A delegate that gets executed then a sequencer is created */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnSequencerCreated, TSharedRef<ISequencer>);

/** A delegate that gets executed a sequencer is initialize and allow modification the initialization params. */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnPreSequencerInit, TSharedRef<ISequencer>, TSharedRef<ISequencerObjectChangeListener>, const FSequencerInitParams&);

/**
 * Sequencer view parameters.
 */
struct FSequencerViewParams
{
	/** Called when building the add track menu */
	FOnGetAddMenuContent OnGetAddMenuContent;

	/** Called when building a context menu for an object binding */
	FOnBuildCustomContextMenuForGuid OnBuildCustomContextMenuForGuid;

	/** Called when this sequencer has received user focus */
	FSimpleDelegate OnReceivedFocus;

	/** Called when this sequencer is initializing the tool menu context */
	FOnInitToolMenuContext OnInitToolMenuContext;

	/** The playback speed options available */
	ISequencer::FOnGetPlaybackSpeeds OnGetPlaybackSpeeds;
	
	/** A menu extender for the add menu */
	TSharedPtr<FExtender> AddMenuExtender;

	/** A toolbar extender for the main toolbar */
	TSharedPtr<FExtender> ToolbarExtender;

	/** Unique name for the sequencer. */
	FString UniqueName;

	/** Whether the sequencer is read-only */
	bool bReadOnly;

	/** Whether the playback range spin box should be visible in TimeRange widget */
	bool bShowPlaybackRangeInTimeSlider;

	/** Style of scrubber to use */
	ESequencerScrubberStyle ScrubberStyle;

	FSequencerViewParams(FString InName = FString())
		: OnGetPlaybackSpeeds(ISequencer::FOnGetPlaybackSpeeds::CreateLambda([]() { return TArray<float>{ 0.1f, 0.25f, 0.5f, 1.f, 2.f, 5.f, 10.f }; }))
		, UniqueName(MoveTemp(InName))
		, bReadOnly(false)
		, bShowPlaybackRangeInTimeSlider(false)
		, ScrubberStyle(ESequencerScrubberStyle::Vanilla)
	{ }
};

/**
 * Sequencer host functionality capabilities. These are no longer
 * based on whether or not there is a Toolkit host as we may have
 * a toolkit host outside of conditions where these are supported.
 */
struct FSequencerHostCapabilities
{
	/** Should we show the Save-As button in the toolbar? */
	bool bSupportsSaveMovieSceneAsset;

	/** Do we support the curve editor */
	bool bSupportsCurveEditor;

	/** Do we support recording */
	bool bSupportsRecording;

	/** Do we support rendering to a movie */
	bool bSupportsRenderMovie;

	/** Do we support adding content from the content browser */
	bool bSupportsAddFromContentBrowser;

	/** Do we support the sidebar widget */
	bool bSupportsSidebar;

	/** Do we support selectability options in the viewport */
	bool bSupportsViewportSelectability;

	/** Do we support drag and drop operation */
	bool bSupportsDragAndDrop;

	/** Do we support toggling between a minimalized, simple view instead of the full Sequencer view */
	bool bSupportsSimpleView;

	FSequencerHostCapabilities()
		: bSupportsSaveMovieSceneAsset(false)
		, bSupportsCurveEditor(false)
		, bSupportsRecording(false)
		, bSupportsRenderMovie(false)
		, bSupportsAddFromContentBrowser(false)
		, bSupportsSidebar(false)
		, bSupportsViewportSelectability(false)
		, bSupportsDragAndDrop(true)
		, bSupportsSimpleView(true)
	{}
};

/**
 * Sequencer initialization parameters.
 */
struct FSequencerInitParams
{
	/** The root movie scene sequence being edited. */
	UMovieSceneSequence* RootSequence;

	/** The asset editor created for this (if any) */
	TSharedPtr<IToolkitHost> ToolkitHost;

	/** View parameters */
	FSequencerViewParams ViewParams;

	/** Immutable capability set specified when our instance is created. Used to specify which feature set is supported. */
	FSequencerHostCapabilities HostCapabilities;

	/** Whether or not sequencer should be edited within the level editor */
	bool bEditWithinLevelEditor;

	/** Domain-specific spawn register for the movie scene */
	TSharedPtr<FMovieSceneSpawnRegister> SpawnRegister;

	/** Accessor for event contexts */
	TAttribute<TArray<UObject*>> EventContexts;

	/** Accessor for playback context */
	TAttribute<UObject*> PlaybackContext;

	/** Accessor for playback client */
	TAttribute<IMovieScenePlaybackClient*> PlaybackClient;

	FSequencerInitParams()
		: RootSequence(nullptr)
		, ToolkitHost(nullptr)
		, bEditWithinLevelEditor(false)
		, SpawnRegister(nullptr)
	{}
};

/**
 * Interface for the Sequencer module.
 */
class ISequencerModule
	: public IModuleInterface
{
public:

	virtual ~ISequencerModule();

	/**
	 * Create a new instance of a standalone sequencer that can be added to other UIs.
	 *
	 * @param InitParams Initialization parameters.
	 * @return The new sequencer object.
	 */
	virtual TSharedRef<ISequencer> CreateSequencer(const FSequencerInitParams& InitParams) = 0;

	/** 
	 * Registers a delegate that will create an editor for a track in each sequencer.
	 *
	 * @param InOnCreateTrackEditor	Delegate to register.
	 * @return A handle to the newly-added delegate.
	 */
	virtual FDelegateHandle RegisterTrackEditor(FOnCreateTrackEditor InOnCreateTrackEditor, TArrayView<FAnimatedPropertyKey> AnimatedPropertyTypes = TArrayView<FAnimatedPropertyKey>()) = 0;

	/**
	 * Unregisters a previously registered delegate for creating a track editor
	 *
	 * @param InHandle	Handle to the delegate to unregister
	 */
	virtual void UnRegisterTrackEditor(FDelegateHandle InHandle) = 0;

	/**
	 * Registers a delegate that will create a decoration editor per-Sequencer.
	 *
	 * @param InOnCreateDecorationEditor Delegate to register.
	 * @return A handle to the newly-added delegate.
	 */
	virtual FDelegateHandle RegisterDecorationEditor(FOnCreateDecorationEditor InOnCreateDecorationEditor) = 0;

	/**
	 * Unregisters a previously registered delegate for creating a decoration editor.
	 *
	 * @param InHandle Handle to the delegate to unregister.
	 */
	virtual void UnRegisterDecorationEditor(FDelegateHandle InHandle) = 0;

	/**
	 * Registers a delegate that will create a track view model for a given track
	 *
	 * @param InCreator Delegate to register
	 * @return A handle to the newly-added delegate
	 */
	virtual FDelegateHandle RegisterTrackModel(FOnCreateTrackModel InCreator) = 0;

	/**
	 * Unregisters a previously registered delegate for creating track view models
	 *
	 * @param InHandle Handle to the delegate to unregister
	 */
	virtual void UnregisterTrackModel(FDelegateHandle InHandle) = 0;

	/**
	 * Registers a delegate that will create a gutter column view model for a given Outliner column
	 *
	 * @param InCreator Delegate to register
	 * @return A handle to the newly added delegate
	 */
	virtual FDelegateHandle RegisterOutlinerColumn(FOnCreateOutlinerColumn InCreator) = 0;

	/**
	 * Unregisters a previously registered delegate for creating Outliner columns
	 *
	 * @param InHandle Handle to the delegate to unregister
	 */
	virtual void UnregisterOutlinerColumn(FDelegateHandle InHandle) = 0;

	/**
	 * Registers a delegate that will create an outliner indicator item
	 *
	 * @param InCreator Delegate to register
	 * @return A handle to the newly added delegate
	 */
	virtual FDelegateHandle RegisterOutlinerIndicator(FOnCreateOutlinerIndicator InCreator) = 0;

	/**
	 * Unregisters a previously registered delegate for creating an outliner indicator item
	 *
	 * @param InHandle Handle to the delegate to unregister
	 */
	virtual void UnregisterOutlinerIndicator(FDelegateHandle InHandle) = 0;

	/**
	 * Registers a delegate that will be called when a sequencer is created
	 *
	 * @param InOnSequencerCreated	Delegate to register.
	 * @return A handle to the newly-added delegate.
	 */
	virtual FDelegateHandle RegisterOnSequencerCreated(FOnSequencerCreated::FDelegate InOnSequencerCreated) = 0;

	/** 
	 * Unregisters a previously registered delegate called when a sequencer is created
	 *
	 * @param InHandle	Handle to the delegate to unregister
	 */
	virtual void UnregisterOnSequencerCreated(FDelegateHandle InHandle) = 0;

	/**
	 * Registers a delegate that will be called just before a sequencer is initialized
	 *
	 * @param InOnPreSequencerInit	Delegate to register.
	 * @return A handle to the newly-added delegate.
	 */
	virtual FDelegateHandle RegisterOnPreSequencerInit(FOnPreSequencerInit::FDelegate InOnPreSequencerInit) = 0;

	/**
	 * Unregisters a previously registered delegate called just before a sequencer is initialized
	 *
	 * @param InHandle	Handle to the delegate to unregister
	 */
	virtual void UnregisterOnPreSequencerInit(FDelegateHandle InHandle) = 0;

	/** 
	 * Registers a delegate that will create editor UI for an object binding in sequencer.
	 *
	 * @param InOnCreateEditorObjectBinding	Delegate to register.
	 * @return A handle to the newly-added delegate.
	 */
	virtual FDelegateHandle RegisterEditorObjectBinding(FOnCreateEditorObjectBinding InOnCreateEditorObjectBinding) = 0;

	/** 
	 * Unregisters a previously registered delegate for creating editor UI for an object binding in sequencer.
	 *
	 * @param InHandle	Handle to the delegate to unregister
	 */
	virtual void UnRegisterEditorObjectBinding(FDelegateHandle InHandle) = 0;

	/**
	 * Register that the specified property type can be animated in sequencer
	 */
	virtual void RegisterPropertyAnimator(FAnimatedPropertyKey Key) = 0;

	/**
	 * Unregister that the specified property type can be animated in sequencer
	 */
	virtual void UnRegisterPropertyAnimator(FAnimatedPropertyKey Key) = 0;

	/**
	 * Check whether the specified property type can be animated by sequeuncer
	 */
	virtual bool CanAnimateProperty(FProperty* Property) = 0;

	/**
	* Get the extensibility manager for object binding menus.
	*
	* @return ObjectBinding Context Menu extensibility manager.
	*/
	virtual TSharedPtr<FExtensibilityManager> GetObjectBindingContextMenuExtensibilityManager() const = 0;

	/**
	 * Get the extensibility manager for add track menus.
	 *
	 * @return Add Track Menu extensibility manager.
	 */
	virtual TSharedPtr<FExtensibilityManager> GetAddTrackMenuExtensibilityManager() const = 0;

	/**
	 * Get the extensibility manager for toolbars.
	 *
	 * @return Toolbar extensibility manager.
	 */
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() const = 0;

	/**
	 * Get the extensibility manager for the view options menu.
	 *
	 * @return Toolbar extensibility manager.
	 */
	virtual TSharedPtr<FExtensibilityManager> GetViewMenuExtensibilityManager() const = 0;

	/**
	 * Get the extensibility manager for the actions menu.
	 *
	 * @return Toolbar extensibility manager.
	 */
	virtual TSharedPtr<FExtensibilityManager> GetActionsMenuExtensibilityManager() const = 0;

	/**
	 * Get the extensibility manager for the sidebar.
	 *
	 * @return Sidebar extensibility manager.
	 */
	virtual TSharedPtr<FExtensibilityManager> GetSidebarExtensibilityManager() const = 0;

	/**
	 * Get the sequencer customization manager, which handles editor customizations applied based on
	 * the currently focused sequence type and other dynamic criteria.
	 */
	virtual TSharedPtr<FSequencerCustomizationManager> GetSequencerCustomizationManager() const = 0;

	/**
	 * Register a new object schema defining how an object is able to be animated through Sequencer
	 */
	virtual void RegisterObjectSchema(TSharedPtr<UE::Sequencer::IObjectSchema> InObjectSchema) = 0;

	/**
	 * Unregister a new object schema defining how an object is able to be animated through Sequencer
	 */
	virtual void UnregisterObjectSchema(TSharedPtr<UE::Sequencer::IObjectSchema> InSchema) = 0;

	/**
	 * Retrieve all the currently registered object schemas
	 */
	virtual TArrayView<const TSharedPtr<UE::Sequencer::IObjectSchema>> GetObjectSchemas() const = 0;

	/**
	 * Find an object schema for the specified object
	 */
	virtual TSharedPtr<UE::Sequencer::IObjectSchema> FindObjectSchema(const UObject* Object) const = 0;

	/**
	 * Register a sequencer channel type using a default channel interface.
	 */
	template<typename ChannelType>
	void RegisterChannelInterface();

	/**
	 * Register a sequencer channel type using the specified interface.
	 */
	template<typename ChannelType>
	void RegisterChannelInterface(TUniquePtr<ISequencerChannelInterface>&& InInterface);

	/**
	 * Find a sequencer channel for the specified channel type name
	 */
	ISequencerChannelInterface* FindChannelEditorInterface(FName ChannelTypeName) const;


	/**
	 * Retrieve the unique identifer for the sequencer selection curve editor filter (of type FSequencerSelectionCurveFilter)
	 */
	static ECurveEditorTreeFilterType GetSequencerSelectionFilterType();

public:

	/**
	 * Register a sequence editor for the specified type of sequence. Sequence editors provide editor-only functionality for particular sequence types.
	 */
	FDelegateHandle RegisterSequenceEditor(UClass* SequenceClass, TUniquePtr<FMovieSceneSequenceEditor>&& InSequenceEditor)
	{
		check(SequenceClass);

		FDelegateHandle NewHandle(FDelegateHandle::GenerateNewHandle);

		SequenceEditors.Add(FSequenceEditorEntry{ NewHandle, SequenceClass, MoveTemp(InSequenceEditor) });

		return NewHandle;
	}

	/**
	 * Unregister a sequence editor for the specified type of sequence.
	 */
	void UnregisterSequenceEditor(FDelegateHandle Handle)
	{
		SequenceEditors.RemoveAll([Handle](const FSequenceEditorEntry& In){ return In.Handle == Handle; });
	}

	/**
	 * Find a sequence editor for the specified sequence class
	 */
	FMovieSceneSequenceEditor* FindSequenceEditor(UClass* SequenceClass) const
	{
		check(SequenceClass);

		UClass* MostRelevantClass = nullptr;
		FMovieSceneSequenceEditor* SequenceEditor = nullptr;

		for (const FSequenceEditorEntry& Entry : SequenceEditors)
		{
			if (SequenceClass->IsChildOf(Entry.ApplicableClass))
			{
				if (!MostRelevantClass || Entry.ApplicableClass->IsChildOf(MostRelevantClass))
				{
					MostRelevantClass = Entry.ApplicableClass;
					SequenceEditor = Entry.Editor.Get();
				}
			}
		}

		return SequenceEditor;
	}

public:

	/** 
	 * Helper template for registering property track editors
	 *
	 * @param InOnCreateTrackEditor	Delegate to register.
	 * @return A handle to the newly-added delegate.
	 */
	template<typename PropertyTrackEditorType>
	FDelegateHandle RegisterPropertyTrackEditor()
	{
		auto PropertyTypes = PropertyTrackEditorType::GetAnimatedPropertyTypes();
		return RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(PropertyTrackEditorType::CreateTrackEditor), PropertyTypes);
	}

public:

	/**
	 * Register a movie renderer for this sequencer.
	 */
	virtual FDelegateHandle RegisterMovieRenderer(TUniquePtr<IMovieRendererInterface>&& InMovieRenderer) = 0;

	/**
	 * Unregister a movie renderer for this sequencer.
	 */
	virtual void UnregisterMovieRenderer(FDelegateHandle InDelegateHandle) = 0;

	/**
	 * Get the movie renderer with the requested name.
	 */
	virtual IMovieRendererInterface* GetMovieRenderer(const FString& InMovieRendererName) = 0;

	/**
	 * Get a list of registered movie renderers.
	 */
	virtual TArray<FString> GetMovieRendererNames() = 0;

private:

	/** Map of sequencer interfaces for movie scene channel types, keyed on channel UStruct name */
	TMap<FName, TUniquePtr<ISequencerChannelInterface>> ChannelToEditorInterfaceMap;

	struct FSequenceEditorEntry
	{
		FDelegateHandle Handle;
		UClass* ApplicableClass;
		TUniquePtr<FMovieSceneSequenceEditor> Editor;
	};

	/** Array of sequence editor entries */
	TArray<FSequenceEditorEntry> SequenceEditors;
};


/**
 * Register a sequencer channel type using a default channel interface.
 */
template<typename ChannelType>
void ISequencerModule::RegisterChannelInterface()
{
	RegisterChannelInterface<ChannelType>(TUniquePtr<ISequencerChannelInterface>(new TSequencerChannelInterface<ChannelType>()));
}

/**
 * Register a sequencer channel type using the specified interface.
 */
template<typename ChannelType>
void ISequencerModule::RegisterChannelInterface(TUniquePtr<ISequencerChannelInterface>&& InInterface)
{
	const FName ChannelTypeName = ChannelType::StaticStruct()->GetFName();
	check(!ChannelToEditorInterfaceMap.Contains(ChannelTypeName));
	ChannelToEditorInterfaceMap.Add(ChannelTypeName, MoveTemp(InInterface));
}

/**
 * Find a sequencer channel for the specified channel type name
 */
inline ISequencerChannelInterface* ISequencerModule::FindChannelEditorInterface(FName ChannelTypeName) const
{
	const TUniquePtr<ISequencerChannelInterface>* Found = ChannelToEditorInterfaceMap.Find(ChannelTypeName);
	ensureMsgf(Found, TEXT("No channel interface found for type ID. Did you call RegisterChannelInterface<> for that type?"));
	return Found ? Found->Get() : nullptr;
}
