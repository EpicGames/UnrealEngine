// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Input/NumericTypeInterface.h"

class IPropertyHandle;
class ISequencer;

namespace UE::Sequencer
{

enum class ENumericIntent
{
	Position,
	Duration
};

struct FSequencerNumericTypeInterface
{
	TSharedRef<INumericTypeInterface<double>> Interface;
	ENumericIntent Intent;

	FSequencerNumericTypeInterface(TSharedRef<INumericTypeInterface<double>> InInterface, ENumericIntent InIntent = ENumericIntent::Position)
		: Interface(InInterface)
		, Intent(InIntent)
	{}

	virtual ~FSequencerNumericTypeInterface() = default;

	SEQUENCER_API virtual int32 GetRelevancyScore(const ISequencer& Sequencer, TSharedPtr<IPropertyHandle> Property) const;
};

} // namespace UE::Sequencer
