// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "KeyPropertyParams.h"
#include "AnimatedPropertyKey.h"

DECLARE_MULTICAST_DELEGATE_OneParam( FOnAnimatablePropertyChanged, const FPropertyChangedParams& );

DECLARE_MULTICAST_DELEGATE_OneParam( FOnObjectPropertyChanged, UObject& );

DECLARE_MULTICAST_DELEGATE_OneParam( FOnPropagateObjectChanges, UObject* );

DECLARE_DELEGATE_RetVal_OneParam( FOnAnimatablePropertyChanged, FFindAnimatablePropertyChangedHandler, const FProperty* );

/**
 * Listens for changes objects and calls delegates when those objects change
 */
class ISequencerObjectChangeListener
{
public:
	/**
	 * A delegate for when a property of a specific FProperty class is changed.  
	 */
	virtual FOnAnimatablePropertyChanged& GetOnAnimatablePropertyChanged( FAnimatedPropertyKey PropertyKey ) = 0;

	/**
	 * A delegate for when a property of a specific path is changed. 
	 */
	virtual FOnAnimatablePropertyChanged& GetOnAnimatablePropertyChanged(const FProperty* Property) = 0;

	/**
	 * A delegate for when object changes should be propagated to/from puppet actors
	 */
	virtual FOnPropagateObjectChanges& GetOnPropagateObjectChanges() = 0;

	/**
	 * A delegate for when any property of a specific object is changed.
	 */
	virtual FOnObjectPropertyChanged& GetOnAnyPropertyChanged(UObject& Object) = 0;

	/**
	 * Add a delegate that will be called before any other method to locate a property changed handler for a property
	 */
	virtual void AddFindPropertyChangedHandler(FFindAnimatablePropertyChangedHandler&&) = 0;

	/**
	 * Report that an object is about to be destroyed (removes any object change delegates bound to the object)
	 */
	virtual void ReportObjectDestroyed(UObject& Object) = 0;

	/**
	 * Triggers all properties as changed for the passed in object.
	 */
	virtual void TriggerAllPropertiesChanged(UObject* Object) = 0;

	UE_DEPRECATED(5.4, "CanKeyProperty taking in only FCanKeyPropertyParams is deprecated. Please use the alternative CanKeyProperty that takes in FCanKeyPropertyParams and an out FPropertyPath.")
	virtual bool CanKeyProperty(FCanKeyPropertyParams KeyPropertyParams) const = 0;

	virtual bool CanKeyProperty(FCanKeyPropertyParams KeyPropertyParams, FPropertyPath& OutPropertyPath) const = 0;

	virtual void KeyProperty(FKeyPropertyParams KeyPropertyParams) const = 0;
};
