// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Containers/ContainersFwd.h"

#define UE_API SEQUENCER_API

class UObject;
class FExtender;
class ISequencer;
class FUICommandList;
template <typename> class TFunctionRef;

namespace UE::Sequencer
{

struct FObjectSchemaRelevancy
{
	UClass* Class;
	uint32 Priority;

	FObjectSchemaRelevancy()
		: Class(nullptr)
		, Priority(0)
	{
	}

	FObjectSchemaRelevancy(UClass* InClass, uint32 InPriority = 0)
		: Class(InClass)
		, Priority(InPriority)
	{
	}

	friend bool operator>(const FObjectSchemaRelevancy& A, const FObjectSchemaRelevancy& B);
};

class IObjectSchema : public TSharedFromThis<IObjectSchema>
{
public:

	static UE_API TMap<const IObjectSchema*, TArray<UObject*>> ComputeRelevancy(TArrayView<UObject* const> InObjects);

	virtual ~IObjectSchema()= default;

	/*~ Begin abstract interface */
	virtual UObject* GetParentObject(UObject* Object) const = 0;

	virtual FObjectSchemaRelevancy GetRelevancy(const UObject* InObject) const = 0;

	virtual TSharedPtr<FExtender> ExtendObjectBindingMenu(TSharedRef<FUICommandList> CommandList, TWeakPtr<ISequencer> WeakSequencer, TArrayView<UObject* const> ContextSensitiveObjects) const = 0;

	UE_API virtual FText GetPrettyName(const UObject* Object) const;

	UE_API virtual UObject* GetRedirectedBoundObject(UObject* Object) const;
	/*~ End abstract implementation */
};

} // namespace UE::Sequencer

#undef UE_API
