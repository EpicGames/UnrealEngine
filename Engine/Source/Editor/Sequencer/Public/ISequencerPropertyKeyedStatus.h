// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class FProperty;
class IPropertyHandle;
enum class EPropertyKeyedStatus : uint8;

class ISequencerPropertyKeyedStatusHandler
{
public:
	virtual ~ISequencerPropertyKeyedStatusHandler() = default;

	DECLARE_DELEGATE_RetVal_OneParam(EPropertyKeyedStatus, FOnGetPropertyKeyedStatus, const IPropertyHandle&);

	/**
	 * Returns the External Handler delegate for a given property
	 * Used for cases where a property is keyed into a track that does not match the property path so requires custom logic to figure out its status
	 */
	virtual FOnGetPropertyKeyedStatus& GetExternalHandler(const FProperty* Property) = 0;
};
