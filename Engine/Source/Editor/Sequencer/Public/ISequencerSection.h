// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelHandle.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Curves/KeyHandle.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Math/Vector2D.h"
#include "Misc/FrameNumber.h"
#include "Misc/Guid.h"
#include "MovieSceneSection.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"

#include "ISequencerSection.generated.h"

class FMenuBuilder;
class FSequencerSectionPainter;
class IDetailsView;
class ISequencer;
class ISequencerSection;
class ISequencerTrackEditor;
class SWidget;
class SOverlay;
struct FGeometry;
struct FKeyHandle;
struct FPointerEvent;
struct FSlateBrush;
struct FTimeToPixel;
template <typename ElementType> class TRange;
struct FMovieSceneChannelMetaData;

namespace UE::Sequencer
{
	class FCategoryModel;
	class FChannelModel;
	class FSectionModel;
	class STrackLane;
	class STrackAreaView;
	struct FViewDensityInfo;

	struct ISectionView
	{
		virtual ~ISectionView() = default;

		virtual TSharedRef<FTimeToPixel> GetTimeToPixel() const = 0;
	};

	struct FCreateSectionViewWidgetParams
	{
		static constexpr int32 CompoundTrackLaneViewOrder = 0;
		static constexpr int32 DefaultWidgetOrder = 10;
		static constexpr int32 ChannelViewOrder = 20;

		TSharedRef<SOverlay> Overlay;
		TSharedRef<ISectionView> SectionView;
		TSharedRef<STrackLane> TrackLane;
		TSharedRef<STrackAreaView> TrackAreaView;
		TSharedRef<FSectionModel> SectionModel;
	};
}

/** Enumerates which edge is being resized */
UENUM()
enum ESequencerSectionResizeMode : int
{
	SSRM_LeadingEdge,
	SSRM_TrailingEdge
};

namespace SequencerSectionConstants
{
	/** How far the user has to drag the mouse before we consider the action dragging rather than a click */
	const float SectionDragStartDistance = 5.0f;

	/** The size of each key */
	const FVector2D KeySize(12.0f, 12.0f);

	const float DefaultSectionGripSize = 8.0f;

	const float DefaultSectionHeight = 27.f;

	const FName SelectionColorName("SelectionColor");

	const FName SelectionInactiveColorName("SelectionColorInactive");
}

/**
 * Parameters for the callback used when a section wants to customize how its properties details context menu looks like.
 */
struct FSequencerSectionPropertyDetailsViewCustomizationParams
{
	FSequencerSectionPropertyDetailsViewCustomizationParams(TSharedRef<ISequencerSection> InSectionInterface, TWeakPtr<ISequencer> InSequencerWeak, ISequencerTrackEditor& InTrackEditor)
		: SectionInterface(InSectionInterface)
		, SequencerWeak(InSequencerWeak)
		, TrackEditor(InTrackEditor)
	{}

	FGuid ParentObjectBindingGuid;
	TSharedRef<ISequencerSection> SectionInterface;
	TWeakPtr<ISequencer> SequencerWeak;
	ISequencerTrackEditor& TrackEditor;
};

/**
 * Interface that should be implemented for the UI portion of a section
 */
class ISequencerSection
{
public:
	/** Structure used during key area creation to group channels by their group name */
	struct FChannelData
	{
		/** Handle to the channel */
		FMovieSceneChannelHandle Channel;

		/** The channel's editor meta data */
		const FMovieSceneChannelMetaData& MetaData;
	};

	virtual ~ISequencerSection()= default;
	/**
	 * The MovieSceneSection data being visualized
	 */
	virtual UMovieSceneSection* GetSectionObject() = 0;

	/**
	 * Called when the section should be painted
	 *
	 * @param Painter		Structure that affords common painting operations
	 * @return				The new LayerId
	 */
	virtual int32 OnPaintSection( FSequencerSectionPainter& InPainter ) const = 0;

	/**
	 * Allows each section to have it's own unique widget for advanced editing functionality
	 * OnPaintSection will still be called if a widget is provided.  OnPaintSection is still used for the background section display
	 * 
	 * @return The generated widget 
	 */
	virtual TSharedRef<SWidget> GenerateSectionWidget() { return SNullWidget::NullWidget; }

	/**
	 * Create view widgets for the section by adding the necessary widgets to the overlay widget
	 */
	SEQUENCER_API virtual void CreateViewWidgets(const UE::Sequencer::FCreateSectionViewWidgetParams& Params);

	/**
	 * Called when the section is double clicked
	 *
	 * @param SectionGeometry	Geometry of the section
	 * @param MouseEvent		Event causing the double click
	 * @return A reply in response to double clicking the section
	 */
	virtual FReply OnSectionDoubleClicked( const FGeometry& SectionGeometry, const FPointerEvent& MouseEvent ) { return FReply::Unhandled(); }

	/**
	 * Called when the section is double clicked
	 *
	 * @param SectionGeometry	Geometry of the section
	 * @param MouseEvent		Event causing the double click
	 * @param ObjectBinding		The object guid bound to this section
	 * @return A reply in response to double clicking the section
	 */
	virtual FReply OnSectionDoubleClicked( const FGeometry& SectionGeometry, const FPointerEvent& MouseEvent, const FGuid& ObjectBinding) { return FReply::Unhandled(); }

	/**
	 * Called when a key on this section is double clicked
	 *
	 * @param KeyHandles			The array of keys that were clicked
	 * @return A reply in response to double clicking the key
	 */
	virtual FReply OnKeyDoubleClicked(const TArray<FKeyHandle>& KeyHandles ) { return FReply::Unhandled(); }

	/**
	 * @return The display name of the section in the section view
	 */
	virtual FText GetSectionTitle() const { return FText(); }

	/**
	 * @return An icon brush for the section's outliner row, or nullptr for no icon
	 */
	virtual const FSlateBrush* GetIconBrush() const { return nullptr; }

	/**
	 * @return The ToolTip for the section in the section view. By default, the section title
	 */
	virtual FText GetSectionToolTip() const { return GetSectionTitle(); }

	/**
	 * @return The local section time
	 */
	virtual TOptional<FFrameTime> GetSectionTime(FSequencerSectionPainter& InPainter) const { return TOptional<FFrameTime>(); }

	/**
	 * @return The amount of padding to apply to non-interactive portions of the section interface (such as section text)
	 */
	virtual FMargin GetContentPadding() const { return FMargin(11.f, 6.f); }

	/**
	 * Generates the inner layout for this section
	 *
	 * @param LayoutBuilder	The builder utility for creating section layouts
	 */	
	SEQUENCER_API virtual void GenerateSectionLayout( class ISectionLayoutBuilder& LayoutBuilder );

	/**
	 * Create a custom category model
	 */
	virtual TSharedPtr<UE::Sequencer::FCategoryModel> ConstructCategoryModel(FName InCategoryName, const FText& InDisplayText, TArrayView<const FChannelData> Channels) const { return nullptr; }

	/**
	 * Create a custom channel model
	 */
	virtual TSharedPtr<UE::Sequencer::FChannelModel> ConstructChannelModel(FName InChannelName, const FMovieSceneChannelHandle& InChannelHandle) const { return nullptr; }

	/**
	 * @return The height of the section
	 */
	UE_DEPRECATED(5.4, "Please call GetSectionHeight(const FViewDensityInfo& ViewDensity) instead.")
	SEQUENCER_API virtual float GetSectionHeight() const;
	SEQUENCER_API virtual float GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const;


	/**
	 * @return The width of the section drag handles
	 */
	virtual float GetSectionGripSize() const { return SequencerSectionConstants::DefaultSectionGripSize; }

	/**
	 * @return The height of the section drag handles
	 */
	virtual float GetSectionGripHeight(float InSectionHeight) const { return InSectionHeight; }

	/**
	 * @ return The size of keyframe widgets
	 */
	virtual FVector2D GetKeySize() const { return SequencerSectionConstants::KeySize; }

	/**
	 * @return Whether or not the user can resize this section.
	 */
	virtual bool SectionIsResizable() const {return true;}

	/**
	 * @return Whether this section is read only.
	 */
	virtual bool IsReadOnly() const {return false;}

	/**
	 * @return Whether this section can be deleted by the user.
	 */
	virtual bool IsDeletable() const { return true; }

	/**
	 * Controls whether a given outliner column toggle (mute, solo, lock, deactivate, etc.)
	 * is available for this section's outliner item. Default is true for all columns.
	 * Override to return false for specific columns on sections like transitions that should
	 * not expose certain controls.
	 */
	virtual bool SupportsOutlinerColumnToggle(FName ColumnName) const { return true; }

	/**
	 * Controls whether the section's key area should be visible when the section's parent
	 * outliner item is collapsed. Default is true.
	 * Override to return false for sections that draw their own visualization but don't want
	 * standard key rendering when collapsed.
	 */
	virtual bool ShouldShowKeyAreaWhenCollapsed() const { return true; }

	/**
	 * Ticks the section during the Slate tick
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  ClippedGeometry The space for this widget clipped against the parent widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick( const FGeometry& AllottedGeometry, const FGeometry& ClippedGeometry, const double InCurrentTime, const float InDeltaTime ) {}

	/**
	 * Builds up the section context menu for the outliner
	 *
	 * @param MenuBuilder	The menu builder to change
	 * @param ObjectBinding The object guid bound to this section
	 */
	virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding) {}

	/**
	 * Builds up the section sidebar menu for the outliner
	 *
	 * @param MenuBuilder	The menu builder to change
	 * @param ObjectBinding The object guid bound to this section
	 */
	virtual void BuildSectionSidebarMenu(FMenuBuilder& MenuBuilder, const FGuid& ObjectBinding)
	{
		BuildSectionContextMenu(MenuBuilder, ObjectBinding);
	}

	/**
	 * Called when the user requests that a category from this section be deleted. 
	 *
	 * @param CategoryNamePath An array of category names which is the path to the category to be deleted.
	 * @returns Whether or not the category was deleted.
	 */
	virtual bool RequestDeleteCategory( const TArray<FName>& CategoryNamePath ) { return false; }

	/**
	 * Called when the user requests that a key area from this section be deleted.
	 *
	 * @param KeyAreaNamePath An array of names representing the path of to the key area to delete, starting with any categories which contain the key area.
	 * @returns Whether or not the key area was deleted.
	 */
	virtual bool RequestDeleteKeyArea( const TArray<FName>& KeyAreaNamePath ) { return false; }

	/**
	 * Resize the section 
	 *
	 * @param ResizeMode Resize either the leading or the trailing edge of the section
	 * @param ResizeTime The time to resize to
	 */
	virtual void BeginResizeSection() {}
	SEQUENCER_API virtual void ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeFrameNumber);

	/**
	 * Slips the section by a specific factor
	 *
	 * @param SlipTime The amount to slip this section by
	 */
	virtual void BeginSlipSection() {}
	virtual void SlipSection(FFrameNumber SlipTime) {}

	/**
	Dilation starts with a drag operation
	*/
	virtual void BeginDilateSection() {};
	/**
	New Range that's set as we Dilate
	@param NewRange The NewRange.
	@param DilationFactor The factor we have dilated from the beginning of the drag
	*/
	virtual void DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor) {};



	/**
	 * Called when the properties context menu is being built, so this section can customize how the menu's details view looks like.
	 *
	 * @param DetailsView The details view widget
	 * @param InParams Information about the current operation
	 */
	virtual void CustomizePropertiesDetailsView(TSharedRef<IDetailsView> DetailsView, const FSequencerSectionPropertyDetailsViewCustomizationParams& InParams) const {}
};

class FSequencerSection : public ISequencerSection
{
public:
	FSequencerSection(UMovieSceneSection& InSection)
		: WeakSection(&InSection)
	{}

	SEQUENCER_API virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override;

	virtual UMovieSceneSection* GetSectionObject() override final
	{
		return WeakSection.Get();
	}

	virtual bool IsReadOnly() const override { return WeakSection.IsValid() ? WeakSection.Get()->IsReadOnly() : false; }

	virtual void DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor) override
	{
		if (GetSectionObject())
		{
			GetSectionObject()->SetRange(NewRange);
		}
	}

protected:
	TWeakObjectPtr<UMovieSceneSection> WeakSection;
};
