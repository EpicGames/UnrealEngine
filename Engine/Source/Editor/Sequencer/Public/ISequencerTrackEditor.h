// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Misc/Attribute.h"
#include "Templates/SubclassOf.h"
#include "Framework/Commands/UICommandList.h"
#include "ISequencerSection.h"
#include "MovieSceneTrack.h"
#include "SequencerKeyParams.h"
#include "SequencerAddKeyOperation.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModelPtr.h"

class FExtender;
class ISequencer;
class FMenuBuilder;
class FPaintArgs;
class FSlateWindowElementList;
class SHorizontalBox;
class UMovieScene;
class UMovieSceneTrack;
class UMovieSceneSequence;

namespace UE::Sequencer
{
	class ITrackExtension;
	class FObjectBindingModel;
}



struct FBuildColumnWidgetParams : UE::Sequencer::FCreateOutlinerViewParams
{
	FBuildColumnWidgetParams(UE::Sequencer::TViewModelPtr<UE::Sequencer::FViewModel> InViewModelItem, const UE::Sequencer::FCreateOutlinerViewParams& InBaseParams)
		: UE::Sequencer::FCreateOutlinerViewParams(InBaseParams)
		, ViewModel(InViewModelItem.AsModel())
	{}

	/**  */
	UE::Sequencer::FViewModelPtr ViewModel;
};

/** Data structure containing information required to build an edit widget */
struct FBuildEditWidgetParams : FBuildColumnWidgetParams
{
	FBuildEditWidgetParams(const FBuildColumnWidgetParams& Other)
		: FBuildColumnWidgetParams(Other)
		, RowIndex(0)
		, TrackInsertRowIndex(0)
	{}

	/** Attribute that specifies when the node relating to this edit widget is hovered */
	TAttribute<bool> NodeIsHovered;

	/** Index of current track row */
	int32 RowIndex;

	/** Track row index for any newly created sections */
	int32 TrackInsertRowIndex;
};

struct FSequencerDragDropParams
{
	FSequencerDragDropParams()
		: Track(nullptr)
		, RowIndex(INDEX_NONE)
		{}

	FSequencerDragDropParams(UMovieSceneTrack* InTrack, int32 InRowIndex, FGuid InTargetObjectGuid, FFrameNumber InFrameNumber, const TRange<FFrameNumber>& InFrameRange)
		: Track(InTrack)
		, RowIndex(InRowIndex)
		, TargetObjectGuid(InTargetObjectGuid)
		, FrameNumber(InFrameNumber)
		, FrameRange(InFrameRange)
	{}

	/** The track that is receiving this drop event */
	TWeakObjectPtr<UMovieSceneTrack> Track;
	 
	/** The row index to drop onto */
	int32 RowIndex;

	/** The object guid this asset is dropped onto, if applicable */
	FGuid TargetObjectGuid;

	/** The frame number that this drop event is being dropped at */
	FFrameNumber FrameNumber;

	/** The frame range that this drop event is being dropped at */
	TRange<FFrameNumber> FrameRange;
};

/**
 * Interface for sequencer track editors.
 */
class ISequencerTrackEditor
{
public:

	/** The display name text to use in the UX. Currently only used for sorting. */
	virtual FText GetDisplayName() const = 0;

	/**
	 * Add keys for the following sections based on an external value if possible
	 *
	 * @param InKeyTime   The time at which to add keys
	 * @param Operation   Structure containing all the sections and channels to key
	 * @param InSequencer The sequencer UI that is applying the key operation
	 * @param OutResults  The key handles that were added with this operation
	 */
	virtual void ProcessKeyOperation(FFrameNumber InKeyTime, const UE::Sequencer::FKeyOperation& Operation, ISequencer& InSequencer, TArray<UE::Sequencer::FAddKeyResult>* OutResults = nullptr) { Operation.ApplyDefault(InKeyTime, InSequencer, OutResults); }

	/**
	 * Add a new track to the sequence.
	 */
	virtual UMovieSceneTrack* AddTrack(UMovieScene* FocusedMovieScene, const FGuid& ObjectHandle, TSubclassOf<class UMovieSceneTrack> TrackClass, FName UniqueTypeName) = 0;

	/**
	 * Allows the track editors to bind commands.
	 *
	 * @param SequencerCommandBindings The command bindings to map to.
	*/
	virtual void BindCommands(TSharedRef<FUICommandList> SequencerCommandBindings) = 0;

	/**
	 * Builds up the sequencer's "Add Track" menu. for menu items pinned to the top
	 *
	 * @param MenuBuilder The menu builder to change.
	 */
	virtual void BuildPinnedAddTrackMenu(FMenuBuilder& MenuBuilder) = 0;

	/**
	 * Builds up the sequencer's "Add Track" menu.
	 *
	 * @param MenuBuilder The menu builder to change.
	 */
	virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) = 0;

	/**
	 * Deprecated in principle, not practice (due to difficulty deprecating public virtual functions).
	 * Still called from the newer BuildObjectBindingColumnWidgets which should now be preferred
	 */
	virtual void BuildObjectBindingEditButtons(TSharedPtr<SHorizontalBox> EditBox, const FGuid& ObjectBinding, const UClass* ObjectClass){}

	/**
	 * Build a column widget for the specified object binding on the outliner
	 *
	 * @param GetEditBox A function to retrieve the edit box to populate
	 * @param ObjectBinding The object binding this is for.
	 */
	virtual void BuildObjectBindingColumnWidgets(TFunctionRef<TSharedRef<SHorizontalBox>()> GetEditBox, const UE::Sequencer::TViewModelPtr<UE::Sequencer::FObjectBindingModel>& ObjectBinding, const UE::Sequencer::FCreateOutlinerViewParams& InParams, const FName& InColumnName) = 0;

	/**
	 * Builds up the object binding track menu for the outliner.
	 *
	 * @param MenuBuilder The menu builder to change.
	 * @param ObjectBinding The object binding this is for.
	 * @param ObjectClass The class of the object this is for.
	 */
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) = 0;

	/**
	 * Extend the object binding track menu for the specified binding and class
	 *
	 * @param Extender A menu extender for the track menu
	 * @param ObjectBinding The object binding this is for.
	 * @param ObjectClass The class of the object this is for.
	 */
	virtual void ExtendObjectBindingTrackMenu(TSharedRef<FExtender> Extender, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) { }


	/**
	 * Builds up the object binding cpmtext menu for the outliner.
	 *
	 * @param MenuBuilder The menu builder to change.
	 * @param ObjectBinding The object binding this is for.
	 * @param ObjectClass The class of the object this is for.
	 */
	virtual void BuildObjectBindingContextMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) {}

	/**
	 * Builds a widget for the add column for outliner nodes which represent tracks which are edited by this editor.
	 * @param ObjectBinding The object binding associated with the track being edited by this editor.
	 * @param Track The track being edited by this editor.
	 * @param Params Parameter struct containing data relevant to the edit widget
	 * @returns The the widget to display in the outliner, or an empty shared ptr if not widget is to be displayed.
	 */
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) = 0;

	/**
	 * Builds a widget for the edit column for outliner nodes which represent tracks which are edited by this editor.
	 * @param ObjectBinding The object binding associated with the track being edited by this editor.
	 * @param Track The track being edited by this editor.
	 * @param Params Parameter struct containing data relevant to the edit widget
	 * @returns The the widget to display in the outliner, or an empty shared ptr if not widget is to be displayed.
	 */
	virtual TSharedPtr<SWidget> BuildOutlinerEditColumnWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) = 0;

	/**
	 * Builds an outliner column widget for the outliner nodes which represent tracks which are edited by this editor.
	 * 
	 * @param Params Parameter struct containing data relevant to the edit widget
	 * @param ColumnName The name of the column. See FCommonOutlinerNames
	 * @returns The the widget to display in the outliner, or an empty shared ptr if not widget is to be displayed.
	 */
	virtual TSharedPtr<SWidget> BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName) = 0;

	/**
	 * Builds the context menu for the track.
	 * @param MenuBuilder The menu builder to use to build the track menu. 
	 */
	virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) = 0;

	/**
	 * Builds the sidebar menu for the track.
	 * @param MenuBuilder The menu builder to use to build the track menu. 
	 */
	virtual void BuildTrackSidebarMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) = 0;

	/**
	 * Called when an asset is dropped into Sequencer. Can potentially consume the asset
	 * so it doesn't get added as a spawnable.
	 *
	 * @param Asset The asset that is dropped in.
	 * @param TargetObjectGuid The object guid this asset is dropped onto, if applicable.
	 * @return true if we want to consume this asset, false otherwise.
	 */
	virtual bool HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) = 0;

	/**
	 * Called when attempting to drop an asset directly onto a track.
	 *
	 * @param DragDropEvent The drag drop event.
	 * @param DragDropParams The drag drop parameters.
	 * @return Whether the drop event can be handled.
	 */
	virtual bool OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams) = 0;

	/**
	 * Called when an asset is dropped directly onto a track.
	 *
	 * @param DragDropEvent The drag drop event.
	 * @param DragDropParams The drag drop parameters.
	 * @return Whether the drop event was handled.
	 */	
	virtual FReply OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams) = 0;

	/**
	 * Called to generate a section layout for a particular section.
	 *
	 * @param SectionObject The section to make UI for.
	 * @param Track The track that owns the section.
	 * @param ObjectBinding the object binding for the track that owns the section, if there is one.
	 */
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) = 0;

	/** Gets an icon brush for this track editor */
	virtual const FSlateBrush* GetIconBrush() const { return nullptr; }

	/** Called when the instance of this track editor is initialized */
	virtual void OnInitialize() = 0;

	/** Called when the instance of this track editor is released */
	virtual void OnRelease() = 0;

	/**
	 * Returns whether a track class is supported by this tool.
	 *
	 * @param TrackClass The track class that could be supported.
	 * @return true if the type is supported.
	 */
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> TrackClass) const = 0;

	/**
	 * Returns whether a sequence is supported by this tool.
	 *
	 * @param InSequence The sequence that could be supported.
	 * @return true if the type is supported.
	 */
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const = 0;

	/**
	 * Ticks this tool.
	 *
	 * @param DeltaTime The time since the last tick.
	 */
	virtual void Tick(float DeltaTime) = 0;

	/**
	 * @return Whether this track handles resize events
	 */
	virtual bool IsResizable(UMovieSceneTrack* InTrack) const
	{
		return false;
	}

	/**
	 * Resize this track
	 */
	virtual void Resize(float NewSize, UMovieSceneTrack* InTrack)
	{
		
	}

	/**
	 * @return The default expansion state of this track
	 */
	virtual bool GetDefaultExpansionState(UMovieSceneTrack* InTrack) const
	{
		return false;
	}

	/**
	 * @return If it supports supports transform key bindings for setting keys.
	 */
	virtual bool HasTransformKeyBindings() const { return false; }

	/** Whether or not we can add or remove a transform key for a selected object
	* @return Returns true if we can.
	**/
	virtual bool CanAddTransformKeysForSelectedObjects() const { return false; }

	/**
	* Adds transform tracks and keys to the selected objects in the level.
	*
	* @param Channel The transform channel to add keys for.
	*/
	virtual void OnAddTransformKeysForSelectedObjects(EMovieSceneTransformChannel Channel) {};

	/**
	* Removes transform keys at the current time from the selected objects in the level.
	*
	* @param Channel The transform channel to remove keys for.
	*/
	virtual void OnDeleteTransformKeysForSelectedObjects(EMovieSceneTransformChannel Channel) {};

	/**
	* If true this track has priority when setting transform keys and should be the only one to set them
	*
	*/
	virtual bool HasTransformKeyOverridePriority() const { return false; }

	/**
	* Handle this object being implicitly added
	*/
	virtual void ObjectImplicitlyAdded(UObject* InObject)  {}

	/**
	* Handle this object being implicitly removed
	*/
	virtual void ObjectImplicitlyRemoved(UObject* InObject) {}

	/**
	 * Called before the sequencer restores pre-animated state on all objects before saving the level.
	 */
	virtual void OnPreSaveWorld(UWorld* World) {}

	/**
	 * Called after the sequencer has re-evaluated all objects after saving the level.
	 */
	virtual void OnPostSaveWorld(UWorld* World) {}

public:

	/** Virtual destructor. */
	virtual ~ISequencerTrackEditor() = default;
};
