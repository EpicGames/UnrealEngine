// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWidget.h"

namespace UE::Sequencer
{
/** Interface to interact with the SSequencer widget. */
class ISequencerWidget
{
public:
	
	/** Enable/disable pending focus in sequencer */
	virtual void EnablePendingFocusOnHovering(const bool InEnabled) = 0;
	
	virtual ~ISequencerWidget() = default;
};
}