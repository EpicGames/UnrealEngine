// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Math/UnrealMathSSE.h"
#include "PropertyPath.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/Field.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"
#include "EntitySystem/MovieSceneIntermediatePropertyValue.h"

#define UE_API SEQUENCER_API

class IPropertyHandle;
class UClass;
class UObject;
class UStruct;

enum class ESequencerKeyMode;

/**
 * Parameters for determining if a property can be keyed.
 */
struct FCanKeyPropertyParams
{
	/**
	 * Creates new can key property parameters.
	 * @param InObjectClass the class of the object which has the property to be keyed.
	 * @param InPropertyPath path get from the root object to the property to be keyed.
	 */
	UE_API FCanKeyPropertyParams(const UClass* InObjectClass, const FPropertyPath& InPropertyPath);

	/**
	* Creates new can key property parameters.
	* @param InObjectClass the class of the object which has the property to be keyed.
	* @param InPropertyHandle a handle to the property to be keyed.
	*/
	UE_API FCanKeyPropertyParams(const UClass* InObjectClass, const IPropertyHandle& InPropertyHandle);

	/** The owner struct */
	UE_API const UStruct* FindPropertyOwner(const FProperty* ForProperty) const;

	/** The container from which to find setter functions */
	UE_API const UStruct* FindPropertyContainer(const FProperty* ForProperty) const;

	/** The class of the object which has the property to be keyed. */
	const UClass* ObjectClass;

	/** A path of properties to get from the root object to the property to be keyed. */
	FPropertyPath PropertyPath;
};

/**
 * Parameters for keying a property.
 */
struct FKeyPropertyParams
{
	/**
	* Creates new key property parameters for a manually triggered property change.
	* @param InObjectsToKey an array of the objects who's property will be keyed.
	* @param InPropertyPath path get from the root object to the property to be keyed.
	*/
	UE_API FKeyPropertyParams(TArray<UObject*> InObjectsToKey, const FPropertyPath& InPropertyPath, ESequencerKeyMode InKeyMode);

	/**
	* Creates new key property parameters from an actual property change notification with a property handle.
	* @param InObjectsToKey an array of the objects who's property will be keyed.
	* @param InPropertyHandle a handle to the property to be keyed.
	*/
	UE_API FKeyPropertyParams(TArray<UObject*> InObjectsToKey, const IPropertyHandle& InPropertyHandle, ESequencerKeyMode InKeyMode);

	/** An array of the objects who's property will be keyed. */
	const TArray<UObject*> ObjectsToKey;

	/** A path of properties to get from the root object to the property to be keyed. */
	FPropertyPath PropertyPath;

	/** Keyframing params */
	const ESequencerKeyMode KeyMode;
};

/**
 * Parameters for the property changed callback.
 */
class FPropertyChangedParams
{
public:

	using FSourcePropertyValue = UE::MovieScene::FSourcePropertyValue;

	UE_API FPropertyChangedParams(TArray<UObject*> InObjectsThatChanged, const FPropertyPath& InPropertyPath, const FPropertyPath& InStructPathToKey, ESequencerKeyMode InKeyMode);


	UE_API TPair<const FProperty*, FSourcePropertyValue> GetPropertyAndValue() const;
	UE_API FSourcePropertyValue GetPropertyValue() const;

	/**
	 * Gets the value of the property that changed.
	 */
	template<typename ValueType>
	ValueType GetPropertyValue() const
	{
		FSourcePropertyValue Value = GetPropertyValue();

		// This is unsafe, but it is assumed that the user knows the cast is safe in calling this function
		return *Value.Cast<ValueType>();
	}

	/** Gets the property path as a period seperated string of property names. */
	UE_API FString GetPropertyPathString() const;

	/** An array of the objects that changed. */
	const TArray<UObject*> ObjectsThatChanged;

	/** A path of properties to get from the root object to the property to be keyed. */
	FPropertyPath PropertyPath;

	/** Represents the path of an inner property which should be keyed for a struct property.  If all inner 
	properties should be keyed, this will be empty. */
	FPropertyPath StructPathToKey;

	/** Keyframing params */
	ESequencerKeyMode KeyMode;
};

#undef UE_API
