// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AcquiredResources.h"
#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "Layout/Visibility.h"
#include "ISceneOutlinerColumn.h"
#include "UObject/ObjectKey.h"
#include "ISequencer.h"
#include "Engine/EngineBaseTypes.h"
#include "LevelEditorSequencerIntegration.generated.h"

#define UE_API SEQUENCER_API

class AActor;
class FExtender;
class FMenuBuilder;
class FSequencer;
class FObjectPostSaveContext;
class FObjectPreSaveContext;
class SLevelViewport;
class FUICommandList;
class IAssetViewport;
class ISequencer;
class ULevel;
class UToolMenu;
struct FPropertyAndParent;
struct FPilotedSpawnable;

struct FLevelEditorSequencerIntegrationOptions
{
	FLevelEditorSequencerIntegrationOptions()
		: bRequiresLevelEvents(true)
		, bRequiresActorEvents(false)
		, bForceRefreshDetails(true)
		, bAttachOutlinerColumns(true)
		, bActivateSequencerEdMode(true)
		, bSyncBindingsToActorLabels(true)
	{}

	bool bRequiresLevelEvents : 1;
	bool bRequiresActorEvents : 1;
	bool bForceRefreshDetails : 1;
	bool bAttachOutlinerColumns : 1;
	bool bActivateSequencerEdMode : 1;
	bool bSyncBindingsToActorLabels : 1;
};


class FLevelEditorSequencerBindingData : public TSharedFromThis<FLevelEditorSequencerBindingData>
{
public:
	FLevelEditorSequencerBindingData() 
		: bActorBindingsDirty(true)
		, bPropertyBindingsDirty(true)
	{}

	DECLARE_MULTICAST_DELEGATE(FActorBindingsDataChanged);
	DECLARE_MULTICAST_DELEGATE(FPropertyBindingsDataChanged);

	FActorBindingsDataChanged& OnActorBindingsDataChanged() { return ActorBindingsDataChanged; }
	FPropertyBindingsDataChanged& OnPropertyBindingsDataChanged() { return PropertyBindingsDataChanged; }

	FString GetLevelSequencesForActor(TWeakPtr<FSequencer> Sequencer, const AActor*);
	bool GetIsPropertyBound(TWeakPtr<FSequencer> Sequencer, const struct FPropertyAndParent&);

	bool bActorBindingsDirty;
	bool bPropertyBindingsDirty;

private:
	void UpdateActorBindingsData(TWeakPtr<FSequencer> InSequencer);
	void UpdatePropertyBindingsData(TWeakPtr<FSequencer> InSequencer);

	TMap< FObjectKey, FString > ActorBindingsMap;
	TMap< FObjectKey, TArray<FString> > PropertyBindingsMap;

	FActorBindingsDataChanged ActorBindingsDataChanged;
	FPropertyBindingsDataChanged PropertyBindingsDataChanged;
};

/**
* Tick function whose sole purpose is to update the gizmo position after any root motion may have run on selected characters at the end of the frame.
**/
USTRUCT()
struct FLevelEditorSequencerUpdateGizmoTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

	bool bSequencerEvaluated = false;

	/**
	* Abstract function to execute the tick.
	* @param DeltaTime - frame time to advance, in seconds.
	* @param TickType - kind of tick for this frame.
	* @param CurrentThread - thread we are executing on, useful to pass along as new tasks are created.
	* @param MyCompletionGraphEvent - completion event for this task. Useful for holding the completetion of this task until certain child tasks are complete.
	*/
	virtual void ExecuteTick(float DeltaTime, enum ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	/** Abstract function to describe this tick. Used to print messages about illegal cycles in the dependency graph. */
	virtual FString DiagnosticMessage() override;
	/** Function used to describe this tick for active tick reporting. **/
	virtual FName DiagnosticContext(bool bDetailed) override;
};

template<>
struct TStructOpsTypeTraits<FLevelEditorSequencerUpdateGizmoTickFunction> : public TStructOpsTypeTraitsBase2<FLevelEditorSequencerUpdateGizmoTickFunction>
{
	enum
	{
		WithCopy = false
	};
};


class FLevelEditorSequencerIntegration
{
public:

	static UE_API FLevelEditorSequencerIntegration& Get();

	UE_API void Initialize(const FLevelEditorSequencerIntegrationOptions& Options);

	UE_API void AddSequencer(TSharedRef<ISequencer> InSequencer, const FLevelEditorSequencerIntegrationOptions& Options);

	UE_API void OnSequencerReceivedFocus(TSharedRef<ISequencer> InSequencer);

	UE_API void RemoveSequencer(TSharedRef<ISequencer> InSequencer);

	UE_API TArray<TWeakPtr<ISequencer>> GetSequencers();
	DECLARE_MULTICAST_DELEGATE(FOnSequencersChanged);
	FOnSequencersChanged& GetOnSequencersChanged() { return OnSequencersChanged; };

private:

	/** Called before the world is going to be saved. The sequencer puts everything back to its initial state. */
	UE_API void OnPreSaveWorld(UWorld* World, FObjectPreSaveContext ObjectSaveContext);

	/** Called after the world has been saved. The sequencer updates to the animated state. */
	UE_API void OnPostSaveWorld(UWorld* World, FObjectPostSaveContext ObjectSaveContext);

	/** Called before any number of external actors are going to be saved. The sequencer puts everything back to its initial state. */
	UE_API void OnPreSaveExternalActors(UWorld* World);

	/** Called after any number of external actors has been saved. The sequencer puts everything back to its initial state. */
	UE_API void OnPostSaveExternalActors(UWorld* World);

	/** Called before asset validation is run on assets. The sequencer puts everything back to its initial state. */
	UE_API void OnPreAssetValidation();
	
	/** Called after asset validation has finished. The sequencer re-evaluates to hide the fact we did this from users. */
	UE_API void OnPostAssetValidation();

	/** Called after a level has been added */
	UE_API void OnLevelAdded(ULevel* InLevel, UWorld* InWorld);

	/** Called after a level has been removed */
	UE_API void OnLevelRemoved(ULevel* InLevel, UWorld* InWorld);

	/** Called after a new level has been created. The sequencer editor mode needs to be enabled. */
	UE_API void OnNewCurrentLevel();

	/** Called after a map has been opened. The sequencer editor mode needs to be enabled. */
	UE_API void OnMapOpened(const FString& Filename, bool bLoadAsTemplate);

	/** Called when new actors are dropped in the viewport. */
	UE_API void OnNewActorsDropped(const TArray<UObject*>& DroppedObjects, const TArray<AActor*>& DroppedActors);

	/** Called when viewport tab content changes. */
	UE_API void OnTabContentChanged();

	/** Called when the map is changed. */
	UE_API void OnMapChanged(UWorld* World, EMapChangeType MapChangeType);

	/** Called before a PIE session begins. */
	UE_API void OnPreBeginPIE(bool bIsSimulating);

	/** Called after a PIE session ends. */
	UE_API void OnEndPIE(bool bIsSimulating);

	/** Called after PIE session ends and maps have been cleaned up */
	UE_API void OnEndPlayMap();

	/** Handles the actor selection changing externally .*/
	UE_API void OnActorSelectionChanged( UObject* );

	/** Called when an actor label has changed */
	UE_API void OnActorLabelChanged(AActor* ChangedActor);

	/** Called when sequencer has been evaluated */
	UE_API void OnSequencerEvaluated();

	/** Called when bindings have changed */
	UE_API void OnMovieSceneBindingsChanged();

	/** Called when data has changed */
	UE_API void OnMovieSceneDataChanged(EMovieSceneDataChangeType DataChangeType);

	/** Called when allow edits mode has changed */
	UE_API void OnAllowEditsModeChanged(EAllowEditsMode AllowEditsMode);

	/** Called when the user begins playing/scrubbing */
	UE_API void OnBeginDeferUpdates();

	/** Called when the user stops playing/scrubbing */
	UE_API void OnEndDeferUpdates();

	/** Called to determine whether a binding is visible in the tree view */
	UE_API bool IsBindingVisible(const FMovieSceneBinding& InBinding);

	UE_API void RegisterMenus();

	UE_API void MakeBrowseToSelectedActorSubMenu(UToolMenu* Menu);
	UE_API void BrowseToSelectedActor(AActor* Actor, FSequencer* Sequencer, FMovieSceneSequenceID SequenceId);

	UE_API bool IsPropertyReadOnly(const FPropertyAndParent& InPropertyAndParent);

private:

	UE_API void ActivateSequencerEditorMode();
	UE_API void DeactivateSequencerEditorMode();
	UE_API void AddLevelViewportMenuExtender();
	UE_API void BindDetailHandler(const FLevelEditorSequencerIntegrationOptions& Options);
	UE_API void ActivateDetailHandler();
	UE_API void AttachOutlinerColumn();
	UE_API void DetachOutlinerColumn();
	UE_API void ActivateRealtimeViewports();
	UE_API void RestoreRealtimeViewports();
	UE_API void RestoreToSavedState(UWorld* World);
	UE_API void ResetToAnimatedState(UWorld* World);

	UE_API void BackupSpawnablePilotData();
	UE_API void RestoreSpawnablePilotData();

	struct FSequencerAndOptions
	{
		TWeakPtr<FSequencer> Sequencer;
		FLevelEditorSequencerIntegrationOptions Options;
		FAcquiredResources AcquiredResources;
		TSharedRef<FLevelEditorSequencerBindingData> BindingData;
	};
	TArray<FSequencerAndOptions> BoundSequencers;

public:
	
	UE_API TSharedRef< ISceneOutlinerColumn > CreateSequencerInfoColumn( ISceneOutliner& SceneOutliner ) const;
	UE_API TSharedRef< ISceneOutlinerColumn > CreateSequencerSpawnableColumn( ISceneOutliner& SceneOutliner ) const;

private:

	UE_API void IterateAllSequencers(TFunctionRef<void(FSequencer&, const FLevelEditorSequencerIntegrationOptions& Options)>) const;
	UE_API void UpdateDetails(bool bForceRefresh = false);

	UE_API FLevelEditorSequencerIntegration();
	UE_API ~FLevelEditorSequencerIntegration();

private:
	FAcquiredResources AcquiredResources;

	TSharedPtr<class FDetailKeyframeHandlerWrapper> KeyFrameHandler;

	TArray<FPilotedSpawnable> PilotedSpawnables;

	bool bDeferUpdates;

	FOnSequencersChanged OnSequencersChanged;

	FLevelEditorSequencerUpdateGizmoTickFunction UpdateGizmoTickFunction;

};

#undef UE_API
