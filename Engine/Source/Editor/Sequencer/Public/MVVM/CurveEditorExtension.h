// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/Extensions/DynamicExtensionContainer.h"
#include "MVVM/Extensions/ICurveEditorTreeItemExtension.h"
#include "CurveDataAbstraction.h"
#include "CurveEditorTypes.h"
#include "Tree/ICurveEditorTreeItem.h"
#include "STemporarilyFocusedSpinBox.h"
#include "Engine/TimerHandle.h"

#define UE_API SEQUENCER_API

class FTabManager;
class ISequencer;
class IPropertyTypeCustomization;
class SCurveEditorPanel;
class SCurveEditorTree;
class SCurveEditorTreeFilterStatusBar;
class USequencerSettings;
struct FTimeSliderArgs;

namespace UE
{
namespace Sequencer
{
class FSequencerEditorViewModel;
class FSequencerCurveEditorApp;

/** Public API for interacting with Curve Editor in Sequencer. */
class FCurveEditorExtension : public IDynamicExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, FCurveEditorExtension)

	FCurveEditorExtension();
	~FCurveEditorExtension();

	UE_API virtual void OnCreated(TSharedRef<FViewModel> InWeakOwner) override;

	/** Creates the curve editor view-model and widget */
	UE_API void CreateCurveEditor(const FTimeSliderArgs& TimeSliderArgs);

	/** Gets the curve editor view-model */
	UE_API TSharedPtr<FCurveEditor> GetCurveEditor() const;

	/** Opens the curve editor */
	UE_API void OpenCurveEditor();
	/** Returns whether the curve editor is open */
	UE_API bool IsCurveEditorOpen() const;
	/** Closes the curve editor */
	UE_API void CloseCurveEditor();

	/** Curve editor tree widget */
	UE_API TSharedPtr<SCurveEditorTree> GetCurveEditorTreeView() const;

	/**
	 * Synchronize curve editor selection with sequencer outliner selection on the next update.
	 */
	UE_API void RequestSyncSelection();

public:

	static UE_API const FName CurveEditorTabName;

private:
	
	TSharedPtr<ISequencer> GetSequencer() const;
	USequencerSettings* GetSequencerSettings() const;
	TSharedPtr<FTabManager> GetTabManager() const;

	EVisibility GetPopoutTransportControlsVisibility() const;

	/** The sequencer editor we are extending with a curve editor */
	TWeakPtr<FSequencerEditorViewModel> WeakOwnerModel;
	
	/** Manages the Curve Editor x Sequencer integration. */
	TPimplPtr<FSequencerCurveEditorApp> CurveEditorApp;
	
	friend class FSequencerCurveEditorApp; // Engineering compromise to access FSequencerCurveEditorApp without exposing to public API
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
