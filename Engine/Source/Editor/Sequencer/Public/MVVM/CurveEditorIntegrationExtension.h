// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/Extensions/DynamicExtensionContainer.h"
#include "MVVM/Extensions/ICurveEditorTreeItemExtension.h"
#include "CurveEditorTypes.h"
#include "Tree/ICurveEditorTreeItem.h"

#define UE_API SEQUENCER_API

namespace UE
{
namespace Sequencer
{

class FSequenceModel;
class FCurveEditorExtension;

/**
 * Extension for managing integration between outliner items and the curve editor.
 *
 * It relies on the following:
 *
 * - Extension owner (generally the root view-model) implements ICurveEditorExtension, to get
 *   access to the curve editor itself.
 *
 * - Outliner items implementing ICurveEditorTreeItemExtension (or its default shim) if they 
 *   want to show up in the curve editor.
 */
class UE_DEPRECATED(5.8, "This class has been removed.") FCurveEditorIntegrationExtension : public IDynamicExtension
{
public:

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, FCurveEditorIntegrationExtension)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.8, "This class has been removed.")
	UE_API FCurveEditorIntegrationExtension();

	UE_API virtual void OnCreated(TSharedRef<FViewModel> InWeakOwner) override;

	/** 
	 * Keeps the curve editor items up-to-date with the sequencer outliner by adding/removing 
	 * entries as needed.
	 */
	UE_DEPRECATED(5.8, "This function has been removed.")
	UE_API void UpdateCurveEditor();

	/**
	 * Clears the curve editor of all contents.
	 */
	UE_DEPRECATED(5.8, "This function has been removed.")
	UE_API void ResetCurveEditor();

private:

	/** Update curve editor items when sequencer outliner items change */
	UE_API void OnHierarchyChanged();

	class FSequencerCurveEditorApp* GetCurveEditorIntegration() const;
	
	/** Finds the curve editor extension on the top-level sequencer editor view-model */
	UE_API FCurveEditorExtension* GetCurveEditorExtension();

private:

	TWeakPtr<FSequenceModel> WeakOwnerModel;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
