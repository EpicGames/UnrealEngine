// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "MVVM/Extensions/DynamicExtensionContainer.h"
#include "MVVM/ViewModelTypeID.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectKey.h"

#define UE_API SEQUENCER_API

class ISequencer;
class UMovieSceneTrack;
class UMovieSceneDecorationContainerObject;

namespace UE
{
namespace Sequencer
{

class FDecorationOutlinerModel;
struct FViewModelChildren;

// Central storage for decoration outliner models, keyed by individual decoration objects.
// Registered on FSequenceModel's shared data, following FSectionModelStorageExtension pattern.
class FDecorationModelStorageExtension
	: public IDynamicExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, FDecorationModelStorageExtension);

	UE_API FDecorationModelStorageExtension();

	// Finds an existing model for the decoration, or creates a new one.
	UE_API TSharedPtr<FDecorationOutlinerModel> FindOrCreateModel(
		UObject* Decoration,
		UMovieSceneDecorationContainerObject* Container,
		UMovieSceneTrack* ParentTrack,
		bool& bOutNewlyCreated);

	// Finds an existing model for the decoration, returns nullptr if not found.
	UE_API TSharedPtr<FDecorationOutlinerModel> FindModel(
		const UObject* Decoration) const;

	// Ensures decoration models for the given container are present in OutlinerChildren.
	// Creates one FDecorationOutlinerModel per visible decoration on the container.
	// Handles creation, re-adoption from recycled, NeedsReconstruct checks, and stale
	// model removal. Also cleans up stale decoration models on child section outliners.
	UE_API void SyncDecorationModels(
		UMovieSceneDecorationContainerObject* Container,
		UMovieSceneTrack* ParentTrack,
		FViewModelChildren& OutlinerChildren,
		ISequencer* InSequencer);

	UE_API virtual void OnReinitialize() override;

private:

	TMap<FObjectKey, TWeakPtr<FDecorationOutlinerModel>> DecorationToModel;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
