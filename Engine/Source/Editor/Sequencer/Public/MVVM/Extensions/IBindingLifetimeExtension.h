// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "Math/Range.h"

#define UE_API SEQUENCER_API

struct FGuid;

namespace UE
{
	namespace Sequencer
	{

		class IBindingLifetimeExtension
		{
		public:

			UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IBindingLifetimeExtension)

			virtual ~IBindingLifetimeExtension() = default;

			virtual const TArray<FFrameNumberRange>& GetInverseLifetimeRange() const = 0;
		};

	} // namespace Sequencer
} // namespace UE

#undef UE_API
