// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "SequencerTimeSliderController.h"

namespace UE::Sequencer
{


/**
 * Extension class that can be added to the sequence model in order to define a custom clock implementation
 */
class IClockExtension
{
public:
	

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(SEQUENCER_API, IClockExtension)

	virtual ~IClockExtension()= default;


	virtual TSharedPtr<INumericTypeInterface<double>> MakePositionNumericTypeInterface() const
	{
		return nullptr;
	}
	virtual TSharedPtr<INumericTypeInterface<double>> MakeDurationNumericTypeInterface() const
	{
		return nullptr;
	}
	virtual double GetDesiredTimeSliderHeight(TSharedPtr<ISequencer> Sequencer) const
	{
		return 22.0;
	}
	virtual float GetCustomMajorTickSize(TSharedPtr<ISequencer> Sequencer) const
	{
		return 9.0;
	}
	virtual bool DrawTicks(TSharedPtr<ISequencer> Sequencer, FSlateWindowElementList& OutDrawElements, const TRange<double>& ViewRange, const FSequencerTimeSliderController::FScrubRangeToScreen& RangeToScreen, FSequencerTimeSliderController::FDrawTickArgs& InOutArgs) const
	{
		return false;
	}
	virtual bool ShouldShowPlayRateCombo(TSharedPtr<const ISequencer> Sequencer) const
	{
		return true;
	}
	virtual bool SupportsSnapping() const
	{
		return false;
	}
	virtual bool ShouldSnapFrameTime() const
	{
		return SupportsSnapping();
	}
	virtual FFrameTime SnapFrameTime(FFrameTime InFrameTime) const
	{
		return InFrameTime;
	}
};

} // namespace UE::Sequencer

