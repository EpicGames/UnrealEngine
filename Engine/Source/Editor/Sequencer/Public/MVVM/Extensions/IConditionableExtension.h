// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/Extensions/HierarchicalCacheExtension.h"

#define UE_API SEQUENCER_API

class UMovieSceneCondition;

namespace UE
{
	namespace Sequencer
	{
		enum class EConditionableConditionState
		{
			None,
			HasConditionEvaluatingFalse,
			HasConditionEvaluatingTrue,
			HasConditionEditorForceTrue,
		};

		/**
		 * An extension for models that can have conditions
		 */
		class IConditionableExtension
		{
		public:

			UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IConditionableExtension)

			virtual ~IConditionableExtension() = default;

			/* Returns the condition in question for this model if applicable. */
			virtual const UMovieSceneCondition* GetCondition() const = 0;

			/** Returns condition state for this item */
			virtual EConditionableConditionState GetConditionState() const = 0;

			/** Set this item's condition to evaluate true for editor preview purposes */
			virtual void SetConditionEditorForceTrue(bool bEditorForceTrue) = 0;
		};

		enum class ECachedConditionState
		{
			None = 0,

			HasCondition = 1 << 0,
			ConditionEvaluatingTrue = 1 << 1,
			EditorForceTrue = 1 << 2,
			ChildHasCondition = 1 << 3,
			SectionHasCondition = 1 << 4,
			ParentHasCondition = 1 << 5,
			ParentHasConditionEvaluatingTrue = 1 << 6,

			InheritedFromChildren = ChildHasCondition,
		};
		ENUM_CLASS_FLAGS(ECachedConditionState)

		class FConditionStateCacheExtension
			: public TFlagStateCacheExtension<ECachedConditionState>
		{
		public:

			UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, FConditionStateCacheExtension);

		private:

			UE_API ECachedConditionState ComputeFlagsForModel(const FViewModelPtr& ViewModel) override;
			UE_API void PostComputeChildrenFlags(const FViewModelPtr& ViewModel, ECachedConditionState& OutThisModelFlags, ECachedConditionState& OutPropagateToParentFlags) override;
		};

	} // namespace Sequencer
} // namespace UE

#undef UE_API
