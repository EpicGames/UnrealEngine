// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "CurveEditorTypes.h"
#include "Tree/ICurveEditorTreeItem.h"

#define UE_API SEQUENCER_API

struct FGuid;
class FCurveEditor;

namespace UE
{
namespace Sequencer
{

/**
 * Extension interface for view-models that want to show up in the curve editor.
 */
class ICurveEditorTreeItemExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ICurveEditorTreeItemExtension)

	virtual ~ICurveEditorTreeItemExtension()= default;

	/** Gets the curve editor item for this view-model */
	virtual TSharedPtr<ICurveEditorTreeItem> GetCurveEditorTreeItem() const = 0;

	/** Whether this view-model has any curves and needs to have a curve editor item created */
	virtual bool HasCurves() const = 0;

	/** Get optional unique path name */
	virtual TOptional<FString> GetUniquePathName() const = 0;

	/** Gets the curve editor ID for this view-model */
	virtual FCurveEditorTreeItemID GetCurveEditorItemID() const = 0;

	/** Called when this view-model has a curve editor item created for it */
	virtual void OnAddedToCurveEditor(FCurveEditorTreeItemID ItemID, TSharedPtr<FCurveEditor> CurveEditor) = 0;
	/** Called when this view-model's curve editor item is removed */
	virtual void OnRemovedFromCurveEditor(TSharedPtr<FCurveEditor> CurveEditor) = 0;
};

/**
 * Default partial implementation of ICurveEditorTreeItemExtension
 */
class FCurveEditorTreeItemExtensionShim : public ICurveEditorTreeItemExtension
{
public:

	FCurveEditorTreeItemID GetCurveEditorItemID() const override { return CurveEditorTreeItemID; }
	void OnAddedToCurveEditor(FCurveEditorTreeItemID ItemID, TSharedPtr<FCurveEditor> CurveEditor) { CurveEditorTreeItemID = ItemID; }
	void OnRemovedFromCurveEditor(TSharedPtr<FCurveEditor> CurveEditor) { CurveEditorTreeItemID = FCurveEditorTreeItemID::Invalid(); }

private:

	FCurveEditorTreeItemID CurveEditorTreeItemID;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
