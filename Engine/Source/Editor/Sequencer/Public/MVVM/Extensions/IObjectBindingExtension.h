// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCER_API

struct FGuid;

namespace UE
{
namespace Sequencer
{

class IObjectBindingExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IObjectBindingExtension)

	virtual ~IObjectBindingExtension()= default;

	virtual FGuid GetObjectGuid() const = 0;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
