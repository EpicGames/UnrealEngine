// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "Containers/Set.h"
#include "Containers/Array.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"

#define UE_API SEQUENCER_API

struct FGeometry;
class FSlateRect;
class FWidgetStyle;
class FSlateWindowElementList;
class UMovieSceneSection;

namespace UE::Sequencer
{

class FVirtualTrackArea;

/** Stores initial row index of a section before drag starts */
struct FInitialRowIndex
{
	UMovieSceneSection* Section = nullptr;
	int32 RowIndex = 0;
};

/** Context data for section vertical drag operations */
struct FSectionVerticalDragContext
{
	/** The primary section being dragged (the one under the mouse) */
	UMovieSceneSection* Section = nullptr;

	/** Mouse position in virtual track area coordinates */
	FVector2D VirtualMousePos = FVector2D::ZeroVector;

	/** Mouse position in local widget coordinates */
	FVector2D LocalMousePos = FVector2D::ZeroVector;

	/** All sections being dragged together (for multi-selection) */
	const TSet<UMovieSceneSection*>* DraggedSections = nullptr;

	/** Initial row indices of ALL sections before drag started (snapshot) */
	const TArray<FInitialRowIndex>* InitialSectionRowIndices = nullptr;

	/** Whether dragged sections started on different rows */
	bool bSectionsAreOnDifferentRows = false;

	/** Lowest row index among dragged sections */
	TOptional<int32> LowestDraggedRow;

	/** Highest row index among dragged sections */
	TOptional<int32> HighestDraggedRow;

	/** Previous mouse Y position */
	TOptional<float> PrevMousePosY;
};

/**
 * Extension interface for handling section vertical drag operations.
 * Tracks implement this to provide custom drag behavior.
 */
class ISectionDragOwnerExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ISectionDragOwnerExtension)

	virtual ~ISectionDragOwnerExtension() = default;

	/** Called when a vertical drag operation begins */
	virtual void OnBeginSectionVerticalDrag() {}

	/**
	 * Called each frame during drag to handle vertical movement.
	 * Return trues if anything changed
	 */
	virtual bool OnSectionVerticalDrag(const FSectionVerticalDragContext& Context) { return false; }

	/**
	 * Called when drag ends (mouse released).
	 */
	virtual void OnEndSectionVerticalDrag(const FSectionVerticalDragContext& Context) {}

	/**
	 * Called during paint to draw any drag preview indicators.
	 * Returns the updated LayerId after drawing
	 */
	virtual int32 OnPaintSectionDragPreview(
		const FGeometry& TrackAreaGeometry,
		const FVirtualTrackArea& VirtualTrackArea,
		const FSlateRect& MyCullingRect,
		FSlateWindowElementList& OutDrawElements,
		int32 LayerId,
		const FWidgetStyle& InWidgetStyle,
		bool bParentEnabled) const { return LayerId; }
};

} // namespace UE::Sequencer

#undef UE_API
