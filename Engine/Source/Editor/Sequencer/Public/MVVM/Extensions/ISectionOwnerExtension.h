// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

#define UE_API SEQUENCER_API

class UMovieSceneSection;

namespace UE::Sequencer
{

struct FViewModelChildren;

class ISectionOwnerExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ISectionOwnerExtension)

	virtual ~ISectionOwnerExtension()= default;

	virtual FViewModelChildren GetSectionModels() = 0;

	TArray<UMovieSceneSection*> GetSections() 
	{ 
		TArray<UMovieSceneSection*> Sections; 
		for (const TViewModelPtr<FSectionModel>& Item : GetSectionModels().IterateSubList<FSectionModel>())
		{
			if (UMovieSceneSection* Section = Item->GetSection())
			{
				Sections.Add(Section);
			}
		}

		return Sections;
	}
};

} // namespace UE::Sequencer

#undef UE_API
