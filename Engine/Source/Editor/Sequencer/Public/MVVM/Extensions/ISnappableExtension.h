// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerCoreFwd.h"
#include "MVVM/ViewModelTypeID.h"
#include "Curves/KeyHandle.h"
#include "Templates/MemoryOps.h"

#define UE_API SEQUENCER_API

struct FGuid;

class IKeyArea;
class UMovieSceneSection;

namespace UE
{
namespace Sequencer
{

/** Structure defining a point to snap to in the sequencer */
struct FSnapPoint
{
	enum ESnapType { Key, SectionBounds, CustomSection, PlaybackRange, CurrentTime, InOutRange, Mark };

	FSnapPoint(ESnapType InType, FFrameNumber InTime, float InWeighting = -1.f)
		: Time(InTime)
		, Weighting(InWeighting)
		, Type(InType)
	{
		if (InWeighting == -1.f)
		{
			if (Type != UE::Sequencer::FSnapPoint::Key)
			{
				Weighting = 10.f;
			}
			else
			{
				Weighting = 1.f;
			}
		}
	}

	/** The time of the snap */
	FFrameNumber Time;

	float Weighting = 1.f;

	/** The type of snap */
	ESnapType Type;

private:

	// uninitialized default constructor - only for use by containers
	FSnapPoint()
	{}
};

class ISnapField
{
public:
	virtual ~ISnapField() = default;

	virtual void AddSnapPoint(const FSnapPoint& SnapPoint) = 0;
};

/** Interface that defines how to construct an FSequencerSnapField */
struct ISnapCandidate
{
	enum class EDefaultApplicability
	{
		None                   = 0,

		Keys                   = 1 << 0,
		SectionBounds          = 1 << 1,
		CustomSnaps            = 1 << 2,

		All = Keys | SectionBounds | CustomSnaps,
	};

	enum class ESnapCapabilities
	{
		None                            = 0,

		IsKeyApplicable                 = 1 << 0,
		AreSectionBoundsApplicable      = 1 << 1,
		AreSectionCustomSnapsApplicable = 1 << 2,

		All = IsKeyApplicable | AreSectionBoundsApplicable | AreSectionCustomSnapsApplicable,
	};

	virtual ~ISnapCandidate() = default;

	/** Retrieve the capabilities of this snap candidate instance as a bitmask. Zero bits will prevent virtual functions being called as an optimziation*/
	virtual void GetCapabilities(EDefaultApplicability& OutDefaultApplicability, ESnapCapabilities& OutCapabilities) const { OutDefaultApplicability = EDefaultApplicability::All; OutCapabilities = ESnapCapabilities::None; }

	/** Return true to include the specified key in the snap field */
	virtual bool IsKeyApplicable(FKeyHandle KeyHandle, const UE::Sequencer::FViewModelPtr& Owner) const { return true; }

	/** Return true to include the specified section's bounds in the snap field */
	virtual bool AreSectionBoundsApplicable(UMovieSceneSection* Section) const { return true; }

	/** Return true to include the specified section's custom snap points in the snap field */
	virtual bool AreSectionCustomSnapsApplicable(UMovieSceneSection* Section) const { return true; }
};

class ISnappableExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ISnappableExtension)

	virtual ~ISnappableExtension()= default;

	virtual void AddToSnapField(const ISnapCandidate& Candidate, ISnapField& SnapField) const = 0;
};

ENUM_CLASS_FLAGS(ISnapCandidate::ESnapCapabilities);

} // namespace Sequencer
} // namespace UE

#undef UE_API
