// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "SequencerTimeDomainOverride.h"

namespace UE::Sequencer
{


/**
 * Extension class that can be added to sections and tracks in order to define which time domain their data is defined in
 */
class ITimeDomainExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(SEQUENCER_API, ITimeDomainExtension)

	virtual ~ITimeDomainExtension()= default;

	virtual ETimeDomain GetDomain() const = 0;
};

} // namespace UE::Sequencer

