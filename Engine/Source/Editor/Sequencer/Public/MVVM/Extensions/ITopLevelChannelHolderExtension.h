// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "MVVM/ViewModels/ViewModel.h"

#define UE_API SEQUENCER_API

namespace UE::Sequencer
{

struct FViewModelChildren;

class ITopLevelChannelHolderExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ITopLevelChannelHolderExtension)

	virtual ~ITopLevelChannelHolderExtension()= default;

	/** Get the top-level channels for this item (used for key area iteration) */
	virtual FViewModelChildren GetTopLevelChannels() = 0;
};

} // namespace UE::Sequencer

#undef UE_API
