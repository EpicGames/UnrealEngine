// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "MVVM/ViewModels/ViewModel.h"

#define UE_API SEQUENCER_API

class UMovieSceneTrack;
class ISequencerTrackEditor;

namespace UE::Sequencer
{

class ITrackExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ITrackExtension)

	virtual ~ITrackExtension()= default;

	virtual UMovieSceneTrack* GetTrack() const = 0;

	virtual TSharedPtr<ISequencerTrackEditor> GetTrackEditor() const = 0;
};

} // namespace UE::Sequencer

#undef UE_API
