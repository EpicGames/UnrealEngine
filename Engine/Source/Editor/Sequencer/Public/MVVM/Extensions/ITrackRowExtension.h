// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "MVVM/ViewModels/ViewModel.h"

#define UE_API SEQUENCER_API

class UMovieSceneTrack;

namespace UE::Sequencer
{

class ITrackRowExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, ITrackRowExtension)

	virtual ~ITrackRowExtension()= default;

	virtual int32 GetRowIndex() const = 0;
	virtual bool SetRowIndex(int32 InRowIndex) = 0;

	virtual UMovieSceneTrack* GetParentTrack() const = 0;
};

} // namespace UE::Sequencer

#undef UE_API
