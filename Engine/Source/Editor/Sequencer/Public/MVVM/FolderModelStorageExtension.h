// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectKey.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/Extensions/DynamicExtensionContainer.h"
#include "EventHandlers/ISequenceDataEventHandler.h"

#define UE_API SEQUENCER_API

class UMovieSceneFolder;

namespace UE
{
namespace Sequencer
{

class FFolderModel;
class FSequenceModel;

class UE_API FFolderModelStorageExtension
	: public IDynamicExtension
	, private UE::MovieScene::TIntrusiveEventHandler<UE::MovieScene::ISequenceDataEventHandler>
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID(FFolderModelStorageExtension)

	FFolderModelStorageExtension();

	virtual void OnCreated(TSharedRef<FViewModel> InWeakOwner) override;
	virtual void OnReinitialize() override;

	TSharedPtr<FFolderModel> CreateModelForFolder(UMovieSceneFolder* InFolder, TSharedPtr<FViewModel> DesiredParent = nullptr);

	TSharedPtr<FFolderModel> FindModelForFolder(UMovieSceneFolder* InFolder) const;

private:

	void OnRootFolderAdded(UMovieSceneFolder*) override;
	void OnRootFolderRemoved(UMovieSceneFolder*) override;

private:

	TMap<FObjectKey, TWeakPtr<FFolderModel>> FolderToModel;

	FSequenceModel* OwnerModel;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
