// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "EventHandlers/ISequenceDataEventHandler.h"
#include "EventHandlers/MovieSceneDataEventContainer.h"
#include "MVVM/Extensions/DynamicExtensionContainer.h"
#include "MVVM/ViewModelTypeID.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"

#define UE_API SEQUENCER_API

class UMovieSceneTrack;
namespace UE::MovieScene { class ISequenceDataEventHandler; }
struct FMovieSceneBinding;

namespace UE
{
namespace Sequencer
{

class FObjectBindingModel;
class FPlaceholderObjectBindingModel;
class FSequenceModel;
class FViewModel;
struct FViewModelChildren;

class FObjectBindingModelStorageExtension
	: public IDynamicExtension
	, private UE::MovieScene::TIntrusiveEventHandler<UE::MovieScene::ISequenceDataEventHandler>
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, FObjectBindingModelStorageExtension);

	UE_API FObjectBindingModelStorageExtension();

	/**
	 * Implementation function for creating a new model for a binding from its ID. May return a placeholder if the binding does not yet exist.
	 * @param Binding The binding to create a model for
	 */
	UE_API TSharedPtr<FViewModel> GetOrCreateModelForBinding(const FGuid& Binding);

	/**
	 * Implementation function for creating a new model for a binding
	 * @param Binding The binding to create a model for
	 */
	UE_API TSharedPtr<FViewModel> GetOrCreateModelForBinding(const FMovieSceneBinding& Binding);

	UE_API TSharedPtr<FObjectBindingModel> FindModelForObjectBinding(const FGuid& InObjectBindingID) const;

	UE_API virtual void OnCreated(TSharedRef<FViewModel> InWeakOwner) override;
	UE_API virtual void OnReinitialize() override;

private:

	UE_API void OnBindingAdded(const FMovieSceneBinding& Binding) override;
	UE_API void OnBindingRemoved(const FGuid& ObjectBindingID) override;
	UE_API void OnTrackAddedToBinding(UMovieSceneTrack* Track, const FGuid& Binding) override;
	UE_API void OnTrackRemovedFromBinding(UMovieSceneTrack* Track, const FGuid& Binding) override;
	UE_API void OnBindingParentChanged(const FGuid& Binding, const FGuid& NewParent) override;


	UE_API TSharedPtr<FObjectBindingModel> CreateModelForObjectBinding(const FMovieSceneBinding& Binding);

	UE_API TSharedPtr<FViewModel> CreatePlaceholderForObjectBinding(const FGuid& ObjectID);
	UE_API TSharedPtr<FViewModel> FindPlaceholderForObjectBinding(const FGuid& InObjectBindingID) const;

	UE_API void Compact();

private:

	TMap<FGuid, TWeakPtr<FObjectBindingModel>> ObjectBindingToModel;
	TMap<FGuid, TWeakPtr<FPlaceholderObjectBindingModel>> ObjectBindingToPlaceholder;

	FSequenceModel* OwnerModel;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
