// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "MVVM/Extensions/DynamicExtensionContainer.h"
#include "MVVM/ViewModelTypeID.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectKey.h"

#define UE_API SEQUENCER_API

class ISequencerSection;
class UMovieSceneSection;

namespace UE
{
namespace Sequencer
{

class FSectionModel;
class FSequenceModel;

class FSectionModelStorageExtension
	: public IDynamicExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, FSectionModelStorageExtension);

	UE_API FSectionModelStorageExtension();

	UE_API TSharedPtr<FSectionModel> CreateModelForSection(UMovieSceneSection* InSection, TSharedRef<ISequencerSection> SectionInterface);

	UE_API TSharedPtr<FSectionModel> FindModelForSection(const UMovieSceneSection* InSection) const;

	UE_API virtual void OnCreated(TSharedRef<FViewModel> InOwner) override;
	UE_API virtual void OnReinitialize() override;

private:

	void NotifyDecorationEditors(UMovieSceneSection* InSection);

	TWeakPtr<FViewModel> WeakOwner;
	TMap<FObjectKey, TWeakPtr<FSectionModel>> SectionToModel;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
