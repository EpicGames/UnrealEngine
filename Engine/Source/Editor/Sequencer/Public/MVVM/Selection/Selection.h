// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Selection/SequencerCoreSelectionTypes.h"
#include "MVVM/Selection/SequencerCoreSelection.h"
#include "MVVM/Selection/SequencerOutlinerSelection.h"
#include "MVVM/ViewModels/ChannelModel.h"


struct FKeyHandle;
class UMovieSceneTrack;
class UMovieSceneSection;

namespace UE::Sequencer
{

class FChannelModel;
class FEditorViewModel;
struct FIndirectOutlinerSelectionIterator;


/**
 * Main key selection class that stores selected key handles, and the channel they originated from
 */
struct FKeySelection : TUniqueFragmentSelectionSet<FKeyHandle, FChannelModel>
{
	/**
	 * Overridden Select function for passing by TViewModelPtr instead of TWeakViewModelPtr
	 */
	void Select(TViewModelPtr<FChannelModel> InOwner, FKeyHandle InKey, bool* OutNewlySelected = nullptr)
	{
		TUniqueFragmentSelectionSet<FKeyHandle, FChannelModel>::Select(InOwner, InKey, OutNewlySelected);
	}

	/**
	 * Overridden SelectRange function for passing by TViewModelPtr instead of TWeakViewModelPtr
	 */
	template<typename RangeType>
	void SelectRange(TViewModelPtr<FChannelModel> InOwner, RangeType Range, bool* OutAnySelected = nullptr)
	{
		TUniqueFragmentSelectionSet<FKeyHandle, FChannelModel>::SelectRange(InOwner, Forward<RangeType>(Range), OutAnySelected);
	}

	/**
	 * Overridden Deselect function to remove duplicate keys on deselection
	 */
	void Deselect(FKeyHandle InKey);

	/**
	 * Overridden Empty function to remove duplicate keys on deselection
	 */
	void Empty();
};


/**
 * Main track-area selection class storing the selection of sections, layer-bars and other trac-area elements
 */
struct FTrackAreaSelection : TSelectionSetBase<FTrackAreaSelection, FWeakViewModelPtr>
{
	/**
	 * Filter this selection set based on the specified filter type
	 */
	template<typename FilterType>
	TFilteredViewModelSelectionIterator<FWeakViewModelPtr, FilterType> Filter() const
	{
		return TFilteredViewModelSelectionIterator<FWeakViewModelPtr, FilterType>{ &GetSelected() };
	}

	SEQUENCER_API bool OnSelectItem(const FWeakViewModelPtr& WeakViewModel);
};


/**
 * Main track-area marked frame selection class
 */
struct FMarkedFrameSelection : TSelectionSetBase<FMarkedFrameSelection, int32>
{
};



/**
 * Selection class for an ISequence instance that manages all the different types of selection and event marshaling
 */
class FSequencerSelection : public FSequencerCoreSelection
{
public:

	/** Selection set representing view-models that are selected on the outliner */
	FOutlinerSelection Outliner;

	/** Selection set representing view-models that are selected on the track-area */
	FTrackAreaSelection TrackArea;

	/** Selection set representing selected keys */
	FKeySelection KeySelection;

	/** Selection set representing selected marked frames */
	FMarkedFrameSelection MarkedFrames;

public:

	/**
	 * Default constructor that initializes the selection sets
	 */
	SEQUENCER_API FSequencerSelection();

	/**
	 * Initialize this selection container with the specified view model. Must only be called once.
	 */
	SEQUENCER_API void Initialize(TViewModelPtr<FEditorViewModel> InViewModel);

	/**
	 * Empty every selection set represented by this class
	 */
	SEQUENCER_API void Empty();

	/**
	 * Iterate all the outliner nodes that are considered 'indirectly' selected because they have
	 * track area selection
	 */
	SEQUENCER_API FIndirectOutlinerSelectionIterator IterateIndirectOutlinerSelection() const;

public:

	/*~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	BEGIN BACKWARDS COMPAT */

	/** DEPRECATED: Gets a set of the outliner nodes that have selected keys or sections */
	const TSet<TWeakViewModelPtr<IOutlinerExtension>>& GetNodesWithSelectedKeysOrSections() const
	{
		return NodesWithKeysOrSections;
	}

	/** DEPRECATED */
	bool NodeHasSelectedKeysOrSections(TWeakViewModelPtr<IOutlinerExtension> Model) const
	{
		return NodesWithKeysOrSections.Contains(Model);
	}

	/** DEPRECATED */
	void GetSelectedOutlinerItems(TArray<TSharedRef<FViewModel>>& OutItems) const;

	/** DEPRECATED */
	void GetSelectedOutlinerItems(TArray<TSharedPtr<FViewModel>>& OutItems) const;

	/** DEPRECATED: Gets the outliner nodes that have selected keys or sections */
	TSet<FGuid> GetBoundObjectsGuids();

	/** DEPRECATED */
	void EmptySelectedOutlinerNodesWithoutSections(const TArray<UMovieSceneSection*>& Sections);

	/** DEPRECATED */
	TSet<UMovieSceneSection*> GetSelectedSections() const;

	/** DEPRECATED */
	TSet<UMovieSceneTrack*> GetSelectedTracks() const;

	/** DEPRECATED */
	TSet<TPair<UMovieSceneTrack*, int32>> GetSelectedTrackRows() const;

	/*~ END BACKWARDS COMPAT
	~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~*/

private:

	void OnHierarchyChanged();
	void RevalidateSelection();

	virtual TSharedPtr<      FOutlinerSelection> GetOutlinerSelection()       override { return TSharedPtr<      FOutlinerSelection>(AsShared(), &Outliner); }
	virtual TSharedPtr<const FOutlinerSelection> GetOutlinerSelection() const override { return TSharedPtr<const FOutlinerSelection>(AsShared(), &Outliner); }

	SEQUENCER_API virtual void PreSelectionSetChangeEvent(FSelectionBase* InSelectionSet) override;
	SEQUENCER_API virtual void PreBroadcastChangeEvent() override;
	
	TSet<TWeakViewModelPtr<IOutlinerExtension>> NodesWithKeysOrSections;
};

/** Iterator class for iterating the outliner nodes that have keys or sections selected on them,
 * including the filtering mechanisms that exist on the actual selection sets */
struct FIndirectOutlinerSelectionIterator
{
	const TSet<TWeakViewModelPtr<IOutlinerExtension>>* SelectionSet;

	/**
	 * Turn this iterator into a filtered iterator that only visits the specified viewmodel or extension types
	 */
	template<typename FilterType>
	TFilteredViewModelSelectionIterator<TWeakViewModelPtr<IOutlinerExtension>, FilterType> Filter() const
	{
		return TFilteredViewModelSelectionIterator<TWeakViewModelPtr<IOutlinerExtension>, FilterType>{ SelectionSet };
	}

	/**
	 * Gather all the view models contained in this iterator into the specified container
	 */
	template<typename FilterType, typename ContainerType>
	void Gather(ContainerType& OutContainer) const
	{
		OutContainer.Reserve(SelectionSet->Num());
		for (TViewModelPtr<FilterType> Model : Filter<FilterType>())
		{
			OutContainer.Add(Model);
		}
	}

	FORCEINLINE TSelectionSetIteratorState<TWeakViewModelPtr<IOutlinerExtension>> begin() const { return TSelectionSetIteratorState<TWeakViewModelPtr<IOutlinerExtension>>(SelectionSet->begin(), SelectionSet->end()); }
	FORCEINLINE TSelectionSetIteratorState<TWeakViewModelPtr<IOutlinerExtension>> end() const   { return TSelectionSetIteratorState<TWeakViewModelPtr<IOutlinerExtension>>(SelectionSet->end(), SelectionSet->end()); }
};

} // namespace UE::Sequencer