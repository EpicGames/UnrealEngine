// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectKey.h"
#include "MVVM/Extensions/DynamicExtensionContainer.h"
#include "MVVM/ViewModelPtr.h"
#include "EventHandlers/ISequenceDataEventHandler.h"
#include "ISequencerModule.h"

class UMovieSceneTrack;

namespace UE
{
namespace Sequencer
{

class FTrackModel;
class FSequenceModel;
class FViewModel;
class ITrackExtension;

class FTrackModelStorageExtension
	: public IDynamicExtension
	, private UE::MovieScene::TIntrusiveEventHandler<UE::MovieScene::ISequenceDataEventHandler>
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(SEQUENCER_API, FTrackModelStorageExtension)

	FTrackModelStorageExtension(const TArray<FOnCreateTrackModel>& InTrackModelCreators);

	virtual void OnCreated(TSharedRef<FViewModel> InWeakOwner) override;
	virtual void OnReinitialize() override;

	TSharedPtr<FTrackModel> CreateModelForTrack(UMovieSceneTrack* InTrack, TSharedPtr<FViewModel> DesiredParent = nullptr);

	/** Find the model for a track - returns FTrackModel for backwards compatibility */
	SEQUENCER_API TSharedPtr<FTrackModel> FindModelForTrack(UMovieSceneTrack* InTrack) const;

	/** Find the registered track extension for a track */
	SEQUENCER_API TViewModelPtr<ITrackExtension> FindTrackExtensionForTrack(UMovieSceneTrack* InTrack) const;

	/** Register a track extension for a track */
	SEQUENCER_API void RegisterTrackExtensionForTrack(UMovieSceneTrack* InTrack, TViewModelPtr<ITrackExtension> InModel);

private:

	void OnTrackAdded(UMovieSceneTrack*) override;
	void OnTrackRemoved(UMovieSceneTrack*) override;

private:

	TArray<FOnCreateTrackModel> TrackModelCreators;

	TMap<FObjectKey, TWeakViewModelPtr<ITrackExtension>> TrackToModel;
	FSequenceModel* OwnerModel;
};

} // namespace Sequencer
} // namespace UE

