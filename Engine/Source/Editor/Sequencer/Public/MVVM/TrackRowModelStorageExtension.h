// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "MVVM/Extensions/DynamicExtensionContainer.h"
#include "MVVM/ViewModelTypeID.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "UObject/ObjectKey.h"

#define UE_API SEQUENCER_API

class UMovieSceneTrack;

namespace UE
{
namespace Sequencer
{

class FSequenceModel;
class FTrackRowModel;
class FViewModel;

class FTrackRowModelStorageExtension
	: public IDynamicExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, FTrackRowModelStorageExtension)

	UE_API FTrackRowModelStorageExtension();

	UE_API virtual void OnCreated(TSharedRef<FViewModel> InWeakOwner) override;
	UE_API virtual void OnReinitialize() override;

	UE_API TSharedPtr<FTrackRowModel> CreateModelForTrackRow(UMovieSceneTrack* InTrack, int32 InRowIndex, TSharedPtr<FViewModel> DesiredParent = nullptr);

	UE_API TSharedPtr<FTrackRowModel> FindModelForTrackRow(UMovieSceneTrack* InTrack, int32 InRowIndex) const;

private:

	using KeyType = TTuple<FObjectKey, int32>;

	TMap<KeyType, TWeakPtr<FTrackRowModel>> TrackToModel;

	FSequenceModel* OwnerModel;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
