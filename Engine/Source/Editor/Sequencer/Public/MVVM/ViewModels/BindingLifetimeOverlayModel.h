// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Extensions/ITrackLaneExtension.h"
#include "MVVM/Extensions/IBindingLifetimeExtension.h"
#include "MVVM/Extensions/LinkedOutlinerExtension.h"
#include "MVVM/Extensions/ViewModelExtensionCollection.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "Math/Range.h"
#include "Templates/SharedPointer.h"
#include "SequencerEditorViewModel.h"

#define UE_API SEQUENCER_API

namespace UE::Sequencer { template <typename T> struct TAutoRegisterViewModelTypeID; }
struct FFrameNumber;

namespace UE
{
	namespace Sequencer
	{
		class FBindingLifetimeOverlayModel
			: public FViewModel
			, public FLinkedOutlinerExtension
			, public ITrackLaneExtension
		{
		public:

			UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FBindingLifetimeOverlayModel, FViewModel
				, FLinkedOutlinerExtension
				, ITrackLaneExtension
			);

			UE_API FBindingLifetimeOverlayModel(TWeakPtr<FViewModel> LayerRoot, TWeakPtr<FSequencerEditorViewModel> InEditorViewModel, TViewModelPtr<IBindingLifetimeExtension> InBindingLifetimeTrack);
			UE_API ~FBindingLifetimeOverlayModel();

			/*~ ITrackLaneExtension Interface */
			UE_API TSharedPtr<ITrackLaneWidget> CreateTrackLaneView(const FCreateTrackLaneViewParams& InParams) override;
			UE_API FTrackLaneVirtualAlignment ArrangeVirtualTrackLaneView() const override;

			UE_API const TArray<FFrameNumberRange>& GetInverseLifetimeRange() const;

		private:

			TWeakPtr<FSequencerEditorViewModel> WeakEditorViewModel;
			TViewModelPtr<IBindingLifetimeExtension> BindingLifetimeTrack;
		};

	} // namespace Sequencer
} // namespace UE

#undef UE_API
