// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelEditorData.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Fonts/SlateFontInfo.h"
#include "Internationalization/Text.h"
#include "MVVM/Extensions/IDeletableExtension.h"
#include "MVVM/Extensions/IGeometryExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/IRecyclableExtension.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/Extensions/ITrackLaneExtension.h"
#include "MVVM/Extensions/LinkedOutlinerExtension.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/OutlinerItemModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API SEQUENCER_API

class FSequencerSectionCategoryNode;
class SWidget;
class UMovieSceneSection;
namespace UE::Sequencer { template <typename T> struct TAutoRegisterViewModelTypeID; }

namespace UE
{
namespace Sequencer
{

/**
 * Model for a single channel category inside a section.
 * For instance, this represents the "Location" category of a single transform section, which would
 * contain the X, Y, and Z translation channels.
 */
class FCategoryModel
	: public FViewModel
	, public FLinkedOutlinerExtension
	, public FGeometryExtensionShim
	, public ITrackLaneExtension
	, public FLinkedOutlinerComputedSizingShim
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FCategoryModel, FViewModel, FLinkedOutlinerExtension, IGeometryExtension, ITrackLaneExtension);

	UE_API explicit FCategoryModel(FName InCategoryName);

	/** Whether any of the channels within this category are animated.  */
	UE_API bool IsAnimated() const;

	/** Returns the category's name */
	FName GetCategoryName() const { return CategoryName; }

	/** Returns the desired sizing for the track area row */
	UE_API virtual FOutlinerSizing GetDesiredSizing() const;

	/*~ ITrackLaneExtension */
	UE_API TSharedPtr<ITrackLaneWidget> CreateTrackLaneView(const FCreateTrackLaneViewParams& InParams) override;
	UE_API FTrackLaneVirtualAlignment ArrangeVirtualTrackLaneView() const override;

private:

	UE_API FLinearColor GetKeyBarColor() const;

private:

	FViewModelListHead Children;
	FName CategoryName;
};

/**
 * Model for the outliner entry associated with all sections' channel categories of a given common name.
 * For instance, this represents the "Location" category entry in the Sequence outliner, which would contain
 * the X, Y, and Z translation channels of all the corresponding sections in the track area.
 */
class FCategoryGroupModel
	: public FOutlinerItemModel
	, public ITrackAreaExtension
	, public ICompoundOutlinerExtension
	, public IDeletableExtension
	, public IRecyclableExtension
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FCategoryGroupModel, FOutlinerItemModel, ITrackAreaExtension, ICompoundOutlinerExtension, IDeletableExtension, IRecyclableExtension);

	UE_API explicit FCategoryGroupModel(FName InCategoryName, const FText& InDisplayText, FGetMovieSceneTooltipText InGetGroupTooltipTextDelegate);

	UE_API ~FCategoryGroupModel();

	/**
	 * @return Whether any of the channels within this category group have any keyframes on them
	 */
	UE_API bool IsAnimated() const;

	FName GetCategoryName() const
	{
		return CategoryName;
	}

	FText GetDisplayText() const
	{
		return DisplayText;
	}

	UE_API void AddCategory(TWeakViewModelPtr<FCategoryModel> InCategory);

	UE_API TArrayView<const TWeakViewModelPtr<FCategoryModel>> GetCategories() const;

	/** Creates a new FCategoryModel for the track area and links it to this category group.
	 *  The caller is responsible for parenting the returned model in the appropriate
	 *  track-area child list (e.g. SectionModel's TopLevelChannel list). */
	UE_API TSharedPtr<FCategoryModel> CreateTrackAreaCategory();

public:

	/*~ ICompoundOutlinerExtension */
	UE_API FOutlinerSizing RecomputeSizing() override;

	/*~ IOutlinerExtension */
	UE_API FOutlinerSizing GetOutlinerSizing() const override;
	UE_API FText GetLabel() const override;
	UE_API FSlateFontInfo GetLabelFont() const override;
	UE_API FText GetLabelToolTipText() const override;
	UE_API TSharedPtr<SWidget> CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName) override;

	/*~ ITrackAreaExtension */
	UE_API FTrackAreaParameters GetTrackAreaParameters() const override;
	UE_API FViewModelVariantIterator GetTrackAreaModelList() const override;

	/*~ IDeletableExtension */
	UE_API bool CanDelete(FText* OutErrorMessage) const override;
	UE_API void Delete() override;

	/*~ IRecyclableExtension */
	UE_API void OnRecycle() override;

	/*~ FOutlinerItemModelMixin */
	UE_API virtual void BuildSidebarMenu(FMenuBuilder& MenuBuilder) override;

private:

	TArray<TWeakViewModelPtr<FCategoryModel>> Categories;
	FName CategoryName;
	FText DisplayText;
	FGetMovieSceneTooltipText GetGroupTooltipTextDelegate;
	FOutlinerSizing ComputedSizing;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
