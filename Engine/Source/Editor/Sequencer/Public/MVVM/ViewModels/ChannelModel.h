// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneChannelOverrideContainer.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "EventHandlers/ISignedObjectEventHandler.h"
#include "Fonts/SlateFontInfo.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "MVVM/Extensions/IDeletableExtension.h"
#include "MVVM/Extensions/IGeometryExtension.h"
#include "MVVM/Extensions/IKeyExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/IRecyclableExtension.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/Extensions/ITrackLaneExtension.h"
#include "MVVM/Extensions/LinkedOutlinerExtension.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/OutlinerItemModel.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Tree/ICurveEditorTreeItem.h"
#include "UObject/NameTypes.h"

#define UE_API SEQUENCER_API

class FCurveModel;
class FSequencerSectionKeyAreaNode;
class IKeyArea;
class ISequencerSection;
class SWidget;
class UMovieSceneSection;
namespace UE::Sequencer { template <typename T> struct TAutoRegisterViewModelTypeID; }
struct FMovieSceneChannel;
struct FMovieSceneChannelHandle;
struct FSequencerChannelPaintArgs;

namespace UE
{
namespace Sequencer
{

class FSectionModel;
class FSequenceModel;
class FChannelGroupOutlinerModel;

/**
 * Model for a single channel inside a section.
 * For instance, this represents the "Location.X" channel of a single transform section.
 */
class FChannelModel
	: public FViewModel
	, public FLinkedOutlinerExtension
	, public FGeometryExtensionShim
	, public FLinkedOutlinerComputedSizingShim
	, public ITrackLaneExtension
	, public IRecyclableExtension
	, public IKeyExtension
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FChannelModel, FViewModel, FLinkedOutlinerExtension, IGeometryExtension, ITrackLaneExtension, IRecyclableExtension, IKeyExtension);

	UE_API FChannelModel(FName InChannelName, TWeakPtr<ISequencerSection> InSection, FMovieSceneChannelHandle InChannel);
	UE_API ~FChannelModel();

	UE_API void Initialize(TWeakPtr<ISequencerSection> InSection, FMovieSceneChannelHandle InChannel);

	/** Returns the section object that owns the associated channel */
	UE_API UMovieSceneSection* GetSection() const;

	/** Returns the object that owns the associated channel. May return the same as GetSection(). */
	UE_API UObject* GetOwningObject() const;

	/** Returns the associated channel object */
	UE_API FMovieSceneChannel* GetChannel() const;

	/** Returns whether this channel has any keyframes on it */
	UE_API bool IsAnimated() const;

	/** Returns the channel's name */
	FName GetChannelName() const { return ChannelName; }

	/** Returns the key area for the channel */
	TSharedPtr<IKeyArea> GetKeyArea() const { return KeyArea; }

	/** Returns the desired sizing for the track area row */
	UE_API FOutlinerSizing GetDesiredSizing() const;

	UE_API int32 CustomPaint(const FSequencerChannelPaintArgs& CustomPaintArgs, int32 LayerId) const;

	/*~ ITrackLaneExtension */
	UE_API TSharedPtr<ITrackLaneWidget> CreateTrackLaneView(const FCreateTrackLaneViewParams& InParams) override;
	UE_API FTrackLaneVirtualAlignment ArrangeVirtualTrackLaneView() const override;

	/*~ IRecyclableExtension */
	UE_API void OnRecycle() override;

	/*~ IKeyExtension */
	UE_API bool UpdateCachedKeys(TSharedPtr<FCachedKeys>& OutCachedKeys) const override;
	UE_API bool GetFixedExtents(double& OutFixedMin, double& OutFixedMax) const override;
	UE_API void DrawKeys(TArrayView<const FKeyHandle> InKeyHandles, TArrayView<FKeyDrawParams> OutKeyDrawParams) override;
	UE_API TUniquePtr<FCurveModel> CreateCurveModel() override;

	/*~ Begin virtual interface */
	virtual void BuildContextMenu(FMenuBuilder& MenuBuilder, TViewModelPtr<FChannelGroupOutlinerModel> GroupOwner) {}
	virtual TSharedPtr<SWidget> CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName) { return nullptr; }
	/*~ End virtual interface */
private:

	UE_API FLinearColor GetKeyBarColor() const;

private:
	
	TSharedPtr<IKeyArea> KeyArea;
	const FName ChannelName;
};

/**
 * Model for the outliner entry associated with all sections' channels of a given common name.
 * For instance, this represents the "Location.X" entry in the Sequencer outliner.
 */
class FChannelGroupModel
	: public FViewModel
	, public ITrackAreaExtension
	, public IRecyclableExtension
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FChannelGroupModel, FViewModel, ITrackAreaExtension, IRecyclableExtension);

	UE_API FChannelGroupModel(FName InChannelName, const FText& InDisplayText);
	UE_API FChannelGroupModel(FName InChannelName, const FText& InDisplayText, const FText& InTooltipText);
	UE_API FChannelGroupModel(FName InChannelName, const FText& InDisplayText, FGetMovieSceneTooltipText InGetTooltipTextDelegate);
	UE_API ~FChannelGroupModel();

	/** Returns whether any of the channels within this group have any keyframes on them */
	UE_API bool IsAnimated() const;

	/** Returns the common name for all channels in this group */
	FName GetChannelName() const { return ChannelName; }

	/** Returns the label for this group */
	FText GetDisplayText() const { return DisplayText; }

	/** Returns the tooltip for this group */
	UE_API FText GetTooltipText() const;

	/** Gets all the channel models in this group */
	UE_API TArrayView<const TWeakViewModelPtr<FChannelModel>> GetChannels() const;
	/** Adds a channel model to this group */
	UE_API void AddChannel(TWeakViewModelPtr<FChannelModel> InChannel);

	/** Get the key area of the channel associated with the given section */
	UE_API TSharedPtr<IKeyArea> GetKeyArea(TSharedPtr<FSectionModel> InOwnerSection) const;

	/** Get the key area of the channel associated with the given section */
	UE_API TSharedPtr<IKeyArea> GetKeyArea(const UMovieSceneSection* InOwnerSection) const;

	/** Get the channel model at the given index in the list of channels */
	UE_API TSharedPtr<FChannelModel> GetChannel(int32 Index) const;

	/** Get the channel model of the channel associated with the given section */
	UE_API TSharedPtr<FChannelModel> GetChannel(TSharedPtr<FSectionModel> InOwnerSection) const;

	/** Get the channel model of the channel associated with the given section */
	UE_API TSharedPtr<FChannelModel> GetChannel(const UMovieSceneSection* InOwnerSection) const;

	/** Get the key areas of all channels */
	UE_API TArray<TSharedRef<IKeyArea>> GetAllKeyAreas() const;

	/** Gets a serial number representing if the list of channels has changed */
	UE_API uint32 GetChannelsSerialNumber() const;

	/** Notifies this view-model that the list of channels has changed */
	UE_API void OnChannelOverridesChanged();

public:

	/*~ ITrackAreaExtension */
	UE_API FTrackAreaParameters GetTrackAreaParameters() const override;
	UE_API FViewModelVariantIterator GetTrackAreaModelList() const override;

	/*~ IRecyclableExtension */
	UE_API void OnRecycle() override;

	UE_API void CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels);
	UE_API bool HasCurves() const;
	UE_API TOptional<FString> GetUniquePathName() const;

	UE_API void BuildChannelOverrideMenu(FMenuBuilder& MenuBuilder);

	UE_API void CleanupChannels();

protected:

	UE_API void UpdateMutability();

protected:

	TArray<TWeakViewModelPtr<FChannelModel>> Channels;
	uint32 ChannelsSerialNumber;
	FName ChannelName;
	FText DisplayText;
	FGetMovieSceneTooltipText GetTooltipTextDelegate;
};


/**
 * Model for the outliner entry associated with all sections' channels of a given common name.
 * For instance, this represents the "Location.X" entry in the Sequencer outliner.
 */
class FChannelGroupOutlinerModel
	: public TOutlinerModelMixin<FChannelGroupModel>
	, public ICompoundOutlinerExtension
	, public IDeletableExtension
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FChannelGroupOutlinerModel, FChannelGroupModel, FOutlinerItemModelMixin, ICompoundOutlinerExtension, IDeletableExtension);

	UE_API FChannelGroupOutlinerModel(FName InChannelName, const FText& InDisplayText, FGetMovieSceneTooltipText InGetTooltipTextDelegate);
	UE_API ~FChannelGroupOutlinerModel();

public:

	/*~ ICompoundOutlinerExtension */
	UE_API FOutlinerSizing RecomputeSizing() override;

	/*~ IOutlinerExtension */
	UE_API FOutlinerSizing GetOutlinerSizing() const override;
	UE_API FText GetLabel() const override;
	UE_API FSlateFontInfo GetLabelFont() const override;
	UE_API FText GetLabelToolTipText() const override;
	UE_API TSharedPtr<SWidget> CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName) override;

	/*~ ICurveEditorTreeItem */
	UE_API void CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels) override;

	/*~ IDeletableExtension */
	UE_API bool CanDelete(FText* OutErrorMessage) const override;
	UE_API void Delete() override;

	/*~ ICurveEditorTreeItemExtension */
	UE_API bool HasCurves() const override;
	UE_API void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	
	UE_API void OnUpdated();
	
	UE_API TOptional<FString> GetUniquePathName() const override;

private:

	UE_API EVisibility GetKeyEditorVisibility() const;
	
private:

	FOutlinerSizing ComputedSizing;
	TWeakViewModelPtr<FChannelModel> WeakCommonChannelModel;
};


/**
 * Utility class for building menus that add, edit, and remove channel overrides.
 */
class FChannelGroupOverrideHelper
{
public:

	static UE_API void BuildChannelOverrideMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FSequenceModel> SequenceModel);

private:

	static UE_API bool GetChannelsFromOutlinerSelection(TSharedPtr<FSequenceModel> SequenceModel, TArray<TViewModelPtr<FChannelModel>>& OutChannelModels);
	static UE_API void BuildChannelOverrideMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FSequenceModel> SequenceModel, UMovieSceneChannelOverrideContainer::FOverrideCandidates OverrideCandidates);
	static UE_API void OverrideChannels(TSharedPtr<FSequenceModel> SequenceModel, TSubclassOf<UMovieSceneChannelOverrideContainer> OverrideClass);
	static UE_API void RemoveChannelOverrides(TSharedPtr<FSequenceModel> SequenceModel);
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
