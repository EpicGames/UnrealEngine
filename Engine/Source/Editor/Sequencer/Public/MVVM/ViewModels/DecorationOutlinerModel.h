// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ISequencerTrackEditor.h"
#include "OutlinerItemModel.h"
#include "MVVM/Extensions/IDeactivatableExtension.h"
#include "MVVM/Extensions/IDeletableExtension.h"
#include "MVVM/Extensions/ILockableExtension.h"
#include "MVVM/Extensions/IMutableExtension.h"
#include "MVVM/Extensions/ISectionOwnerExtension.h"
#include "MVVM/Extensions/ITopLevelChannelHolderExtension.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"

struct FMovieSceneChannelProxy;

#define UE_API SEQUENCER_API

namespace UE::Sequencer
{

// Outliner model for a single decoration on a track or section.
// One instance is created per visible decoration (IMovieSceneChannelDecoration or
// IMovieSceneSectionProviderDecoration). For section-provider decorations, section
// models are hosted directly in the track area and channel items appear as outliner
// children. For channel decorations, FChannelGroupOutlinerModel instances are direct
// outliner children.
class FDecorationOutlinerModel
	: public FOutlinerItemModel
	, public ISectionOwnerExtension
	, public ITrackAreaExtension
	, public ILockableExtension
	, public IDeletableExtension
	, public ITopLevelChannelHolderExtension
	, public IMutableExtension
	, public IDeactivatableExtension
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(SEQUENCER_API, FDecorationOutlinerModel
		, FOutlinerItemModel
		, ISectionOwnerExtension
		, ITrackAreaExtension
		, ILockableExtension
		, IDeletableExtension
		, ITopLevelChannelHolderExtension
		, IMutableExtension
		, IDeactivatableExtension)

	UE_API explicit FDecorationOutlinerModel(UMovieSceneTrack* InParentTrack, UMovieSceneDecorationContainerObject* InDecorationContainer, UObject* InDecoration);
	UE_API virtual ~FDecorationOutlinerModel() override;

	/* FViewModel Interface */
	UE_API void OnDecorationSignatureChanged();
	virtual void OnConstruct() override;

	/* FOutlinerItemModel */
	UE_API virtual void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	UE_API virtual void BuildSidebarMenu(FMenuBuilder& MenuBuilder) override;

	/* IOutlinerExtension */
	UE_API virtual FOutlinerSizing GetOutlinerSizing() const override;
	UE_API TSharedRef<SWidget> BuildAddDecorationSubMenu(TSharedPtr<FEditorViewModel> EditorViewModel) const;
	virtual TSharedPtr<SWidget> CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName) override;
	UE_API virtual FText GetLabel() const override;
	UE_API virtual const FSlateBrush* GetIconBrush() const override;

	/* ISectionOwnerExtension */
	UE_API virtual FViewModelChildren GetSectionModels() override;

	/* ITopLevelChannelHolderExtension */
	UE_API virtual FViewModelChildren GetTopLevelChannels() override;

	/* ITrackAreaExtension */
	UE_API virtual FTrackAreaParameters GetTrackAreaParameters() const override;
	UE_API virtual FViewModelVariantIterator GetTrackAreaModelList() const override;
	UE_API virtual FViewModelVariantIterator GetTopLevelChildTrackAreaModels() const override;

	/* IDeletableExtension */
	UE_API virtual bool CanDelete(FText* OutErrorMessage) const override;
	UE_API virtual void Delete() override;

	/* ILockableExtension */
	UE_API virtual ELockableLockState GetLockState() const override;
	UE_API virtual void SetIsLocked(bool bIsLocked) override;

	/* IMutableExtension */
	UE_API virtual bool IsMuted() const override;
	UE_API virtual void SetIsMuted(bool bIsMuted) override;

	/* IDeactivatableExtension */
	UE_API virtual bool IsDeactivated() const override;
	UE_API virtual void SetIsDeactivated(bool bInIsDeactivated) override;

	/* IOutlinerExtension */
	UE_API virtual bool IsDimmed() const override;

	TWeakObjectPtr<UMovieSceneDecorationContainerObject> GetDecorationContainer() const { return WeakDecorationContainer; }

	// Get the decoration object this model represents. Used for sidebar property display.
	UE_API UObject* GetDecoration() const;

	// Returns true if the decoration's state has changed since the last OnConstruct.
	UE_API bool NeedsReconstruct() const;

private:

	// Outliner children: channel groups for both section-provider and channel decorations.
	FViewModelListHead DecoratorList{EViewModelListType::Outliner};

	// Section models for section-provider decorations. Used for the track area display.
	FViewModelListHead SectionModelList{EViewModelListType::TrackArea};

	TWeakObjectPtr<UMovieSceneTrack> WeakParentTrack;

	TWeakObjectPtr<UMovieSceneDecorationContainerObject> WeakDecorationContainer;

	// The specific decoration this model represents.
	TWeakObjectPtr<UObject> WeakDecoration;

	TSharedPtr<ISequencerTrackEditor> TrackEditor;

	// Channel proxy for IMovieSceneChannelDecoration. Stored to keep channel handles valid.
	TSharedPtr<FMovieSceneChannelProxy> ChannelDecorationProxy;

	// Tracks models (channel and category) added to section models so they can be removed during cleanup.
	TArray<TWeakPtr<FViewModel>> TrackedSectionChildModels;

	// Handle for this decoration's on signature changed delegate.
	FDelegateHandle SignatureChangedHandle;

	bool bDecorationSignatureChanged = false;

	FGuid DecorationSignature;
};

#undef UE_API
}
