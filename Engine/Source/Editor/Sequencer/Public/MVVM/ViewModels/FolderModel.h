// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/OutlinerItemModel.h"
#include "EventHandlers/IFolderEventHandler.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/IRenameableExtension.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/Extensions/IGroupableExtension.h"
#include "MVVM/Extensions/IDraggableOutlinerExtension.h"
#include "MVVM/Extensions/ISortableExtension.h"
#include "MVVM/Extensions/IDeletableExtension.h"
#include "MVVM/Extensions/IDeactivatableExtension.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"

#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UMovieSceneFolder;
class FActorDragDropOp;

namespace UE
{
namespace Sequencer
{

class FLayerBarModel;
class FSequenceModel;
class FSequencerOutlinerDragDropOp;

class FFolderModel
	: public FEvaluableOutlinerItemModel
	, public IRenameableExtension
	, public ITrackAreaExtension
	, public IGroupableExtension
	, public IDraggableOutlinerExtension
	, public ISortableExtension
	, public IOutlinerDropTargetOutlinerExtension
	, public IDeletableExtension
	, private UE::MovieScene::TIntrusiveEventHandler<UE::MovieScene::IFolderEventHandler>
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE(FFolderModel
		, FEvaluableOutlinerItemModel
		, IRenameableExtension
		, ITrackAreaExtension
		, IGroupableExtension
		, IDraggableOutlinerExtension
		, ISortableExtension
		, IOutlinerDropTargetOutlinerExtension
		, IDeletableExtension);

	FFolderModel(UMovieSceneFolder* InFolder);
	~FFolderModel();

	/** Get the folder data */
	UMovieSceneFolder* GetFolder() const;

	/*~ FOutlinerItemModel */
	void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	void BuildSidebarMenu(FMenuBuilder& MenuBuilder) override;

	/*~ IFolderEventHandler */
	void OnTrackAdded(UMovieSceneTrack* Track) override;
	void OnTrackRemoved(UMovieSceneTrack* Track) override;
	void OnObjectBindingAdded(const FGuid& ObjectBinding) override;
	void OnObjectBindingRemoved(const FGuid& ObjectBinding) override;
	void OnChildFolderAdded(UMovieSceneFolder* Folder) override;
	void OnChildFolderRemoved(UMovieSceneFolder* Folder) override;
	void OnPostUndo() override;

	/*~ IOutlinerExtension */
	FOutlinerSizing GetOutlinerSizing() const override;
	FText GetLabel() const override;
	const FSlateBrush* GetIconBrush() const override;
	FSlateColor GetIconTint() const override;
	FSlateColor GetLabelColor() const override;
	TSharedPtr<SWidget> CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName) override;

	/*~ ITrackAreaExtension */
	FTrackAreaParameters GetTrackAreaParameters() const override;
	FViewModelVariantIterator GetTrackAreaModelList() const override;

	/*~ IGroupableExtension */
	void GetIdentifierForGrouping(TStringBuilder<128>& OutString) const override;

	/*~ IRenameableExtension */
	bool CanRename() const override;
	void Rename(const FText& NewName) override;

	/*~ IDraggableOutlinerExtension */
	bool CanDrag() const override;

	/*~ ISortableExtension */
	void SortChildren() override;
	FSortingKey GetSortingKey() const override;
	void SetCustomOrder(int32 InCustomOrder) override;

	/*~ IOutlinerDropTargetOutlinerExtension */
	TOptional<EItemDropZone> CanAcceptDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone) override;
	void PerformDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone) override;

	/*~ IDeletableExtension */
	bool CanDelete(FText* OutErrorMessage) const override;
	void Delete() override;

	/*~ IMutableExtension */
	bool IsMuted() const override;
	void SetIsMuted(bool bIsMuted) override;

	/*~ ISolableExtension */
	bool IsSolo() const override;
	void SetIsSoloed(bool bIsSoloed) override;

private:

	/*~ FViewModel interface */
	void OnConstruct() override;

	static TMap<TWeakObjectPtr<UMovieSceneFolder>, FColor> InitialFolderColors;
	static bool bFolderPickerWasCancelled;

	void SetFolderColor();
	void OnColorPickerPicked(FLinearColor NewFolderColor);
	void OnColorPickerClosed(const TSharedRef<SWindow>& Window);
	void OnColorPickerCancelled(FLinearColor NewFolderColor);

	void RepopulateChildren();

	void PerformDropActors(const FViewModelPtr& TargetModel, TSharedPtr<FActorDragDropOp> ActorDragDropEvent, TSharedPtr<FViewModel> AttachAfter);
	void PerformDropOutliner(const FViewModelPtr& TargetModel, TSharedPtr<FSequencerOutlinerDragDropOp> OutlinerDragDropEvent, TSharedPtr<FViewModel> AttachAfter);

private:

	FViewModelListHead TrackAreaList;
	TSharedPtr<FLayerBarModel> LayerBar;
	TWeakObjectPtr<UMovieSceneFolder> WeakFolder;
	FSequenceModel* OwnerModel;
};

} // namespace Sequencer
} // namespace UE

