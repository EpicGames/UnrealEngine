// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Extensions/IDraggableTrackAreaExtension.h"
#include "MVVM/Extensions/ILayerBarExtension.h"
#include "MVVM/Extensions/ISelectableExtension.h"
#include "MVVM/Extensions/ISnappableExtension.h"
#include "MVVM/Extensions/IStretchableExtension.h"
#include "MVVM/Extensions/ITrackLaneExtension.h"
#include "MVVM/Extensions/LinkedOutlinerExtension.h"
#include "MVVM/Extensions/ViewModelExtensionCollection.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "Math/Range.h"
#include "Templates/SharedPointer.h"
#include "Misc/Attribute.h"
#include "Internationalization/Text.h"

#define UE_API SEQUENCER_API

namespace UE::Sequencer { class ILayerBarExtension; }
namespace UE::Sequencer { template <typename T> struct TAutoRegisterViewModelTypeID; }
struct FFrameNumber;

namespace UE
{
namespace Sequencer
{

class FSequenceModel;

class FLayerBarModel
	: public FViewModel
	, public FLinkedOutlinerExtension
	, public ITrackLaneExtension
	, public ISelectableExtension
	, public ISnappableExtension
	, public IDraggableTrackAreaExtension
	, public IStretchableExtension
	, protected TViewModelExtensionCollection<ILayerBarExtension>
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FLayerBarModel, FViewModel
		, FLinkedOutlinerExtension
		, ITrackLaneExtension
		, ISelectableExtension
		, ISnappableExtension
		, IDraggableTrackAreaExtension
		, IStretchableExtension
	);

	UE_API FLayerBarModel(TWeakPtr<FViewModel> LayerRoot);
	UE_API ~FLayerBarModel();

	/*~ ITrackLaneExtension Interface */
	UE_API TSharedPtr<ITrackLaneWidget> CreateTrackLaneView(const FCreateTrackLaneViewParams& InParams) override;
	UE_API FTrackLaneVirtualAlignment ArrangeVirtualTrackLaneView() const override;

	/*~ ISelectableExtension Interface */
	UE_API ESelectionIntent IsSelectable() const override;

	/*~ ISnappableExtension Interface */
	UE_API void AddToSnapField(const ISnapCandidate& Candidate, ISnapField& SnapField) const override;

	/*~ IDraggableTrackAreaExtension Interface */
	UE_API bool CanDrag() const override;
	UE_API void OnBeginDrag(IDragOperation& DragOperation) override;
	UE_API void OnEndDrag(IDragOperation& DragOperation) override;

	/*~ IStretchableExtension Interface */
	UE_API void OnInitiateStretch(IStretchOperation& StretchOperation, EStretchConstraint Constraint, FStretchParameters* InOutGlobalParameters) override;

public:

	UE_API TRange<FFrameNumber> ComputeRange() const;

	UE_API void Offset(FFrameNumber Offset);

	UE_API void SetOverlayText(TAttribute<FText> InText);
	UE_API const TAttribute<FText>& GetOverlayText() const;

private:

	/*~ FViewModel interface */
	UE_API void OnConstruct() override;
	UE_API void OnDestruct() override;

	/*~ TViewModelExtensionCollection Interface */
	UE_API void OnExtensionsDirtied() override;

	TAttribute<FText> OverlayText;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
