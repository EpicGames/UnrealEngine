// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "MVVM/ViewModels/OutlinerItemModel.h"
#include  "MVVM/ViewModels/BindingLifetimeOverlayModel.h"
#include "MVVM/Extensions/IRenameableExtension.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/Extensions/IGroupableExtension.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/ISortableExtension.h"
#include "MVVM/Extensions/IDraggableOutlinerExtension.h"
#include "MVVM/Extensions/IDeletableExtension.h"

#define UE_API SEQUENCER_API


struct FMovieSceneBinding;

class UMovieScene;
class UMovieSceneTrack;
class FMenuBuilder;
class FPropertyPath;
class FStructOnScope;
enum class ECheckBoxState : uint8;

namespace UE
{
namespace Sequencer
{

class FSequenceModel;
class FLayerBarModel;
class FTrackModelStorageExtension;

/** Enumeration specifying what kind of object binding this is */
enum class EObjectBindingType
{
	Possessable, Spawnable, Unknown
};

class FObjectBindingModel
	: public FEvaluableOutlinerItemModel
	, public IObjectBindingExtension
	, public IDraggableOutlinerExtension
	, public ITrackAreaExtension
	, public IGroupableExtension
	, public IRenameableExtension
	, public ISortableExtension
	, public IDeletableExtension
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FObjectBindingModel
		, FEvaluableOutlinerItemModel
		, IObjectBindingExtension
		, IDraggableOutlinerExtension
		, ITrackAreaExtension
		, IGroupableExtension
		, IRenameableExtension
		, ISortableExtension
		, IDeletableExtension);

	UE_API FObjectBindingModel(FSequenceModel* OwnerModel, const FMovieSceneBinding& InBinding);
	UE_API ~FObjectBindingModel();

	static UE_API EViewModelListType GetTopLevelChildTrackAreaGroupType();

	UE_API void AddTrack(UMovieSceneTrack* Track);
	UE_API void RemoveTrack(UMovieSceneTrack* Track);

	/*~ IObjectBindingExtension */
	UE_API FGuid GetObjectGuid() const override;

	/*~ IRenameableExtension */
	UE_API bool CanRename() const override;
	UE_API void Rename(const FText& NewName) override;

	/*~ IOutlinerExtension */
	UE_API FOutlinerSizing GetOutlinerSizing() const override;
	UE_API FText GetLabel() const override;
	UE_API FSlateColor GetLabelColor() const override;
	UE_API FText GetLabelToolTipText() const override;
	UE_API const FSlateBrush* GetIconBrush() const override;
	UE_API TSharedPtr<SWidget> CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName) override;

	/*~ ITrackAreaExtension */
	UE_API FTrackAreaParameters GetTrackAreaParameters() const override;
	UE_API FViewModelVariantIterator GetTrackAreaModelList() const override;
	UE_API FViewModelVariantIterator GetTopLevelChildTrackAreaModels() const override;

	/*~ IGroupableExtension */
	UE_API void GetIdentifierForGrouping(TStringBuilder<128>& OutString) const override;

	/*~ ISortableExtension */
	UE_API void SortChildren() override;
	UE_API FSortingKey GetSortingKey() const override;
	UE_API void SetCustomOrder(int32 InCustomOrder) override;

	/*~ FOutlinerItemModel */
	UE_API void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	UE_API void BuildOrganizeContextMenu(FMenuBuilder& MenuBuilder) override;
	UE_API bool GetDefaultExpansionState() const override;
	UE_API void BuildSidebarMenu(FMenuBuilder& MenuBuilder) override;

	/*~ IDraggableOutlinerExtension */
	UE_API bool CanDrag() const override;

	/*~ IDeletableExtension */
	UE_API bool CanDelete(FText* OutErrorMessage) const override;
	UE_API void Delete() override;

	/*~ IMutableExtension */
	UE_API bool IsMuted() const override;
	UE_API void SetIsMuted(bool bIsMuted) override;

	/*~ ISolableExtension */
	UE_API bool IsSolo() const override;
	UE_API void SetIsSoloed(bool bIsSoloed) override;

public:

	UE_API virtual void SetParentBindingID(const FGuid& InObjectBindingID);
	UE_API virtual FGuid GetDesiredParentBinding() const;
	UE_API virtual EObjectBindingType GetType() const;
	UE_API virtual FText GetTooltipForSingleObjectBinding() const;
	UE_API virtual const UClass* FindObjectClass() const;
	UE_API virtual bool SupportsRebinding() const;
	virtual FSlateColor GetInvalidBindingLabelColor() const { return FLinearColor::Red; }

public:

	UE_API TSharedRef<SWidget> GetAddTrackMenuContent();

protected:

	/*~ FViewModel interface */
	UE_API void OnConstruct() override;

private:

	struct FPropertyMenuData
	{
		FString MenuName;
		FPropertyPath PropertyPath;

		bool operator< (const FPropertyMenuData& Other) const
		{
			int32 CompareResult = MenuName.Compare(Other.MenuName);
			return CompareResult < 0;
		}
	};

	UE_API void AddPropertyMenuItems(FMenuBuilder& AddTrackMenuBuilder, int32 NumStartingBlocks, TArray<FPropertyPath> KeyablePropertyPaths, int32 PropertyNameIndexStart);
	UE_API void AddPropertyMenuItem(FMenuBuilder& AddTrackMenuBuilder, const FPropertyMenuData& KeyablePropertyMenuData);
	UE_API void HandleAddTrackSubMenuNew(FMenuBuilder& AddTrackMenuBuilder, TArray<FPropertyPath> KeyablePropertyPaths, int32 PropertyNameIndexStart);
	UE_API void HandlePropertyMenuItemExecute(FPropertyPath PropertyPath);

	UE_API void AddTagMenu(FMenuBuilder& MenuBuilder);
	UE_API ECheckBoxState GetTagCheckState(FName TagName);
	UE_API void ToggleTag(FName TagName);
	UE_API void HandleDeleteTag(FName TagName);
	UE_API void HandleAddTag(FName TagName);
	UE_API void HandleTemplateActorClassPicked(UClass* ChosenClass);

	UE_API void OnFinishedChangingDynamicBindingProperties(const FPropertyChangedEvent& ChangeEvent, TSharedPtr<FStructOnScope> ValueStruct);

	UE_API UObject* GetFirstBoundObject(FGuid InGuid);

protected:

	FGuid ObjectBindingID;
	FGuid ParentObjectBindingID;
	FViewModelListHead TrackAreaList;
	FViewModelListHead TopLevelChildTrackAreaList;
	TSharedPtr<FLayerBarModel> LayerBar;
	TSharedPtr<FBindingLifetimeOverlayModel> BindingLifetimeOverlayModel;
	FSequenceModel* OwnerModel;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
