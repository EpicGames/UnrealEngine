// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ICastable.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/Extensions/ICurveEditorTreeItemExtension.h"
#include "MVVM/Extensions/IDeactivatableExtension.h"
#include "MVVM/Extensions/IDimmableExtension.h"
#include "MVVM/Extensions/IGeometryExtension.h"
#include "MVVM/Extensions/IHoveredExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/IPinnableExtension.h"
#include "MVVM/Extensions/IMutableExtension.h"
#include "MVVM/Extensions/ISoloableExtension.h"
#include "MVVM/Extensions/HierarchicalCacheExtension.h"
#include "CurveEditorTypes.h"
#include "Tree/ICurveEditorTreeItem.h"

#define UE_API SEQUENCER_API

class UMovieSceneSection;
class UMovieSceneSequence;
class UMovieSceneTrack;
class FBoolProperty;
class FSequencer;
class IDetailsView;

namespace UE
{
namespace Sequencer
{

class FSequencerEditorViewModel;

class FOutlinerItemModelMixin
	: public FOutlinerExtensionShim
	, public FGeometryExtensionShim
	, public FPinnableExtensionShim
	, public FHoveredExtensionShim
	, public IDimmableExtension
	, public FCurveEditorTreeItemExtensionShim
	, public ICurveEditorTreeItem
{
public:

	using Implements = TImplements<IOutlinerExtension, IGeometryExtension, IPinnableExtension, IHoveredExtension, IDimmableExtension, ICurveEditorTreeItemExtension>;

	UE_API FOutlinerItemModelMixin();

	UE_API TSharedPtr<FSequencerEditorViewModel> GetEditor() const;
	
	/*~ IOutlinerExtension */
	UE_API FName GetIdentifier() const override;
	UE_API bool IsExpanded() const override;
	UE_API void SetExpansion(bool bInIsExpanded) override;
	UE_API bool IsFilteredOut() const override;
	UE_API TSharedPtr<SWidget> CreateContextMenuWidget(const FCreateOutlinerContextMenuWidgetParams& InParams) override;
	UE_API FSlateColor GetLabelColor() const override;

	/*~ ICurveEditorTreeItemExtension */
	UE_API virtual bool HasCurves() const override;
	UE_API virtual TSharedPtr<ICurveEditorTreeItem> GetCurveEditorTreeItem() const override;
	UE_API virtual TOptional<FString> GetUniquePathName() const override;

	/*~ ICurveEditorTreeItem */
	UE_API virtual TSharedPtr<SWidget> GenerateCurveEditorTreeWidget(const FName& InColumnName, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID, const TSharedRef<ITableRow>& InTableRow) override;
	UE_API virtual void CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels) override;
	UE_API virtual bool PassesFilter(const FCurveEditorTreeFilter* InFilter) const override;

	/*~ IPinnableExtension */
	UE_API bool IsPinned() const override;
	UE_API void SetPinned(const bool bInIsPinned) override;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPinStateChanged, const TViewModelPtr<IPinnableExtension>& /*InPinnableExtension*/, const bool /*bInPinned*/)
	/** Delegate that fires when the pin state of this item changes */
	static UE_API FOnPinStateChanged& OnPinStateChanged() { return PinStateChangedDelegate; }

	/*~ IDimmableExtension */
	UE_API bool IsDimmed() const override;

	/** Get context menu contents. */
	UE_API virtual void BuildContextMenu(FMenuBuilder& MenuBuilder);

	UE_API virtual void BuildSidebarMenu(FMenuBuilder& MenuBuilder);

protected:

	/** Set identifier for computing node paths */
	UE_API void SetIdentifier(FName InNewIdentifier);

	/** Get the default expansion state if it wasn't saved in the movie-scene data */
	UE_API virtual bool GetDefaultExpansionState() const;

	/** Set expansion state without saving it in the movie-scene data */
	UE_API void SetExpansionWithoutSaving(bool bInIsExpanded);

	UE_API virtual void BuildSectionColorTintsMenu(FMenuBuilder& MenuBuilder);
	UE_API virtual void BuildOrganizeContextMenu(FMenuBuilder& MenuBuilder);
	UE_API virtual void BuildDisplayOptionsMenu(FMenuBuilder& MenuBuilder);
	UE_API virtual void BuildTrackOptionsMenu(FMenuBuilder& MenuBuilder, const TArray<UMovieSceneTrack*>& InTracks);
	UE_API virtual void BuildTrackRowOptionsMenu(FMenuBuilder& MenuBuilder);

private:

	virtual FViewModel* AsViewModel() = 0;
	virtual const FViewModel* AsViewModel() const = 0;

protected:
	FViewModelListHead OutlinerChildList;

private:

	UE_API bool IsRootModelPinned() const;
	UE_API void ToggleRootModelPinned();

	UE_API ECheckBoxState SelectedModelsSoloState() const;
	UE_API void ToggleSelectedModelsSolo();

	UE_API ECheckBoxState SelectedModelsMuteState() const;
	UE_API void ToggleSelectedModelsMuted();

	UE_API TArray<UMovieSceneSection*> GetSelectedSections() const;
	UE_API TArray<UMovieSceneTrack*> GetSelectedTracks() const;
	UE_API TArray<TPair<UMovieSceneTrack*, int32>> GetSelectedTrackRows() const;

	UE_API void AddEvalOptionsPropertyMenuItem(FMenuBuilder& InMenuBuilder, const FBoolProperty* InProperty, TFunction<bool(UMovieSceneTrack*)> InValidator = nullptr);
	UE_API void AddDisplayOptionsPropertyMenuItem(FMenuBuilder& InMenuBuilder, const FBoolProperty* InProperty, TFunction<bool(UMovieSceneTrack*)> InValidator = nullptr);

private:

	ICastable* CastableThis;
	FName TreeItemIdentifier;
	FCurveEditorTreeItemID CurveEditorItemID;
	mutable bool bInitializedExpansion;
	mutable bool bInitializedPinnedState;

	static FOnPinStateChanged PinStateChangedDelegate;
};

//
// Note: You must add the base class you use as a template parameter to the UE_SEQUENCER_DECLARE_CASTABLE list
// 
template<typename BaseType>
class TOutlinerModelMixin : public BaseType, public FOutlinerItemModelMixin
{
public:

	template<typename... ArgTypes>
	TOutlinerModelMixin(ArgTypes&&... InArgs)
		: BaseType(Forward<ArgTypes>(InArgs)...)
	{
		this->RegisterChildList(&this->OutlinerChildList);
	}

	TOutlinerModelMixin(const TOutlinerModelMixin<BaseType>&) = delete;
	TOutlinerModelMixin<BaseType> operator=(const TOutlinerModelMixin<BaseType>&) = delete;

	TOutlinerModelMixin(TOutlinerModelMixin<BaseType>&&) = delete;
	TOutlinerModelMixin<BaseType> operator=(TOutlinerModelMixin<BaseType>&&) = delete;

	virtual FViewModel* AsViewModel() { return this; }
	virtual const FViewModel* AsViewModel() const { return this; }
};

class FOutlinerItemModel : public TOutlinerModelMixin<FViewModel>
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FOutlinerItemModel, FViewModel, FOutlinerItemModelMixin);
};

class FEvaluableOutlinerItemModel
	: public FOutlinerItemModel
	, public IMutableExtension
	, public ISoloableExtension
	, public IDeactivatableExtension
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FEvaluableOutlinerItemModel
		, FOutlinerItemModel
		, IMutableExtension
		, ISoloableExtension
		, IDeactivatableExtension);

	/*~ IDeactivatableExtension */
	UE_API virtual bool IsDeactivated() const override;
	UE_API virtual void SetIsDeactivated(const bool bInIsDeactivated) override;
};

class FOutlinerCacheExtension
	: public FHierarchicalCacheExtension
{
public:

	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, FOutlinerCacheExtension);

	FOutlinerCacheExtension()
	{
		ModelListFilter = EViewModelListType::Outliner;
	}
};


} // namespace Sequencer
} // namespace UE

#undef UE_API
