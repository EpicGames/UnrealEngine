// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "Misc/Guid.h"

class UClass;
namespace UE::Sequencer { class FSequenceModel; }
namespace UE::Sequencer { template <typename T> struct TAutoRegisterViewModelTypeID; }
struct FMovieSceneBinding;
struct FMovieScenePossessable;
struct FSlateBrush;

namespace UE
{
namespace Sequencer
{

class FPossessableModel
	: public FObjectBindingModel
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE(FPossessableModel, FObjectBindingModel);

	FPossessableModel(FSequenceModel* OwnerModel, const FMovieSceneBinding& InBinding, const FMovieScenePossessable& InPossessable);
	~FPossessableModel();

	/*~ FObjectBindingModel */
	EObjectBindingType GetType() const override;
	const FSlateBrush* GetIconOverlayBrush() const override;
	const UClass* FindObjectClass() const override;
	bool SupportsRebinding() const override;

	/*~ FViewModel interface */
	void OnConstruct() override;

	/*~ FOutlinerItemModel interface */
	FText GetIconToolTipText() const override;

	/*~ FObjectBindingModel interface */
	void Delete() override;
	FSlateColor GetInvalidBindingLabelColor() const override;

};

} // namespace Sequencer
} // namespace UE

