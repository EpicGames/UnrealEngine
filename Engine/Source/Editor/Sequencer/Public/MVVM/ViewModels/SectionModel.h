// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelHandle.h"
#include "MovieSceneSection.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "MVVM/Extensions/ILayerBarExtension.h"
#include "MVVM/Extensions/ITrackLaneExtension.h"
#include "MVVM/Extensions/IGeometryExtension.h"
#include "MVVM/Extensions/IObjectModelExtension.h"
#include "MVVM/Extensions/ISelectableExtension.h"
#include "MVVM/Extensions/ISnappableExtension.h"
#include "MVVM/Extensions/IDraggableTrackAreaExtension.h"
#include "MVVM/Extensions/IStretchableExtension.h"
#include "MVVM/Extensions/LinkedOutlinerExtension.h"
#include "MVVM/Extensions/IConditionableExtension.h"
#include "Delegates/DelegateCombinations.h"
#include "EventHandlers/ISignedObjectEventHandler.h"
#include "EventHandlers/ISectionEventHandler.h"

#define UE_API SEQUENCER_API

class ISequencerSection;
class UMovieSceneSection;
struct FMovieSceneChannelProxy;

namespace UE
{
namespace Sequencer
{

class ITrackExtension;
class ISectionOwnerExtension;
class FChannelModel;
struct FOverlappingSections;

/**
 * Model for a sequencer section
 */
class FSectionModel
	: public FViewModel
	, public FLinkedOutlinerExtension
	, public IObjectModelExtension
	, public ILayerBarExtension
	, public ITrackLaneExtension
	, public ISelectableExtension
	, public ISnappableExtension
	, public IDraggableTrackAreaExtension
	, public IStretchableExtension
	, public IConditionableExtension
	, public UE::MovieScene::TIntrusiveEventHandler<UE::MovieScene::ISignedObjectEventHandler>
	, public UE::MovieScene::TIntrusiveEventHandler<UE::MovieScene::ISectionEventHandler>
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FSectionModel, FViewModel
		, FLinkedOutlinerExtension
		, IObjectModelExtension
		, ILayerBarExtension
		, ITrackLaneExtension
		, ISelectableExtension
		, ISnappableExtension
		, IDraggableTrackAreaExtension
		, IStretchableExtension
		, IConditionableExtension
	);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnModelUpdated, FSectionModel*)

	UE_DEPRECATED(5.5, "This member is no longer supported, please subscribe to UMovieSceneSignedObject::OnModifiedDirectly.")
	FOnModelUpdated OnUpdated;

	UE_API FSectionModel();
	UE_API ~FSectionModel();

	UE_API void InitializeSection(TSharedPtr<ISequencerSection> InSectionInterface);

	/** Gets the time range of the section */
	UE_API TRange<FFrameNumber> GetRange() const;

	/** Gets the section object */
	UE_API UMovieSceneSection* GetSection() const;

	/** Gets the section interface */
	UE_API TSharedPtr<ISequencerSection> GetSectionInterface() const;

	/** Gets the parent section owner model (which could be a track, track row, or something else like a mixer layer) */
	UE_API TViewModelPtr<ISectionOwnerExtension> GetParentSectionOwnerModel() const;

	/** Gets the ancestor track model */
	UE_API TViewModelPtr<ITrackExtension> GetParentTrackModel() const;

	/** Gets the ancestor track model as an ITrackExtension */
	UE_API TViewModelPtr<ITrackExtension> GetParentTrackExtension() const;

	/** Gets an array of sections that underlap the specified section */
	UE_API TArray<FOverlappingSections> GetUnderlappingSections();
	/** Gets an array of sections whose easing bounds underlap the specified section */
	UE_API TArray<FOverlappingSections> GetEasingSegments();

	UE_API int32 GetPreRollFrames() const;
	UE_API int32 GetPostRollFrames() const;

	/** Returns whether this section model needs to be rebuilt, ie. the channel proxy is no longer valid */
	bool NeedsLayout() const
	{
		return !PreviousLayoutChannelProxy.Pin().IsValid() || (GetSection() && GetSection()->GetRowIndex() != PreviousLayoutRowIndex);
	}

	/** Set the channel proxy that was most recently used to layout this section model */
	void SetLayoutChannelProxy(TWeakPtr<FMovieSceneChannelProxy> InPreviousLayoutChannelProxy)
	{
		PreviousLayoutChannelProxy = InPreviousLayoutChannelProxy;
	}

	/** Set the row index that was most recently used to layout this section model */
	void SetLayoutRowIndex(int32 InRowIndex)
	{
		PreviousLayoutRowIndex = InRowIndex;
	}

public:

	/*~ IObjectModelExtension Interface */
	UE_API virtual void InitializeObject(TWeakObjectPtr<> InWeakObject) override;
	UE_API virtual UObject* GetObject() const override;

	/*~ ILayerBarExtension Interface */
	UE_API TRange<FFrameNumber> GetLayerBarRange() const override;
	UE_API void OffsetLayerBar(FFrameNumber Amount) override;

	/*~ ITrackLaneExtension Interface */
	UE_API TSharedPtr<ITrackLaneWidget> CreateTrackLaneView(const FCreateTrackLaneViewParams& InParams) override;
	UE_API FTrackLaneVirtualAlignment ArrangeVirtualTrackLaneView() const override;

	/*~ ISelectableExtension Interface */
	UE_API ESelectionIntent IsSelectable() const override;

	/*~ ISnappableExtension Interface */
	UE_API void AddToSnapField(const ISnapCandidate& Candidate, ISnapField& SnapField) const override;

	/*~ ISignedObjectEventHandler Interface */
	UE_API void OnModifiedDirectly(UMovieSceneSignedObject*) override;

	/*~ ISectionEventHandler Interface */
	UE_API void OnRowChanged(UMovieSceneSection*) override;

	/*~ IDraggableTrackAreaExtension Interface */
	UE_API bool CanDrag() const override;
	UE_API void OnBeginDrag(IDragOperation& DragOperation) override;
	UE_API void OnEndDrag(IDragOperation& DragOperation) override;

	/*~ IStretchableExtension Interface */
	UE_API void OnInitiateStretch(IStretchOperation& StretchOperation, EStretchConstraint Constraint, FStretchParameters* InOutGlobalParameters) override;
	UE_API EStretchResult OnBeginStretch(const IStretchOperation& StretchOperation, const FStretchScreenParameters& ScreenParameters, FStretchParameters* InOutParameters) override;
	UE_API void OnStretch(const IStretchOperation& StretchOperation, const FStretchScreenParameters& ScreenParameters, FStretchParameters* InOutParameters) override;
	UE_API void OnEndStretch(const IStretchOperation& StretchOperation, const FStretchScreenParameters& ScreenParameters, FStretchParameters* InOutParameters) override;

	/*~ IConditionableExtension Interface */
	UE_API const UMovieSceneCondition* GetCondition() const override;
	UE_API EConditionableConditionState GetConditionState() const override;
	UE_API void SetConditionEditorForceTrue(bool bEditorForceTrue) override;

private:

	UE_API void UpdateCachedData();

private:

	FViewModelListHead ChannelList;
	TSharedPtr<ISequencerSection> SectionInterface;
	TWeakObjectPtr<UMovieSceneSection> WeakSection;
	TRange<FFrameNumber> SectionRange;
	TRange<FFrameNumber> LayerBarRange;

	TWeakPtr<FMovieSceneChannelProxy> PreviousLayoutChannelProxy;
	int32 PreviousLayoutRowIndex = -1;
};

struct FOverlappingSections
{
	/** The range for the overlap */
	TRange<FFrameNumber> Range;
	/** The sections that occupy this range, sorted by overlap priority */
	TArray<TWeakPtr<FSectionModel>> Sections;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
