// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/OutlinerItemModel.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/Extensions/ILockableExtension.h"
#include "MVVM/Extensions/IResizableExtension.h"
#include "MVVM/Extensions/IDeletableExtension.h"
#include "MVVM/Extensions/IConditionableExtension.h"
#include "MVVM/Extensions/ISectionOwnerExtension.h"
#include "MVVM/Extensions/ITopLevelChannelHolderExtension.h"
#include "EventHandlers/ISectionEventHandler.h"
#include "EventHandlers/ISignedObjectEventHandler.h"
#include "EventHandlers/MovieSceneDataEventContainer.h"

#include "UObject/WeakObjectPtr.h"

#define UE_API SEQUENCER_API

class ISequencer;
class ISequencerTrackEditor;
class UMovieSceneSection;

namespace UE
{
namespace Sequencer
{

class FSectionModel;

/**
 * View model representing a section in the outliner hierarchy.
 * Used when a layer/track row contains multiple sections and we want to group
 * their channels separately in the outliner to avoid overlap.
 *
 * This acts as an intermediate level between the layer and the channels,
 * showing the section name in the outliner with its channels underneath.
 * This is an alternative to graying out non-shared channel areas.
 */
class FSectionOutlinerModel
	: public FEvaluableOutlinerItemModel
	, public ITrackAreaExtension
	, public ILockableExtension
	, public IResizableExtension
	, public IDeletableExtension
	, public IConditionableExtension
	, public ISectionOwnerExtension
	, public ITopLevelChannelHolderExtension
	, public UE::MovieScene::ISectionEventHandler
	, public UE::MovieScene::TIntrusiveEventHandler<UE::MovieScene::ISignedObjectEventHandler>
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FSectionOutlinerModel
		, FEvaluableOutlinerItemModel
		, ITrackAreaExtension
		, ILockableExtension
		, IResizableExtension
		, IDeletableExtension
		, IConditionableExtension
		, ISectionOwnerExtension
		, ITopLevelChannelHolderExtension);

	UE_API explicit FSectionOutlinerModel(UMovieSceneSection* InSection, TSharedPtr<FSectionModel> InSectionModel);
	UE_API ~FSectionOutlinerModel();

	/*~ FViewModel interface */
	UE_API virtual void OnConstruct() override;


	UE_API FViewModelChildren GetTrackAreaChildren();

	UE_API UMovieSceneSection* GetSection() const { return WeakSection.Get(); }

	UE_API TSharedPtr<FSectionModel> GetSectionModel() const;

	UE_API UMovieSceneTrack* GetTrack() const;

	/*~ FEvaluableOutlinerItemModel */
	UE_API bool IsDeactivated() const override;
	UE_API void SetIsDeactivated(bool bInIsDeactivated) override;

	/*~ FOutlinerItemModel */
	UE_API void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	UE_API void BuildSidebarMenu(FMenuBuilder& MenuBuilder) override;
	UE_API void CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels) override;

	/*~ IOutlinerExtension */
	UE_API FOutlinerSizing GetOutlinerSizing() const override;
	UE_API TSharedPtr<SWidget> CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName) override;
	UE_API bool SupportsOutlinerColumn(const FName& InColumnName) const override;
	UE_API FText GetLabel() const override;
	UE_API FSlateColor GetLabelColor() const override;
	UE_API FSlateFontInfo GetLabelFont() const override;
	UE_API const FSlateBrush* GetIconBrush() const override;

	/*~ ISectionOwnerExtension */
	UE_API FViewModelChildren GetSectionModels() override;

	/*~ ITopLevelChannelHolderExtension */
	UE_API FViewModelChildren GetTopLevelChannels() override;

	/*~ IResizableExtension */
	UE_API bool IsResizable() const override;
	UE_API void Resize(float NewSize) override;

	/*~ ITrackAreaExtension */
	UE_API FTrackAreaParameters GetTrackAreaParameters() const override;
	UE_API FViewModelVariantIterator GetTrackAreaModelList() const override;
	UE_API FViewModelVariantIterator GetTopLevelChildTrackAreaModels() const override;

	/*~ IDimmableExtension */
	UE_API bool IsDimmed() const override;

	/*~ IDeletableExtension */
	UE_API bool CanDelete(FText* OutErrorMessage) const override;
	UE_API void Delete() override;

	/*~ ILockableExtension Interface */
	UE_API ELockableLockState GetLockState() const override;
	UE_API void SetIsLocked(bool bIsLocked) override;

	/*~ IConditionableExtension Interface */
	UE_API const UMovieSceneCondition* GetCondition() const override;
	UE_API EConditionableConditionState GetConditionState() const override;
	UE_API void SetConditionEditorForceTrue(bool bEditorForceTrue) override;

	/*~ IMutableExtension */
	UE_API bool IsMuted() const override;
	UE_API void SetIsMuted(bool bIsMuted) override;

	/*~ ISoloableExtension */
	UE_API bool IsSolo() const override;
	UE_API void SetIsSoloed(bool bIsSoloed) override;

	/*~ ISectionEventHandler */
	UE_API void OnDecorationAdded(UObject* AddedDecoration) override;
	UE_API void OnDecorationRemoved(UObject* RemovedDecoration) override;

#if WITH_EDITOR
	/*~ ISignedObjectEventHandler */
	UE_API void OnPostUndo() override;
#endif

	// Refreshes decoration models under this section outliner.
	UE_API void RefreshDecorations();

private:

	FViewModelListHead TopLevelChannelList;
	FViewModelListHead SectionModelList{EViewModelListType::TrackArea};
	TWeakObjectPtr<UMovieSceneSection> WeakSection;
	TWeakViewModelPtr<FSectionModel> WeakSectionModel;

	TSharedPtr<ISequencerTrackEditor> TrackEditor;

	UE::MovieScene::TNonIntrusiveEventHandler<UE::MovieScene::ISectionEventHandler> SectionEventHandler;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
