// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Containers/SortedMap.h"
#include "Misc/Guid.h"
#include "Delegates/DelegateCombinations.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "MovieSceneSequenceID.h"
#include "EventHandlers/ISignedObjectEventHandler.h"
#include "EventHandlers/ISequenceDataEventHandler.h"

#include "SequencerCoreFwd.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Extensions/IGeometryExtension.h"
#include "MVVM/Extensions/ISortableExtension.h"
#include "MVVM/Extensions/ICurveEditorTreeItemExtension.h"

#define UE_API SEQUENCER_API

class ISequencer;
class FSequencer;
class FCurveEditor;
class UMovieScene;
class UMovieSceneSequence;

namespace UE
{
namespace Sequencer
{

class FEditorViewModel;
class FSequenceModel;
class FSequencerEditorViewModel;

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnInitializeSequenceModel, TSharedPtr<FEditorViewModel>, TSharedPtr<FSequenceModel>);

class FSequenceModel
	: public FViewModel
	, public ISortableExtension
	, public IOutlinerDropTargetOutlinerExtension
	, public UE::MovieScene::ISignedObjectEventHandler
	, public UE::MovieScene::ISequenceDataEventHandler
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FSequenceModel, FViewModel, ISortableExtension, IOutlinerDropTargetOutlinerExtension);

	static UE_API FOnInitializeSequenceModel CreateExtensionsEvent;

	UE_API FSequenceModel(TWeakPtr<FSequencerEditorViewModel> InEditorViewModel);

	UE_API void InitializeExtensions();

	UE_API FMovieSceneSequenceID GetSequenceID() const;
	UE_API UMovieSceneSequence* GetSequence() const;
	UE_API UMovieScene* GetMovieScene() const;
	UE_API void SetSequence(UMovieSceneSequence* InSequence, FMovieSceneSequenceID InSequenceID);

	UE_API TSharedPtr<FSequencerEditorViewModel> GetEditor() const;
	UE_API TSharedPtr<ISequencer> GetSequencer() const;
	UE_API TSharedPtr<FSequencer> GetSequencerImpl() const;

	TSharedPtr<FViewModel> GetBottomSpacer() const { return BottomSpacer; }

	/*~ ISignedObjectEventHandler */
	UE_API void OnPostUndo() override;
	UE_API void OnModifiedIndirectly(UMovieSceneSignedObject*) override;
	UE_API void OnModifiedDirectly(UMovieSceneSignedObject*) override;

	/*~ ISortableExtension */
	UE_API void SortChildren() override;
	UE_API FSortingKey GetSortingKey() const override;
	UE_API void SetCustomOrder(int32 InCustomOrder) override;

	/*~ Begin ISequenceDataEventHandler */
	UE_API virtual void OnDecorationAdded(UObject* AddedDecoration) override;
	UE_API virtual void OnDecorationRemoved(UObject* RemovedDecoration) override;
	/*~ End ISequenceDataEventHandler */

	/*~ IOutlinerDropTargetOutlinerExtension */
	UE_API TOptional<EItemDropZone> CanAcceptDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone) override;
	UE_API void PerformDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone) override;

	static UE_API EViewModelListType GetDecorationModelListType();

private:

	FViewModelListHead RootOutlinerItems;
	FViewModelListHead DecorationModelList;

	TWeakObjectPtr<UMovieSceneSequence> WeakSequence;
	TWeakPtr<FSequencerEditorViewModel> WeakEditor;

	TSharedPtr<FViewModel> BottomSpacer;
	FMovieSceneSequenceID SequenceID;

	MovieScene::TNonIntrusiveEventHandler<MovieScene::ISignedObjectEventHandler> SequenceEventHandler;
	MovieScene::TNonIntrusiveEventHandler<MovieScene::ISignedObjectEventHandler> MovieSceneEventHandler;
	MovieScene::TNonIntrusiveEventHandler<MovieScene::ISequenceDataEventHandler> MovieSceneDataEventHandler;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
