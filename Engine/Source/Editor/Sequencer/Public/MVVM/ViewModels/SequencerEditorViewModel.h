// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/UICommandList.h"
#include "MVVM/ViewModels/EditorViewModel.h"
#include "SequencerCustomizationManager.h"
#include "Toolkits/AssetEditorToolkit.h"

#define UE_API SEQUENCER_API

struct FMovieSceneSequenceID;

class ISequencer;
class FSequencer;
class UMovieSceneSequence;
struct FSequencerHostCapabilities;

namespace UE::Sequencer
{

class FSequenceModel;
class FSequencerSelection;
class STrackAreaView;
struct ITrackAreaHotspot;

/**
 * Main view-model for the Sequencer editor.
 */
class FSequencerEditorViewModel : public FEditorViewModel
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FSequencerEditorViewModel, FEditorViewModel);

	UE_API FSequencerEditorViewModel(TSharedRef<ISequencer> InSequencer, const FSequencerHostCapabilities& InHostCapabilities);
	UE_API ~FSequencerEditorViewModel();

	/** Retrieve this editor's selection class */
	UE_API TSharedPtr<FSequencerSelection> GetSelection() const;

	// @todo_sequencer_mvvm remove this later
	UE_API TSharedPtr<ISequencer> GetSequencer() const;
	// @todo_sequencer_mvvm remove this ASAP
	UE_API TSharedPtr<FSequencer> GetSequencerImpl() const;

	// @todo_sequencer_mvvm move this to the root view-model
	UE_API void SetSequence(UMovieSceneSequence* InRootSequence);

	UE_API TViewModelPtr<FSequenceModel> GetRootSequenceModel() const;

public:

	/** Adjust sequencer customizations based on the currently focused sequence */
	UE_API bool UpdateSequencerCustomizations(const UMovieSceneSequence* PreviousFocusedSequence);

	/** Get the active customization infos */
	UE_API TArrayView<const FSequencerCustomizationInfo> GetActiveCustomizationInfos() const;

	/** Build a combined menu extender */
	UE_API TSharedPtr<FExtender> GetSequencerMenuExtender(
			TSharedPtr<FExtensibilityManager> ExtensibilityManager, const TArray<UObject*>& ContextObjects,
			TFunctionRef<const FOnGetSequencerMenuExtender&(const FSequencerCustomizationInfo&)> Endpoint, FViewModelPtr InViewModel) const;

	UE_DEPRECATED(5.8, "GetPinnedTrackArea() is deprecated. Use GetTrackArea() as there's no need for a separate pinned track area.")
	UE_API TSharedPtr<FTrackAreaViewModel> GetPinnedTrackArea() const;

	/** Gets the current hotspots across any of our track areas */
	UE_API TSharedPtr<ITrackAreaHotspot> GetHotspot() const;

	void HandleDataHierarchyChanged() {}

protected:

	UE_API virtual void PreInitializeEditorImpl() override;
	UE_API virtual void InitializeEditorImpl() override;
	UE_API virtual TSharedPtr<FViewModel> CreateRootModelImpl() override;
	UE_API virtual TSharedPtr<FOutlinerViewModel> CreateOutlinerImpl() override;
	UE_API virtual TSharedPtr<FTrackAreaViewModel> CreateTrackAreaImpl() override;
	UE_API virtual TSharedPtr<FSequencerCoreSelection> CreateSelectionImpl() override;
	UE_API virtual USequencerScriptingLayer* CreateScriptingLayerImpl() override;
	UE_API virtual bool IsReadOnly() const override;

	UE_API void OnTrackAreaHotspotChanged(TSharedPtr<ITrackAreaHotspot> NewHotspot);

protected:

	/** Owning sequencer */
	TWeakPtr<ISequencer> WeakSequencer;
	
	/** Whether we have a curve editor extension */
	bool bSupportsCurveEditor;
	
	/** The current hotspot, from any of our track areas */
	TSharedPtr<ITrackAreaHotspot> CurrentHotspot;

	/** Active customizations. */
	TArray<TUniquePtr<ISequencerCustomization>> ActiveCustomizations;
	TArray<FSequencerCustomizationInfo> ActiveCustomizationInfos;
};

} // namespace UE::Sequencer

#undef UE_API
