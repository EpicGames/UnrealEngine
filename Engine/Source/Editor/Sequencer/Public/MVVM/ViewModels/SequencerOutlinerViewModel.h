// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/DragAndDrop.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MovieSceneTimeUnit.h"
#include "SequencerKeyCollection.h"
#include "ISequencerModule.h"

class FMenuBuilder;

namespace UE
{
namespace Sequencer
{

class FSequenceModel;

class FSequencerOutlinerViewModel : public FOutlinerViewModel
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE(FSequencerOutlinerViewModel, FOutlinerViewModel);

	FSequencerOutlinerViewModel();

	void BuildContextMenu(FMenuBuilder& MenuBuilder);
	void BuildCustomContextMenuForGuid(FMenuBuilder& MenuBuilder, FGuid ObjectBinding);

	/*~ FOutlinerViewModel */
	TSharedPtr<SWidget> CreateContextMenuWidget() override;
	TSharedRef<FDragDropOperation> InitiateDrag(TArray<TWeakViewModelPtr<IOutlinerExtension>>&& InDraggedModels) override;
	void RequestUpdate() override;

	FFrameNumber GetNextKey(const TArray<TSharedRef<UE::Sequencer::FViewModel>>& InNodes, FFrameNumber FrameNumber, EMovieSceneTimeUnit TimeUnit, const TRange<FFrameNumber>& Range, const bool bAllowLoop);
	FFrameNumber GetPreviousKey(const TArray<TSharedRef<UE::Sequencer::FViewModel>>& InNodes, FFrameNumber FrameNumber, EMovieSceneTimeUnit TimeUnit, const TRange<FFrameNumber>& Range, const bool bAllowLoop);

private:

	FFrameNumber GetNextKeyInternal(const TArray<TSharedRef<UE::Sequencer::FViewModel>>& InNodes, FFrameNumber FrameNumber, EMovieSceneTimeUnit TimeUnit, const TRange<FFrameNumber>& Range, EFindKeyDirection Direction, const bool bAllowLoop);

public:

	/** Called when the add menu is created */
	FOnGetAddMenuContent OnGetAddMenuContent;

	/** Called when object is clicked in track list */
	FOnBuildCustomContextMenuForGuid OnBuildCustomContextMenuForGuid;
};

} // namespace Sequencer
} // namespace UE

