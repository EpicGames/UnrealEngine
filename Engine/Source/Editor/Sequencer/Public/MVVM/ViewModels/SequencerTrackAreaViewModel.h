// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/TrackAreaViewModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "ISequencerModule.h"

class FMenuBuilder;
class ISequencer;

namespace UE
{
namespace Sequencer
{

class STrackAreaView;

class FSequencerTrackAreaViewModel
	: public FTrackAreaViewModel
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE(FSequencerTrackAreaViewModel, FTrackAreaViewModel);

	FSequencerTrackAreaViewModel(TSharedRef<ISequencer> InSequencer);

	void InitializeDefaultEditTools(UE::Sequencer::STrackAreaView& InTrackArea);

	virtual FFrameRate GetTickResolution() const override;
	virtual TRange<double> GetViewRange() const override;

private:

	TWeakPtr<ISequencer> WeakSequencer;
};

} // namespace Sequencer
} // namespace UE

