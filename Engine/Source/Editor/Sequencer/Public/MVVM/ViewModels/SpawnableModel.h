// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "Misc/Guid.h"

class UClass;
namespace UE::Sequencer { class FSequenceModel; }
namespace UE::Sequencer { template <typename T> struct TAutoRegisterViewModelTypeID; }
struct FMovieSceneBinding;
struct FMovieSceneSpawnable;
struct FSlateBrush;

namespace UE
{
namespace Sequencer
{

class FSpawnableModel
	: public FObjectBindingModel
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE(FSpawnableModel, FObjectBindingModel);

	FSpawnableModel(FSequenceModel* InOwnerModel, const FMovieSceneBinding& InBinding, const FMovieSceneSpawnable& InSpawnable);
	~FSpawnableModel();

	/*~ FObjectBindingModel */
	EObjectBindingType GetType() const override;
	const FSlateBrush* GetIconOverlayBrush() const override;
	FText GetTooltipForSingleObjectBinding() const override;
	const UClass* FindObjectClass() const override;

	/*~ FViewModel interface */
	void OnConstruct() override;

	/*~ FOutlinerItemModel interface */
	FText GetIconToolTipText() const override;

	/*~ FObjectBindingModel interface */
	void Delete() override;
	FSlateColor GetInvalidBindingLabelColor() const override { return FSlateColor::UseSubduedForeground(); }

};

} // namespace Sequencer
} // namespace UE

