// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/OutlinerItemModel.h"
#include "MVVM/Extensions/IGeometryExtension.h"
#include "MVVM/Extensions/IGroupableExtension.h"
#include "MVVM/Extensions/IRenameableExtension.h"
#include "MVVM/Extensions/IResizableExtension.h"
#include "MVVM/Extensions/ISortableExtension.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/Extensions/ILockableExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Extensions/IDeletableExtension.h"
#include "MVVM/Extensions/IDraggableOutlinerExtension.h"
#include "MVVM/Extensions/IDeletableExtension.h"
#include "MVVM/Extensions/IConditionableExtension.h"
#include "MVVM/Extensions/ISectionDragOwnerExtension.h"
#include "MVVM/Extensions/ISectionOwnerExtension.h"
#include "MVVM/Extensions/ITopLevelChannelHolderExtension.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "EventHandlers/ISignedObjectEventHandler.h"

#include "MovieSceneSignedObject.h"

#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API SEQUENCER_API

class ISequencer;
class ISequencerSection;
class ISequencerTrackEditor;
class UMovieSceneTrack;

namespace UE
{
namespace Sequencer
{

class FTrackModel
	: public FEvaluableOutlinerItemModel
	, public IRenameableExtension
	, public IResizableExtension
	, public ITrackExtension
	, public ISectionOwnerExtension
	, public ITopLevelChannelHolderExtension
	, public ITrackAreaExtension
	, public ILockableExtension
	, public IGroupableExtension
	, public ISortableExtension
	, public IDraggableOutlinerExtension
	, public IDeletableExtension
	, public IConditionableExtension
	, public IOutlinerDropTargetOutlinerExtension
	, public ISectionDragOwnerExtension
	, public UE::MovieScene::TIntrusiveEventHandler<UE::MovieScene::ISignedObjectEventHandler>
	, public UE::MovieScene::IDeferredSignedObjectFlushSignal
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FTrackModel
		, FEvaluableOutlinerItemModel
		, IRenameableExtension
		, IResizableExtension
		, ITrackExtension
		, ISectionOwnerExtension
		, ITopLevelChannelHolderExtension
		, ITrackAreaExtension
		, ILockableExtension
		, IGroupableExtension
		, ISortableExtension
		, IDraggableOutlinerExtension
		, IDeletableExtension
		, IConditionableExtension
		, IOutlinerDropTargetOutlinerExtension
		, ISectionDragOwnerExtension);

	UE_API explicit FTrackModel(UMovieSceneTrack* Track);
	UE_API ~FTrackModel();

public:

	static UE_API EViewModelListType GetTopLevelChannelType();
	static UE_API EViewModelListType GetTopLevelChannelGroupType();

	/*~ ITrackExtension */
	UE_API UMovieSceneTrack* GetTrack() const override;
	UE_API TSharedPtr<ISequencerTrackEditor> GetTrackEditor() const override;

	/*~ ISectionOwnerExtension */
	UE_API FViewModelChildren GetSectionModels() override;

	/*~ ITopLevelTrackHolderExtension */
	UE_API FViewModelChildren GetTopLevelChannels() override;

	/*~ ISignedObjectEventHandler */
	UE_API void OnModifiedDirectly(UMovieSceneSignedObject*) override;
	UE_API void OnModifiedIndirectly(UMovieSceneSignedObject*) override;

	/* IDeferredSignedObjectFlushSignal */
	UE_API virtual void OnDeferredModifyFlush() override;

	/*~ IOutlinerExtension */
	UE_API FOutlinerSizing GetOutlinerSizing() const override;
	UE_API TSharedPtr<SWidget> CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName) override;
	UE_API FSlateFontInfo GetLabelFont() const override;
	UE_API const FSlateBrush* GetIconBrush() const override;
	UE_API FText GetLabel() const override;
	UE_API FSlateColor GetLabelColor() const override;
	UE_API FText GetLabelToolTipText() const override;
	
	/*~ IDimmableExtension */
	UE_API bool IsDimmed() const override;

	/*~ IResizableExtension */
	UE_API bool IsResizable() const override;
	UE_API void Resize(float NewSize) override;

	/*~ ILockableExtension Interface */
	UE_API ELockableLockState GetLockState() const override;
	UE_API void SetIsLocked(bool bIsLocked) override;

	/*~ IConditionableExtension Interface */
	UE_API const UMovieSceneCondition* GetCondition() const override;
	UE_API EConditionableConditionState GetConditionState() const override;
	UE_API void SetConditionEditorForceTrue(bool bEditorForceTrue) override;

	/*~ ITrackAreaExtension */
	UE_API FTrackAreaParameters GetTrackAreaParameters() const override;
	UE_API FViewModelVariantIterator GetTrackAreaModelList() const override;
	UE_API FViewModelVariantIterator GetTopLevelChildTrackAreaModels() const override;

	/*~ ICurveEditorTreeItem */
	UE_API void CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels) override;

	/*~ IGroupableIdentifier */
	UE_API void GetIdentifierForGrouping(TStringBuilder<128>& OutString) const override;

	/*~ IRenameableExtension */
	UE_API bool CanRename() const override;
	UE_API void Rename(const FText& NewName) override;
	UE_API bool IsRenameValidImpl(const FText& NewName, FText& OutErrorMessage) const override;

	/*~ ISortableExtension */
	UE_API void SortChildren() override;
	UE_API FSortingKey GetSortingKey() const override;
	UE_API void SetCustomOrder(int32 InCustomOrder) override;

	/*~ IDraggableOutlinerExtension */
	UE_API bool CanDrag() const override;

	/*~ IOutlinerDropTargetOutlinerExtension */
	UE_API virtual TOptional<EItemDropZone> CanAcceptDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone) override;
	UE_API virtual void PerformDrop(const FViewModelPtr& TargetModel, const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone) override;

	/*~ FOutlinerItemModel */
	UE_API bool HasCurves() const override;
	UE_API void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	UE_API bool GetDefaultExpansionState() const override;
	UE_API void BuildSidebarMenu(FMenuBuilder& MenuBuilder) override;

	/*~ IDeletableExtension */
	UE_API bool CanDelete(FText* OutErrorMessage) const override;
	UE_API void Delete() override;

	/*~ IMutableExtension */
	UE_API bool IsMuted() const override;
	UE_API void SetIsMuted(bool bIsMuted) override;

	/*~ ISoloableExtension */
	UE_API bool IsSolo() const override;
	UE_API void SetIsSoloed(bool bIsSoloed) override;

	/*~ FViewModel interface */
	UE_API virtual void OnConstruct() override;


protected:

	/** Rebuilds the track's layout and child models. Override to customize track layout behavior. */
	UE_API virtual void ForceUpdate();

	/*~ ISectionDragOwnerExtension */
	UE_API virtual void OnBeginSectionVerticalDrag() override;
	UE_API virtual bool OnSectionVerticalDrag(const FSectionVerticalDragContext& Context) override;
	UE_API virtual void OnEndSectionVerticalDrag(const FSectionVerticalDragContext& Context) override;

protected:

	/** Shift all content at StartIndex and below down by 1 row */
	UE_API virtual void ShiftRowsDownFromIndex(int32 StartIndex, const TSet<UMovieSceneSection*>& ExcludeSections);

private:

	UE_API bool FindBoundObjects(TArray<UObject*>& OutBoundObjects) const;

	int32 GetMaxRowIndexForSectionDrag(const TSet<UMovieSceneSection*>& ExcludeSections) const;

	/** A second children list for the sections inside this track */
	FViewModelListHead SectionList;
	FViewModelListHead TopLevelChannelList;

	/** The actual track wrapped by this data model */
	TWeakObjectPtr<UMovieSceneTrack> WeakTrack;

	// @todo_sequencer_mvvm: move all the track editor behavior into the view model
	TSharedPtr<ISequencerTrackEditor> TrackEditor;

	bool bNeedsUpdate;

	int32 PreviousLayoutNumRows = -1;

	/** Tracks if this track was auto-expanded during a section drag operation */
	bool bAutoExpandedDuringDrag = false;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
