// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "HAL/PlatformCrt.h"
#include "ISectionLayoutBuilder.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Templates/Invoke.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"

class FText;
class ISequencerSection;
namespace UE::Sequencer { class FViewModel; }
struct FMovieSceneChannelHandle;

namespace UE
{
namespace Sequencer
{

class FSectionModel;

struct FHierarchicalModelListRefresher
{
	FHierarchicalModelListRefresher();
	FHierarchicalModelListRefresher(TSharedPtr<FViewModel> InRoot, FViewModelChildren InExistingChildren);

	// Delete copy operations
	FHierarchicalModelListRefresher(const FHierarchicalModelListRefresher&) = delete;
	FHierarchicalModelListRefresher& operator=(const FHierarchicalModelListRefresher&) = delete;

	// Allow move operations
	FHierarchicalModelListRefresher(FHierarchicalModelListRefresher&&) = default;
	FHierarchicalModelListRefresher& operator=(FHierarchicalModelListRefresher&&) = default;

	bool IsValid() const;
	void Reset();

	void Link(TSharedPtr<FViewModel> Item);

	void RecurseInto(TSharedPtr<FViewModel> Item, FViewModelChildren InExistingChildren);
	TSharedPtr<FViewModel> GetCurrentParent() const;
	EViewModelListType GetCurrentType() const;
	void Pop();

	template<typename ModelType, typename Predicate>
	TSharedPtr<ModelType> FindItem(Predicate&& InPredicate) const
	{
		TViewModelPtr<ModelType> Model = FindExistingItem<ModelType>(InPredicate);
		return Model ? Model : FindRecycledItem<ModelType>(InPredicate);
	}

	template<typename ModelType, typename Predicate>
	TSharedPtr<ModelType> FindExistingItem(Predicate&& InPredicate) const
	{
		// Look for existing children
		for (const TViewModelPtr<ModelType>& Item : ListData.Last().Children.IterateSubList<ModelType>())
		{
			if (Invoke(InPredicate, *Item))
			{
				return Item;
			}
		}

		return nullptr;
	}

	template<typename ModelType, typename Predicate>
	TViewModelPtr<ModelType> FindRecycledItem(Predicate&& InPredicate) const
	{
		// Look for recycled children
		for (const TViewModelPtr<ModelType>& Item : ListData.Last().Parent->GetChildrenOfType<ModelType>(EViewModelListType::Recycled))
		{
			if (Invoke(InPredicate, *Item))
			{
				return Item;
			}
		}

		return nullptr;
	}

private:

	void ConditionalRecycleChildren(const TSharedPtr<FViewModel>& InModel, FViewModelChildren InExistingChildren);

	struct FListData
	{
		explicit FListData(TSharedPtr<FViewModel> InParent, const FViewModelChildren& InExistingChildren);

		// The data model that owns the list we're refreshing.
		TSharedPtr<FViewModel> Parent;
		// The list we're refreshing.
		FViewModelChildren Children;
		// The tail of the list we're refreshing.
		TSharedPtr<FViewModel> AttachTail;
	};

	TArray<FListData, TInlineAllocator<8>> ListData;

	TArray<TArray<FScopedViewModelListHead>> RecycledLists;
};

class SEQUENCER_API FTrackModelLayoutBuilder
	: private ISectionLayoutBuilder
{
public:
	FTrackModelLayoutBuilder(TSharedPtr<FViewModel> InSharedOutlinerRoot);
	~FTrackModelLayoutBuilder();

	void RefreshLayout(TSharedPtr<FSectionModel> InSection);

public:

	// ISectionLayoutBuilder interface

	virtual void PushCategory( FName CategoryName, const FText& DisplayLabel, FGetMovieSceneTooltipText GetGroupTooltipTextDelegate, TFunction<TSharedPtr<UE::Sequencer::FCategoryModel>(FName, const FText&)> OptionalFactory ) override;
	virtual void SetTopLevelChannel( const FMovieSceneChannelHandle& Channel, TFunction<TSharedPtr<UE::Sequencer::FChannelModel>(FName, const FSectionModel&, const FMovieSceneChannelHandle&)> OptionalFactory ) override;
	virtual void AddChannel( const FMovieSceneChannelHandle& Channel, TFunction<TSharedPtr<UE::Sequencer::FChannelModel>(FName, const FSectionModel&, const FMovieSceneChannelHandle&)> OptionalFactory ) override;
	virtual void PopCategory() override;

private:

	void AddChannel( const FMovieSceneChannelHandle& Channel, bool bIsTopLevel, TFunction<TSharedPtr<UE::Sequencer::FChannelModel>(FName, const FSectionModel&, const FMovieSceneChannelHandle&)> OptionalFactory );

private:

	TSharedPtr<FViewModel> Root;
	FHierarchicalModelListRefresher OutlinerList;
	FHierarchicalModelListRefresher TrackAreaList;
	TSharedPtr<FSectionModel> Section;
};

} // namespace Sequencer
} // namespace UE

