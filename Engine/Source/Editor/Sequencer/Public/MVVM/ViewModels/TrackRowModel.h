// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/OutlinerItemModel.h"
#include "MVVM/Extensions/ISectionOwnerExtension.h"
#include "MVVM/Extensions/ITopLevelChannelHolderExtension.h"
#include "MVVM/Extensions/ITrackRowExtension.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"
#include "MVVM/Extensions/ILockableExtension.h"
#include "MVVM/Extensions/IRenameableExtension.h"
#include "MVVM/Extensions/IResizableExtension.h"
#include "MVVM/Extensions/IDeletableExtension.h"
#include "MVVM/Extensions/IConditionableExtension.h"
#include "MVVM/Extensions/IDraggableOutlinerExtension.h"

#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API SEQUENCER_API

class ISequencer;
class ISequencerTrackEditor;
class UMovieSceneTrack;

namespace UE
{
namespace Sequencer
{

class FSectionModel;

class FTrackRowModel
	: public FEvaluableOutlinerItemModel
	, public ITrackAreaExtension
	, public ILockableExtension
	, public ITrackRowExtension
	, public ISectionOwnerExtension
	, public ITopLevelChannelHolderExtension
	, public IRenameableExtension
	, public IResizableExtension
	, public IDeletableExtension
	, public IConditionableExtension
	, public IDraggableOutlinerExtension
{
public:

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FTrackRowModel
		, FEvaluableOutlinerItemModel
		, ITrackAreaExtension
		, ILockableExtension
		, ITrackRowExtension
		, ISectionOwnerExtension
		, ITopLevelChannelHolderExtension
		, IRenameableExtension
		, IResizableExtension
		, IDeletableExtension
		, IConditionableExtension
		, IDraggableOutlinerExtension);

	UE_API explicit FTrackRowModel(UMovieSceneTrack* InTrack, int32 InRowIndex);
	UE_API ~FTrackRowModel();

	UE_API void Initialize();

	/* Refreshes the layout of the row model including all child sections/other outliner items.
	 * Pass in whether to force children layout (e.g. if this is a new track row) */
	UE_API virtual void RefreshLayout(bool bChildrenNeedLayout=true);

	/*~ FOutlinerItemModel */
	UE_API void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	UE_API void BuildSidebarMenu(FMenuBuilder& MenuBuilder) override;
	UE_API void CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels) override;

	/*~ IOutlinerExtension */
	UE_API FOutlinerSizing GetOutlinerSizing() const override;
	UE_API TSharedPtr<SWidget> CreateOutlinerViewForColumn(const FCreateOutlinerViewParams& InParams, const FName& InColumnName) override;
	UE_API FText GetLabel() const override;
	UE_API FSlateColor GetLabelColor() const override;
	UE_API FSlateFontInfo GetLabelFont() const override;
	UE_API const FSlateBrush* GetIconBrush() const override;

	/*~ IRenameableExtension */
	UE_API bool CanRename() const override;
	UE_API void Rename(const FText& NewName) override;
	UE_API bool IsRenameValidImpl(const FText& NewName, FText& OutErrorMessage) const override;

	/*~ IResizableExtension */
	UE_API bool IsResizable() const override;
	UE_API void Resize(float NewSize) override;

	/*~ ITrackRowExtension */
	UE_API int32 GetRowIndex() const override;
	UE_API bool SetRowIndex(int32 NewRowIndex) override;
	UE_API UMovieSceneTrack* GetParentTrack() const override;

	/*~ ISectionOwnerExtension */
	UE_API FViewModelChildren GetSectionModels() override;

	/*~ ITopLevelChannelHolderExtension */
	UE_API FViewModelChildren GetTopLevelChannels() override;

	/*~ ITrackAreaExtension */
	UE_API FTrackAreaParameters GetTrackAreaParameters() const override;
	UE_API FViewModelVariantIterator GetTrackAreaModelList() const override;
	UE_API FViewModelVariantIterator GetTopLevelChildTrackAreaModels() const override;

	/*~ IDimmableExtension */
	UE_API bool IsDimmed() const override;

	/*~ IDeletableExtension */
	UE_API bool CanDelete(FText* OutErrorMessage) const override;
	UE_API void Delete() override;

	/*~ ILockableExtension Interface */
	UE_API ELockableLockState GetLockState() const override;
	UE_API void SetIsLocked(bool bIsLocked) override;

	/*~ IConditionableExtension Interface */
	UE_API const UMovieSceneCondition* GetCondition() const override;
	UE_API EConditionableConditionState GetConditionState() const override;
	UE_API void SetConditionEditorForceTrue(bool bEditorForceTrue) override;

	/*~ IMutableExtension */
	UE_API bool IsMuted() const override;
	UE_API void SetIsMuted(bool bIsMuted) override;

	/*~ ISolableExtension */
	UE_API bool IsSolo() const override;
	UE_API void SetIsSoloed(bool bIsSoloed) override;

	/*~ IDraggableOutlinerExtension */
	UE_API virtual bool CanDrag() const override;

	/*~ Deprecated ITrackExtension functions */
	UMovieSceneTrack* GetTrack() const { return GetParentTrack(); }
	TSharedPtr<ISequencerTrackEditor> GetTrackEditor() const { return TrackEditor; }

private:

	FViewModelListHead SectionList;
	FViewModelListHead TopLevelChannelList;
	TWeakObjectPtr<UMovieSceneTrack> WeakTrack;
	int32 RowIndex;

	// @todo_sequencer_mvvm: move all the track editor behavior into the view model
	TSharedPtr<ISequencerTrackEditor> TrackEditor;

	friend class FTrackModel;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
