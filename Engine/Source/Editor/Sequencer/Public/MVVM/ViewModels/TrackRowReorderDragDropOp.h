// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MVVM/ViewModels/OutlinerViewModelDragDropOp.h"
#include "MVVM/ViewModelPtr.h"

namespace UE
{
namespace Sequencer
{

class FTrackModel;
class FTrackRowModel;
class IOutlinerExtension;
class ITrackRowExtension;

/**
 * Drag/drop operation for reordering track rows/mixer layers within a track.
 */
class FTrackRowReorderDragDropOp : public FOutlinerViewModelDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FTrackRowReorderDragDropOp, FOutlinerViewModelDragDropOp)

	/**
	 * Construct a new drag/drop operation for reordering track rows
	 */
	static SEQUENCER_API TSharedRef<FTrackRowReorderDragDropOp> New(
		TArray<TWeakViewModelPtr<IOutlinerExtension>>&& InDraggedViewModels,
		FText InDefaultText,
		const FSlateBrush* InDefaultIcon);

	/**
	 * Check if all dragged track rows belong to the same parent track
	 * @return The parent track model, or nullptr if invalid/mixed parents
	 */
	SEQUENCER_API TSharedPtr<FTrackModel> GetCommonParentTrack() const;

	/**
	 * Execute the reorder operation, delegating to track row models
	 * @param TargetRowIndex The row index to move the dragged rows to
	 * @return true if the reorder succeeded
	 */
	SEQUENCER_API bool ExecuteReorder(int32 TargetRowIndex);

	/*
	* We override this to ensure the drop model is always the track and the target is always a track row even if we're dragging below
	* a nested channel.
	*/
	SEQUENCER_API virtual void GetTargetAndDropModel(const FViewModelPtr& CurrentViewModel, EItemDropZone InItemDropZone, FViewModelPtr& DropModel, FViewModelPtr& TargetModel) const override;

protected:
	/** Protected construction - use New() factory function */
	FTrackRowReorderDragDropOp()
	{}

private:
	/** Cached common parent track (validated to be same for all dragged rows) */
	mutable TWeakViewModelPtr<FTrackModel> CachedCommonParentTrack;
	mutable bool bCommonParentTrackCached = false;
};

} // namespace Sequencer
} // namespace UE
