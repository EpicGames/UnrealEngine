// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "MVVM/Extensions/ITrackLaneExtension.h"
#include "MVVM/Views/KeyRenderer.h"
#include "MVVM/Views/STrackAreaLaneView.h"
#include "MVVM/ViewModelPtr.h"

#include "Templates/SharedPointer.h"

#define UE_API SEQUENCER_API

class FSequencer;
struct FSequencerSelectedKey;

namespace UE::Sequencer
{

struct ITrackAreaHotspot;

class FTrackAreaViewModel;
class FChannelModel;
class FSectionModel;
class STrackLane;

/** Cache structure that stores all the sequencer-specific things that contribute to the cached screen-space renderered key states */
struct FChannelViewKeyCachedState
{
	/** The current track area hotspot */
	TWeakPtr<ITrackAreaHotspot> WeakHotspot;
	/** DO NOT DEREFERENCE. Raw ptr to the last used hotspot. Only used to differentiate an expired hotspot from an explicitly null one. */
	ITrackAreaHotspot* RawHotspotDoNotUse = nullptr;
	/** Weak ptr to the last used non-linear transform for rendering these keys */
	TWeakPtr<INonLinearTimeTransform> NonLinearTransform;
	/** The min/max tick value relating to the FMovieSceneSubSequenceData::ValidPlayRange bounds, or the current playback range */
	FFrameNumber ValidPlayRangeMin, ValidPlayRangeMax;

	/** The current view range */
	TRange<FFrameTime> VisibleRange;

	/** The size of the keys to draw */
	FVector2D KeySizePx;

	/** The value of FSequencerSelection::GetSerialNumber when this cache was created */
	uint32 SelectionSerial;

	/** The value of FSequencerSelectionPreview::GetSelectionHash when this cache was created */
	uint32 SelectionPreviewHash;

	/** The value of FSequencerFilterBar::GetFilterSerial from the active filter bar when this cache was created */
	uint32 FilterSerial;

	/** Whether to draw a visual representation of the curve on this track as well */
	bool bShowCurve;

	/** Whether to show key bars */
	bool bShowKeyBars;

	/** Whether to gather keys for all children and collapse them into this renderer (normally only true when an item is collapsed) */
	bool bCollapseChildren;

	/** Whether this channel is hovered */
	bool bIsChannelHovered;

	/** Default constructor for SWidget construction - not to be used under other circumstances */
	FChannelViewKeyCachedState();

	/** Real constructor for use when rendering */
	FChannelViewKeyCachedState(TRange<FFrameTime> InVisibleRange, TSharedPtr<ITrackAreaHotspot> Hotspot, FViewModelPtr Model, FSequencer* Sequencer);

	/** Compare this cache to another, returning what (if anything) has changed */
	EViewDependentCacheFlags CompareTo(const FChannelViewKeyCachedState& Other) const;
};

/**
 * Widget for displaying a channel within a section
 */
class SChannelView
	: public STrackAreaLaneView
{
public:
	SLATE_BEGIN_ARGS(SChannelView) {}
		SLATE_ATTRIBUTE(FLinearColor, KeyBarColor)
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const FViewModelPtr& InViewModel, TSharedPtr<STrackAreaView> InTrackAreaView);

	/*~ SWidget */
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

protected:

	UE_API int32 DrawLane(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;

	UE_API void CreateKeysUnderMouse(const FVector2D& MousePosition, const FGeometry& AllottedGeometry, const TSet<FSequencerSelectedKey>& InPressedKeys, TArray<FSequencerSelectedKey>& OutKeys);
	UE_API void GetKeysUnderMouse(const FVector2D& MousePosition, const FGeometry& AllottedGeometry, TArray<FSequencerSelectedKey>& OutKeys) const;
	UE_API void GetKeysAtPixelX(float LocalMousePixelX, TArray<FSequencerSelectedKey>& OutKeys) const;

	UE_API TViewModelPtr<FSectionModel> GetSection() const;

	UE_API TSharedPtr<FSequencer> LegacyGetSequencer() const;

	UE_API void EndCreateKeyTransactionIfActive();

protected:

	FKeyRenderer KeyRenderer;
	mutable TOptional<FChannelViewKeyCachedState> KeyRendererCache;

	TAttribute<FLinearColor> KeyBarColor;

	/** True while a Create Key transaction started in OnMouseButtonDown is still open, so OnMouseButtonUp can close it regardless of which button or modifier state arrives on release. */
	bool bCreateKeyTransactionActive = false;
};

} // namespace UE::Sequencer

#undef UE_API
