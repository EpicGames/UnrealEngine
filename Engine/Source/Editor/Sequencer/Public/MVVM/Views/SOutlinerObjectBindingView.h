// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Misc/Guid.h"

#include "MVVM/Views/SOutlinerItemViewBase.h"
#include "MVVM/ViewModels/OutlinerItemModel.h"

#include "PropertyPath.h"

namespace UE
{
namespace Sequencer
{

class FObjectBindingModel;
class FSequencerEditorViewModel;

/**
 * A widget for displaying an object binding in the sequencer outliner
 */
class SOutlinerObjectBindingView
	: public SOutlinerItemViewBase
{
public:

	void Construct(
			const FArguments& InArgs, 
			TSharedPtr<FObjectBindingModel> InViewModel, 
			TSharedPtr<FSequencerEditorViewModel> InEditor,
			const TSharedRef<ISequencerTreeViewRow>& InTableRow);
};

} // namespace Sequencer
} // namespace UE

