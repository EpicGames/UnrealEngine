// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerCoreFwd.h"
#include "Widgets/SCompoundWidget.h"
#include "MVVM/ViewModelPtr.h"

class UMovieSceneTrack;

namespace UE::Sequencer
{

class IOutlinerExtension;
class FEditorViewModel;

class SOutlinerTrackColorPicker
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SOutlinerTrackColorPicker){}
	SLATE_END_ARGS()

	SEQUENCER_API void Construct(const FArguments& InArgs, TWeakViewModelPtr<IOutlinerExtension> InWeakOutlinerExtension, const TSharedPtr<FEditorViewModel>& EditorViewModel);

private:

	FSlateColor GetTrackColorTint() const;

	/** Called when the user clicks the track color */
	FReply OnSetTrackColor();
	void OnColorPickerPicked(FLinearColor NewFolderColor);
	void OnColorPickerClosed(const TSharedRef<SWindow>& Window);
	void OnColorPickerCancelled(FLinearColor NewFolderColor);

	/** Gets the track out of the underlying Node structure. Can return null. */
	UMovieSceneTrack* GetTrackFromNode() const;

private:

	TWeakViewModelPtr<IOutlinerExtension> WeakOutlinerExtension;
};

} // namespace UE::Sequencer
