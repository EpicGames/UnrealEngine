// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerCoreFwd.h"
#include "MVVM/Views/SOutlinerItemViewBase.h"

class UMovieSceneTrack;

namespace UE
{
namespace Sequencer
{

class FViewModel;
class FSequencerEditorViewModel;

/**
 * A widget for displaying a track in the sequencer outliner
 */
class SOutlinerTrackView
	: public SOutlinerItemViewBase
{
public:

	void Construct(
			const FArguments& InArgs, 
			TWeakViewModelPtr<IOutlinerExtension> InWeakOutlinerExtension,
			TWeakPtr<FSequencerEditorViewModel> InWeakEditor,
			const TSharedRef<ISequencerTreeViewRow>& InTableRow);

	FSlateColor GetTrackColorTint() const;

	/** Called when the user clicks the track color */
	FReply OnSetTrackColor();
	void OnColorPickerPicked(FLinearColor NewFolderColor);
	void OnColorPickerClosed(const TSharedRef<SWindow>& Window);
	void OnColorPickerCancelled(FLinearColor NewFolderColor);

	/** Gets the track out of the underlying Node structure. Can return null. */
	UMovieSceneTrack* GetTrackFromNode() const;
};

} // namespace Sequencer
} // namespace UE

