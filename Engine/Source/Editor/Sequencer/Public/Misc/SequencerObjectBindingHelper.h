// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "PropertyPath.h"

class ISequencer;
class UClass;
class UStruct;

/**
 * Helper class for using object bindings in the Sequencer editor.
 */
class FSequencerObjectBindingHelper
{
public:

	/**
	 * Gets the keyable properties on the given object, for the given Sequencer editor.
	 *
	 * @param Object            The object to key.
	 * @param Sequencer         The Sequencer that will be responsible for animating the properties.
	 * @param KeyablePropertyPaths  The list of properties that can be keyed.
	 */
	SEQUENCER_API static void GetKeyablePropertyPaths(const UObject* Object, TSharedRef<ISequencer> Sequencer, TArray<FPropertyPath>& KeyablePropertyPaths);

	/**
	 * Gets the keyable properties on the given object, for the given Sequencer editor.
	 *
	 * @param Class             The class of the UObject to animate.
	 * @param ValuePtr          A pointer to object, or one of its struct members.
	 * @param PropertySource    The type of the ValuePtr.
	 * @param PropertyPath      The path to ValuePtr (should be empty if ValuePtr is the object itself).
	 * @param Sequencer         The Sequencer that will be responsible for animating the properties.
	 * @param KeyablePropertyPaths  The list of properties that can be keyed.
	 */
	SEQUENCER_API static void GetKeyablePropertyPaths(const UClass* Class, const void* ValuePtr, const UStruct* PropertySource, FPropertyPath PropertyPath, TSharedRef<ISequencer> Sequencer, TArray<FPropertyPath>& KeyablePropertyPaths);

};

