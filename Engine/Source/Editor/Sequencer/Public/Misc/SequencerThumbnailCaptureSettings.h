// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencer.h"
#include "Misc/FrameNumber.h"
#include "SequencerThumbnailCaptureSettings.generated.h"

class ISequencer;
class UMovieSceneSection;

/** Specifies which frame should be captured in a track. */
UENUM()
enum class ESequencerThumbnailCaptureTimeLocation : uint8
{
	/** Use the first frame */
	FirstFrame,
	/** Use the middle frame */
	MiddleFrame,
	/** Use the last frame */
	LastFrame,
	/** Use the frame at which the scrubber is currently positioned */
	CurrentFrame
};

/** Configures how a thumbnail is supposed to be captured for a level sequence */
USTRUCT()
struct FSequencerThumbnailCaptureSettings
{
	GENERATED_BODY()

	/** Specifies which frame should be captured in a track. */
	UPROPERTY(EditAnywhere, Category = General)
	ESequencerThumbnailCaptureTimeLocation CaptureFrameLocationRule = ESequencerThumbnailCaptureTimeLocation::CurrentFrame;
};

namespace UE::Sequencer
{
	/** @return The frame that Rule wants to get from Sequencer */
	SEQUENCER_API FFrameNumber GetFrameByRule(const ISequencer& Sequencer, ESequencerThumbnailCaptureTimeLocation Rule);
}