// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/NameTypes.h"

#define UE_API SEQUENCER_API

class UBlueprint;
class UMovieSceneSequence;

struct FMovieSceneSequenceEditor
{
	virtual ~FMovieSceneSequenceEditor()= default;


	/**
	 * Attempt to find a sequence editor for the specified sequence
	 *
	 * @param Sequence        The sequence to get an editor for
	 * @return The sequence editor ptr, or null if one is not available for the specified type of sequence
	 */
	static UE_API FMovieSceneSequenceEditor* Find(UMovieSceneSequence* InSequence);

	/**
	 * Check whether the specified sequence supports events
	 *
	 * @param Sequence        The sequence to test
	 */
	UE_API bool SupportsEvents(UMovieSceneSequence* InSequence) const;

	/**
	 * Check whether the specified sequence supports director blueprints
	 *
	 * @param Sequence        The sequence to test
	 */
	UE_API bool SupportsDirectorBlueprint(UMovieSceneSequence* Sequence) const;

	/**
	 * Access the director blueprint for the specified sequence
	 *
	 * @param Sequence        The sequence to access the director blueprint for
	 * @return The sequence's director blueprint or nullptr if it does not have one (or cannot)
	 */
	UE_API UBlueprint* FindDirectorBlueprint(UMovieSceneSequence* Sequence) const;

	/**
	 * Access the director blueprint for the specified sequence
	 *
	 * @param Sequence        The sequence to access the director blueprint for
	 * @return The sequence's director blueprint or nullptr if it does not have one (or cannot)
	 */
	UE_API UBlueprint* GetOrCreateDirectorBlueprint(UMovieSceneSequence* Sequence) const;

private:

	virtual bool CanCreateEvents(UMovieSceneSequence* InSequence) const { return false; }
	virtual bool CanCreateDirectorBlueprint(UMovieSceneSequence* Sequence) const { return false; }
	virtual UBlueprint* GetBlueprintForSequence(UMovieSceneSequence* InSequence) const { return nullptr; }
	virtual UBlueprint* CreateBlueprintForSequence(UMovieSceneSequence* InSequence) const { return nullptr; }
};

#undef UE_API
