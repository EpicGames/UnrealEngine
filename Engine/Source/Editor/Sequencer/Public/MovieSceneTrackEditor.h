// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "ISequencer.h"
#include "Framework/Commands/UICommandList.h"
#include "ScopedTransaction.h"
#include "MovieSceneTrack.h"
#include "ISequencerTrackEditor.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"

#define UE_API SEQUENCER_API

class FMenuBuilder;
class FPaintArgs;
class FSlateWindowElementList;
class SHorizontalBox;

/**
 * Result of keying
 */
struct FKeyPropertyResult
{
	FKeyPropertyResult()
		: bTrackModified(false)
		, bHandleCreated(false)
		, bTrackCreated(false)
		, bKeyCreated(false) {}

	inline void operator |= (const FKeyPropertyResult& A)
	{
		bTrackModified |= A.bTrackModified;
		bHandleCreated |= A.bHandleCreated;
		bTrackCreated |= A.bTrackCreated;
		bKeyCreated |= A.bKeyCreated;
		SectionsCreated.Append(A.SectionsCreated);
		SectionsKeyed.Append(A.SectionsKeyed);
	}

	/* Was the track modified in any way? */
	bool bTrackModified;

	/* Was a handle/binding created? */
	bool bHandleCreated;

	/* Was a track created? */
	bool bTrackCreated;

	/* Was a key created? */
	bool bKeyCreated;

	/* Was a section created */
	TArray<TWeakObjectPtr<UMovieSceneSection> > SectionsCreated;

	/* Was a section keyed */
	TArray<TWeakObjectPtr<UMovieSceneSection> > SectionsKeyed;
};

/** Delegate for adding keys for a property
 * FFrameNumber - The time at which to add the key.
 * return - KeyPropertyResult - 
 */
DECLARE_DELEGATE_RetVal_OneParam(FKeyPropertyResult, FOnKeyProperty, FFrameNumber)


/** Delegate for whether a property can be keyed
 * FFrameNumber - The time at which to add the key.
 * return - True if the property can be keyed, otherwise false.
 */
DECLARE_DELEGATE_RetVal_OneParam(bool, FCanKeyProperty, FFrameNumber)


/**
 * Base class for handling key and section drawing and manipulation
 * of a UMovieSceneTrack class.
 *
 * @todo Sequencer Interface needs cleanup
 */
class FMovieSceneTrackEditor
	: public TSharedFromThis<FMovieSceneTrackEditor>
	, public ISequencerTrackEditor
{
public:

	/** Constructor */
	UE_API FMovieSceneTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Destructor */
	UE_API virtual ~FMovieSceneTrackEditor();

public:

	/** @return The current movie scene */
	UE_API UMovieSceneSequence* GetMovieSceneSequence() const;

	/**
	 * @return The current local time at which we should add a key
	 */
	UE_API FFrameNumber GetTimeForKey();
	
	/**
	 * Initiate keying when there is more than one object to key (ie. drag and drop assets). 
	 * This will allow for different behaviors, ie. keying all at the current time or keying 
	 * one after the other.
	 */
	static UE_API void BeginKeying(FFrameNumber InFrameNumber);
	static UE_API void EndKeying();

	UE_API void UpdatePlaybackRange();

	UE_API void AnimatablePropertyChanged( FOnKeyProperty OnKeyProperty );

	struct FFindOrCreateHandleResult
	{
		FGuid Handle;
		bool bWasCreated;
	};
	
	UE_API FFindOrCreateHandleResult FindOrCreateHandleToObject( UObject* Object, bool bCreateHandleIfMissing = true, const FName& CreatedFolderName = NAME_None );

	struct FFindOrCreateTrackResult
	{
		FFindOrCreateTrackResult() : Track(nullptr), bWasCreated(false) {}

		UMovieSceneTrack* Track;
		bool bWasCreated;
	};

	UE_API FFindOrCreateTrackResult FindOrCreateTrackForObject( const FGuid& ObjectHandle, TSubclassOf<UMovieSceneTrack> TrackClass, FName PropertyName = NAME_None, bool bCreateTrackIfMissing = true );

	template<typename TrackClass>
	struct FFindOrCreateRootTrackResult
	{
		FFindOrCreateRootTrackResult() : Track(nullptr), bWasCreated(false) {}

		TrackClass* Track;
		bool bWasCreated;
	};

	/**
	 * Find or add a track of the specified type in the focused movie scene.
	 *
	 * @param TrackClass The class of the track to find or add.
	 * @return The track results.
	 */
	template<typename TrackClass>
	FFindOrCreateRootTrackResult<TrackClass> FindOrCreateRootTrack()
	{
		FFindOrCreateRootTrackResult<TrackClass> Result;
		bool bTrackExisted;

		UMovieScene* MovieScene = GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene();
		Result.Track = MovieScene->template FindTrack<TrackClass>();
		bTrackExisted = Result.Track != nullptr;

		if (Result.Track == nullptr)
		{
			Result.Track = MovieScene->template AddTrack<TrackClass>();
		}

		Result.bWasCreated = bTrackExisted == false && Result.Track != nullptr;
		return Result;
	}

	/** @return The sequencer bound to this handler */
	UE_API TSharedPtr<ISequencer> GetSequencer() const;

public:

	// ISequencerTrackEditor interface

	UE_API virtual FText GetDisplayName() const override;

	UE_API virtual UMovieSceneTrack* AddTrack(UMovieScene* FocusedMovieScene, const FGuid& ObjectHandle, TSubclassOf<class UMovieSceneTrack> TrackClass, FName UniqueTypeName) override;

	UE_API virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;

	UE_API virtual void BindCommands(TSharedRef<FUICommandList> SequencerCommandBindings) override;
	UE_API virtual void BuildPinnedAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	UE_API virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	UE_API virtual void BuildObjectBindingColumnWidgets(TFunctionRef<TSharedRef<SHorizontalBox>()> GetEditBox, const UE::Sequencer::TViewModelPtr<UE::Sequencer::FObjectBindingModel>& ObjectBinding, const UE::Sequencer::FCreateOutlinerViewParams& InParams, const FName& InColumnName) override;
	UE_API virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	UE_API virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	UE_API virtual TSharedPtr<SWidget> BuildOutlinerEditColumnWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	UE_API virtual TSharedPtr<SWidget> BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName) override;
	UE_API virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override;
	UE_API virtual void BuildTrackSidebarMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override;
	UE_API virtual bool HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) override;
	UE_API virtual bool OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams) override;
	UE_API virtual FReply OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams) override;

	UE_API virtual void OnInitialize() override;
	UE_API virtual void OnRelease() override;

	virtual bool SupportsType( TSubclassOf<class UMovieSceneTrack> TrackClass ) const override = 0;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override { return true; }
	UE_API virtual void Tick(float DeltaTime) override;

protected:

	/**
	 * Gets the currently focused movie scene, if any.
	 *
	 * @return Focused movie scene, or nullptr if no movie scene is focused.
	 */
	UE_API UMovieScene* GetFocusedMovieScene() const;

private:

	/** The sequencer bound to this handler.  Used to access movie scene and time info during auto-key */
	TWeakPtr<ISequencer> WeakSequencerPtr;

	/** The key time to use during a multi key operation. Only used if bKeying is true */
	static UE_API TOptional<FFrameNumber> NextKeyTime;

	/** Indicates whether we're currently in a keying operation where multiple keys may be created (ie. drag and drop) */
	static UE_API bool bKeying;
};

#undef UE_API
