// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Scripting/SequencerScriptingLayer.h"

#include "SequencerModuleScriptingLayer.generated.h"

UCLASS(MinimalAPI)
class USequencerModuleScriptingLayer : public USequencerScriptingLayer
{
public:

	GENERATED_BODY()

	virtual void Initialize(TSharedPtr<UE::Sequencer::FEditorViewModel> InViewModel);

	/** Retrieve the outliner */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	USequencerModuleOutlinerScriptingObject* GetOutliner();
};