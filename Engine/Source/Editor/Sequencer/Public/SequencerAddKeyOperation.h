// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "SequencerCoreFwd.h"
#include "Misc/FrameNumber.h"
#include "SequencerKeyParams.h"
#include "Templates/SharedPointer.h"

#define UE_API SEQUENCER_API

class IKeyArea;
class ISequencer;
class ISequencerSection;
class ISequencerDecorationEditor;
class ISequencerTrackEditor;
class UMovieSceneSection;
class UMovieSceneTrack;
struct FFrameNumber;

namespace UE::Sequencer
{
	class FChannelModel;
	struct FKeyOperation;
}

namespace UE
{
namespace Sequencer
{

class FChannelGroupModel;
class FViewModel;
class ITrackExtension;
class ISectionOwnerExtension;
class IOutlinerExtension;

/**
 * Temporary structure used for consistent add-key behavior for a set of display nodes
 * Ultimately the operation will call ISequencerTrackEditor::ProcessKeyOperation for each track editor that needs to add keys.
 */
struct FAddKeyOperation
{
	/**
	 * Construct an operation from any set of display nodes. Each node in the set will receive keys for all decendant key areas.
	 *
	 * @param InModels A set of all the models to key
	 */
	static UE_API FAddKeyOperation FromNodes(const TSet<TWeakViewModelPtr<IOutlinerExtension>>& InModels);


	/**
	 * Construct an operation from a single display node. Every key area underneath this node will receive keys.
	 */
	static UE_API FAddKeyOperation FromNode(TWeakPtr<FViewModel> InModel);


	/**
	 * Construct an operation from some key areas on a track.
	 */
	static UE_API FAddKeyOperation FromKeyAreas(ISequencerTrackEditor* TrackEditor, const TArrayView<TSharedRef<IKeyArea>> InKeyAreas);


	/**
	 * Construct an operation from a set of channel models. Each channel model's key area will receive a key.
	 */
	static UE_API FAddKeyOperation FromChannelModels(const TSet<TWeakViewModelPtr<FChannelModel>>& InChannelModels);


	/**
	 * Commit this operation by choosing the section(s) to key for each key area, and adding a key at the specified time
	 *
	 * @param InKeyTime     The time to add keys at
	 * @param InSequencer   The sequencer instance that is performing this operations
	 * &param OutResults    The key handles that were added with this operation
	 */
	UE_API void Commit(FFrameNumber InKeyTime, ISequencer& InSequencer, TArray<FAddKeyResult> *OutResults = nullptr);

private:

	/**
	 * Add a set of nodes to this operation that have already had child nodes removed (ie only parent nodes should exist in the set)
	 *
	 * @param InNodes     A set of nodes to add to this operation that contains no child nodes
	 */
	UE_API void AddPreFilteredNodes(TArrayView<const TWeakPtr<FViewModel>> InNodes);


	/**
	 * Add any keyable areas to the list of potential things to key
	 *
	 * @param InTrackEditor         The track editor
	 * @param InKeyAnythingBeneath  A node to search within for key areas
	 */
	UE_API bool ConsiderKeyableAreas(TViewModelPtr<ISectionOwnerExtension> InSectionOwner, FViewModelPtr InKeyAnythingBeneath);


	/**
	 * Add key areas for a key area display node to this operation
	 *
	 * @param InTrackNode         The current track node
	 * @param InKeyAreaNode       The key area node to add key areas from
	 */
	UE_API bool ProcessKeyArea(ISequencerTrackEditor* InTrackEditor, TViewModelPtr<FChannelGroupModel> InChannelGroupModel);


	/**
	 * Add a key area to this operation
	 *
	 * @param InTrackEditor       The track editor responsible for the key area
	 * @param InKeyArea           The key area to add
	 */
	UE_API bool ProcessKeyArea(ISequencerTrackEditor* InTrackEditor, TSharedPtr<IKeyArea> InKeyArea);


	/**
	 * Retrieve the operation that relates to a specific track editor instance
	 */
	UE_API FKeyOperation& GetTrackOperation(ISequencerTrackEditor* TrackEditor);

private:

	FAddKeyOperation() {}

	/** Map of key operations stored by their track editor. */
	TMap<ISequencerTrackEditor*, FKeyOperation> OperationsByTrackEditor;

	/** Pending decoration key operations collected during ProcessKeyArea. */
	struct FDecorationKeyOp
	{
		UMovieSceneSection* Section;
		FMovieSceneChannelHandle ChannelHandle;
	};
	TArray<FDecorationKeyOp> DecorationKeyOperations;
};

} // namespace Sequencer
} // namespace UE

#undef UE_API
