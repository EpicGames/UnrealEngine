// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Misc/Optional.h"

#define UE_API SEQUENCER_API

class ISequencer;

namespace UE::Sequencer
{

	class FSequencerBakingSetupRestore 
	{
	public:
		FSequencerBakingSetupRestore() = delete;
		UE_API FSequencerBakingSetupRestore(TSharedPtr<ISequencer>& SequencerPtr);
		UE_API ~FSequencerBakingSetupRestore();
		
	private:
		TWeakPtr<ISequencer> WeakSequencer;
		TOptional<bool> bRestoreShouldEvaluateSubSequencesInIsolation;
		
	};

} // namespace UE::Sequencer

#undef UE_API
