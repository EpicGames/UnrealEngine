// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerChannelInterface.h"
#include "SequencerChannelTraits.h"
#include "MovieSceneSection.h"
#include "IKeyArea.h"
#include "CurveModel.h"

namespace Sequencer
{

/** Concept for detecting deprecated CreateCurveEditorModel signatures that will no longer be called in later versions */
struct CLegacyCurveModelCreatable
{
	template<typename T>
	auto Requires()
		-> decltype(CreateCurveEditorModel(*((TMovieSceneChannelHandle<T>*)0), (UMovieSceneSection*)nullptr, TSharedPtr<ISequencer>().ToSharedRef()));
};

struct CLegacyKeyEditorCreatable
{
	template<typename T>
	auto Requires()
		-> decltype(CreateKeyEditor(
			*((TMovieSceneChannelHandle<T>*)nullptr),
			(UMovieSceneSection*)nullptr,
			FGuid(),
			TWeakPtr<FTrackInstancePropertyBindings>(),
			TWeakPtr<ISequencer>()
		));
};

template<typename T>
UE_DEPRECATED(5.5, "CreateCurveEditorModel(const TMovieSceneChannelHandle<T>&, UMovieSceneSection*, TSharedRef<ISequencer>) has been deprecated. Please update your signature to use FCreateCurveEditorModelParams")
void CreateCurveEditorModelDeprecatedSignature()
{
}

template<typename T>
UE_DEPRECATED(5.5, "CreateKeyEditor(const TMovieSceneChannelHandle<T>&, UMovieSceneSection*, const FGuid&, TWeakPtr<FTrackInstancePropertyBindings>, TWeakPtr<ISequencer>); has been deprecated. Please update your signature to use FCreateKeyEditorParams")
void CreateKeyEditorDeprecatedSignature()
{
}

} // namespace Sequencer

/**
 * Templated channel interface that calls overloaded functions matching the necessary channel types.
 * Designed this way to allow for specific customization of key-channel behavior without having to reimplement swathes of boilerplate.
 * This base interface implements common functions that do not require extended editor data.
 *
 * Behavior can be overridden for any channel type by declaring an overloaded function for the relevant channel type in the same namespace as the channel.
 * For instance, to implement how to retrieve key times from a channel, implement the following function:
 *
 * void GetKeyTimes(FMyChannelType* InChannel, TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes);
 */
template<typename ChannelType>
struct TSequencerChannelInterfaceCommon : ISequencerChannelInterface
{
	/**
	 * Delete the specified keys. If all keys are removed, the current value at that time will be set as the default value for the channel
	 *
	 * @param Channel               The channel to remove the keys from
	 * @param InHandles             The key handles to delete
	 * @param InTime                The time at which to evaluate for the default value if there are no keys remaining
	 */
	virtual void DeleteKeys_Raw(FMovieSceneChannel* InChannel, TArrayView<const FKeyHandle> InHandles, FFrameNumber InTime) const override
	{
		using namespace Sequencer;
		DeleteKeys(static_cast<ChannelType*>(InChannel), InHandles, InTime);
	}

	/**
	 * Copy all the keys specified in KeyMask to the specified clipboard
	 *
	 * @param Channel               The channel to copy from
	 * @param Section               The section that owns the channel
	 * @param KeyAreaName           The name of the key area
	 * @param ClipboardBuilder      The structure responsible for building clipboard information for each key
	 * @param KeyMask               A specific set of keys to copy
	 */
	virtual void CopyKeys_Raw(FMovieSceneChannel* InChannel, const UMovieSceneSection* Section, FName KeyAreaName, FMovieSceneClipboardBuilder& ClipboardBuilder, TArrayView<const FKeyHandle> KeyMask) const override
	{
		using namespace Sequencer;
		CopyKeys(static_cast<ChannelType*>(InChannel), Section, KeyAreaName, ClipboardBuilder, KeyMask);
	}

	/**
	 * Paste the specified key track into the specified channel
	 *
	 * @param Channel               The channel to copy from
	 * @param Section               The section that owns the channel
	 * @param KeyTrack              The source clipboard data to paste
	 * @param SrcEnvironment        The environment the source data was copied from
	 * @param DstEnvironment        The environment we're pasting into
	 * @param OutPastedKeys         Array to receive key handles for any pasted keys
	 */
	virtual void PasteKeys_Raw(FMovieSceneChannel* InChannel, UMovieSceneSection* Section, const FMovieSceneClipboardKeyTrack& KeyTrack, const FMovieSceneClipboardEnvironment& SrcEnvironment, const FSequencerPasteEnvironment& DstEnvironment, TArray<FKeyHandle>& OutPastedKeys) const override
	{
		using namespace Sequencer;
		PasteKeys(static_cast<ChannelType*>(InChannel), Section, KeyTrack, SrcEnvironment, DstEnvironment, OutPastedKeys);
	}

	/**
	 * Get an editable key struct for the specified key
	 *
	 * @param Channel               The channel on which the key resides
	 * @param KeyHandle             Handle of the key to get
	 * @return A shared editable key struct
	 */
	virtual TSharedPtr<FStructOnScope> GetKeyStruct_Raw(FMovieSceneChannelHandle InChannel, FKeyHandle KeyHandle) const override
	{
		using namespace Sequencer;
		return GetKeyStruct(InChannel.Cast<ChannelType>(), KeyHandle);
	}

	/**
	 * Check whether an editor on the sequencer node tree can be created for the specified channel
	 *
	 * @param Channel               The channel to check
	 * @return true if a key editor should be constructed, false otherwise
	 */
	virtual bool CanCreateKeyEditor_Raw(const FMovieSceneChannel* InChannel) const override
	{
		using namespace Sequencer;
		return CanCreateKeyEditor(static_cast<const ChannelType*>(InChannel));
	}

	/**
	 * Extend the key context menu
	 *
	 * @param MenuBuilder           The menu builder used to create this context menu
	 * @param Channels              Array of channels and handles that are being shown in the context menu
	 * @param InSequencer           The currently active sequencer
	 */
	virtual void ExtendKeyMenu_Raw(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, TArrayView<const FExtendKeyMenuParams> ChannelsAndHandles, TWeakPtr<ISequencer> InSequencer) const override
	{
		using namespace Sequencer;
		TArray<TExtendKeyMenuParams<ChannelType>> TypedChannels;

		for (const FExtendKeyMenuParams& Ptr : ChannelsAndHandles)
		{
			TExtendKeyMenuParams<ChannelType> TypedChannelAndHandles;
			TypedChannelAndHandles.Section   = Ptr.Section;
			TypedChannelAndHandles.WeakOwner = Ptr.WeakOwner;
			TypedChannelAndHandles.Handles   = Ptr.Handles;
			TypedChannelAndHandles.Channel   = Ptr.Channel.Cast<ChannelType>();

			TypedChannels.Add(MoveTemp(TypedChannelAndHandles));
		}

		ExtendKeyMenu(MenuBuilder, MenuExtender, MoveTemp(TypedChannels), InSequencer);
	}

	/**
	 * Extend the section context menu
	 *
	 * @param MenuBuilder           The menu builder used to create this context menu
	 * @param InMenuExtender        The menu extender to use
	 * @param InChannels            Array of type specific channels that exist in the selected sections
	 * @param InWeakSections        Array of sections being shown on the context menu
	 * @param InWeakSequencer       The currently active sequencer
	 */
	virtual void ExtendSectionMenu_Raw(FMenuBuilder& MenuBuilder
		, TSharedPtr<FExtender> InMenuExtender
		, TArrayView<const FMovieSceneChannelHandle> InChannels
		, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections
		, TWeakPtr<ISequencer> InWeakSequencer) const override
	{
		using namespace Sequencer;
		TArray<TMovieSceneChannelHandle<ChannelType>> TypedChannels;

		for (const FMovieSceneChannelHandle& RawHandle : InChannels)
		{
			TypedChannels.Add(RawHandle.Cast<ChannelType>());
		}

		ExtendSectionMenu(MenuBuilder, InMenuExtender, MoveTemp(TypedChannels), InWeakSections, InWeakSequencer);
	}

	/**
	 * Extend the section sidebar menu
	 *
	 * @param MenuBuilder           The menu builder used to create this context menu
	 * @param InMenuExtender        The menu extender to use
	 * @param InChannels            Array of type specific channels that exist in the selected sections
	 * @param InWeakSections        Array of sections being shown on the context menu
	 * @param InWeakSequencer       The currently active sequencer
	 */
	virtual TSharedPtr<ISidebarChannelExtension> ExtendSidebarMenu_Raw(FMenuBuilder& MenuBuilder
		, TSharedPtr<FExtender> InMenuExtender
		, TArrayView<const FMovieSceneChannelHandle> InChannels
		, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections
		, TWeakPtr<ISequencer> InWeakSequencer) const override
	{
		using namespace Sequencer;
		TArray<TMovieSceneChannelHandle<ChannelType>> TypedChannels;

		for (const FMovieSceneChannelHandle& RawHandle : InChannels)
		{
			TypedChannels.Add(RawHandle.Cast<ChannelType>());
		}

		return ExtendSidebarMenu(MenuBuilder, InMenuExtender, MoveTemp(TypedChannels), InWeakSections, InWeakSequencer);
	}

	/**
	 * Gather information on how to draw the specified keys
	 *
	 * @param Channel               The channel to query
	 * @param InKeyHandles          Array of handles to duplicate
	 * @param InOwner               The section that owns the channel
	 * @param OutKeyDrawParams      Pre-sized array to receive key draw parameters. Invalid key handles will not be assigned to this array. Must match size of InKeyHandles.
	 */
	virtual void DrawKeys_Raw(FMovieSceneChannel* InChannel, TArrayView<const FKeyHandle> InKeyHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams) const override
	{
		check(InKeyHandles.Num() == OutKeyDrawParams.Num());

		using namespace Sequencer;
		DrawKeys(static_cast<ChannelType*>(InChannel), InKeyHandles, InOwner, OutKeyDrawParams);
	}

	/**
	 * Draw additional content in addition to keys for a particular channel
	 *
	 * @param InChannel          The channel to draw extra display information for
	 * @param InOwner            The owning movie scene section for this channel
	 * @param PaintArgs          Paint arguments containing the draw element list, time-to-pixel converter and other structures
	 * @param LayerId            The slate layer to paint onto
	 * @return The new slate layer ID for subsequent elements to paint onto
	 */
	virtual int32 DrawExtra_Raw(FMovieSceneChannel* InChannel, const UMovieSceneSection* InOwner, const FSequencerChannelPaintArgs& PaintArgs, int32 LayerId) const override
	{
		using namespace Sequencer;
		return DrawExtra(static_cast<ChannelType*>(InChannel), InOwner, PaintArgs, LayerId);
	}

	/**
	 * Retrieve the interpolation mode for this curve at the current time
	 *
	 * @param InChannel          The channel to query
	 * @param InTime             The time at which to compute the interpolation mode
	 * @param InDefaultMode      Default interp mode to use
	 * @return This curve's desired interpolation mode
	 */
	virtual EMovieSceneKeyInterpolation GetInterpolationMode_Raw(FMovieSceneChannel* InChannel, FFrameNumber InTime, EMovieSceneKeyInterpolation InDefaultMode) const override
	{
		using namespace Sequencer;
		return GetInterpolationMode(static_cast<ChannelType*>(InChannel), InTime, InDefaultMode);
	}

	/**
	 * Whether this channel supports curve models
	 */
	virtual bool SupportsCurveEditorModels_Raw(const FMovieSceneChannelHandle& InChannel) const override
	{
		using namespace Sequencer;
		return SupportsCurveEditorModels(InChannel.Cast<ChannelType>());
	}

	/**
	 * Whether this channel should draw a curve on its editor UI
	 *
	 * @param Channel               The channel to query
	 * @param InSection             The section that owns the channel
	 * @return true to show the curve on the UI, false otherwise
	 */
	virtual bool ShouldShowCurve_Raw(const FMovieSceneChannel* Channel, UMovieSceneSection* InSection) const override
	{
		using namespace Sequencer;
		return ShouldShowCurve(static_cast<const ChannelType*>(Channel), InSection);
	}

	/**
	 * Create a new model for this channel that can be used on the curve editor interface
	 *
	 * @return (Optional) A new model to be added to a curve editor
	 */
	virtual TUniquePtr<FCurveModel> CreateCurveEditorModel_Raw(const FMovieSceneChannelHandle& InChannel, const UE::Sequencer::FCreateCurveEditorModelParams& Params) const override
	{
		using namespace Sequencer;

		if constexpr (TModels_V<CLegacyCurveModelCreatable, ChannelType>)
		{
			// Emit deprecation warning
			CreateCurveEditorModelDeprecatedSignature<ChannelType>();
			TUniquePtr<FCurveModel> Result = CreateCurveEditorModel(InChannel.Cast<ChannelType>(), Params.OwningSection, Params.Sequencer);
			if (Result)
			{
				return Result;
			}
		}

		return CreateCurveEditorModel(InChannel.Cast<ChannelType>(), Params);
	}

	/**
	 * Create a new channel model for this type of channel
	 *
	 * @param InChannelHandle    The channel handle to create a model for
	 * @param InSectionModel     The section that owns this channel model
	 * @param InChannelName      The identifying name of this channel
	 * @return (Optional) A new model to be added to a curve editor
	 */
	virtual TSharedPtr<UE::Sequencer::FChannelModel> CreateChannelModel_Raw(const FMovieSceneChannelHandle& InChannelHandle, const UE::Sequencer::FSectionModel& InSection, FName InChannelName) const override
	{
		using namespace Sequencer;
		return CreateChannelModel(InChannelHandle.Cast<ChannelType>(), InSection, InChannelName);
	}

	/**
	 * Create a new channel view for this type of channel
	 *
	 * @param InChannelHandle    The channel handle to create a model for
	 * @param InWeakModel        The model that is creating the view. Should not be Pinned persistently.
	 * @param Parameters         View construction parameters
	 * @return (Optional) A new model to be added to a curve editor
	 */
	virtual TSharedPtr<UE::Sequencer::STrackAreaLaneView> CreateChannelView_Raw(const FMovieSceneChannelHandle& InChannelHandle, TWeakPtr<UE::Sequencer::FChannelModel> InWeakModel, const UE::Sequencer::FCreateTrackLaneViewParams& Parameters) const override
	{
		using namespace Sequencer;
		return CreateChannelView(InChannelHandle.Cast<ChannelType>(), InWeakModel, Parameters);
	}

	/**
	 * Create an editor on the sequencer node tree
	 *
	 * @param Channel               The channel to check
	 * @param Params                Creation parameters containing all the necessary structures for creating the key editor
	 * @return The editor widget to display on the node tree
	 */
	virtual TSharedRef<SWidget> CreateKeyEditor_Raw(const FMovieSceneChannelHandle& Channel, const UE::Sequencer::FCreateKeyEditorParams& Params) const override
	{
		using namespace Sequencer;

		if constexpr (TModels_V<CLegacyKeyEditorCreatable, ChannelType>)
		{
			// Emit deprecation warning
			CreateKeyEditorDeprecatedSignature<ChannelType>();
			return CreateKeyEditor(Channel.Cast<ChannelType>(), Params.OwningSection, Params.ObjectBindingID, Params.PropertyBindings, Params.Sequencer);
		}

		return CreateKeyEditor(Channel.Cast<ChannelType>(), Params);
	}
};

template<typename ChannelType, bool HasExtendedData> struct TSequencerChannelInterfaceBase;


/**
 * Extended base interface for channel types that do not specify extended editor data (ie, TMovieSceneChannelTraits<>::ExtendedEditorDataType is void)
 */
template<typename ChannelType>
struct TSequencerChannelInterfaceBase<ChannelType, false> : TSequencerChannelInterfaceCommon<ChannelType>
{
	/**
	 * Add (or update) a key to the specified channel using it's current value at that time, or some external value specified by the extended editor data
	 *
	 * @param Channel               The channel to add a key to
	 * @param SectionToKey          The Section to Key
	 * @param ExtendedEditorData    A pointer to the extended editor data for this channel of type TMovieSceneChannelTraits<>::ExtendedEditorDataType
	 * @param InTime                The time at which to add a key
	 * @param InSequencer           The currently active sequencer
	 * @param ObjectBindingID       The object binding ID for the track that this channel resides within
	 * @param PropertyBindings      (Optional) Property bindings where this channel exists on a property track
	 * @return A handle to the new or updated key
	 */
	virtual FKeyHandle AddOrUpdateKey_Raw(FMovieSceneChannel* InChannel, UMovieSceneSection* SectionToKey, const void* ExtendedEditorData, FFrameNumber InTime, ISequencer& Sequencer, const FGuid& ObjectBindingID, FTrackInstancePropertyBindings* PropertyBindings) const override
	{
		using namespace Sequencer;
		return AddOrUpdateKey(static_cast<ChannelType*>(InChannel), SectionToKey, InTime, Sequencer, ObjectBindingID, PropertyBindings);
	}
};

/**
 * Extended base interface for channel types that specify extended editor data (ie, TMovieSceneChannelTraits<>::ExtendedEditorDataType is not void)
 */
template<typename ChannelType>
struct TSequencerChannelInterfaceBase<ChannelType, true> : TSequencerChannelInterfaceCommon<ChannelType>
{
	typedef typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType ExtendedEditorDataType;

	/**
	 * Add (or update) a key to the specified channel using it's current value at that time, or some external value specified by the extended editor data
	 *
	 * @param Channel               The channel to add a key to
	 * @param SectionToKey          The Section to Key
	 * @param ExtendedEditorData    A pointer to the extended editor data for this channel of type TMovieSceneChannelTraits<>::ExtendedEditorDataType
	 * @param InTime                The time at which to add a key
	 * @param InSequencer           The currently active sequencer
	 * @param ObjectBindingID       The object binding ID for the track that this channel resides within
	 * @param PropertyBindings      (Optional) Property bindings where this channel exists on a property track
	 * @return A handle to the new or updated key
	 */
	virtual FKeyHandle AddOrUpdateKey_Raw(FMovieSceneChannel* InChannel, UMovieSceneSection* SectionToKey, const void* ExtendedEditorData, FFrameNumber InTime, ISequencer& Sequencer, const FGuid& ObjectBindingID, FTrackInstancePropertyBindings* PropertyBindings) const override
	{
		using namespace Sequencer;

		// Extended data must be available for this interface
		check(ExtendedEditorData);

		const auto* TypedEditorData = static_cast<const ExtendedEditorDataType*>(ExtendedEditorData);
		return AddOrUpdateKey(static_cast<ChannelType*>(InChannel), SectionToKey, *TypedEditorData, InTime, Sequencer, ObjectBindingID, PropertyBindings);
	}
};

/** Generic sequencer channel interface to any channel type */
template<typename ChannelType>
struct TSequencerChannelInterface : TSequencerChannelInterfaceBase<ChannelType, !std::is_same_v<typename TMovieSceneChannelTraits<ChannelType>::ExtendedEditorDataType, void>>
{
};
