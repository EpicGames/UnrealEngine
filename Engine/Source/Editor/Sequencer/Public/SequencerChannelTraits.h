// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "Curves/KeyHandle.h"
#include "Evaluation/MovieSceneRootOverridePath.h"
#include "Evaluation/MovieSceneTimeTransform.h"
#include "ISequencer.h"
#include "ISequencerChannelInterface.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "MovieSceneClipboard.h"
#include "MovieSceneSection.h"
#include "MVVM/Views/KeyDrawParams.h"
#include "SequencerClipboardReconciler.h"
#include "SequencerKeyStructGenerator.h"
#include "Templates/Decay.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/SNullWidget.h"
#include "TimeToPixel.h"

class FCurveModel;
class FMenuBuilder;
class FSequencerSectionPainter;
class FStructOnScope;
class FTrackInstancePropertyBindings;
class SWidget;
class UMovieSceneTrack;
class UObject;
struct FGeometry;
struct FKeyDrawParams;
struct FMovieSceneChannel;
template <typename T> struct TMovieSceneExternalValue;
template <typename ValueType> struct TMovieSceneChannelData;

/** Utility struct representing a number of selected keys on a single channel */
template<typename ChannelType>
struct TExtendKeyMenuParams
{
	/** The section on which the channel resides */
	TWeakObjectPtr<UMovieSceneSection> Section;

	/** The owning object. Probably the same as Section, but can be different. */
	TWeakObjectPtr<UObject> WeakOwner;

	/** The channel on which the keys reside */
	TMovieSceneChannelHandle<ChannelType> Channel;

	/** An array of key handles to operante on */
	TArray<FKeyHandle> Handles;
};

struct FSequencerChannelPaintArgs
{
	FSlateWindowElementList& DrawElements;

	const FPaintArgs& WidgetPaintArgs;
	const FGeometry& Geometry;
	const FSlateRect& MyCullingRect;
	const FWidgetStyle& WidgetStyle;

	FTimeToPixel TimeToPixel;

	bool bParentEnabled = true;
};

namespace UE::Sequencer
{
	class FChannelModel;
	class STrackAreaLaneView;

	struct FCreateTrackLaneViewParams;
}

/**
 * Stub/default implementations for ISequencerChannelInterface functions.
 * Custom behaviour should be implemented by overloading the relevant function with the necessary channel/data types.
 * For example, to overload how to draw keys for FMyCustomChannelType, implement the following function in the same namespace as FMyCustomChannelType:
 *
 * void DrawKeys(FMyCustomChannelType* Channel, TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyDrawParams> OutKeyDrawParams)
 * {
 * ...
 * }
 */
namespace Sequencer
{
	/**
	 * Extend the specified selected section context menu
	 *
	 * @param MenuBuilder    The menu builder that will construct the section context menu
	 * @param Channels       An array of all channels that are currently selected, in no particular order
	 * @param Sections       An array of all sections that the selected channels reside in
	 * @param InSequencer    The sequencer that is currently active
	 */
	template<typename ChannelType>
	void ExtendSectionMenu(FMenuBuilder& MenuBuilder
		, TSharedPtr<FExtender> InMenuExtender
		, TArray<TMovieSceneChannelHandle<ChannelType>>&& InChannels
		, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections
		, TWeakPtr<ISequencer> InWeakSequencer)
	{}

	/**
	 * Extend the specified selected sidebar menu
	 *
	 * @param MenuBuilder     The menu builder that will construct the section context menu
	 * @param InMenuExtender  The extender for the menu
	 * @param InChannels      An array of all channels that are currently selected, in no particular order
	 * @param InWeakSections  An array of all sections that the selected channels reside in
	 * @param InWeakSequencer The sequencer that is currently active
	 */
	template<typename ChannelType>
	TSharedPtr<ISidebarChannelExtension> ExtendSidebarMenu(FMenuBuilder& MenuBuilder
		, TSharedPtr<FExtender> InMenuExtender
		, TArray<TMovieSceneChannelHandle<ChannelType>>&& InChannels
		, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections
		, TWeakPtr<ISequencer> InWeakSequencer)
	{
		return nullptr;
	}

	/**
	 * Extend the specified selected key context menu
	 *
	 * @param MenuBuilder    The menu builder that will construct the section context menu
	 * @param Channels       An array of all channels that are currently selected, in no particular order
	 * @param Sections       An array of all sections that the selected channels reside in
	 * @param InSequencer    The sequencer that is currently active
	 */
	template<typename ChannelType>
	void ExtendKeyMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> MenuExtender, TArray<TExtendKeyMenuParams<ChannelType>>&& InChannels, TWeakPtr<ISequencer> InSequencer)
	{}


	/**
	 * Get a transient key structure that can be added to a details panel to enable editing of a single key
	 *
	 * @param ChannelHandle  Handle to the channel in which the key resides
	 * @param KeyHandle      A handle to the key to edit
	 * @return A shared struct object, or nullptr
	 */
	template<typename ChannelType>
	TSharedPtr<FStructOnScope> GetKeyStruct(const TMovieSceneChannelHandle<ChannelType>& ChannelHandle, FKeyHandle KeyHandle)
	{
		return FSequencerKeyStructGenerator::Get().CreateKeyStructInstance(ChannelHandle, KeyHandle);
	}

	/**
	 * Check whether the specified channel can create a key editor widget that should be placed on the sequencer node tree 
	 *
	 * @param InChannel      The channel to check
	 * @return true if a key editor can be created, false otherwise
	 */
	inline bool CanCreateKeyEditor(const FMovieSceneChannel* InChannel)
	{
		return false;
	}



	/**
	 * Create a key editor widget for the specified channel with the channel's specialized editor data. Such widgets are placed on the sequencer node tree for a given key area node.
	 *
	 * @param InChannel          The channel to create a key editor for
	 * @param Params             Creation parameters containing all the necessary structures for creating the key editor
	 * @return The key editor widget
	 */
	inline TSharedRef<SWidget> CreateKeyEditor(const FMovieSceneChannelHandle& InChannel,const UE::Sequencer::FCreateKeyEditorParams& Params)
	{
		return SNullWidget::NullWidget;
	}

	/**
	 * Add a key at the specified time (or update an existing key) with the channel's current value at that time
	 *
	 * @param InChannel          The channel to create a key for
	 * @param InChannelData      The channel's data
	 * @param InTime             The time at which to add a key
	 * @param InSequencer        The currently active sequencer
	 * @param InDefaultValue     The default value to use if evaluation of the channel failed
	 * @return A handle to the added (or updated) key
	 */
	template<typename ChannelType, typename ValueType>
	FKeyHandle EvaluateAndAddKey(ChannelType* InChannel, const TMovieSceneChannelData<ValueType>& InChannelData, FFrameNumber InTime, ISequencer& InSequencer, ValueType InDefaultValue = ValueType{})
	{
		using namespace UE::MovieScene;

		ValueType ValueAtTime = InDefaultValue;
		EvaluateChannel(InChannel, InTime, ValueAtTime);

		EMovieSceneKeyInterpolation InterpolationMode = GetInterpolationMode(InChannel, InTime, InSequencer.GetKeyInterpolation());
		return AddKeyToChannel(InChannel, InTime, ValueAtTime, InterpolationMode);
	}
	


	/**
	 * Retrieve a channel's external value, and add it to the channel as a new key (or update an existing key with its value)
	 *
	 * @param InChannel          The channel to create a key for
	 * @param InExternalValue    The external value definition
	 * @param InTime             The time at which to add a key
	 * @param InSequencer        The currently active sequencer
	 * @param InObjectBindingID  The object binding ID that this section's track is bound to
	 * @param InPropertyBindings Optionally supplied helper for accessing an object's property pertaining to this channel
	 * @return (Optional) A handle to the added (or updated) key
	 */
	template<typename ChannelType, typename ValueType>
	TOptional<FKeyHandle> AddKeyForExternalValue(
		ChannelType*                               InChannel,
		const TMovieSceneExternalValue<ValueType>& InExternalValue,
		FFrameNumber                               InTime,
		ISequencer&                                InSequencer,
		const FGuid&                               InObjectBindingID,
		FTrackInstancePropertyBindings*            InPropertyBindings
		)
	{
		using namespace UE::MovieScene;

		// Add a key for the current value of the valid first object we can find
		if (InExternalValue.OnGetExternalValue && InObjectBindingID.IsValid())
		{
			for (TWeakObjectPtr<> WeakObject : InSequencer.FindBoundObjects(InObjectBindingID, InSequencer.GetFocusedTemplateID()))
			{
				UObject* Object = WeakObject.Get();
				if (Object)
				{
					TOptional<ValueType> Value = InExternalValue.OnGetExternalValue(*Object, InPropertyBindings);
					if (Value.IsSet())
					{
						EMovieSceneKeyInterpolation InterpolationMode = GetInterpolationMode(InChannel, InTime, InSequencer.GetKeyInterpolation());
						return AddKeyToChannel(InChannel, InTime, Value.GetValue(), InterpolationMode);
					}
				}
			}
		}
		return TOptional<FKeyHandle>();
	}


	/**
	 * Add or update a key for this channel's current value
	 *
	 * @param InChannel          The channel to create a key for
	 * @param InSectionToKey     The Section to key
	 * @param InTime             The time at which to add a key
	 * @param InSequencer        The currently active sequencer
	 * @param InObjectBindingID  The object binding ID that this section's track is bound to
	 * @param InPropertyBindings Optionally supplied helper for accessing an object's property pertaining to this channel
	 * @return A handle to the added (or updated) key
	 */
	template<typename ChannelType>
	FKeyHandle AddOrUpdateKey(
		ChannelType*                    InChannel,
		UMovieSceneSection*             InSectionToKey,
		FFrameNumber                    InTime,
		ISequencer&                     InSequencer,
		const FGuid&                    InObjectBindingID,
		FTrackInstancePropertyBindings* InPropertyBindings
		)
	{
		return EvaluateAndAddKey(InChannel, InChannel->GetData(), InTime, InSequencer);
	}


	/**
	 * Add or update a key for this channel's current value, using an external value if possible
	 *
	 * @param InChannel          The channel to create a key for
	 * @param InSectionToKey     The Section to key
	 * @param InExternalValue    The external value definition
	 * @param InTime             The time at which to add a key
	 * @param InSequencer        The currently active sequencer
	 * @param InObjectBindingID  The object binding ID that this section's track is bound to
	 * @param InPropertyBindings Optionally supplied helper for accessing an object's property pertaining to this channel
	 * @return A handle to the added (or updated) key
	 */
	template<typename ChannelType, typename ValueType>
	FKeyHandle AddOrUpdateKey(
		ChannelType*                               InChannel,
		UMovieSceneSection*                        SectionToKey,
		const TMovieSceneExternalValue<ValueType>& InExternalValue,
		FFrameNumber                               InTime,
		ISequencer&                                InSequencer,
		const FGuid&                               InObjectBindingID,
		FTrackInstancePropertyBindings*            InPropertyBindings)
	{
		using namespace UE::MovieScene;

		TOptional<FKeyHandle> Handle = AddKeyForExternalValue(InChannel, InExternalValue, InTime, InSequencer, InObjectBindingID, InPropertyBindings);
		if (!Handle.IsSet())
		{
			ValueType ValueAtTime{};
			EvaluateChannel(InChannel, InTime, ValueAtTime);
		
			EMovieSceneKeyInterpolation InterpolationMode = GetInterpolationMode(InChannel, InTime, InSequencer.GetKeyInterpolation());
			Handle = AddKeyToChannel(InChannel, InTime, ValueAtTime, InterpolationMode);
		}

		return Handle.GetValue();
	}

	/**
	 * Gather key draw information from a channel for a specific set of keys
	 *
	 * @param InChannel          The channel to draw keys
	 * @param InHandles          Array of key handles that should be displayed
	 * @param InOwner            The owning movie scene section for this channel
	 * @param OutKeyDrawParams   Array to receive key draw information. Must be exactly the size of InHandles
	 */
	SEQUENCER_API void DrawKeys(FMovieSceneChannel* Channel, TArrayView<const FKeyHandle> InHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams);

	/**
	 * Draw additional content in addition to keys for a particular channel
	 *
	 * @param InChannel          The channel to draw extra display information for
	 * @param InOwner            The owning movie scene section for this channel
	 * @param PaintArgs          Paint arguments containing the draw element list, time-to-pixel converter and other structures
	 * @param LayerId            The slate layer to paint onto
	 * @return The new slate layer ID for subsequent elements to paint onto
	 */
	SEQUENCER_API int32 DrawExtra(FMovieSceneChannel* InChannel, const UMovieSceneSection* InOwner, const FSequencerChannelPaintArgs& PaintArgs, int32 LayerId);

	/**
	 * Delete the specified keys. If all keys are removed, the current value at that time will be set as the default value for the channel
	 *
	 * @param Channel               The channel to remove the keys from
	 * @param InHandles             The key handles to delete
	 * @param InTime                The time at which to evaluate for the default value if there are no keys remaining
	 */
	template<typename ChannelType>
	typename TEnableIf<TMovieSceneChannelTraits<ChannelType>::SupportsDefaults>::Type DeleteKeys(ChannelType* InChannel, TArrayView<const FKeyHandle> InHandles, FFrameNumber InTime)
	{
		using namespace UE::MovieScene;

		using CurveValueType = typename ChannelType::CurveValueType;
		CurveValueType Value;

		EvaluateChannel(InChannel, InTime, Value);

		InChannel->DeleteKeys(InHandles);

		if (InChannel->GetNumKeys() == 0)
		{
			InChannel->SetDefault(Value);
		}
	}

	template<typename ChannelType>
	typename TEnableIf<!TMovieSceneChannelTraits<ChannelType>::SupportsDefaults>::Type DeleteKeys(ChannelType* InChannel, TArrayView<const FKeyHandle> InHandles, FFrameNumber InTime)
	{
		InChannel->DeleteKeys(InHandles);
	}

	/**
	 * Copy the specified keys from a channel
	 *
	 * @param InChannel          The channel to duplicate keys in
	 * @param InSection          The section that owns this channel
	 * @param KeyAreaName        The name of the key area representing this channel
	 * @param ClipboardBuilder   Structure for populating the clipboard
	 * @param InHandles          Array of key handles that should be copied
	 */
	template<typename ChannelType>
	void CopyKeys(ChannelType* InChannel, const UMovieSceneSection* InSection, FName KeyAreaName, FMovieSceneClipboardBuilder& ClipboardBuilder, TArrayView<const FKeyHandle> InHandles)
	{
		UMovieSceneTrack* Track = InSection ? InSection->GetTypedOuter<UMovieSceneTrack>() : nullptr;
		if (!Track)
		{
			return;
		}

		FMovieSceneClipboardKeyTrack* KeyTrack = nullptr;

		auto ChannelData = InChannel->GetData();
		TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
		auto Values = ChannelData.GetValues();

		for (FKeyHandle Handle : InHandles)
		{
			const int32 KeyIndex = ChannelData.GetIndex(Handle);
			if (KeyIndex != INDEX_NONE)
			{
				FFrameNumber KeyTime  = Times[KeyIndex];
				auto         KeyValue = Values[KeyIndex];

				if (!KeyTrack)
				{
					KeyTrack = &ClipboardBuilder.FindOrAddKeyTrack<decltype(KeyValue)>(KeyAreaName, *Track);
				}

				KeyTrack->AddKey(KeyTime, KeyValue);
			}
		}
	}

	/**
	 * Paste the clipboard contents onto a channel
	 *
	 * @param InChannel          The channel to duplicate keys in
	 * @param Section            The section that owns this channel
	 * @param KeyTrack           The clipboard track to paste
	 * @param SrcEnvironment     The source clipboard environment that was originally copied
	 * @param DstEnvironment     The destination clipboard environment that we're copying to
	 * @param OutPastedKeys      Array of key handles that should receive any pasted keys
	 */
	template<typename ChannelType>
	void PasteKeys(ChannelType* InChannel, UMovieSceneSection* Section, const FMovieSceneClipboardKeyTrack& KeyTrack, const FMovieSceneClipboardEnvironment& SrcEnvironment, const FSequencerPasteEnvironment& DstEnvironment, TArray<FKeyHandle>& OutPastedKeys)
	{
		FFrameTime PasteAt = DstEnvironment.CardinalTime;

		auto ChannelData = InChannel->GetData();

		TRange<FFrameNumber> NewRange = Section->GetRange();

		auto ForEachKey = [Section, PasteAt, &NewRange, &ChannelData, &OutPastedKeys, &SrcEnvironment, &DstEnvironment](const FMovieSceneClipboardKey& Key)
		{
			FFrameTime SrcTime = Key.GetTime();
			SrcTime = SrcTime * SrcEnvironment.TimeTransform.Inverse();

			FFrameTime DstTime = FFrameRate::TransformTime(SrcTime, SrcEnvironment.TickResolution, DstEnvironment.TickResolution);
			DstTime = DstTime * DstEnvironment.TimeTransform;

			FFrameNumber Time = (PasteAt + DstTime).FloorToFrame();

			NewRange = TRange<FFrameNumber>::Hull(NewRange, TRange<FFrameNumber>(Time));

			typedef typename TDecay<decltype(ChannelData.GetValues()[0])>::Type KeyType;
			KeyType NewKey = Key.GetValue<KeyType>();

			FKeyHandle KeyHandle = ChannelData.UpdateOrAddKey(Time, NewKey);
			OutPastedKeys.Add(KeyHandle);
			return true;
		};
		KeyTrack.IterateKeys(ForEachKey);

		Section->SetRange(NewRange);
	}

	inline EMovieSceneKeyInterpolation GetInterpolationMode(const FMovieSceneChannel*, FFrameNumber, EMovieSceneKeyInterpolation InDefaultMode)
	{
		return InDefaultMode;
	}

	/**
	 * Whether the specified channel handle supports curve models or not
	 */
	SEQUENCER_API bool SupportsCurveEditorModels(const FMovieSceneChannelHandle& ChannelHandle);

	/**
	 * Create a new model for the specified channel that can be used on the curve editor interface
	 *
	 * @return (Optional) A new model to be added to a curve editor
	 */
	SEQUENCER_API TUniquePtr<FCurveModel> CreateCurveEditorModel(const FMovieSceneChannelHandle& ChannelHandle, const UE::Sequencer::FCreateCurveEditorModelParams& Params);

	/**
	 * Create a new channel model for this type of channel
	 *
	 * @param InChannelHandle    The channel handle to create a model for
	 * @param InSectionModel     The section that owns this channel model
	 * @param InChannelName      The identifying name of this channel
	 * @return (Optional) A new model to be used as part of the Sequencer MVVM framework
	 */
	inline TSharedPtr<UE::Sequencer::FChannelModel> CreateChannelModel(const FMovieSceneChannelHandle& InChannelHandle, const UE::Sequencer::FSectionModel& InSection, FName InChannelName)
	{
		return nullptr;
	}

	/**
	 * Create a new channel view for this type of channel
	 *
	 * @param InChannelHandle    The channel handle to create a model for
	 * @param InWeakModel        The model that is creating the view. Should not be Pinned persistently.
	 * @param Parameters         View construction parameters
	 * @return (Optional) A new view to be shown on the track area
	 */
	inline TSharedPtr<UE::Sequencer::STrackAreaLaneView> CreateChannelView(const FMovieSceneChannelHandle& InChannelHandle, TWeakPtr<UE::Sequencer::FChannelModel> InWeakModel, const UE::Sequencer::FCreateTrackLaneViewParams& Parameters)
	{
		return nullptr;
	}	

	/**
	 * Whether this channel should draw a curve on its editor UI
	 *
	 * @param Channel               The channel to query
	 * @param InSection             The section that owns the channel
	 * @return true to show the curve on the UI, false otherwise
	 */
	inline bool ShouldShowCurve(const FMovieSceneChannel* Channel, UMovieSceneSection* InSection)
	{
		return false;
	}

}	// namespace Sequencer
