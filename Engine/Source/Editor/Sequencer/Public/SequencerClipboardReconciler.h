// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Curves/KeyHandle.h"
#include "Misc/Optional.h"
#include "MovieSceneClipboard.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

namespace UE
{
namespace Sequencer
{
	class FChannelModel;
}
}


struct FSequencerPasteEnvironment : FMovieSceneClipboardEnvironment
{
	void ReportPastedKey(FKeyHandle KeyHandle, TSharedPtr<UE::Sequencer::FChannelModel> ChannelModel) const
	{
		if (OnKeyPasted)
		{
			OnKeyPasted(KeyHandle, ChannelModel);
		}
	}

	TFunction<void(FKeyHandle, TSharedPtr<UE::Sequencer::FChannelModel>)> OnKeyPasted;
};

/** Struct responsible for adding key areas to a group */
struct FSequencerClipboardPasteGroup
{
	/** Constructor that takes an array to add our group of key areas to */
	FSequencerClipboardPasteGroup(TArray<TArray<TSharedPtr<UE::Sequencer::FChannelModel>>>& InOwnerArray)
		: OwnerArray(InOwnerArray)
		, Index(INDEX_NONE)
	{}

	/** Add a key area to this group */
	void Add(TSharedPtr<UE::Sequencer::FChannelModel> InChannel)
	{
		if (Index == INDEX_NONE)
		{
			Index = OwnerArray.AddDefaulted();
		}
		OwnerArray[Index].Add(InChannel);
	}

private:
	/** Owner array to create our group within */
	TArray<TArray<TSharedPtr<UE::Sequencer::FChannelModel>>>& OwnerArray;

	/** Index into the above array at which our group resides */
	int32 Index;
};

/**
 * Class responsible for reconciling copied key-tracks to a set of paste destinations
 * Reconciler should be populated with all pasted destinations (sets of keyareas grouped together),
 * before reconciling and pasting the clipboard with the reconciled result.
 *
 * Comparable track types can be associated by name using AddTrackAlias
 */
class FSequencerClipboardReconciler
{
public:
	/** Construction from existing clipboard data */
	FSequencerClipboardReconciler(TSharedRef<const FMovieSceneClipboard> InClipboard);

	/** Attempt to reconcile the source clipboard data to the current paste destinations */
	bool Reconcile();

	/** Perform the paste using the specified environment */
	bool Paste(const FSequencerPasteEnvironment& PasteEnvironment);

public:

	/** Add a new paste destination (to consist of one or more key areas) */
	FSequencerClipboardPasteGroup AddDestinationGroup();

	/** Add a rule specifying that 2 names should be considered synonymous when reconciling */
	SEQUENCER_API static void AddTrackAlias(FName Alias1, FName Alias2);

public:

	/** Check if this reconciler can auto paste (that is to say that there are some exact matches for this reconciler) */
	bool CanAutoPaste() const;

private:

	/** Implementation of the paste method */
	bool PasteImpl(const FSequencerPasteEnvironment& PasteEnvironment);

	/**
	 * Find a matching track group for the given source, optionally allowing aliases
	 *
	 * @param Destination 	The destination key areas to find a matching group in the source for
	 * @param Source 		A source group of clipboard tracks to match
	 * @param IndexMap 		A map to which any matches should be added (destination index -> source index)
	 * @param bAllowAliases	true to allow synonymous names to match, false if only exact matches should be allowed
	 * @return true if any matches were found, false otherwise
	 */
	bool FindMatchingGroup(const TArray<TSharedPtr<UE::Sequencer::FChannelModel>>& Destination, const TArray<FMovieSceneClipboardKeyTrack>& Source, TMap<int32, int32>& Map, bool bAllowAliases);

	/** Reconcile a single key area group, to one or more destination groups */
	bool ReconcileOneToMany();

private:

	/** Structure expressing how a paste should be performed for a particular destination group */
	struct FPasteMetaData
	{
		enum EMethod
		{
			Compress, Expand, Apply, ApplyRepeating, Custom
		};

		FPasteMetaData(int32 InSourceGroup, EMethod InMethod)
			: SourceGroup(InSourceGroup)
			, Method(InMethod)
		{}

		/** The index into the Source groups to paste into this entry */
		int32 SourceGroup;

		/** Map of destination key area index -> source track index. Used where Method == Custom */
		TMap<int32, int32> DestToSrcMap;

		/** The method to use when pasting */
		EMethod Method;
	};

	/** Map of meta data for each paste destination group */
	TMap<int32, FPasteMetaData> MetaData;

	/** The clipboard from which we are pasting */
	TSharedRef<const FMovieSceneClipboard> Clipboard;

	/** Array of paste destinations. A paste destination will consist of one or more key areas. */
	TArray<TArray<TSharedPtr<UE::Sequencer::FChannelModel>>> PasteDestination;

	/** Optional cached reconciliation result for the current data set */
	TOptional<bool> bReconcileResult;

	/** If we have found some *exact* matches between the source/destination tracks, we can auto paste */
	bool bCanAutoPaste;

	/** Static map of synonyms for a given key area name */
	SEQUENCER_API static TMap<FName, TArray<FName>> KeyAreaAliases;
};
