// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

UE_DECLARE_TCOMMANDS(class FSequencerCommands, SEQUENCER_API)

class FSequencerCommands : public TCommands<FSequencerCommands>
{

public:
	FSequencerCommands() : TCommands<FSequencerCommands>
	(
		"Sequencer",
		NSLOCTEXT("Contexts", "Sequencer", "Sequencer"),
		NAME_None, // "MainFrame" // @todo Fix this crash
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
	{}
	
	/** Toggle play */
	TSharedPtr< FUICommandInfo > TogglePlay;

	/** Toggle play viewport*/
	TSharedPtr< FUICommandInfo > TogglePlayViewport;

	/** Scrub time in the viewport*/
	TSharedPtr< FUICommandInfo > ScrubTimeViewport;

	/** Play forward */
	TSharedPtr< FUICommandInfo > PlayForward;

	/** Jump to start of playback */
	TSharedPtr< FUICommandInfo > JumpToStart;

	/** Jump to end of playback */
	TSharedPtr< FUICommandInfo > JumpToEnd;

	/** Jump to start of playback */
	TSharedPtr< FUICommandInfo > JumpToStartViewport;

	/** Jump to end of playback */
	TSharedPtr< FUICommandInfo > JumpToEndViewport;

	/** Shuttle forward */
	TSharedPtr< FUICommandInfo > ShuttleForward;

	/** Shuttle backward */
	TSharedPtr< FUICommandInfo > ShuttleBackward;

	/** Pause */
	TSharedPtr< FUICommandInfo > Pause;
	
	/** Restores real time speed */
	TSharedPtr< FUICommandInfo > RestorePlaybackSpeed;

	/** Step forward */
	TSharedPtr< FUICommandInfo > StepForward;

	/** Step backward */
	TSharedPtr< FUICommandInfo > StepBackward;

	/** Step forward */
	TSharedPtr< FUICommandInfo > StepForwardViewport;

	/** Step backward */
	TSharedPtr< FUICommandInfo > StepBackwardViewport;

	/** Jump forward */
	TSharedPtr< FUICommandInfo > JumpForward;

	/** Jump backward */
	TSharedPtr< FUICommandInfo > JumpBackward;

	/** Step to next key */
	TSharedPtr< FUICommandInfo > StepToNextKey;

	/** Step to previous key */
	TSharedPtr< FUICommandInfo > StepToPreviousKey;

	/** Step to next shot */
	TSharedPtr< FUICommandInfo > StepToNextShot;

	/** Step to previous shot */
	TSharedPtr< FUICommandInfo > StepToPreviousShot;

	/** Set start playback range */
	TSharedPtr< FUICommandInfo > SetStartPlaybackRange;

	/** Set end playback range */
	TSharedPtr< FUICommandInfo > SetEndPlaybackRange;

	/** Focus the view range on the current playback time without changing zoom level */
	TSharedPtr< FUICommandInfo > FocusPlaybackTime;

	/** Reset the view range to the playback range */
	TSharedPtr< FUICommandInfo > ResetViewRange;

	/** Zoom to fit the selected sections and keys */
	TSharedPtr< FUICommandInfo > ZoomToFit;

	/** Zoom to fit the selection range (falls back to time bounds if selection is empty) */
	TSharedPtr< FUICommandInfo > ZoomToFitSelectionRange;

	/** Zoom into the view range */
	TSharedPtr< FUICommandInfo > ZoomInViewRange;

	/** Zoom out of the view range */
	TSharedPtr< FUICommandInfo > ZoomOutViewRange;

	/** Navigate backward */
	TSharedPtr< FUICommandInfo > NavigateBackward;

	/** Navigate forward */
	TSharedPtr< FUICommandInfo > NavigateForward;

	/** Set the selection range to the next shot. */
	TSharedPtr< FUICommandInfo > SetSelectionRangeToNextShot;

	/** Set the selection range to the previous shot. */
	TSharedPtr< FUICommandInfo > SetSelectionRangeToPreviousShot;

	/** Set the playback range to all the shots. */
	TSharedPtr< FUICommandInfo > SetPlaybackRangeToAllShots;

	/** Toggle locking the playback range. */
	TSharedPtr< FUICommandInfo > TogglePlaybackRangeLocked;

	/** Toggle clean playback mode. */
	TSharedPtr< FUICommandInfo > ToggleCleanPlaybackMode;

	/** Reruns construction scripts on bound actors every frame. */
	TSharedPtr< FUICommandInfo > ToggleRerunConstructionScripts;

	/** When enabled, enables a single asynchronous evaluation once per-frame. When disabled, forces a full blocking evaluation every time this sequence is evaluated (should be avoided for real-time content). */
	TSharedPtr< FUICommandInfo > ToggleAsyncEvaluation;

	/** When enabled, all blendable tracks will always cache their initial values to ensure that they are able to correctly blend in/out when dynamic weights are being used. */
	TSharedPtr< FUICommandInfo > ToggleDynamicWeighting;

	/** When enabled, the sequence considers looping as a jump/cut. When disabled, the sequence considers looping to be seamless. */
	TSharedPtr< FUICommandInfo > ToggleLoopCuts;

	/** Toggle resetting the playhead when navigating in and out of subsequences */
	TSharedPtr< FUICommandInfo > ToggleResetPlayheadWhenNavigating;

	/** While scrubbing, don't restrict the playhead to any range */
	TSharedPtr< FUICommandInfo > SetPlayheadScrubbingLimitNone;

	/** While scrubbing, restrict the playhead to the playback range */
	TSharedPtr< FUICommandInfo > SetPlayheadScrubbingLimitPlaybackRange;

	/** While scrubbing, restrict the playhead to the selection range */
	TSharedPtr< FUICommandInfo > SetPlayheadScrubbingLimitSelectionRange;

	/** Toggle constraining the playback range to the section bounds */
	TSharedPtr< FUICommandInfo > ToggleKeepPlaybackRangeInSectionBounds;

	/** Toggle auto expand outliner tree on child selection */
	TSharedPtr< FUICommandInfo > ToggleAutoExpandNodesOnSelection;
	
	/** Toggle auto expand outliner tree on child selection */
	TSharedPtr< FUICommandInfo > ToggleAutoExpandCurveEditorOnSelection;

	/**
	 * Toggle whether unlocking a camera cut track should return the viewport to its original location, or keep it where
	 * the camera cut was.
	 */
	TSharedPtr< FUICommandInfo > ToggleRestoreOriginalViewportOnCameraCutUnlock;

	/** Toggle whether to preview camera cuts while in SIE, or after ejecting from the player controller in PIE. */
	TSharedPtr< FUICommandInfo > TogglePreviewCameraCutsInSimulate;

	/** Expand/collapse nodes */
	TSharedPtr< FUICommandInfo > ToggleExpandCollapseNodes;

	/** Expand/collapse nodes and descendants */
	TSharedPtr< FUICommandInfo > ToggleExpandCollapseNodesAndDescendants;

	/** Expand all nodes */
	TSharedPtr< FUICommandInfo > ExpandAllNodes;

	/** Collapse all nodes */
	TSharedPtr< FUICommandInfo > CollapseAllNodes;

	/** Sort all nodes and descendants */
	TSharedPtr< FUICommandInfo > SortAllNodesAndDescendants;

	/** Sets the upper bound of the selection range */
	TSharedPtr< FUICommandInfo > SetSelectionRangeEnd;

	/** Sets the lower bound of the selection range */
	TSharedPtr< FUICommandInfo > SetSelectionRangeStart;

	/** Clear and reset the selection range */
	TSharedPtr< FUICommandInfo > ClearSelectionRange;

	/** Select all keys that fall into the selection range*/
	TSharedPtr< FUICommandInfo > SelectKeysInSelectionRange;

	/** Select all sections that fall into the selection range*/
	TSharedPtr< FUICommandInfo > SelectSectionsInSelectionRange;

	/** Select all markers that fall into the selection range*/
	TSharedPtr< FUICommandInfo > SelectMarkersInSelectionRange;

	/** Select all keys, sections and markers that fall into the selection range*/
	TSharedPtr< FUICommandInfo > SelectAllInSelectionRange;

	/** Select all keys, sections and markers forward from the current time */
	TSharedPtr< FUICommandInfo > SelectForward;

	/** Select all keys, sections and markers backward from the current time */
	TSharedPtr< FUICommandInfo > SelectBackward;

	/** Select none */
	TSharedPtr< FUICommandInfo > SelectNone;

	/** Add selected actors to sequencer */
	TSharedPtr< FUICommandInfo > AddActorsToSequencer;

	/** Add selection from content browser */
	TSharedPtr< FUICommandInfo > AddSelectionFromContentBrowser;

	/** Sets a key at the current time for the selected actor */
	TSharedPtr< FUICommandInfo > SetKey;

	/** Sets the interp tangent mode for the selected keys to smart auto */
	TSharedPtr< FUICommandInfo > SetInterpolationCubicSmartAuto;

	/** Sets the interp tangent mode for the selected keys to auto */
	TSharedPtr< FUICommandInfo > SetInterpolationCubicAuto;

	/** Sets the interp tangent mode for the selected keys to user */
	TSharedPtr< FUICommandInfo > SetInterpolationCubicUser;

	/** Sets the interp tangent mode for the selected keys to break */
	TSharedPtr< FUICommandInfo > SetInterpolationCubicBreak;

	/** Toggles the interp tangent weight mode for the selected keys */
	TSharedPtr< FUICommandInfo > ToggleWeightedTangents;

	/** Sets the interp tangent mode for the selected keys to linear */
	TSharedPtr< FUICommandInfo > SetInterpolationLinear;

	/** Sets the interp tangent mode for the selected keys to constant */
	TSharedPtr< FUICommandInfo > SetInterpolationConstant;

	/** Trim section to the left, keeping the right portion */
	TSharedPtr< FUICommandInfo > TrimSectionLeft;

	/** Trim section to the right, keeping the left portion */
	TSharedPtr< FUICommandInfo > TrimSectionRight;

	/** Trim or extend closest sections to the left for the selected tracks (or all tracks if none selected) to the current time */
	TSharedPtr< FUICommandInfo > TrimOrExtendSectionLeft;

	/** Trim or extend closest sections to the right for the selected tracks (or all tracks if none selected) to the current time */
	TSharedPtr< FUICommandInfo > TrimOrExtendSectionRight;

	/** Translate the selected keys and section to the left */
	TSharedPtr< FUICommandInfo > TranslateLeft;

	/** Translate the selected keys and section to the right */
	TSharedPtr< FUICommandInfo > TranslateRight;

	/** Split section */
	TSharedPtr< FUICommandInfo > SplitSection;

	/** Set the auto change mode to Key. */
	TSharedPtr< FUICommandInfo > SetAutoKey;

	/** Set the auto change mode to Track. */
	TSharedPtr< FUICommandInfo > SetAutoTrack;

	/** Set the auto change mode to None. */
	TSharedPtr< FUICommandInfo > SetAutoChangeAll;

	/** Set the auto change mode to None. */
	TSharedPtr< FUICommandInfo > SetAutoChangeNone;

	/** Set allow edits to all. */
	TSharedPtr< FUICommandInfo > AllowAllEdits;

	/** Set allow edits to sequencer only. */
	TSharedPtr< FUICommandInfo > AllowSequencerEditsOnly;

	/** Set allow edits to levels only. */
	TSharedPtr< FUICommandInfo > AllowLevelEditsOnly;

	/** Turns autokey on and off. */
	TSharedPtr< FUICommandInfo > ToggleAutoKeyEnabled;

	/** Set mode to just key changed attribute. */
	TSharedPtr< FUICommandInfo > SetKeyChanged;

	/** Set mode to key changed attribute and others in it's group. */
	TSharedPtr< FUICommandInfo > SetKeyGroup;

	/** Set mode to key all. */
	TSharedPtr< FUICommandInfo > SetKeyAll;

	/** Toggle showing marked frames */
	TSharedPtr< FUICommandInfo> ToggleShowMarkedFrames;

	/** Toggle on/off a Mark at the current time **/
	TSharedPtr< FUICommandInfo> ToggleMarkAtPlayPosition;

	/** Step to next mark */
	TSharedPtr< FUICommandInfo > StepToNextMark;

	/** Step to previous mark */
	TSharedPtr< FUICommandInfo > StepToPreviousMark;

	/** Toggle locking marks */
	TSharedPtr< FUICommandInfo > ToggleMarksLocked;

	/** Toggle show marked frames globally */
	TSharedPtr< FUICommandInfo > ToggleShowMarkedFramesGlobally;

	/** Clear global marked frames */
	TSharedPtr< FUICommandInfo > ClearGlobalMarkedFrames;

	/** Rotates through the supported formats for displaying times/frames/timecode. */
	TSharedPtr< FUICommandInfo > ChangeTimeDisplayFormat;

	/** Toggle the visibility of the goto box. */
	TSharedPtr< FUICommandInfo > ToggleShowGotoBox;

	/** Toggle the visibility of the transform box. */
	TSharedPtr< FUICommandInfo > ToggleShowTransformBox;

	/** Toggle the visibility of the stretch box. */
	TSharedPtr< FUICommandInfo > ToggleShowStretchBox;

	/** Opens the director blueprint for a sequence. */
	TSharedPtr< FUICommandInfo > OpenDirectorBlueprint;

	/** Opens the tagged binding manager. */
	TSharedPtr< FUICommandInfo > OpenTaggedBindingManager;

	/** Opens the node group manager. */
	TSharedPtr< FUICommandInfo > OpenNodeGroupsManager;

	/** Sets the tree search widget as the focused widget in Slate for easy typing. */
	TSharedPtr< FUICommandInfo > QuickTreeSearch;

	/** Move selected nodes to new folder. */
	TSharedPtr< FUICommandInfo > MoveToNewFolder;

	/** Remove selected nodes from folder. */
	TSharedPtr< FUICommandInfo > RemoveFromFolder;

	/** Toggles display of the transport controls panel. */
	TSharedPtr< FUICommandInfo > ToggleShowTransportControl;

	/** Toggles popped out state of the transport controls panel. */
	TSharedPtr< FUICommandInfo > ToggleTransportControlsPoppedOut;

	/** Turns the range slider on and off. */
	TSharedPtr< FUICommandInfo > ToggleShowRangeSlider;

	/** Turns the bottom time slider on and off. */
	TSharedPtr< FUICommandInfo > ToggleShowBottomTimeSlider;

	/** Turns snapping on and off. */
	TSharedPtr< FUICommandInfo > ToggleIsSnapEnabled;

	/** Toggles whether or not keys should snap to other elements. */
	TSharedPtr< FUICommandInfo > ToggleSnapKeyTimesToElements;

	/** Toggles whether or not sections should snap to other elements. */
	TSharedPtr< FUICommandInfo > ToggleSnapSectionTimesToElements;

	/** Toggles whether or not snap to key times while scrubbing. */
	TSharedPtr< FUICommandInfo > ToggleSnapPlayTimeToKeys;

	/** Toggles whether or not snap to section bounds while scrubbing. */
	TSharedPtr< FUICommandInfo > ToggleSnapPlayTimeToSections;

	/** Toggles whether or not snap to markers while scrubbing. */
	TSharedPtr< FUICommandInfo > ToggleSnapPlayTimeToMarkers;

	/** Toggles whether or not the play time should snap to the pressed key. */
	TSharedPtr< FUICommandInfo > ToggleSnapPlayTimeToPressedKey;

	/** Toggles whether or not the play time should snap to the dragged key. */
	TSharedPtr< FUICommandInfo > ToggleSnapPlayTimeToDraggedKey;

	/** Toggles whether or not to snap curve values to the interval. */
	TSharedPtr< FUICommandInfo > ToggleSnapCurveValueToInterval;

	/** Toggles whether or not to snap keys/sections/playhead to the interval which forces whole frames. */
	TSharedPtr< FUICommandInfo > ToggleForceWholeFrames;

	/** Finds the viewed sequence asset in the content browser. */
	TSharedPtr< FUICommandInfo > FindInContentBrowser;

	/** Browse to the selected objects. */
	TSharedPtr< FUICommandInfo > BrowseToObject;

	/** Toggles whether to show layer bars to edit keyframes in bulk. */
	TSharedPtr< FUICommandInfo > ToggleLayerBars;

	/** Show/hide key bar connectors for quickly retiming pairs of keys*/
	TSharedPtr< FUICommandInfo > ToggleKeyBars;

	/** Toggles whether to show channel colors in the track area. */
	TSharedPtr< FUICommandInfo > ToggleChannelColors;

	/** Toggles whether to show the info button in the playback controls. */
	TSharedPtr< FUICommandInfo > ToggleShowInfoButton;

	/** Turns auto scroll on and off. */
	TSharedPtr< FUICommandInfo > ToggleAutoScroll;

	/** Toggles whether or not the curve editor should be shown. */
	TSharedPtr< FUICommandInfo > ToggleShowCurveEditor;

	/** Toggles whether or not the curve editor time range should be linked to the sequencer. */
	TSharedPtr< FUICommandInfo > ToggleLinkCurveEditorTimeRange;

	/** Toggles whether or not the curve editor time tree view should be linked with the sequencer filters. */
	TSharedPtr< FUICommandInfo > ToggleLinkFiltersWithCurveEditor;

	/** Toggles visualization of pre and post roll */
	TSharedPtr< FUICommandInfo > ToggleShowPreAndPostRoll;

	/** Enable the move tool */
	TSharedPtr< FUICommandInfo > MoveTool;

	/** Enable the marquee selection tool */
	TSharedPtr< FUICommandInfo > MarqueeTool;

	/** Open a panel that enables exporting the sequence to a movie */
	TSharedPtr< FUICommandInfo > RenderMovie;

	/** Create camera and set it as the current camera cut */
	TSharedPtr< FUICommandInfo > CreateCamera;

	/** Paste from the sequencer clipboard history */
	TSharedPtr< FUICommandInfo > PasteFromHistory;

	/** Convert the selected possessed objects to spawnables. These will be spawned and destroyed by sequencer as per object's the spawn track. */
	TSharedPtr< FUICommandInfo > ConvertToSpawnable;

	/** Convert the selected spawnable objects to possessables. The newly created possessables will be created in the current level. */
	TSharedPtr< FUICommandInfo > ConvertToPossessable;

	/** Saves the current state of this object as the default spawnable state. */
	TSharedPtr< FUICommandInfo > SaveCurrentSpawnableState;

	/** Restores all animated state for the current sequence. */
	TSharedPtr< FUICommandInfo > RestoreAnimatedState;

	/** Attempts to fix possessables whose object class don't match the object class of their currently bound objects. */
	TSharedPtr< FUICommandInfo > FixPossessableObjectClass;

	/** Rebinds all possessable references with their current bindings. */
	TSharedPtr< FUICommandInfo > RebindPossessableReferences;

	/** Toggle whether we should evaluate sub sequences in isolation */
	TSharedPtr< FUICommandInfo > ToggleEvaluateSubSequencesInIsolation;

	/** Sets a transform key at the current time for the selected actor */
	TSharedPtr< FUICommandInfo > AddTransformKey;

	/** Sets a translation key at the current time for the selected actor */
	TSharedPtr< FUICommandInfo > AddTranslationKey;

	/** Sets a rotation key at the current time for the selected actor */
	TSharedPtr< FUICommandInfo > AddRotationKey;

	/** Sets a scale key at the current time for the selected actor */
	TSharedPtr< FUICommandInfo > AddScaleKey;

	/** Deletes the transform key at the current time for the selected actor */
	TSharedPtr< FUICommandInfo > DeleteTransformKey;

	/** Deletes the translation key at the current time for the selected actor */
	TSharedPtr< FUICommandInfo > DeleteTranslationKey;

	/** Deletes the rotation key at the current time for the selected actor */
	TSharedPtr< FUICommandInfo > DeleteRotationKey;

	/** Deletes the scale key at the current time for the selected actor */
	TSharedPtr< FUICommandInfo > DeleteScaleKey;

	/** Set the key to a specified time */
	TSharedPtr< FUICommandInfo > SetKeyTime;

	/** Set the selected key's time to the current time */
	TSharedPtr< FUICommandInfo > Rekey;

	/** Snap selected keys to frame */
	TSharedPtr< FUICommandInfo > SnapToFrame;

	/** Toggle piloting the last camera or the camera cut camera */
	TSharedPtr< FUICommandInfo > TogglePilotCamera;

	/** Forcibly refresh the UI */
	TSharedPtr< FUICommandInfo > RefreshUI;

	/** Toggles selectability of viewport objects to Sequencer items only */
	TSharedPtr< FUICommandInfo > ToggleLimitViewportSelection;

	/** Toggles the visibility of the Sequencer sidebar */
	TSharedPtr< FUICommandInfo > ToggleSidebarVisible;

	/** Toggles the state of the Sequencer selection details between docked in the Sequencer tab and being popped out to its own tab window */
	TSharedPtr< FUICommandInfo > ToggleSelectionDetailsPoppedOut;

	/** Toggles the sidebar "Selection" drawer open or closed */
	TSharedPtr< FUICommandInfo > ToggleSidebarSelectionDrawerOpen;

	/** Toggles the docked state of the current sidebar drawer. Undocks the docked sidebar drawer if docked or docks the sidebar drawer if there is one open and no currently docked drawer. */
	TSharedPtr< FUICommandInfo > ToggleSidebarDrawerDock;

	/** Aligns layer bars, sections, or key selections to the current playhead location */
	TSharedPtr< FUICommandInfo > AlignSelectionToPlayhead;

	/** Toggle the pinned state of Sequencer track selection */
	TSharedPtr< FUICommandInfo > ToggleTrackSelectionPin;

	/** Toggle the locked state of Sequencer track selection */
	TSharedPtr< FUICommandInfo > ToggleTrackSelectionLock;

	/** Toggle the deactived state of Sequencer track selection */
    TSharedPtr< FUICommandInfo > ToggleTrackSelectionDeactive;

	/** Toggle the muted state of Sequencer track selection */
	TSharedPtr< FUICommandInfo > ToggleTrackSelectionMute;

	/** Toggle the soloed state of Sequencer track selection */
	TSharedPtr< FUICommandInfo > ToggleTrackSelectionSolo;

	/** Toggle between the toolbar panel being visible and hidden */
	TSharedPtr< FUICommandInfo > ToggleToolBarVisible;

	/** Toggle between the transport controls panel being visible and hidden */
	TSharedPtr< FUICommandInfo > ToggleTransportPanelVisible;

	/** Toggle between the transport controls panel being popped out in its own tab window and docked within its inline content */
	TSharedPtr< FUICommandInfo > ToggleTransportPanelDocked;

	/** Toggle between Sequencer simplified view and full views */
	TSharedPtr< FUICommandInfo > ToggleSimpleView;

	/**
	 * Initialize commands
	 */
	SEQUENCER_API virtual void RegisterCommands() override;
};
