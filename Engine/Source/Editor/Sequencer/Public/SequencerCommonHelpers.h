// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Input/SNumericDropDown.h"
#include "Containers/ArrayView.h"
#include "Misc/FrameNumber.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"

#define LOCTEXT_NAMESPACE "SequencerHelpers"

class FPropertyPath;
class FSequencer;
class IKeyArea;
class IPropertyHandle;
class ISequencer;
class UMovieSceneDecorationContainerObject;
class UMovieSceneSection;
class UMovieSceneTrack;
struct FKeyHandle;

namespace UE
{
namespace Sequencer
{
	class FChannelModel;
	class FObjectBindingModel;
	class IOutlinerExtension;
	struct FKeySelection;
	struct FViewModelVariantIterator;

} // namespace Sequencer
} // namespace UE


class SEQUENCER_API SequencerHelpers
{
public:
	using FViewModel = UE::Sequencer::FViewModel;

	/**
	 * Gets the key areas from the requested node
	 */
	static void GetAllKeyAreas(TSharedPtr<FViewModel> DataModel, TSet<TSharedPtr<IKeyArea>>& KeyAreas);

	/**
	 * Gets the channels from the requested node
	 */
	static void GetAllChannels(TSharedPtr<FViewModel> DataModel, TSet<TSharedPtr<UE::Sequencer::FChannelModel>>& Channels);

	/**
	 * Get the section index that relates to the specified time
	 * @return the index corresponding to the highest overlapping section, or nearest section where no section overlaps the current time
	 */
	static int32 GetSectionFromTime(TArrayView<UMovieSceneSection* const> InSections, FFrameNumber Time);

	/**
	 * Get descendant nodes
	 */
	static void GetDescendantNodes(TSharedRef<FViewModel> DataModel, TSet<TSharedRef<FViewModel> >& Nodes);

	/**
	 * Gets all sections from the requested node
	 */
	static void GetAllSections(TSharedPtr<FViewModel> DataModel, TSet<TWeakObjectPtr<UMovieSceneSection>>& Sections);

	/**
	 * Perform default selection for the specified mouse event, based on the current hotspot
	 */
	static void PerformDefaultSelection(FSequencer& Sequencer, const FPointerEvent& MouseEvent);
	
	/*
	 * Remove duplicate keys 
	 */
	static void RemoveDuplicateKeys(const UE::Sequencer::FKeySelection& KeySelection, TArrayView<FKeyHandle> KeyHandles);

	/**
	 * Attempt to summon a context menu for the current hotspot
	 */
	static TSharedPtr<SWidget> SummonContextMenu(FSequencer& Sequencer, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/*
	 * Build a sub menu for adding a new track section
	 */
	static void BuildNewSectionMenu(const TWeakPtr<FSequencer>& InWeakSequencer
		, const int32 InRowIndex
		, const TWeakObjectPtr<UMovieSceneTrack>& InTrackWeak
		, FMenuBuilder& MenuBuilder);

	/*
	 * Build an inline menu or sub menu for editing track section(s)
	 */
	static void BuildEditSectionMenu(const TWeakPtr<FSequencer>& InWeakSequencer
		, const TArray<TWeakObjectPtr<>>& InWeakSections
		, FMenuBuilder& MenuBuilder
		, const bool bInSubMenu);

	/*
	* Build an inline menu or sub menu for editing track(s)
	*/
	static void BuildEditTrackMenu(const TWeakPtr<FSequencer>& InWeakSequencer
		, const TArray<TWeakObjectPtr<>>& InWeakTracks
		, FMenuBuilder& MenuBuilder
		, const bool bInSubMenu);

	/*
	 * Build an inline menu for editing decoration(s)
	 */
	static void BuildEditDecorationMenu(const TWeakPtr<FSequencer>& InWeakSequencer
		, const TArray<TWeakObjectPtr<>>& InWeakDecorations
		, FMenuBuilder& MenuBuilder);

	/*
	 * Build a menu for selection the blend algorithm
	 */
	static void BuildBlendingMenu(const TWeakPtr<FSequencer>& InWeakSequencer
		, const TWeakObjectPtr<UMovieSceneTrack>& InTrackWeak
		, FMenuBuilder& MenuBuilder);

	/*
	 * Build a menu for selected object bindings
	 */
	static void BuildObjectBindingMenu(const TWeakPtr<FSequencer>& InWeakSequencer
		, const UE::Sequencer::TViewModelPtr<UE::Sequencer::FObjectBindingModel>& InObjectBindingModel
		, FMenuBuilder& MenuBuilder);

	/**
	 * Gets all section objects from track area models
	 */
	static TArray<TWeakObjectPtr<>> GetSectionObjectsFromTrackAreaModels(const UE::Sequencer::FViewModelVariantIterator& InTrackAreaModels);

	/**
	 * Sorts an array of outliner items by start time of the first layer bar or selection order
	 * @param Sequencer The Sequencer that the outliner items belong to
	 * @param InItems The list of outliner items to sort
	 * @param bInSortByItemOrder If true, sorts by the order of the item in the array instead of by start time of the layer bar
	 * @param bInDescending If true, sorts in reverse order instead of ascending
	 * @param bInTransact If true, creates an editor transaction that can be undone
	 */
	static void SortOutlinerItems(FSequencer& Sequencer
		, const TArray<UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>>& InItems
		, const bool bInSortByItemOrder
		, const bool bInDescending
		, const bool bInTransact = true);

	/**
	 * Check if a decoration container has any compatible decorations that can be added.
	 * Returns true if there are compatible decorations that are not already present.
	 */
	static bool HasCompatibleDecorations(const UMovieSceneDecorationContainerObject* DecorationContainer);

	// Returns true if the container has any decorations that should be visible in the outliner
	// (implements IMovieSceneChannelDecoration or IMovieSceneSectionProviderDecoration).
	static bool HasOutlinerVisibleDecorations(const UMovieSceneDecorationContainerObject* Container);

	/**
	 * Populates a menu with "Add Decoration" entries for compatible decorations.
	 */
	static void BuildDecorationMenu(FMenuBuilder& MenuBuilder
		, UMovieSceneDecorationContainerObject* DecorationContainer
		, const FGuid& ObjectBindingID
		, TWeakPtr<ISequencer> InSequencer
		, const FText& SectionLabel = FText());

	/*
	 * Build an FPropertyPath from an IPropertyHandle. 
	 */
	static FPropertyPath PropertyHandleToPropertyPath(const IPropertyHandle& InPropertyHandle);
};

#undef LOCTEXT_NAMESPACE
