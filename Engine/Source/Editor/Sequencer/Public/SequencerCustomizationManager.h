// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISequencerModule.h"
#include "Misc/EnumClassFlags.h"
#include "MVVM/ViewModelPtr.h"

#define UE_API SEQUENCER_API

class FAssetDragDropOp;
class FClassDragDropOp;
class FActorDragDropOp;
class FFolderDragDropOp;
class UMovieSceneSequence;

/** A delegate that gets executed when a drag/drop event happens on the sequencer. The return value determines if the event was handled by the bound delegate. */
DECLARE_DELEGATE_RetVal_ThreeParams(bool, FOptionalOnDragDrop, const FGeometry&, const FDragDropEvent&, FReply&);

enum class ESequencerDropResult
{
	Unhandled,
	DropDenied,
	DropHandled
};

/** A delegate that gets executed when an asset is dropped on the sequencer. The return value determines if the operation was handled by the bound delegate. */
DECLARE_DELEGATE_RetVal_TwoParams(ESequencerDropResult, FOnAssetsDrop, const TArray<UObject*>&, const FAssetDragDropOp&);

/** A delegate that gets executed when a class is dropped on the sequencer. The return value determines if the operation was handled by the bound delegate. */
DECLARE_DELEGATE_RetVal_TwoParams(ESequencerDropResult, FOnClassesDrop, const TArray<TWeakObjectPtr<UClass>>&, const FClassDragDropOp&);

/** A delegate that gets executed when an actor is dropped on the sequencer. The return value determines if the operation was handled by the bound delegate. */
DECLARE_DELEGATE_RetVal_TwoParams(ESequencerDropResult, FOnActorsDrop, const TArray<TWeakObjectPtr<AActor>>&, const FActorDragDropOp&);

/** A delegate that gets executed when a folder is dropped on the sequencer. The return value determines if the operation was handled by the bound delegate. */
DECLARE_DELEGATE_RetVal_TwoParams(ESequencerDropResult, FOnFoldersDrop, const TArray<FName>&, const FFolderDragDropOp&);


enum class ESequencerPasteSupport : uint8
{
	None = 0,
	Folders = 1 << 0,
	ObjectBindings = 1 << 1,
	Tracks = 1 << 2,
	Sections = 1 << 3,
	Decorations = 1 << 4,
	All = 0xff
};
ENUM_CLASS_FLAGS(ESequencerPasteSupport);

/** A delegate that customizes how the sequencer behaves when data is pasted into it */
DECLARE_DELEGATE_RetVal(ESequencerPasteSupport, FOnSequencerPaste);

/** A delegate that returns an extender */
DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<FExtender>, FOnGetSequencerMenuExtender, UE::Sequencer::FViewModelPtr);

/** Class for specifying customizations to apply to a sequence editor. */
struct FSequencerCustomizationInfo
{
	/** Extender for the "add track" menu. */
	TSharedPtr<FExtender> AddMenuExtender;

	/** Extender for the sequencer toolbar. */
	TSharedPtr<FExtender> ToolbarExtender;

	/** Extender for the actions menu. */
	TSharedPtr<FExtender> ActionsMenuExtender;

	/** Extender for the view menu. */
	TSharedPtr<FExtender> ViewMenuExtender;
	
	/** Extender for the sequencer sidebar. */
	TSharedPtr<FExtender> SidebarExtender;

	/** Extender for the object binding. */
	FOnGetSequencerMenuExtender OnBuildObjectBindingContextMenu;

	/** Called when the sequencer sidebar is being built. */
	FOnGetSequencerMenuExtender OnBuildSidebarMenu;

	/** Called when something is dragged over the sequencer. */
	FOptionalOnDragDrop OnReceivedDragOver;

	/** Called when something is dropped on the sequencer. */
	FOptionalOnDragDrop OnReceivedDrop;

	/** Called when an asset is dropped on the sequencer. Only if OnReceivedDrop hasn't handled the event. */
	FOnAssetsDrop OnAssetsDrop;

	/** Called when a class is dropped on the sequencer. Only if OnReceivedDrop hasn't handled the event. */
	FOnClassesDrop OnClassesDrop;

	/** Called when an actor is dropped on the sequencer. Only if OnReceivedDrop hasn't handled the event. */
	FOnActorsDrop OnActorsDrop;

	/** Called when a folder is dropped on the sequencer. Only if OnReceivedDrop hasn't handled the event. */
	FOnFoldersDrop OnFoldersDrop;

	/** Called when data is pasted into the sequence */
	FOnSequencerPaste OnPaste;
};

/** Class to pass to ISequencerCustomization for building a customization. */
class FSequencerCustomizationBuilder
{
public:
	FSequencerCustomizationBuilder(ISequencer& InSequencer, UMovieSceneSequence& InFocusedSequence)
		: Sequencer(InSequencer), FocusedSequence(InFocusedSequence)
	{}

	/** Gets the sequencer that we are customizing. Customizations can hold on to this reference. */
	ISequencer& GetSequencer() const { return Sequencer; }
	/** Gets the sequence that is currently edited. Customizations can hold on to this reference. */
	UMovieSceneSequence& GetFocusedSequence() const { return FocusedSequence; }
	/** Gets the already registered customizations. */
	const TArray<FSequencerCustomizationInfo>& GetCustomizations() const { return Customizations; }

	/** Adds a new customization for the current sequence. */
	void AddCustomization(const FSequencerCustomizationInfo& Customization) { Customizations.Add(Customization); }

private:
	ISequencer& Sequencer;
	UMovieSceneSequence& FocusedSequence;
	TArray<FSequencerCustomizationInfo> Customizations;
};

/** Class that can figure out what customizations to apply to a given sequence. */
class ISequencerCustomization
{
public:
	virtual ~ISequencerCustomization() = default;
	virtual void RegisterSequencerCustomization(FSequencerCustomizationBuilder& Builder) = 0;
	virtual void UnregisterSequencerCustomization() = 0;
};

DECLARE_DELEGATE_RetVal(ISequencerCustomization*, FOnGetSequencerCustomizationInstance);

/** Manager class for sequencer customizations. */
class FSequencerCustomizationManager
{
public:
	UE_API FSequencerCustomizationManager();

	/**
	 * Registers a sequencer customization against a specific type of sequence.
	 */
	UE_API void RegisterInstancedSequencerCustomization(const UClass* SequenceClass, FOnGetSequencerCustomizationInstance GetCustomizationDelegate);
	/**
	 * Unregisters a sequencer customization that was previously registered against a specific type of sequence.
	 */
	UE_API void UnregisterInstancedSequencerCustomization(const UClass* SequenceClass);

	/**
	 * Gets the sequencer customizations for the given sequence.
	 */
	UE_API void GetSequencerCustomizations(UMovieSceneSequence* FocusedSequence, TArray<TUniquePtr<ISequencerCustomization>>& OutCustomizations);

	/**
	 * Determines whether the new edited sequence requires changing the customizations.
	 */
	UE_API bool NeedsCustomizationChange(const UMovieSceneSequence* OldFocusedSequence, const UMovieSceneSequence* NewFocusedSequence) const;

private:
	FSequencerCustomizationManager(const FSequencerCustomizationManager&) = delete;
	FSequencerCustomizationManager& operator=(const FSequencerCustomizationManager&) = delete;

	struct FCustomizationRegistryEntry
	{
		const UClass* SequenceClass;
		FOnGetSequencerCustomizationInstance Factory;
	};

	TArray<FCustomizationRegistryEntry> CustomizationRegistryEntries;
};

#undef UE_API
