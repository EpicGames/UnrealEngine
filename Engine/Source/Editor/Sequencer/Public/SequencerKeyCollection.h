// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "CoreTypes.h"
#include "Math/Range.h"
#include "MVVM/ViewModels/ViewModelHierarchy.h"
#include "Misc/FrameNumber.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Misc/OptionalFwd.h"
#include "Templates/SharedPointer.h"

class IKeyArea;
class UMovieSceneSection;

namespace UE
{
namespace Sequencer
{
	class FViewModel;
}
}

/** Enumeration used to define how to search for keys */
enum class EFindKeyDirection
{
	Backwards, Forwards
};

enum class EFindKeyType : uint8
{
	FKT_Keys,
	FKT_Sections,
	FKT_All
};

struct FSequencerKeyCollectionSignature
{
	using FViewModel = UE::Sequencer::FViewModel;

	FSequencerKeyCollectionSignature()
	{}

	/** Initialize this key collection from the specified nodes. Only gathers keys from those explicitly specified. */
	SEQUENCER_API static FSequencerKeyCollectionSignature FromNodes(const TArray<TSharedRef<FViewModel>>& InNodes, FFrameNumber InDuplicateThreshold);

	/** Initialize this key collection from the specified nodes. Gathers keys from all child nodes. */
	SEQUENCER_API static FSequencerKeyCollectionSignature FromNodesRecursive(const TArray<TSharedRef<FViewModel>>& InNodes, FFrameNumber InDuplicateThreshold);

	/** Initialize this key collection from the specified node and section index. */
	SEQUENCER_API static FSequencerKeyCollectionSignature FromNodeRecursive(TSharedRef<FViewModel> InNode, UMovieSceneSection* InSection, FFrameNumber InDuplicateThreshold);

	/** Compare this signature for inequality with another */
	SEQUENCER_API friend bool operator!=(const FSequencerKeyCollectionSignature& A, const FSequencerKeyCollectionSignature& B);

	/** Compare this signature for equality with another */
	SEQUENCER_API friend bool operator==(const FSequencerKeyCollectionSignature& A, const FSequencerKeyCollectionSignature& B);

	/** Access the map of keyareas and signatures this signature was generated for */
	const TMap<TSharedRef<IKeyArea>, FGuid>& GetKeyAreas() const
	{
		return KeyAreaToSignature;
	}

	/** Access the map of signatures to section bounds that this signature was generated for */
	const TMap<FGuid, TRange<FFrameNumber>>& GetSectionBounds() const
	{
		return SignatureToSectionBounds;
	}

	/** Access duplicate threshold that this signature was generated for */
	FFrameNumber GetDuplicateThreshold() const
	{
		return DuplicateThresholdTime;
	}

private:

	/** Check whether this signature contains content that cannot be cached (such content causes this signature to never compare equal with another) */
	bool HasUncachableContent() const;

	/** The time at which proximal keys are considered duplicates */
	FFrameNumber DuplicateThresholdTime;

	/** Map of key areas to the section signature with with this signature was generated */
	TMap<TSharedRef<IKeyArea>, FGuid> KeyAreaToSignature;

	/** Map of the section signature to the section bounds */
	TMap<FGuid, TRange<FFrameNumber> > SignatureToSectionBounds;
};

/**
 * A collection of keys gathered recursively from a particular node or nodes
 */
class FSequencerKeyCollection
{
public:

	/**
	 * Search forwards or backwards for the first key within the specified range
	 *
	 * @param Range       The range to search within
	 * @param Direction   Whether to return the first or last key that reside in the given range
	 * @param FindKeyType Whether to find all keys, sections or all
	 * @return (Optional) the time of the key that matched the range
	 */
	SEQUENCER_API TOptional<FFrameNumber> FindFirstKeyInRange(const TRange<FFrameNumber>& Range, EFindKeyDirection Direction, EFindKeyType FindKeyType = EFindKeyType::FKT_All) const;
	UE_DEPRECATED(5.4, "Please use FindFirstKeyInRange which takes EFindKeyType, FindFirstSectionKeyInRange is no longer supported")
	TOptional<FFrameNumber> FindFirstSectionKeyInRange(const TRange<FFrameNumber>& Range, EFindKeyDirection Direction) const { return FindFirstKeyInRange(Range, Direction, EFindKeyType::FKT_Sections); }

	/**
	 * Get a view of all key times that reside within the specified range
	 *
	 * @param Range       The range to search within
	 * @param FindKeyType Whether to find all keys, sections or all
	 * @return A (possibly empty) array view of all the times that lie within the range
	 */
	SEQUENCER_API TArrayView<const FFrameNumber> GetKeysInRange(const TRange<FFrameNumber>& Range, EFindKeyType FindKeyType = EFindKeyType::FKT_All) const;
	UE_DEPRECATED(5.4, "Please use GetKeysInRange which takes EFindKeyType, GetSectionKeysInRange is no longer supported")
	TArrayView<const FFrameNumber> GetSectionKeysInRange(const TRange<FFrameNumber>& Range) const { return GetKeysInRange(Range, EFindKeyType::FKT_Sections); }

	/**
	* Search forwards or backwards for the next key from the specified frame number
	* @param FrameNumber The frame number to search from
	* @param Direction   Whether to return the next key or previous key from that time
	* @param Range       The range to search within
	* @param FindKeyType Whether to find all keys, sections or all
	* @param bAllowLoop  Whether to loop around if past the last key when retrieving the next key (or past the first key when retrieving the previous key)
	* @return (Optional)  Frame number of the key that's next or previous from that time 
	*/
	SEQUENCER_API TOptional<FFrameNumber> GetNextKey(FFrameNumber FrameNumber, EFindKeyDirection Direction, const TRange<FFrameNumber>& Range, EFindKeyType FindKeyType = EFindKeyType::FKT_All, const bool bAllowLoop = true) const;
	UE_DEPRECATED(5.4, "Please use GetNextKey which takes EFindKeyType, GetNextSectionKey is no longer supported")
	TOptional<FFrameNumber> GetNextSectionKey(FFrameNumber FrameNumber, EFindKeyDirection Direction, const TRange<FFrameNumber>& Range) const { return GetNextKey(FrameNumber, Direction, Range, EFindKeyType::FKT_Sections); }

	/**
	 * Access the signature this collection was generated with
	 *
	 * @return The signature that this collection was generated with
	 */
	const FSequencerKeyCollectionSignature& GetSignature() const
	{
		return Signature;
	}

public:

	/**
	 * Update this key collection using the specified signature
	 *
	 * @param InSignature    The signature to generate keys for, containing all key areas to use for the generation
	 * @return true if this collection was updated, or false if it was already up to date
	 */
	bool Update(const FSequencerKeyCollectionSignature& InSignature);

private:

	/** All keys and section times grouped by the supplied threshold */
	TArray<FFrameNumber> AllGroupedTimes;

	/** Times grouped by the supplied threshold */
	TArray<FFrameNumber> GroupedTimes;

	/** Section times grouped by the supplied threshold */
	TArray<FFrameNumber> GroupedSectionTimes;

	/** The signature with which the above array was generated */
	FSequencerKeyCollectionSignature Signature;
};
