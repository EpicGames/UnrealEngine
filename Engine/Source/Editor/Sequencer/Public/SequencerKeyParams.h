// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Curves/KeyHandle.h"
#include "HAL/PlatformCrt.h"
#include "Misc/FrameNumber.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"

class IKeyArea;
class ISequencer;
class ISequencerSection;
class ISequencerTrackEditor;
class UMovieSceneTrack;
struct FFrameNumber;
struct FGuid;
template <typename FuncType> class TFunctionRef;

namespace UE
{
namespace Sequencer
{

struct FAddKeyResult
{
	TSharedPtr<IKeyArea> KeyArea;
	FKeyHandle KeyHandle;
};

/**
 * Structure defining a section that needs to be keyed by specific channels
 */
struct FKeySectionOperation
{
	/**
	 * Default application function that adds keys to the section by calling IKeyArea->AddOrUpdateKey on each channel
	 *
	 * @param InKeyTime      The time at which to add keys
	 * @param InSequencer    The current sequencer instance that is adding the keys
	 */
	SEQUENCER_API void Apply(FFrameNumber InKeyTime, ISequencer& InSequencer) const;

	/** The section to add keys to */
	TSharedPtr<ISequencerSection> Section;

	/** The key areas defining the channels to key */
	TArray<TSharedPtr<IKeyArea>>  KeyAreas;
};

/**
 * Structure defining an opeartion that should add keys to a track
 */
struct FKeyOperation
{
	using CallbackType = void(UMovieSceneTrack*, TArrayView<const FKeySectionOperation>);

	/**
	 * Iterate all the individual operations
	 */
	SEQUENCER_API void IterateOperations(TFunctionRef<CallbackType> Callback) const;


	/**
	 * Apply this operation
	 *
	 * @param InKeyTime      The time at which to add keys
	 * @param InSequencer    The current sequencer instance that is adding the keys
	 * @param OutResults     The key handles that were added with this operation
	 */
	SEQUENCER_API void ApplyDefault(FFrameNumber InKeyTime, ISequencer& InSequencer, TArray<FAddKeyResult>* OutResults = nullptr) const;

	/**
	 * Apply a specific set of section operations
	 *
	 * @param InKeyTime      The time at which to add keys
	 * @param InOperations   The operations to apply
	 * @param InSequencer    The current sequencer instance that is adding the keys
	 * @param OutResults     The key handles that were added with this operation
	 */
	SEQUENCER_API static void ApplyOperations(FFrameNumber InKeyTime, TArrayView<const FKeySectionOperation> InOperations, const FGuid& ObjectBindingID, ISequencer& InSequencer, TArray<FAddKeyResult>* OutResults = nullptr);

public:

	/*~ Public, non-exported API */

	/**
	 * Populate this operation with a specific track, section and channel.
	 * @note This channel may or may not be used for the final operation depending on the result of ChooseOperation()
	 *
	 * @param InTrack        The track object that owns the section. WARNING: Must relate to this operation's track editor.
	 * @param InSection      The section interface for the section to add keys to
	 * @param InKeyArea      They key area relating to the channel to key
	 */
	void Populate(UMovieSceneTrack* InTrack, TSharedPtr<ISequencerSection> InSection, TSharedPtr<IKeyArea> InKeyArea);


	/**
	 * Called before this operation is applied - intializes the sections and adds them to the transaction buffer
	 *
	 * @param InKeyTime      The time that keys will be added at
	 */
	void InitializeOperation(FFrameNumber InKeyTime);


private:

	struct FSectionCandidates
	{
		void FilterOperations(UMovieSceneTrack* Track, FFrameNumber KeyTime);

		TArray<FKeySectionOperation, TInlineAllocator<1>> Operations;
	};

	/** Mapping of track to section key data */
	TMap<UMovieSceneTrack*, FSectionCandidates> CandidatesByTrack;
};


} // namespace Sequencer
} // namespace UE
