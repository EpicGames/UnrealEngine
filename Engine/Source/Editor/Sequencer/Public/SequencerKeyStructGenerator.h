// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Curves/KeyHandle.h"
#include "Misc/FrameNumber.h"
#include "MovieSceneKeyStruct.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/FieldPath.h"
#include "UObject/GCObject.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StructOnScope.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

#include "SequencerKeyStructGenerator.generated.h"

#define UE_API SEQUENCER_API

class FArrayProperty;
class FProperty;
class FSequencerKeyStructGenerator;
class FStructProperty;
class UObject;

/**
 * Struct type that is generated from an FMovieSceneChannel type to create a single edit interface for a key/value pair
 */
UCLASS(MinimalAPI)
class UMovieSceneKeyStructType : public UScriptStruct
{
public:
	GENERATED_BODY()

	UE_API UMovieSceneKeyStructType(const FObjectInitializer& ObjInit);

	UE_API void InitializeStruct(void* InDest, int32 ArrayDim) const override;
	UE_API void DestroyStruct(void* Dest, int32 ArrayDim) const override;

	/**
	 * Check whether this generated struct is complete and ready to be used
	 */
	bool IsComplete() const
	{
		return SourceValuesProperty.Get() && SourceTimesProperty.Get() && DestValueProperty.Get() && DestTimeProperty.Get();
	}

	/** The (external) source TArray<FFrameNumber> property that stores the key times in the channel */
	UPROPERTY()
	TFieldPath<FArrayProperty> SourceTimesProperty;

	/** The (external) source TArray<T> property that stores the key values in the channel */
	UPROPERTY()
	TFieldPath<FArrayProperty> SourceValuesProperty;

	/** The time property for this reflected struct, of type FFrameNumber */
	UPROPERTY()
	TFieldPath<FStructProperty> DestTimeProperty;

	/** The value property for this reflected struct, of the same type as SourceValuesProperty->Inner */
	UPROPERTY()
	TFieldPath<FProperty> DestValueProperty;

private:
	using UStruct::SetSuperStruct;
};

/**
 * Singleton class that is used to create, store and instantiate generated structs for editing single keys on channels
 */
class FSequencerKeyStructGenerator : FGCObject
{
public:

	/**
	 * Access the singlton instance of this class
	 */
	static UE_API FSequencerKeyStructGenerator& Get();


	/**
	 * Allocate a brand new, empty key struct type. Must be fully completed and finalized with FinalizeNewKeyStruct
	 */
	static UE_API UMovieSceneKeyStructType* AllocateNewKeyStruct();


	/**
	 * Allocate a brand new, key struct type, automatically discovering time/value properties from property meta-data. Must be finalized with FinalizeNewKeyStruct
	 */
	static UE_API UMovieSceneKeyStructType* AllocateNewKeyStruct(UScriptStruct* ChannelType);


	/**
	 * Finalize the specified struct type by prepending a time property and linking the struct
	 */
	static UE_API void FinalizeNewKeyStruct(UMovieSceneKeyStructType* InStruct);


	/**
	 * Helper function to locate an array property with the specified meta-data tag
	 */
	static UE_API FArrayProperty* FindArrayPropertyWithTag(UScriptStruct* ChannelStruct, FName MetaDataTag);

public:


	/**
	 * Add the specified struct to this manager with the specified unique name, explicitly keeping it alive through AddReferencedObjects.
	 *
	 * @param InstancedStructName       A unique name to associate with this struct, potentially for later retrieval. May include a specific property postfix such as an object or enum type for dynamic channels.
	 * @param GeneratedStruct           The generated struct to associate with the name
	 */
	UE_API void AddGeneratedStruct(FName InstancedStructName, UMovieSceneKeyStructType* Struct);


	/**
	 * Attempt to locate an existing struct type with the specified name
	 *
	 * @param InstancedStructName       The unique name to look for
	 * @return A generated struct that was added with the specified name, or nullptr if one does not exist.
	 */
	UE_API UMovieSceneKeyStructType* FindGeneratedStruct(FName InstancedStructName);


	/**
	 * Create a new generated key struct by reflecting array properties with 'KeyTimes' and 'KeyValues' meta-data
	 *
	 * @param ChannelType     The struct type of the channel (normally retrieved through ::StaticStruct())
	 * @return A valid pointer to a UMovieSceneKeyStructType struct type, or nullptr if one could not be found or allocated
	 */
	UE_API UMovieSceneKeyStructType* DefaultInstanceGeneratedStruct(UScriptStruct* ChannelType);

public:

	/**
	 * Create a new struct instance using the specified channel and key handles
	 *
	 * @note: Specific generation logic may be implemented by overloading the following free-function for your channel type:
	 * UMovieSceneKeyStructType* InstanceGeneratedStruct(ChannelType* Channel, FSequencerKeyStructGenerator* Generator)
	 *
	 * @return An instance of a key struct for use on an edit UI, or nullptr if no valid one could be generated.
	 */
	template<typename ChannelType>
	TSharedPtr<FStructOnScope> CreateKeyStructInstance(const TMovieSceneChannelHandle<ChannelType>& ChannelHandle, FKeyHandle InHandle);

private:

	/** Mapping of instance name -> generated struct type for reference collection */
	TMap<FName, TObjectPtr<UMovieSceneKeyStructType>> InstanceNameToGeneratedStruct;

	FSequencerKeyStructGenerator(){}
	~FSequencerKeyStructGenerator()= default;

	/**
	 * Create a new struct instance populated with the time and value for the specified key index, but no OnPropertyChangedEvent initialized.
	 */
	UE_API TSharedPtr<FStructOnScope> CreateInitialStructInstance(const void* SourceChannel, UMovieSceneKeyStructType* GeneratedStructType, int32 InitialKeyIndex);

	/**
	 * FGCObject Interface
	 */
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FSequencerKeyStructGenerator");
	}

	/**
	 * Applies reflected values from the key struct instance back into the channel, called on user-interaction with the edit instance
	 */
	template<typename ChannelType>
	static void CopyInstanceToKey(const TMovieSceneChannelHandle<ChannelType>& DestChannelHandle, FKeyHandle DestHandle, FStructOnScope* SourceInstance);
};



/**
 * Function overload used to create a new generated struct for the specified channel.
 * Overload the first parameter for specific channel types to create specific instancing logic (such as that required for object or enum properties):
 * UMovieSceneKeyStructType* InstanceGeneratedStruct(ChannelType* Channel, FSequencerKeyStructGenerator* Generator)
 */
inline UMovieSceneKeyStructType* InstanceGeneratedStruct(void* Channel, FSequencerKeyStructGenerator* Generator)
{
	return nullptr;
}

/**
 * Called to initialize a newly allocated key struct for editing.
 * Empty by default, but can be overridden to perform any per-instance setup required for specific channel's key structs
 */
template<typename ChannelType>
void PostConstructKeyInstance(const TMovieSceneChannelHandle<ChannelType>& ChannelHandle, FKeyHandle InHandle, FStructOnScope* Struct)
{
	ChannelType* Channel = ChannelHandle.Get();
	if (Channel)
	{
		const UMovieSceneKeyStructType* GeneratedStructType = CastChecked<const UMovieSceneKeyStructType>(Struct->GetStruct());
		void* StructPtr = Struct->GetStructMemory();

		const int32 InitialKeyIndex = Channel->GetData().GetIndex(InHandle);

		// Copy the initial value into the struct
		if (InitialKeyIndex != INDEX_NONE)
		{
			const uint8* SrcValueData  = GeneratedStructType->SourceValuesProperty->ContainerPtrToValuePtr<uint8>(Channel);
			uint8*       DestValueData = GeneratedStructType->DestValueProperty->ContainerPtrToValuePtr<uint8>(StructPtr);

			FScriptArrayHelper SourceValuesArray(GeneratedStructType->SourceValuesProperty.Get(), SrcValueData);
			GeneratedStructType->SourceValuesProperty->Inner->CopyCompleteValue(DestValueData, SourceValuesArray.GetRawPtr(InitialKeyIndex));
		}
	}
}

template<typename ChannelType>
TSharedPtr<FStructOnScope> FSequencerKeyStructGenerator::CreateKeyStructInstance(const TMovieSceneChannelHandle<ChannelType>& ChannelHandle, FKeyHandle InHandle)
{
	ChannelType* Channel  = ChannelHandle.Get();
	if (Channel)
	{
		const int32  KeyIndex = Channel->GetData().GetIndex(InHandle);
		if (KeyIndex != INDEX_NONE)
		{
			UMovieSceneKeyStructType* GeneratedStructType = InstanceGeneratedStruct(Channel, this);
			if (!GeneratedStructType)
			{
				GeneratedStructType = DefaultInstanceGeneratedStruct(ChannelType::StaticStruct());
			}

			if (GeneratedStructType && GeneratedStructType->IsComplete())
			{
				TSharedPtr<FStructOnScope> StructInstance = CreateInitialStructInstance(Channel, GeneratedStructType, KeyIndex);

				auto CopyInstanceToKeyLambda = [ChannelHandle, InHandle, StructPtr = StructInstance.Get()](const FPropertyChangedEvent&)
				{
					FSequencerKeyStructGenerator::CopyInstanceToKey(ChannelHandle, InHandle, StructPtr);
				};

				FGeneratedMovieSceneKeyStruct* KeyStruct = reinterpret_cast<FGeneratedMovieSceneKeyStruct*>(StructInstance->GetStructMemory());
				KeyStruct->OnPropertyChangedEvent = CopyInstanceToKeyLambda;

				PostConstructKeyInstance(ChannelHandle, InHandle, StructInstance.Get());

				return StructInstance;
			}
		}
	}

	return nullptr;
}

template<typename ChannelType>
void FSequencerKeyStructGenerator::CopyInstanceToKey(const TMovieSceneChannelHandle<ChannelType>& DestChannelHandle, FKeyHandle DestHandle, FStructOnScope* SourceInstance)
{
	ChannelType* DestinationChannel = DestChannelHandle.Get();
	if (DestinationChannel)
	{
		const int32  KeyIndex = DestinationChannel->GetData().GetIndex(DestHandle);
		if (KeyIndex != INDEX_NONE)
		{
			const UMovieSceneKeyStructType* GeneratedStructType = CastChecked<UMovieSceneKeyStructType>(SourceInstance->GetStruct());

			// Copy the new value back into the channel
			{
				uint8*       DestValueData = GeneratedStructType->SourceValuesProperty->ContainerPtrToValuePtr<uint8>(DestinationChannel);
				const uint8* SrcValueData  = GeneratedStructType->DestValueProperty->ContainerPtrToValuePtr<uint8>(SourceInstance->GetStructMemory());

				FScriptArrayHelper SourceValuesArray(GeneratedStructType->SourceValuesProperty.Get(), DestValueData);
				GeneratedStructType->DestValueProperty->CopyCompleteValue(SourceValuesArray.GetRawPtr(KeyIndex), SrcValueData);
			}

			// Set the new key time
			{
				FFrameNumber SrcTimeData = *GeneratedStructType->DestTimeProperty->ContainerPtrToValuePtr<FFrameNumber>(SourceInstance->GetStructMemory());
				DestinationChannel->SetKeyTime(DestHandle, SrcTimeData);
			}
		}
	}
}

#undef UE_API
