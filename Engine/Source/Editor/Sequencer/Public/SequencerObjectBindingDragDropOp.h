// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModels/OutlinerViewModelDragDropOp.h"

namespace UE
{
namespace MovieScene
{
	struct FFixedObjectBindingID;
} // namespace MovieScene

namespace Sequencer
{

class FSequencerObjectBindingDragDropOp : public FOutlinerViewModelDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE( FSequencerObjectBindingDragDropOp, FOutlinerViewModelDragDropOp )

	virtual TArray<UE::MovieScene::FFixedObjectBindingID> GetDraggedBindings() const = 0;
	virtual TArray<UE::MovieScene::FFixedObjectBindingID> GetDraggedRebindableBindings() const = 0;
};


} // namespace Sequencer
} // namespace UE