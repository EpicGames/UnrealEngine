// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreTypes.h"
#include "Layout/Geometry.h"
#include "Layout/SlateRect.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"

#define UE_API SEQUENCER_API

class FSlateWindowElementList;
class IKeyArea;
class UMovieSceneSection;
class UMovieSceneTrack;
struct FTimeToPixel;

namespace UE
{
namespace Sequencer
{
	class FSectionModel;
}
}


struct FKeyAreaElement
{
	TArray<TSharedRef<IKeyArea>, TInlineAllocator<1>> KeyAreas;
	FGeometry KeyAreaGeometry;

	/** Whether this layout element pertains to one or multiple key areas */
	enum EType { Single, Group };

	EType Type;
};

/** Class that wraps up common section painting functionality */
class FSequencerSectionPainter
{
public:
	UE_NONCOPYABLE(FSequencerSectionPainter)

	/** Constructor */
	UE_API FSequencerSectionPainter(FSlateWindowElementList& OutDrawElements, const FGeometry& InSectionGeometry, TSharedPtr<UE::Sequencer::FSectionModel> Section);

	/** Virtual destructor */
	UE_API virtual ~FSequencerSectionPainter();

	/** Paint the section background with the specified tint override */
	virtual int32 PaintSectionBackground(const FLinearColor& Tint) = 0;

	/** Paint the section background with the tint stored on the track */
	UE_API int32 PaintSectionBackground();

	/** Get the track that this painter is painting sections for */
	UE_API UMovieSceneTrack* GetTrack() const;

	UE_API FLinearColor GetSectionColor() const;

	/** Blend the specified color with the default track color */
	static UE_API FLinearColor BlendColor(FLinearColor InColor);

public:

	/** Get a time-to-pixel converter for the section */
	virtual const FTimeToPixel& GetTimeConverter() const = 0;

public:

	/** The section we are painting */
	TSharedPtr<UE::Sequencer::FSectionModel> SectionModel;

	/** List of slate draw elements - publicly modifiable */
	FSlateWindowElementList& DrawElements;

	/** The full geometry of the section. This is the width of the track area in the case of infinite sections */
	FGeometry SectionGeometry;

	/** The header geometry of the section. This is the same as SectionGeometry for collapsed sections, but just the height of the top row for expanded sections. */
	FGeometry HeaderGeometry;

	/** The full clipping rectangle for the section */
	FSlateRect SectionClippingRect;
	
	/** The layer ID we're painting on */
	int32 LayerId;

	/** Whether our parent widget is enabled or not */
	bool bParentEnabled;

	/** Whether the section is hovered or not */
	bool bIsHighlighted;

	/** Whether the section is selected or not */
	bool bIsSelected;

	/** Whether the section is to be drawn semi-transparent or not */
	float GhostAlpha;
};

#undef UE_API
