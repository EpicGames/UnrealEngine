// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CurveEditorZoomScaleConfig.h"
#include "Filters/SequencerFilterBarConfig.h"
#include "Filters/SequencerFilterAreaConfig.h"
#include "Misc/SequencerThumbnailCaptureSettings.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "FrameNumberDisplayFormat.h"
#include "Sidebar/SidebarState.h"
#include "SequencerSettings.generated.h"

#define UE_API SEQUENCER_API

class ISequencer;
struct FPropertyChangedEvent;
enum class EAutoChangeMode : uint8;
enum class EAllowEditsMode : uint8;
enum class EKeyGroupMode : uint8;
enum class EMovieSceneKeyInterpolation : uint8;

namespace UE::Sequencer
{
	enum class EViewDensity;
}

UENUM()
enum ESequencerSpawnPosition : int
{
	/** Origin. */
	SSP_Origin UMETA(DisplayName="Origin"),

	/** Place in Front of Camera. */
	SSP_PlaceInFrontOfCamera UMETA(DisplayName="Place in Front of Camera"),
};

UENUM()
enum ESequencerZoomPosition : int
{
	/** Playhead. */
	SZP_CurrentTime UMETA(DisplayName="Playhead"),

	/** Mouse Position. */
	SZP_MousePosition UMETA(DisplayName="Mouse Position"),
};

UENUM(BlueprintType)
enum ESequencerLoopMode : int
{
	/** No Looping. */
	SLM_NoLoop UMETA(DisplayName="No Looping"),

	/** Loop Playback Range. */
	SLM_Loop UMETA(DisplayName="Loop"),

	/** Loop Selection Range. */
	SLM_LoopSelectionRange UMETA(DisplayName="Loop Selection Range"),
};

UENUM()
enum ESequencerPlayheadScrubbingLimit : int
{
	SPSL_None UMETA(DisplayName="None"),

	SPSL_PlaybackRange UMETA(DisplayName="Playback Range"),

	SPSL_SelectionRange UMETA(DisplayName="Selection Range")
};

UENUM()
enum class ESequencerTimeWarpDisplay : uint8
{
	UnwarpedTime = 1 UMETA(DisplayName="Unwarped Time"),
	WarpedTime = 2 UMETA(DisplayName="Warped Time"),
	Both = UnwarpedTime | WarpedTime UMETA(DisplayName="Both"),
};
ENUM_CLASS_FLAGS(ESequencerTimeWarpDisplay)

/** Empty class used to house multiple named USequencerSettings */
UCLASS(MinimalAPI)
class USequencerSettingsContainer
	: public UObject
{
public:
	GENERATED_BODY()

	/** Get or create a settings object for the specified name */
	template<class T> 
	static T* GetOrCreate(const TCHAR* InName)
	{
		static const TCHAR* SettingsContainerName = TEXT("SequencerSettingsContainer");

		auto* Outer = FindObject<USequencerSettingsContainer>(GetTransientPackage(), SettingsContainerName);
		if (!Outer)
		{
			Outer = NewObject<USequencerSettingsContainer>(GetTransientPackage(), USequencerSettingsContainer::StaticClass(), SettingsContainerName);
			Outer->AddToRoot();
		}
	
		T* Inst = FindObject<T>( Outer, InName );
		if (!Inst)
		{
			Inst = NewObject<T>( Outer, T::StaticClass(), InName );
			Inst->LoadConfig();
		}

		return Inst;
	}
};

/** Struct for storing reorderable and hidden/visible outliner columns */
USTRUCT(BlueprintType)
struct FColumnVisibilitySetting
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category=General)
	FName ColumnName;

	UPROPERTY(EditAnywhere, Category=General)
	bool bIsVisible;

	bool operator==(const FColumnVisibilitySetting& Other) const
	{
		return ColumnName == Other.ColumnName && bIsVisible == Other.bIsVisible;
	}

	FColumnVisibilitySetting(FName InColumnName, bool InbIsVisible)
		: ColumnName(InColumnName)
		, bIsVisible(InbIsVisible)
	{}

	FColumnVisibilitySetting()
		: ColumnName(NAME_None)
		, bIsVisible(false)
	{}
};

/** Serializable options for sequencer. */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, PerObjectConfig)
class USequencerSettings
	: public UObject
{
public:
	GENERATED_UCLASS_BODY()

	DECLARE_MULTICAST_DELEGATE( FOnEvaluateSubSequencesInIsolationChanged );
	DECLARE_MULTICAST_DELEGATE( FOnShowSelectedNodesOnlyChanged );
	DECLARE_MULTICAST_DELEGATE_OneParam( FOnAllowEditsModeChanged, EAllowEditsMode );
	DECLARE_MULTICAST_DELEGATE(FOnLoopStateChanged);
	DECLARE_MULTICAST_DELEGATE(FOnTimeDisplayFormatChanged);

	/** Gets the current auto change mode. */
	UE_API EAutoChangeMode GetAutoChangeMode() const;
	/** Sets the current auto change mode. */
	UE_API void SetAutoChangeMode(EAutoChangeMode AutoChangeMode);

	/** Gets the current allow edits mode. */
	UE_API EAllowEditsMode GetAllowEditsMode() const;
	/** Sets the current auto-key mode. */
	UE_API void SetAllowEditsMode(EAllowEditsMode AllowEditsMode);
	/** Gets the multicast delegate which is run whenever the allow edits mode is changed. */
	FOnAllowEditsModeChanged& GetOnAllowEditsModeChanged() { return OnAllowEditsModeChangedEvent; }

	/** Returns what channels will get keyed when one channel changes */
	UE_API EKeyGroupMode GetKeyGroupMode() const;
	/** Sets which channels are keyed when a channel is keyed */
	UE_API void SetKeyGroupMode(EKeyGroupMode);

	/** Get the initial Interpolation type for newly created keyframes if the channel does not already have keyframes */
	UE_API EMovieSceneKeyInterpolation GetKeyInterpolation() const;
	/** Sets initial key interpolation for creating new keys on empty channels */
	UE_API void SetKeyInterpolation(EMovieSceneKeyInterpolation InKeyInterpolation);

	/** Get initial spawn position. */
	UE_API ESequencerSpawnPosition GetSpawnPosition() const;
	/** Set initial spawn position. */
	UE_API void SetSpawnPosition(ESequencerSpawnPosition InSpawnPosition);

	/** Get whether to create spawnable cameras. */
	UE_API bool GetCreateSpawnableCameras() const;
	/** Set whether to create spawnable cameras. */
	UE_API void SetCreateSpawnableCameras(bool bInCreateSpawnableCameras);

	/** Gets whether or not to show the time range slider. */
	UE_API bool GetShowRangeSlider() const;
	/** Sets whether or not to show the time range slider. */
	UE_API void SetShowRangeSlider(bool InbShowRangeSlider);

	/** Gets whether or not to show the bottom time slider. */
	UE_API bool GetShowBottomTimeSlider() const;
	/** Sets whether or not to show the bottom time slider. */
	UE_API void SetShowBottomTimeSlider(const bool bInbShow);

	/** Gets whether or not snapping is enabled. */
	UE_API bool GetIsSnapEnabled() const;
	/** Sets whether or not snapping is enabled. */
	UE_API void SetIsSnapEnabled(bool InbIsSnapEnabled);

	/** Gets whether or not to snap keys to other elements. */
	UE_API bool GetSnapKeyTimesToElements() const;
	/** Sets whether or not to snap keys to other elements. */
	UE_API void SetSnapKeyTimesToElements(bool InbSnapKeyTimesToElements);

	/** Gets whether or not to snap sections to other elements. */
	UE_API bool GetSnapSectionTimesToElements() const;
	/** sets whether or not to snap sections to other elements. */
	UE_API void SetSnapSectionTimesToElements( bool InbSnapSectionTimesToElements );

	/** Gets whether or not to snap the play time to keys while scrubbing. */
	UE_API bool GetSnapPlayTimeToKeys() const;
	/** Sets whether or not to snap the play time to keys while scrubbing. */
	UE_API void SetSnapPlayTimeToKeys(bool InbSnapPlayTimeToKeys);

	/** Gets whether or not to snap the play time to section bounds while scrubbing. */
	UE_API bool GetSnapPlayTimeToSections() const;
	/** Sets whether or not to snap the play time to section bounds while scrubbing. */
	UE_API void SetSnapPlayTimeToSections(bool InbSnapPlayTimeToSections);

	/** Gets whether or not to snap the play time to markers while scrubbing. */
	UE_API bool GetSnapPlayTimeToMarkers() const;
	/** Sets whether or not to snap the play time to markers while scrubbing. */
	UE_API void SetSnapPlayTimeToMarkers(bool InbSnapPlayTimeToMarkers);

	/** Gets whether or not to snap the play time to the pressed key. */
	UE_API bool GetSnapPlayTimeToPressedKey() const;
	/** Sets whether or not to snap the play time to the pressed key. */
	UE_API void SetSnapPlayTimeToPressedKey(bool InbSnapPlayTimeToPressedKey);

	/** Gets whether or not to snap the play time to the dragged key. */
	UE_API bool GetSnapPlayTimeToDraggedKey() const;
	/** Sets whether or not to snap the play time to the dragged key. */
	UE_API void SetSnapPlayTimeToDraggedKey(bool InbSnapPlayTimeToDraggedKey);

	/** Gets the snapping interval for curve values. */
	UE_API float GetCurveValueSnapInterval() const;
	/** Sets the snapping interval for curve values. */
	UE_API void SetCurveValueSnapInterval(float InCurveValueSnapInterval);

	/** Gets whether or not to snap keys/sections/playhead to the interval which forces whole frames */
	UE_API bool GetForceWholeFrames() const;
	/** Sets whether or not to snap keys/sections/playhead to the interval which forces whole frames */
	UE_API void SetForceWholeFrames(bool bInForceWholeFrames);

	/** Gets the state for spacing between grid lines */
	UE_API TOptional<float> GetGridSpacing() const;
	/** Sets the grid line spacing state */
	UE_API void SetGridSpacing(TOptional<float> InGridSpacing);

	/** Gets whether or not to snap curve values to the interval. */
	UE_API bool GetSnapCurveValueToInterval() const;
	/** Sets whether or not to snap curve values to the interval. */
	UE_API void SetSnapCurveValueToInterval(bool InbSnapCurveValueToInterval);

	/** Gets whether or not to show selected nodes only. */
	UE_API bool GetShowSelectedNodesOnly() const;
	/** Sets whether or not to show selected nodes only. */
	UE_API void SetShowSelectedNodesOnly(bool Visible);
	FOnShowSelectedNodesOnlyChanged& GetOnShowSelectedNodesOnlyChanged() { return OnShowSelectedNodesOnlyChangedEvent; }

	/** Gets whether left mouse drag does marquee select instead of camera orbit and ALT always moves the camera */
	UE_API bool GetLeftMouseDragDoesMarquee() const;
	/** Sets whether left mouse drag does marquee select instead of camera orbit and ALT always moves the camera */
	UE_API void SetLeftMouseDragDoesMarque(bool bDoMarque);

	/** Get zoom in/out position (mouse position or current time). */
	UE_API ESequencerZoomPosition GetZoomPosition() const;
	/** Set zoom in/out position (mouse position or current time). */
	UE_API void SetZoomPosition(ESequencerZoomPosition InZoomPosition);

	/** Gets whether or not auto-scroll is enabled when playing. */
	UE_API bool GetAutoScrollEnabled() const;
	/** Sets whether or not auto-scroll is enabled when playing. */
	UE_API void SetAutoScrollEnabled(bool bInAutoScrollEnabled);
	
	/** Gets whether or not scrubbing time hot key starts from cursor. */
	UE_API bool GetScrubTimeStartFromCursor() const;
	/** Sets whether or not scrubing time hot key starts from cursor. */
	UE_API void SetScrubTimeStartFromCursor(bool bInValue);

	/** Gets whether or not we stop playing when jumping to start or end. */
	UE_API bool GetStopPlayingWhenJumpingToStartOrEnd() const;

	/** Sets whether or not we stop playing when jumping to start or end. */
	UE_API void SetStopPlayingWhenJumpingToStartOrEnd(bool bInValue);

	/** Gets whether or not to link the curve editor time range. */
	UE_API bool GetLinkCurveEditorTimeRange() const;
	/** Sets whether or not to link the curve editor time range. */
	UE_API void SetLinkCurveEditorTimeRange(bool InbLinkCurveEditorTimeRange);

	/** Gets whether or not to link sequence filters with the curve editor tree */
	UE_DEPRECATED(5.8, "Linked filtering replaces this feature")
	UE_API bool GetLinkFiltersWithCurveEditor() const;
	/** Sets whether or not to link sequence filters with the curve editor tree */
	UE_DEPRECATED(5.8, "Linked filtering replaces this feature")
	UE_API void SetLinkFiltersWithCurveEditor(bool bInLinkFiltersWithCurveEditor);

	/** Return true whether changing the sequencer tree selection will also select the relevant nodes in the curve editor tree if possible */
	bool ShouldSyncCurveEditorSelection() const { return bSynchronizeCurveEditorSelection; }
	/** Assign whether changing the sequencer tree selection will also select the relevant nodes in the curve editor tree if possible */
	UE_API void SyncCurveEditorSelection(bool bInSynchronizeCurveEditorSelection);
	
	/** Return true whether changing the curve editor tree selection will also select the relevant nodes in the outliner tree.*/
	bool ShouldSyncOutlinerSelectionToCurveEditor() const { return bSynchronizeOutlinerSelectionToCurveEditor; }
	/** Assign whether changing the curve editor tree selection will also select the relevant nodes in the outliner tree. */
	UE_API void SyncOutlinerSelectionToCurveEditor(bool bInValue);

	/** Return true if we should filter the curve editor tree to only nodes that are relevant to the current sequencer selection */
	bool ShouldIsolateToCurveEditorSelection() const { return bIsolateCurveEditorToSelection; }
	/** Assign whether we should filter the curve editor tree to only nodes that are relevant to the current sequencer selection */
	UE_API void IsolateCurveEditorToSelection(bool bInIsolateCurveEditorToSelection);

	/** Return true if we should filter the curve editor tree to only nodes that are relevant to the current sequencer selection */
	bool GetCurveEditorVisible() const { return bCurveEditorVisible; }
	/** Assign whether we should filter the curve editor tree to only nodes that are relevant to the current sequencer selection */
	UE_API void SetCurveEditorVisible(bool bCurveEditorVisible);

	/** @return The config affecting the zooming behavior of Sequencer's Curve Editor. */
	const FCurveEditorZoomScaleConfig& GetCurveEditorZoomScaling() { return CurveEditorZoomScaling; }
	/** Sets the config affecting the zooming behaviour of Sequencer's Curve Editor. */
	UE_API void SetCurveEditorZoomScaling(const FCurveEditorZoomScaleConfig& Value);

	/** Gets the loop mode. */
	UE_API ESequencerLoopMode GetLoopMode() const;
	/** Sets the loop mode. */
	UE_API void SetLoopMode(ESequencerLoopMode InLoopMode);

	/** @return true if the cursor reset when navigating in and out of subsequences, false otherwise */
	UE_API bool ShouldResetPlayheadWhenNavigating() const;
	/** Set whether or not the cursor should be reset when navigating in and out of subsequences */
	UE_API void SetResetPlayheadWhenNavigating(bool bInResetPlayheadWhenNavigating);

	/** @return the limit that the playhead should be restricted to when scrubbing */
	UE_API ESequencerPlayheadScrubbingLimit GetPlayheadScrubbingLimit() const;
	/** Set the limit that the playhead should be restricted to when scrubbing */
	UE_API void SetPlayheadScrubbingLimit(ESequencerPlayheadScrubbingLimit InPlayheadScrubbingLimit);
	/** Get the actual frame number range that the playhead should be restricted to when scrubbing */
	UE_API TRange<FFrameNumber> GetPlayheadScrubbingRange(TWeakPtr<ISequencer> InSequencer) const;

	/** @return true if the playback range should be synced to the section bounds, false otherwise */
	UE_API bool ShouldKeepPlayRangeInSectionBounds() const;
	/** Set whether or not the playback range should be synced to the section bounds */
	UE_API void SetKeepPlayRangeInSectionBounds(bool bInKeepPlayRangeInSectionBounds);

	/** Get the number of digits we should zero-pad to when showing frame numbers in sequencer */
	UE_API uint8 GetZeroPadFrames() const;
	/** Set the number of digits we should zero-pad to when showing frame numbers in sequencer */
	UE_API void SetZeroPadFrames(uint8 InZeroPadFrames);

	/** Get the number of frames to increment when jumping forwards/backwards */
	UE_API FFrameNumber GetJumpFrameIncrement() const;
	/** Set the number of frames to increment when jumping forwards/backwards */
	UE_API void SetJumpFrameIncrement(FFrameNumber InJumpFrameIncrement);

	/** Get the time-warp display mode */
	UE_API ESequencerTimeWarpDisplay GetTimeWarpDisplayMode() const;
	/** Set the time-warp display mode */
	UE_API void SetTimeWarpDisplayMode(ESequencerTimeWarpDisplay InTimeWarpDisplay);

	/** @return true if showing layer bars */
	UE_API bool GetShowLayerBars() const;
	/** Set whether to show layer bars */ 
	UE_API void SetShowLayerBars(bool bInShowLayerBars);

	/** @return true if showing key bars */
	UE_API bool GetShowKeyBars() const;
	/** Set whether to show key bars */ 
	UE_API void SetShowKeyBars(bool bInShowKeyBars);

	/** @return true if key areas are infinite */
	UE_API bool GetInfiniteKeyAreas() const;
	/** Set whether to show channel colors */
	UE_API void SetInfiniteKeyAreas(bool bInInfiniteKeyAreas);

	/** @return true if showing channel colors for the key bars */
	UE_API bool GetShowChannelColors() const;
	/** Set whether to show channel colors for the key bars */
	UE_API void SetShowChannelColors(bool bInShowChannelColors);

	/** @return true if showing the info button in the playback controls */
	UE_API bool GetShowInfoButton() const;
	/** Set whether to show the info button in the playback controls */
	UE_API void SetShowInfoButton(bool bInShowInfoButton);

	/** @return true if showing tick lines */
	UE_API bool GetShowTickLines() const;
	/** Set whether to show status bar */
	UE_API void SetShowTickLines(bool bInDrawTickLines);

	/** @return true if showing sequencer toolbar */
	UE_API bool GetShowSequencerToolbar() const;
	/** Set whether to show sequencer toolbar bar */
	UE_API void SetShowSequencerToolbar(bool bInShowSequencerToolbar);

	/** @return true if showing marked frames */
	UE_API bool GetShowMarkedFrames() const;
	/** Set whether to show marked frames */
	UE_API void SetShowMarkedFrames(bool bShowMarkedFrames);

	/** @return true if showing scaling anchors */
	UE_API bool GetShowScalingAnchors() const;
	/** Set whether to show scaling anchors */
	UE_API void SetShowScalingAnchors(bool bShowScalingAnchors);

	/** @return Whether the given channel has curve extents */
	UE_API bool HasKeyAreaCurveExtents(const FString& ChannelName) const;
	/** @ Remove curve extents for the given channel */
	UE_API void RemoveKeyAreaCurveExtents(const FString& ChannelName);
	/** @return Get the key area curve extents for the given channel */
	UE_API void GetKeyAreaCurveExtents(const FString& ChannelName, double& InMin, double& InMax) const;
	/** Set the key area curve extents for the given channel */
	UE_API void SetKeyAreaCurveExtents(const FString& ChannelName, double InMin, double InMax);

	/** @return The key area height when showing curves */
	UE_API float GetKeyAreaHeightWithCurves() const;
	/** Set the key area height when showing curves */
	UE_API void SetKeyAreaHeightWithCurves(float InKeyAreaHeightWithCurves);

	/** @return The tolerance to use when reducing keys */
	UE_API float GetReduceKeysTolerance() const;
	/** Set the tolerance to use when reducing keys */
	UE_API void SetReduceKeysTolerance(float InReduceKeysTolerance);

	/** @return true if deleting keys that fall beyond the section range when trimming */
	UE_API bool GetDeleteKeysWhenTrimming() const;
	/** Set whether to delete keys that fall beyond the section range when trimming */
	UE_API void SetDeleteKeysWhenTrimming(bool bInDeleteKeysWhenTrimming);

	/** @return true if disable sections when baking */
	UE_API bool GetDisableSectionsAfterBaking() const;
	/** Set whether to disable sections when baking, as opposed to deleting */
	UE_API void SetDisableSectionsAfterBaking(bool bInDisableSectionsAfterBaking);

	/** Set the playback range start color */
	UE_API FLinearColor GetPlaybackRangeStartColor() const;
	/** Set the playback range start color */
	UE_API void SetPlaybackRangeStartColor(const FLinearColor& InColor);

	/** Set the playback range end color */
	UE_API FLinearColor GetPlaybackRangeEndColor() const;
	/** Set the playback range end color */
	UE_API void SetPlaybackRangeEndColor(const FLinearColor& InColor);

	/** @return the default marked frame color */
	UE_API FLinearColor GetMarkedFrameColor() const;
	/** Set the default marked frame color */
	UE_API void SetMarkedFrameColor(const FLinearColor& InColor);

	/** @return the section color tints */
	UE_API TArray<FColor> GetSectionColorTints() const;
	/** Set the section color tints */
	UE_API void SetSectionColorTints(const TArray<FColor>& InSectionColorTints);

	/** @return The alpha opacity to use for section background colors */
	UE_API float GetSectionColorAlpha() const;
	/** Set the alpha opacity to use for section background colors */
	UE_API void SetSectionColorAlpha(const float InAlpha);

	/** @return True if horizontal track separator lines should be drawn in the track area */
	UE_API bool ShouldDrawHorizontalTrackSeparatorLines() const;
	/** Set if horizontal track separator lines should be drawn in the track area */
	UE_API void SetDrawHorizontalTrackSeparatorLines(const bool bInDraw);

	/** @return Whether to playback in clean mode (game view, hide viewport UI) */
	UE_API bool GetCleanPlaybackMode() const;
	/** Toggle whether to playback in clean mode */
	UE_API void SetCleanPlaybackMode(bool bInCleanPlaybackMode);

	/** @return Whether to activate realtime viewports when in sequencer */
	UE_API bool ShouldActivateRealtimeViewports() const;
	/** Toggle whether to allow possession of PIE viewports */
	UE_API void SetActivateRealtimeViewports(bool bInActivateRealtimeViewports);

	/** Gets whether or not track defaults will be automatically set when modifying tracks. */
	UE_API bool GetAutoSetTrackDefaults() const;
	/** Sets whether or not track defaults will be automatically set when modifying tracks. */
	UE_API void SetAutoSetTrackDefaults(bool bInAutoSetTrackDefaults);

	/** @return Whether to show debug vis */
	UE_API bool ShouldShowDebugVisualization() const;
	/** Toggle whether to show debug vis */
	UE_API void SetShowDebugVisualization(bool bInShowDebugVisualization);

	/** @return Whether to evaluate sub sequences in isolation */
	UE_API bool ShouldEvaluateSubSequencesInIsolation() const;
	/** Set whether to evaluate sub sequences in isolation */
	UE_API void SetEvaluateSubSequencesInIsolation(bool bInEvaluateSubSequencesInIsolation);
	/** Gets the multicast delegate which is run whenever evaluate sub sequences in isolation is changed. */
	FOnEvaluateSubSequencesInIsolationChanged& GetOnEvaluateSubSequencesInIsolationChanged() { return OnEvaluateSubSequencesInIsolationChangedEvent; }

	/** @return Whether to rerun construction scripts on bound actors every frame */
	UE_API bool ShouldRerunConstructionScripts() const;
	/** Set whether to rerun construction scripts on bound actors every frame */
	UE_API void SetRerunConstructionScripts(bool bInRerunConstructionScripts);

	/** Snaps a time value in seconds to the currently selected interval. */
	UE_API float SnapTimeToInterval(float InTimeValue) const;

	/** Check whether to show pre and post roll in sequencer */
	UE_API bool ShouldShowPrePostRoll() const;
	/** Toggle whether to show pre and post roll in sequencer */
	UE_API void SetShouldShowPrePostRoll(bool bInVisualizePreAndPostRoll);

	/** Check whether whether to recompile the director blueprint when the sequence is evaluated (if one exists) */
	UE_API bool ShouldCompileDirectorOnEvaluate() const;
	/** Assign whether whether to recompile the director blueprint when the sequence is evaluated (if one exists) */
	UE_API void SetCompileDirectorOnEvaluate(bool bInCompileDirectorOnEvaluate);

	uint32 GetTrajectoryPathCap() const { return TrajectoryPathCap; }

	UE_API FOnLoopStateChanged& GetOnLoopStateChanged();

	UE_API FOnTimeDisplayFormatChanged& GetOnTimeDisplayFormatChanged();

	/** What format should we display the UI controls in when representing time in a sequence? */
	EFrameNumberDisplayFormats GetTimeDisplayFormat() const { return FrameNumberDisplayFormat; }
	/** Sets the time display format to the specified type. */
	UE_API void SetTimeDisplayFormat(EFrameNumberDisplayFormats InFormat);

	/** What movie renderer to use */
	FString GetMovieRendererName() const { return MovieRendererName; }
	/** Sets the movie renderer to use */
	UE_API void SetMovieRendererName(const FString& InMovieRendererName);

	/** Gets whether or not to expand the outliner tree view when a child element is selected (from outside of the tree view). */
	bool GetAutoExpandNodesOnSelection() const { return bAutoExpandNodesOnSelection; }
	/** Sets whether or not to expand the outliner tree view when a child element is selected (from outside of the tree view). */
	UE_API void SetAutoExpandNodesOnSelection(bool bInAutoExpandNodesOnSelection);

	/** Gets whether when selecting items in Sequencer's Outliner will expand Curve Editor's tree view to show the item. */
	bool GetAutoExpandCurveEditorOnSelection() const { return bAutoExpandOnCurveEditorSelection; }
	/** Sets whether when selecting items in Sequencer's Outliner will expand Curve Editor's tree view to show the item. */
	UE_API void SetAutoExpandCurveEditorOnSelection(bool bValue);
	
	/**
	 * Gets whether unlocking a camera cut track should return the viewport to its original location, or keep it where
	 * the camera cut was.
	 */
	bool GetRestoreOriginalViewportOnCameraCutUnlock() const { return bRestoreOriginalViewportOnCameraCutUnlock; }
	/**
	 * Sets whether unlocking a camera cut track should return the viewport to its original location, or keep it where
	 * the camera cut was.
	 */
	UE_API void SetRestoreOriginalViewportOnCameraCutUnlock(bool bInRestoreOriginalViewportOnCameraCutUnlock);

	/** Gets the tree view width in pixels */
	float GetTreeViewWidth() const { return TreeViewWidth; }
	/** Sets the tree view width in pixels */
	UE_API void SetTreeViewWidth(float InTreeViewWidth);

	/** Gets the saved view density */
	UE_API UE::Sequencer::EViewDensity GetViewDensity() const;
	/** Sets the saved view density */
	UE_API void SetViewDensity(FName InViewDensity);

	/** Gets the asset browser width */
	float GetAssetBrowserWidth() const { return AssetBrowserWidth; }
	/** Sets the asset browser width */
	UE_API void SetAssetBrowserWidth(float InAssetBrowserWidth);

	/** Gets the asset browser height */
	float GetAssetBrowserHeight() const { return AssetBrowserHeight; }
	/** Sets the asset browser width */
	UE_API void SetAssetBrowserHeight(float InAssetBrowserHeight);

	/** Gets whether the given track filter is enabled */
	UE_API bool IsTrackFilterEnabled(const FString& TrackFilter) const;
	/** Sets whether the track filter should be enabled/disabled */
	UE_API void SetTrackFilterEnabled(const FString& TrackFilter, bool bEnabled);

	/** Get outliner column visibility in display order */
	TArray<FColumnVisibilitySetting> GetOutlinerColumnSettings() const { return ColumnVisibilitySettings; }
	/** Sets the visibility of outliner columns in display order */
	UE_API void SetOutlinerColumnVisibility(const TArray<FColumnVisibilitySetting>& InColumnVisibilitySettings);

	/** Gets the last saved sidebar state */
	UE_API FSidebarState& GetSidebarState();
	/** Sets the sidebar state to be restored on Sequencer initialize */
	UE_API void SetSidebarState(const FSidebarState& InSidebarState);

	UE_API FSequencerFilterBarConfig& FindOrAddTrackFilterBar(const FName InIdentifier, const bool bInSaveConfig);
	UE_API FSequencerFilterBarConfig* FindTrackFilterBar(const FName InIdentifier);
	UE_API bool RemoveTrackFilterBar(const FName InIdentifier);
	
	UE_API FSequencerFilterAreaConfig& FindOrAddTrackFilterArea(const FName InIdentifier, bool bInSaveConfig = true);
	UE_API FSequencerFilterAreaConfig* FindTrackFilterArea(const FName InIdentifier);
	UE_API bool RemoveTrackFilterArea(const FName InIdentifier);
	

	UE_API bool GetIncludePinnedInFilter() const;
	UE_API void SetIncludePinnedInFilter(const bool bInIncludePinned);

	UE_API bool GetAutoExpandNodesOnFilterPass() const;
	UE_API void SetAutoExpandNodesOnFilterPass(const bool bInIncludePinned);

	UE_API bool GetUseFilterSubmenusForCategories() const;
	UE_API void SetUseFilterSubmenusForCategories(const bool bInUseFilterSubmenusForCategories);

	UE_DEPRECATED(5.8, "Use the version that takes in an identifier instead.")
	UE_API bool IsFilterBarVisible() const;
	UE_DEPRECATED(5.8, "Use the version that takes in an identifier instead.")
	UE_API void SetFilterBarVisible(const bool bInVisible);

	UE_DEPRECATED(5.8, "Sequencer now only uses horizontal layout")
	UE_API EFilterBarLayout GetFilterBarLayout() const;
	UE_DEPRECATED(5.8, "Sequencer now only uses horizontal layout")
	UE_API void SetFilterBarLayout(const EFilterBarLayout InLayout);

	UE_API float GetLastFilterBarSizeCoefficient() const;
	UE_API void SetLastFilterBarSizeCoefficient(const float bInSizeCoefficient, const bool bInSaveConfig);
	
	/** Gets the settings that determine how the asset thumbnail is captured when the sequence is saved. */
	const FSequencerThumbnailCaptureSettings& GetThumbnailCaptureSettings() const { return ThumbnailCaptureSettings; }
	/** Sets how the asset thumbnail is captured when the sequence is saved.  */
	UE_API void SetThumbnailCaptureSettings(const FSequencerThumbnailCaptureSettings& InNewValue);

	/** @return True if the Sequence Navigator is visible */
	bool IsNavigationToolVisible() const { return bNavigationToolVisible; }
	/** Sets the visibility of the Sequence Navigator */
	UE_API void SetNavigationToolVisible(const bool bInVisible);

	/** @return True if the selection details are popped out from the Sequencer tab to their own tab */
	bool IsSelectionDetailsPoppedOut() const { return bSelectionDetailsPoppedOut; }
	/** Sets the popped out state of the selection details in the Sequencer tab */
	UE_API void SetSelectionDetailsPoppedOut(const bool bInPoppedOut);

	/** @return True if the transport control panel is visible in the Sequencer tab */
	UE_API bool ShouldShowTransportControls() const;
	/** Sets the visibility of the transport control panel in the Sequencer tab */
	UE_API void SetShowTransportControls(const bool bInShow);

	/** @return True if the transport control panel is popped out from the Sequencer tab to its own tab */
	UE_API bool AreTransportControlsPoppedOut() const;
	/** Sets the popped out state of the transport controls panel in the Sequencer tab */
	UE_API void SetTransportControlsPoppedOut(const bool bInPoppedOut);

	/** @return True if the persisted preference for restoring transport controls in a popped-out state when shown. */
	bool ShouldRestoreTransportControlsPoppedOut() const;
	/** Sets the persisted preference for restoring transport controls in a popped-out state when shown. */
	void SetRestoreTransportControlsPoppedOut(const bool bInRestorePoppedOut);

	/** @return True if the last the time Sequencer was open, it was in the simplified view and not the full view */
	bool IsSimpleView() const { return bSimpleView; }
	/** Switches between full Sequencer and a simplified Sequencer view */
	void SetIsSimpleView(const bool bInSimpleView);

protected:

	/** The auto change mode (auto-key, auto-track or none). */
	UPROPERTY( config, EditAnywhere, Category=Keyframing )
	EAutoChangeMode AutoChangeMode;

	/** Allow edits mode. */
	UPROPERTY( config, EditAnywhere, Category=Keyframing )
	EAllowEditsMode AllowEditsMode;

	/**Key group mode. */
	UPROPERTY(config, EditAnywhere, Category = Keyframing)
	EKeyGroupMode KeyGroupMode;

	/** The interpolation type for the initial keyframe */
	UPROPERTY( config, EditAnywhere, Category=Keyframing )
	EMovieSceneKeyInterpolation KeyInterpolation;

	/** Whether or not track defaults will be automatically set when modifying tracks. */
	UPROPERTY( config, EditAnywhere, Category=Keyframing, meta = (ToolTip = "When setting keys on properties and transforms automatically update the track default values used when there are no keys."))
	bool bAutoSetTrackDefaults;

	/** The default location of a spawnable when it is first dragged into the viewport from the content browser. */
	UPROPERTY( config, EditAnywhere, Category=General )
	TEnumAsByte<ESequencerSpawnPosition> SpawnPosition;

	/** Enable or disable creating of spawnable cameras whenever cameras are created. */
	UPROPERTY( config, EditAnywhere, Category=General )
	bool bCreateSpawnableCameras;

	/** Show the in/out range in the timeline with respect to the start/end range. */
	UPROPERTY( config, EditAnywhere, Category=Timeline )
	bool bShowRangeSlider;

	/** Show the bottom time slider, which is only visible if bShowRangeSlider is false. */
	UPROPERTY( config, EditAnywhere, Category=Timeline )
	bool bShowBottomTimeSlider;

	/** Enable or disable snapping in the timeline. */
	UPROPERTY( config, EditAnywhere, Category=Snapping )
	bool bIsSnapEnabled;

	/** Enable or disable snapping keys to other elements. */
	UPROPERTY( config, EditAnywhere, Category=Snapping )
	bool bSnapKeyTimesToElements;
	
	/** Enable or disable snapping sections to other elements. */
	UPROPERTY( config, EditAnywhere, Category=Snapping )
	bool bSnapSectionTimesToElements;

	/** Enable or disable snapping the playhead to keys while scrubbing. */
	UPROPERTY( config, EditAnywhere, Category=Snapping, meta = (DisplayName = "Snap Playhead to Keys"))
	bool bSnapPlayTimeToKeys;

	/** Enable or disable snapping the playhead to section bounds while scrubbing. */
	UPROPERTY( config, EditAnywhere, Category=Snapping, meta = (DisplayName = "Snap Playhead to Sections"))
	bool bSnapPlayTimeToSections;

	/** Enable or disable snapping the playhead to markers while scrubbing. */
	UPROPERTY( config, EditAnywhere, Category=Snapping, meta = (DisplayName = "Snap Playhead to Markers"))
	bool bSnapPlayTimeToMarkers;

	/** Enable or disable snapping the playhead to the pressed key. */
	UPROPERTY( config, EditAnywhere, Category=Snapping, meta = (DisplayName = "Snap Playhead to Pressed Key"))
	bool bSnapPlayTimeToPressedKey;

	/** Enable or disable snapping the playhead to the dragged key. */
	UPROPERTY( config, EditAnywhere, Category=Snapping, meta = (DisplayName = "Snap Playhead to Dragged Key"))
	bool bSnapPlayTimeToDraggedKey;

	/** The curve value interval to snap to. */
	float CurveValueSnapInterval;

	/** grid line spacing state */
	TOptional<float> GridSpacing;

	/** Enable or disable snapping the curve value to the curve value interval. */
	UPROPERTY( config, EditAnywhere, Category=Snapping )
	bool bSnapCurveValueToInterval;

	/** Enable or disable snapping keys/sections/playhead to the interval which forces whole frames */
	UPROPERTY(config, EditAnywhere, Category = Snapping)
	bool bForceWholeFrames;

	/** Only show selected nodes in the tree view. */
	UPROPERTY( config, EditAnywhere, Category=General )
	bool bShowSelectedNodesOnly;

	/** Defines whether to jump back to the start of the sequence when a recording is started */
	UPROPERTY(config, EditAnywhere, Category=General)
	bool bRewindOnRecord;

	/** Defines whether left mouse drag does marquee select instead of camera orbit */
	UPROPERTY(config, EditAnywhere, Category = General)
	bool bLeftMouseDragDoesMarquee;

	/** Whether to zoom in on the current position or the current time in the timeline. */
	UPROPERTY( config, EditAnywhere, Category=Timeline )
	TEnumAsByte<ESequencerZoomPosition> ZoomPosition;

	/** Enable or disable auto scroll in the timeline when playing. */
	UPROPERTY( config, EditAnywhere, Category=Timeline )
	bool bAutoScrollEnabled;

	/** When enabled, playing will stop when you jump to  start or end of range*/
	UPROPERTY(config, EditAnywhere, Category = Timeline)
	bool bStopPlayingWhenJumpingToStartOrEnd;

	/** When enabled, scrubbing time hotkey will move time from initial cursor position  */
	UPROPERTY(config, EditAnywhere, Category = Timeline)
	bool bScrubTimeStartFromCursor;

	/** Enable or disable linking the curve editor time range to the sequencer timeline's time range. */
	UPROPERTY( config, EditAnywhere, Category=CurveEditor )
	bool bLinkCurveEditorTimeRange;

	/** Enable or disable linking sequence filters with the curve editor tree */
	UE_DEPRECATED(5.8, "Linked filtering replaces this feature")
	UPROPERTY(config)
	bool bLinkFiltersWithCurveEditor_DEPRECATED;

	/** When enabled, changing the sequencer tree selection will also select the relevant nodes in the curve editor tree if possible. */
	UPROPERTY(config, EditAnywhere, Category=CurveEditor, meta = (DisplayName = "Synchronize Curve Editor Selection to Outliner"))
	bool bSynchronizeCurveEditorSelection;

	/** When enabled, changing the curve editor tree selection will also select the relevant nodes in the outliner tree. */
	UPROPERTY(config, EditAnywhere, Category=CurveEditor)
	bool bSynchronizeOutlinerSelectionToCurveEditor; 
	
	/** When enabled, changing the sequencer tree selection will isolate (auto-filter) the selected nodes in the curve editor. */
	UPROPERTY(config, EditAnywhere, Category=CurveEditor)
	bool bIsolateCurveEditorToSelection;
	
	/** When enabled, selecting items in Sequencer's Outliner will expand Curve Editor's tree view to show the item. */
	UPROPERTY(config, EditAnywhere, Category=CurveEditor)
	bool bAutoExpandOnCurveEditorSelection;
	
	/** Whether the curve editor is visible */
	UPROPERTY(config, EditAnywhere, Category=CurveEditor)
	bool bCurveEditorVisible;

	/** Affects the zooming behavior of in Sequencer's Curve Editor. */
	UPROPERTY(config, EditAnywhere, Category=CurveEditor)
	FCurveEditorZoomScaleConfig CurveEditorZoomScaling;

	/** The loop mode of the playback in timeline. */
	UPROPERTY( config )
	TEnumAsByte<ESequencerLoopMode> LoopMode;

	/** Enable or disable resetting the playhead when navigating in and out of subsequences. */
	UPROPERTY(config, EditAnywhere, Category = Timeline, meta = (DisplayName = "Reset Playhead When Navigating"))
	bool bResetPlayheadWhenNavigating;

	/** The limit that the playhead should be restricted to when scrubbing */
	UPROPERTY(config, EditAnywhere, Category = Timeline, meta = (DisplayName = "Playhead Scrubbing Limit"))
	TEnumAsByte<ESequencerPlayheadScrubbingLimit> PlayheadScrubbingLimit;

	/** Enable or disable keeping the playback range constrained to the section bounds. */
	UPROPERTY( config, EditAnywhere, Category=Timeline )
	bool bKeepPlayRangeInSectionBounds;

	/** The number of zeros to pad the frame numbers by. */
	UPROPERTY( config, EditAnywhere, Category=Timeline )
	uint8 ZeroPadFrames;

	/** The number of frames to jump by when jumping forward or backwards. */
	UPROPERTY( config, EditAnywhere, Category=Timeline )
	FFrameNumber JumpFrameIncrement;

	/** Controls how time-warped time is displayed on the timeline. */
	UPROPERTY( config, EditAnywhere, Category=Timeline )
	ESequencerTimeWarpDisplay TimeWarpDisplay;

	/** Enable or disable the layer bars to edit keyframes in bulk. */
	UPROPERTY( config, EditAnywhere, Category=Timeline )
	bool bShowLayerBars;

	/** Enable or disable key bar connections. */
	UPROPERTY( config, EditAnywhere, Category=Timeline )
	bool bShowKeyBars;

	/** Enable or disable setting key area sections as infinite by default. */
	UPROPERTY( config, EditAnywhere, Category=Timeline )
	bool bInfiniteKeyAreas;

	/** Enable or disable displaying channel bar colors for the key bars. */
	UPROPERTY(config, EditAnywhere, Category = Timeline)
	bool bShowChannelColors;

	/** Enable or disable displaying the info button in the playback controls. */
	UPROPERTY(config, EditAnywhere, Category = Timeline)
	bool bShowInfoButton;

	/** Enable or disable displaying the tick lines. */
	UPROPERTY(config, EditAnywhere, Category = Timeline)
	bool bShowTickLines;

	/** Enable or disable displaying the sequencer toolbar. */
	UPROPERTY(config, EditAnywhere, Category = Timeline)
	bool bShowSequencerToolbar;

	/** Enable or disable showing marked frames */
	UPROPERTY(config, EditAnywhere, Category = Timeline)
	bool bShowMarkedFrames;

	/** Enable or disable showing scaling anchors */
	UPROPERTY(config, EditAnywhere, Category = Timeline)
	bool bShowScalingAnchors;

	/** The key area curve extents, stored per channel name */
	UPROPERTY(config, EditAnywhere, Category = Timeline)
	FString KeyAreaCurveExtents;

	/** The key area height when showing curves */
	UPROPERTY(config, EditAnywhere, Category = Timeline)
	float KeyAreaHeightWithCurves;

	/** The tolerance to use when reducing keys */
	UPROPERTY(config, EditAnywhere, Category = Timeline)
	float ReduceKeysTolerance;

	/** Enable or disable deleting keys that fall beyond the section range when trimming. */
	UPROPERTY(config, EditAnywhere, Category = Timeline)
	bool bDeleteKeysWhenTrimming;

	/** Whether to disable sections after baking as opposed to deleting. */
	UPROPERTY(config, EditAnywhere, Category = Timeline)
	bool bDisableSectionsAfterBaking;
	
	/** PLayback range start color */
	UPROPERTY(config, EditAnywhere, Category = Timeline)
	FLinearColor PlaybackRangeStartColor;

	/** PLayback range end color */
	UPROPERTY(config, EditAnywhere, Category = Timeline)
	FLinearColor PlaybackRangeEndColor;

	/** Default marked frame color */
	UPROPERTY(config, EditAnywhere, Category = Timeline)
	FLinearColor MarkedFrameColor;

	/** Section color tints */
	UPROPERTY(config, EditAnywhere, Category = General)
	TArray<FColor> SectionColorTints;

	/** Alpha transparency to use for section background colors */
	UPROPERTY(config, EditAnywhere, Category = General, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.1", UIMax = "1.0"))
	float SectionColorAlpha;

	/** Draw horizontal track separator lines in the track area */
	UPROPERTY(config, EditAnywhere, Category = General)
	bool bDrawHorizontalTrackSeparatorLines;

	/** When enabled, sequencer will playback in clean mode (game view, hide viewport UI) */
	UPROPERTY(config, EditAnywhere, Category = General)
	bool bCleanPlaybackMode;

	/** When enabled, sequencer will activate 'Realtime' in viewports */
	UPROPERTY(config, EditAnywhere, Category = General)
	bool bActivateRealtimeViewports;

	/** When enabled, entering a sub sequence will evaluate that sub sequence in isolation, rather than from the root sequence */
	UPROPERTY(config, EditAnywhere, Category=Playback)
	bool bEvaluateSubSequencesInIsolation;

	/** When enabled, construction scripts will be rerun on bound actors for every frame */
	UPROPERTY(config, EditAnywhere, Category=Playback)
	bool bRerunConstructionScripts;

	/** Enable or disable showing of debug visualization. */
	UPROPERTY( config, EditAnywhere, Category=General )
	bool bShowDebugVisualization;

	/** Enable or disable showing of pre and post roll visualization. */
	UPROPERTY( config, EditAnywhere, Category=General )
	bool bVisualizePreAndPostRoll;

	/** Whether to recompile the director blueprint when the sequence is evaluated (if one exists) */
	UPROPERTY(config, EditAnywhere, Category=General)
	bool bCompileDirectorOnEvaluate;

	/** Specifies the maximum number of keys to draw when rendering trajectories in viewports */
	UPROPERTY(config, EditAnywhere, Category = General)
	uint32 TrajectoryPathCap;

	/** What format do we display time in to the user? */
	UPROPERTY(config, EditAnywhere, Category=General)
	EFrameNumberDisplayFormats FrameNumberDisplayFormat;

	/** Which movie renderer to use */
	UPROPERTY(config, EditAnywhere, Category=General)
	FString MovieRendererName;

	/** Whether to expand the sequencer tree view when a child element is selected (from outside of the tree view). */
	UPROPERTY(config, EditAnywhere, Category = General)
	bool bAutoExpandNodesOnSelection;

	/**
	 * Whether unlocking a camera cut track should return the viewport to its original location, or keep it where the
	 * camera cut was.
	 * WARNING: Disabling this will make previewing camera cut blends useless, since it will blend to the same position.
	 */
	UPROPERTY(config, EditAnywhere, Category=General)
	bool bRestoreOriginalViewportOnCameraCutUnlock;

	/** The tree view width in pixels */
	UPROPERTY(config, EditAnywhere, Category = General, meta = (ClampMin = "150", UIMin = "150"))
	float TreeViewWidth;

	UPROPERTY(config, EditAnywhere, Category = General)
	FName ViewDensity;

	/** The width for the asset browsers in Sequencer */
	UPROPERTY(config, EditAnywhere, Category = General)
	float AssetBrowserWidth;

	/** The height for the asset browsers in Sequencer */
	UPROPERTY(config, EditAnywhere, Category = General)
	float AssetBrowserHeight;

	/** The track filters that are enabled */
	UPROPERTY(config, EditAnywhere, Category = General)
	TArray<FString> TrackFilters;

	/** List of all columns and their visibility, in the order to be displayed in the outliner view */
	UPROPERTY(config, EditAnywhere, Category = General)
	TArray<FColumnVisibilitySetting> ColumnVisibilitySettings;

	/** The state of a sidebar to be restored when each Sequencer type is initialized */
	UPROPERTY(config)
	TMap<FName, FSidebarState> SidebarState;

	/** 
	 * Saved settings for each unique filter bar instance mapped by instance identifier.
	 * Filter bar instances live in a filter bar areas. 
	 * @see TrackFilterAreas For settings independent whether the filter bar is in linked or instanced mode, such as visibility.
	 */
	UPROPERTY(config, EditAnywhere, Category = Filtering)
	TMap<FName, FSequencerFilterBarConfig> TrackFilterBars;
	
	/**
	 * Saved settings for each filter UI area. 
	 * @see TrackFilterBars For settings that are specific to the linked or instanced mode, such as the filter active state.
	 */
	UPROPERTY(config, EditAnywhere, Category = Filtering)
	TMap<FName, FSequencerFilterAreaConfig> TrackFilterAreas;

	/** Apply filtering to pinned tracks that would otherwise ignore filters */
	UPROPERTY(config, EditAnywhere, Category = Filtering)
	bool bIncludePinnedInFilter;

	/** Automatically expand tracks that pass filters */
	UPROPERTY(config, EditAnywhere, Category = Filtering)
	bool bAutoExpandNodesOnFilterPass;

	/** Display the filter menu categories as submenus instead of sections */
	UPROPERTY(config, EditAnywhere, Category = Filtering)
	bool bUseFilterSubmenusForCategories;

	/** Last saved visibility of the filter bar to restore after closed */
	UPROPERTY(config, EditAnywhere, Category = Filtering)
	bool bFilterBarVisible;

	/** Last saved layout orientation of the filter bar to restore after closed */
	UE_DEPRECATED(5.8, "Sequencer now only uses horizontal layout")
	UPROPERTY(config)
	EFilterBarLayout LastFilterBarLayout_DEPRECATED;

	/** Last saved size of the filter bar to restore after closed */
	UPROPERTY(config)
	float LastFilterBarSizeCoefficient;

	/** Controls how the thumbnail is captured when the sequence is saved. */
	UPROPERTY(config, EditAnywhere, Category = General, meta = (EditCondition="ShouldShowThumbnailCaptureSettings()", EditConditionHides))
	FSequencerThumbnailCaptureSettings ThumbnailCaptureSettings;

	/** Last saved visibility of the Sequence Navigator to restore after closed */
	UPROPERTY(config, EditAnywhere, Category = General, meta = (DisplayName = "Sequence Navigator Visible"))
	bool bNavigationToolVisible;

	/** Last saved popped out state of the selection details */
	UPROPERTY(config, EditAnywhere, Category = General)
	bool bSelectionDetailsPoppedOut;

	/** Last saved visibility of the transport controls */
	UPROPERTY(config, EditAnywhere, Category = General)
	bool bShowTransportControls;

	/** Last saved state of the transport controls: popped out to its own tab (that can be docked/undocked,
	 * thus the term of "Popout" is used) instead of "Dock" or popped into inline content */
	UPROPERTY(config, EditAnywhere, Category = General)
	bool bTransportControlsPoppedOut;

	/** Hidden persisted preference for whether transport controls should reopen popped out when shown again */
	UPROPERTY(config)
	bool bRestoreTransportControlsPoppedOut = false;

	/** Last saved state of the Sequencer view: full or simple */
	UPROPERTY(config)
	bool bSimpleView;

	UFUNCTION()
	static UE_API bool ShouldShowThumbnailCaptureSettings();

	FOnEvaluateSubSequencesInIsolationChanged OnEvaluateSubSequencesInIsolationChangedEvent;
	FOnShowSelectedNodesOnlyChanged OnShowSelectedNodesOnlyChangedEvent;
	FOnAllowEditsModeChanged OnAllowEditsModeChangedEvent;
	FOnLoopStateChanged OnLoopStateChangedEvent;
	FOnTimeDisplayFormatChanged OnTimeDisplayFormatChangedEvent;
};

#undef UE_API
