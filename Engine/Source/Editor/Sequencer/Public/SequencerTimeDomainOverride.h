// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::Sequencer
{


enum class ETimeDomain : uint8
{
	Warped,
	Unwarped
};

/**
 * RAII struct that temporarily overrides time operations within Sequencer to operate in warped or unwarped space.
 * By default all SetLocalTime behavior is in warped space, since this comprises the majority of tracks.
 * This class is very similar to TGuardValue<ETimeDomain>, except that it is moveable.
 */
struct FTimeDomainOverride
{
public:

	FTimeDomainOverride(ETimeDomain* Ptr, ETimeDomain NewValue)
		: ValuePtr(Ptr)
	{
		if (Ptr)
		{
			OriginalValue = *Ptr;
			*Ptr = NewValue;
		}
	}

	FTimeDomainOverride(const FTimeDomainOverride& Other) = delete;
	FTimeDomainOverride& operator=(const FTimeDomainOverride& Other) = delete;

	FTimeDomainOverride(FTimeDomainOverride&& Other)
		: ValuePtr(Other.ValuePtr)
		, OriginalValue(Other.OriginalValue)
	{
		Other.ValuePtr = nullptr;
	}

	~FTimeDomainOverride()
	{
		if (ValuePtr)
		{
			*ValuePtr = OriginalValue;
		}
	}

private:

	ETimeDomain* ValuePtr;
	ETimeDomain OriginalValue;
};

} // namespace UE::Sequencer

