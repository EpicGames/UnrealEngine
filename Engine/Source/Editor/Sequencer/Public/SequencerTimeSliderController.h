// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "ITimeSlider.h"
#include "ISequencerModule.h"
#include "TimeSliderArgs.h"

class FSlateWindowElementList;
struct FContextMenuSuppressor;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;
class FSlateFontMeasure;
class FSequencer;
class IPropertyTypeCustomization;
class USequencerSettings;

/**
 * A time slider controller for sequencer
 * Draws and manages time data for a Sequencer
 */
class FSequencerTimeSliderController : public ITimeSliderController, public TSharedFromThis<FSequencerTimeSliderController>
{
public:
	FSequencerTimeSliderController( const FTimeSliderArgs& InArgs, TWeakPtr<FSequencer> InWeakSequencer );
	~FSequencerTimeSliderController();

	/**
	* Determines the optimal spacing between tick marks in the slider for a given pixel density
	* Increments until a minimum amount of slate units specified by MinTick is reached
	*
	* @param InPixelsPerInput	The density of pixels between each input
	* @param MinTick			The minimum slate units per tick allowed
	* @param MinTickSpacing	The minimum tick spacing in time units allowed
	* @return the optimal spacing in time units
	*/
	float DetermineOptimalSpacing(float InPixelsPerInput, uint32 MinTick, float MinTickSpacing) const;

	virtual const FTimeSliderArgs& GetTimeSliderArgs() const { return TimeSliderArgs; }

	/** ITimeSliderController Interface */
	virtual int32 OnPaintTimeSlider( bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	virtual FReply OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointEvent) override;
	virtual FReply OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseButtonDoubleClick(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseMove(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseWheel(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnTimeSliderMouseMove(SWidget& OwnerWidget, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FCursorReply OnCursorQuery( const SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;
	virtual double ComputeHeight() const override;
	/** End ITimeSliderController Interface */

	/** Get the owning sequencer instance */
	TSharedPtr<ISequencer> GetSequencer() const;

	/** Get the current play rate for this controller */
	virtual FFrameRate GetDisplayRate() const override { return TimeSliderArgs.DisplayRate.Get(); }

	/** Get the current tick resolution for this controller */
	virtual FFrameRate GetTickResolution() const override { return TimeSliderArgs.TickResolution.Get(); }

	/** Get the current view range for this controller */
	virtual FAnimatedRange GetViewRange() const override { return TimeSliderArgs.ViewRange.Get(); }

	/** Get the current clamp range for this controller in seconds. */
	virtual FAnimatedRange GetClampRange() const override { return TimeSliderArgs.ClampRange.Get(); }

	/** Get the current play range for this controller */
	virtual TRange<FFrameNumber> GetPlayRange() const override { return TimeSliderArgs.PlaybackRange.Get(TRange<FFrameNumber>()); }

	/** Get the time bounds for this controller. The time bounds should be a subset of the playback range. */
	virtual TRange<FFrameNumber> GetTimeBounds() const override { return TimeSliderArgs.TimeBounds.Get(TRange<FFrameNumber>()); }

	/** Get the selection range */
	virtual TRange<FFrameNumber> GetSelectionRange() const override { return TimeSliderArgs.SelectionRange.Get(TRange<FFrameNumber>()); }

	/** Get the current time for the Scrub handle which indicates what range is being evaluated. */
	virtual FFrameTime GetScrubPosition() const override { return TimeSliderArgs.ScrubPosition.Get(FFrameTime()); }

	/** Set the current time for the handle when playback is stopped*/
	virtual void SetStoppedPosition(FFrameTime InTime) override;

	/** Get the current time for the Scrub handle which indicates what range is being evaluated. */
	virtual void SetScrubPosition(FFrameTime InTime, bool bEvaluate) override { CommitScrubPosition(InTime, GetPlaybackStatus() == ETimeSliderPlaybackStatus::Scrubbing, bEvaluate); }

	/** Set the playback status for the controller*/
	virtual void SetPlaybackStatus(ETimeSliderPlaybackStatus InStatus) override;

	/** Get the playback status for the controller, by default it is ETimeSliderPlaybackStatus::Stopped */
	virtual ETimeSliderPlaybackStatus GetPlaybackStatus() const override;

	/**
	 * Clamp the given range to the clamp range 
	 *
	 * @param NewRangeMin		The new lower bound of the range
	 * @param NewRangeMax		The new upper bound of the range
	 */	
	virtual void ClampViewRange(double& NewRangeMin, double& NewRangeMax);

	/**
	 * Set a new range based on a min, max and an interpolation mode
	 * 
	 * @param NewRangeMin		The new lower bound of the range
	 * @param NewRangeMax		The new upper bound of the range
	 * @param Interpolation		How to set the new range (either immediately, or animated)
	 */
	virtual void SetViewRange( double NewRangeMin, double NewRangeMax, EViewRangeInterpolation Interpolation ) override;

	/**
	 * Set a new clamp range based on a min, max
	 * 
	 * @param NewRangeMin		The new lower bound of the clamp range
	 * @param NewRangeMax		The new upper bound of the clamp range
	 */
	virtual void SetClampRange( double NewRangeMin, double NewRangeMax ) override;

	/**
	 * Set a new playback range based on a min, max
	 * 
	 * @param RangeStart		The new lower bound of the playback range
	 * @param RangeDuration		The total number of frames that we play for
	 */
	virtual void SetPlayRange( FFrameNumber RangeStart, int32 RangeDuration ) override;

	/**
	 * Set a new selection range
	 * 
	 * @param NewRange		The new selection range
	 */
	virtual void SetSelectionRange(const TRange<FFrameNumber>& NewRange) override;

	/**
	 * Zoom the range by a given delta.
	 * 
	 * @param InDelta		The total amount to zoom by (+ve = zoom out, -ve = zoom in)
	 * @param ZoomBias		Bias to apply to lower/upper extents of the range. (0 = lower, 0.5 = equal, 1 = upper)
	 */
	bool ZoomByDelta( float InDelta, float ZoomBias = 0.5f );

	/**
	 * Pan the range by a given delta
	 * 
	 * @param InDelta		The total amount to pan by (+ve = pan forwards in time, -ve = pan backwards in time)
	 */
	void PanByDelta( float InDelta );

	/**
	 * Draws major tick lines in the section view                                                              
	 */
	virtual int32 OnPaintViewArea( const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, bool bEnabled, const FPaintViewAreaArgs& Args ) const override;

	/**
	 * Call this method when the user's interaction has changed the scrub position
	 *
	 * @param NewValue				Value resulting from the user's interaction
	 * @param bIsScrubbing			True if done via scrubbing, false if just releasing scrubbing
	 * @param bEvaluate				If true evaluate, if not just change time
	 */
	void CommitScrubPosition(FFrameTime NewValue, bool bIsScrubbing, bool bEvaluate);

public:

	struct FScrubberMetrics
	{
		/** The extents of the current frame that the scrubber is on, in pixels */
		TRange<float> FrameExtentsPx;
		/** The pixel range that the scrubber handle (thumb) occupies */
		TRange<float> HandleRangePx;
		/** The style of the scrubber handle */
		ESequencerScrubberStyle Style;
		/** The style of the scrubber handle */
		bool bDrawExtents;
	};

	/** Utility struct for converting between scrub range space and local/absolute screen space */
	struct FScrubRangeToScreen
	{
		double ViewStart;
		float PixelsPerInput;

		FScrubRangeToScreen(const TRange<double>& InViewInput, const FVector2D& InWidgetSize)
		{
			float ViewInputRange = InViewInput.Size<double>();

			ViewStart = InViewInput.GetLowerBoundValue();
			PixelsPerInput = ViewInputRange > 0 ? (InWidgetSize.X / ViewInputRange) : 0;
		}

		/** Local Widget Space -> Curve Input domain. */
		double LocalXToInput(float ScreenX) const
		{
			return PixelsPerInput > 0 ? (ScreenX / PixelsPerInput) + ViewStart : ViewStart;
		}

		/** Local Widget Space -> Curve Input domain. */
		double LocalDeltaXToDeltaInput(float ScreenDeltaX) const
		{
			return PixelsPerInput > 0 ? (ScreenDeltaX / PixelsPerInput) : 0;
		}

		/** Curve Input domain -> local Widget Space */
		float InputToLocalX(double Input) const
		{
			return (Input - ViewStart) * PixelsPerInput;
		}
	};

	struct FDrawTickArgs
	{
		/** Geometry of the area */
		FGeometry AllottedGeometry;
		/** Culling rect of the area */
		FSlateRect CullingRect;
		/** Color of each tick */
		FLinearColor TickColor;
		/** Color of each tick's text, if displayed */
		FLinearColor TickTextColor;
		/** Offset in Y where to start the tick */
		float TickOffset;
		/** Height in of major ticks */
		float MajorTickHeight;
		/** Start layer for elements */
		int32 StartLayer;
		/** Draw effects to apply */
		ESlateDrawEffect DrawEffects;
		/** Whether or not to only draw major ticks */
		bool bOnlyDrawMajorTicks;
		/** Whether or not to mirror labels */
		bool bMirrorLabels;
		
	};

	bool IsEvaluating() const { return bIsEvaluating; }

	/** Set that's evaluating */
	void SetIsEvaluating()
	{
		bIsEvaluating = true;
	}

protected:

	virtual FReply OnMouseMoveImpl(SWidget& WidgetOwner
		, const FGeometry& InGeometry
		, const FPointerEvent& InPointerEvent
		, const bool bInFromTimeSlider);

	virtual FReply OnMouseButtonUp_ContextMenu(SWidget& WidgetOwner, const FGeometry& InGeometry, const FPointerEvent& InPointerEvent);
	virtual FReply OnMouseButtonUp_MouseDragType(SWidget& WidgetOwner
		, const FGeometry& InGeometry
		, const FPointerEvent& InPointerEvent
		, const FScrubRangeToScreen& InRangeToScreen
		, const FFrameTime& InMouseTime);

	virtual FReply OnMouseMove_LeftAndMiddleMouseDown(const FGeometry& InGeometry
		, const FPointerEvent& InPointerEvent
		, const bool bInFromTimeSlider
		, const FScrubRangeToScreen& InRangeToScreen
		, const USequencerSettings& InSequencerSettings);
	virtual FReply OnMouseMove_RightMouseDown(const FGeometry& InGeometry
		, const FPointerEvent& InPointerEvent, const USequencerSettings& InSequencerSettings);

	virtual void HandleDragMark_NoDragType(const int32 InMarkIndex);
	virtual void HandleDragMark(const FFrameNumber& InLastDiffFrame);

	void ResetMouseDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent);

	/**
	 * Fire the End delegate for any transactional drag currently in progress and clear MouseDragType.
	 */
	void EndAnyInProgressTransactionalDrag();

	/**
	 * Draw time tick marks
	 *
	 * @param OutDrawElements	List to add draw elements to
	 * @param ViewRange			The currently visible time range in seconds
	 * @param RangeToScreen		Time range to screen space converter
	 * @param InArgs			Parameters for drawing the tick lines
	 */
	void DrawTicks( FSlateWindowElementList& OutDrawElements, const TRange<double>& ViewRange, const FScrubRangeToScreen& RangeToScreen, FDrawTickArgs& InArgs ) const;

	/**
	 * Draw time tick marks
	 *
	 * @param OutDrawElements	List to add draw elements to
	 * @param ViewRange			The currently visible time range in seconds
	 * @param RangeToScreen		Time range to screen space converter
	 * @param InArgs			Parameters for drawing the tick lines
	 */
	void DrawLinearTicks(FSlateWindowElementList& OutDrawElements, const TRange<double>& ViewRange, const FScrubRangeToScreen& RangeToScreen, FDrawTickArgs& InArgs) const;

	/**
	 * Draw horizontal separator lines between tracks
	 */
	void DrawHorizontalTrackSeparatorLines(FSlateWindowElementList& OutDrawElements, FDrawTickArgs& InArgs) const;

	/**
	 * Draw the selection range.
	 *
	 * @return The new layer ID.
	 */
	int32 DrawSelectionRange(const FGeometry& InGeometry,
		const FSlateRect& InCullingRect,
		FSlateWindowElementList& OutDrawElements,
		int32 LayerId,
		const FScrubRangeToScreen& InRangeToScreen,
		const FPaintPlaybackRangeArgs& InRangeArgs) const;

	/**
	 * Draw the playback range.
	 *
	 * @return the new layer ID
	 */
	int32 DrawPlaybackRange(const FGeometry& InGeometry,
		const FSlateRect& InCullingRect,
		FSlateWindowElementList& OutDrawElements,
		int32 LayerId,
		const FScrubRangeToScreen& InRangeToScreen,
		const FPaintPlaybackRangeArgs& InRangeArgs) const;

	/** Draw playback range start/end markers only. */
	int32 DrawPlaybackRangeMarkers(const FGeometry& InGeometry,
		const FSlateRect& InCullingRect,
		FSlateWindowElementList& OutDrawElements,
		int32 LayerId,
		const FScrubRangeToScreen& InRangeToScreen,
		const FPaintPlaybackRangeArgs& InRangeArgs) const;

	/** Draw dimmed regions outside the playback range only. */
	int32 DrawPlaybackRangeExcludedRegions(const FGeometry& InGeometry,
		const FSlateRect& InCullingRect,
		FSlateWindowElementList& OutDrawElements,
		int32 LayerId,
		const FScrubRangeToScreen& InRangeToScreen,
		const FPaintPlaybackRangeArgs& InRangeArgs) const;

	/**
	 * Draw the playback range.
	 *
	 * @return the new layer ID
	 */
	int32 DrawSubSequenceRange(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const;

	/** Draw subsequence range start/end markers only. */
	int32 DrawSubSequenceRangeMarkers(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const;

	/** Draw dimmed regions outside the subsequence range only. */
	int32 DrawSubSequenceRangeExcludedRegions(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const;

	/** Draw subsequence boundary hash marks only. */
	int32 DrawSubSequenceRangeHashMarks(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const;

	/**
	 * Draw the vertical frames.
	 *
	 * @return the new layer ID
	 */
	int32 DrawVerticalFrames(const FGeometry& AllottedGeometry, const FScrubRangeToScreen& RangeToScreen, FSlateWindowElementList& OutDrawElements, int32 LayerId, const ESlateDrawEffect& DrawEffects) const;

	/**
	 * Draw the marked frames.
	 *
	 * @return the new layer ID
	 */
	int32 DrawMarkedFrames(const FGeometry& InGeometry,
		const FScrubRangeToScreen& InRangeToScreen,
		FSlateWindowElementList& OutDrawElements,
		int32 LayerId,
		const ESlateDrawEffect& InDrawEffects,
		bool bInDrawLabels) const;

	/**
	 * Draw any scaling anchors.
	 *
	 * @return the new layer ID
	 */
	int32 DrawScalingAnchors( const FGeometry& AllottedGeometry, const FScrubRangeToScreen& RangeToScreen, FSlateWindowElementList& OutDrawElements, int32 LayerId, const ESlateDrawEffect& DrawEffects ) const;

	/**
	 * Hit test the lower bound of a range
	 */
	bool HitTestRangeStart(const FScrubRangeToScreen& RangeToScreen, const TRange<double>& Range, float HitPixel) const;

	/**
	 * Hit test the upper bound of a range
	 */
	bool HitTestRangeEnd(const FScrubRangeToScreen& RangeToScreen, const TRange<double>& Range, float HitPixel) const;

public:

	/**
	 * Hit test marks
	 *
	 * @return The mark index hit
	 */
	bool HitTestMark(const FGeometry& AllottedGeometry, const FScrubRangeToScreen& RangeToScreen, float HitPixel, bool bTestLabelBox, int32* OutMarkIndex = nullptr, FFrameNumber* OutMarkFrameNumber = nullptr) const;

protected:

	/**
	 * Get marked frame label box size
	 */
	void GetMarkLabelGeometry(const FGeometry& AllottedGeometry, const FScrubRangeToScreen& RangeToScreen, const FMovieSceneMarkedFrame& MarkedFrame, FVector2D& OutPosition, FVector2D& OutSize, bool& bIsDrawLeft) const;

	FFrameTime SnapTimeToNearestKey(const FPointerEvent& MouseEvent, const FScrubRangeToScreen& RangeToScreen, float CursorPos, FFrameTime InTime) const;

public:
	void SetPlaybackRangeStart(FFrameNumber NewStart);
	void SetPlaybackRangeEnd(FFrameNumber NewEnd);

	void SetSelectionRangeStart(FFrameNumber NewStart);
	void SetSelectionRangeEnd(FFrameNumber NewEnd);

	FFrameTime ComputeScrubTimeFromMouse(const FGeometry& Geometry, const FPointerEvent& MouseEvent, FScrubRangeToScreen RangeToScreen) const;
	FFrameTime ComputeFrameTimeFromMouse(const FGeometry& Geometry, FVector2D ScreenSpacePosition, FScrubRangeToScreen RangeToScreen, bool CheckSnapping = true) const;

	/**
	 * Get the pixel matrics of the Scrubber
	 * @param ScrubTime			The qualified time of the scrubber
	 * @param RangeToScreen		Range to screen helper
	 * @param DilationPixels	Number of pixels to dilate the handle by
	 * return FScrubberMetrics struct
	 */
	virtual FScrubberMetrics GetScrubPixelMetrics(const FQualifiedFrameTime& ScrubTime, const FScrubRangeToScreen& RangeToScreen, float DilationPixels = 0.f) const;

	void ClearMarkSelection();
	TOptional<FFrameNumber> GetMarkFrameNumber(const int32 InMarkIndex) const;
	void AddMarkAtFrame(const FFrameNumber InFrameNumber);
	void SetMarkAtFrame(const int32 InMarkIndex, const FFrameNumber InFrameNumber);
	void DeleteSelectedMarks();
	void DeleteAllMarks();

protected:
	virtual TSharedPtr<SWidget> OpenContextMenu(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent);

	FFrameTime SnapSequencerTime(FFrameTime InTime) const;

	FScrubberMetrics GetHitTestScrubPixelMetrics(const FScrubRangeToScreen& RangeToScreen) const;

	void ResetMouseInput();

	/** Pointer back to the sequencer object */
	TWeakPtr<FSequencer> WeakSequencer;

	FTimeSliderArgs TimeSliderArgs;

	/** Brush for drawingthe fill area on the scrubber */
	const FSlateBrush* ScrubFillBrush;
	
	/** Brush for drawing a downwards facing scrub handle */
	const FSlateBrush* FrameBlockScrubHandleDownBrush, *VanillaScrubHandleDownBrush;

	/** Font measure service */
	TSharedPtr<FSlateFontMeasure> FontMeasureService;

	/** Font info for the marked frames labels */
	FSlateFontInfo SmallLayoutFont;
	FSlateFontInfo SmallBoldLayoutFont;
	
	/** Total mouse delta during dragging **/
	float DistanceDragged;

	/** Drag operations for this time controller */
	enum EDragType
	{
		DRAG_NONE,
		DRAG_SCRUBBING_TIME,
		DRAG_SETTING_RANGE,
		DRAG_PLAYBACK_START,
		DRAG_PLAYBACK_END,
		DRAG_SELECTION_START,
		DRAG_SELECTION_END,
		DRAG_MARK
	};
	EDragType MouseDragType;
	
	/** If mouse down was in time scrubbing region, only allow setting time when mouse is pressed down in the region */
	bool bMouseDownInRegion;

	/** If we are currently panning the panel */
	bool bPanning;

	/** Mouse down position */
	TOptional<FVector2D> MouseDownPosition;
	
	/** Last mouse position */
	TOptional<FVector2D> LastMousePosition;

	/** Geometry on mouse down */
	FGeometry MouseDownGeometry;

	/** Playback range when the mouse is first pressed down */
	TRange<FFrameNumber> MouseDownPlaybackRange;

	/** Selection range when the mouse is first pressed down */
	TRange<FFrameNumber> MouseDownSelectionRange;

	/** Range stack */
	TArray<TRange<double>> ViewRangeStack;

	/** Index of mark being hovered */
	int32 HoverMarkIndex;

	/** When > 0, we should not show context menus */
	int32 ContextMenuSuppression;

	/** If not evaluating, we draw the time box to be yellow and not default */
	bool bIsEvaluating = true;

	friend FContextMenuSuppressor;
};

struct FContextMenuSuppressor
{
	FContextMenuSuppressor(TSharedRef<FSequencerTimeSliderController> InTimeSliderController)
		: TimeSliderController(InTimeSliderController)
	{
		++TimeSliderController->ContextMenuSuppression;
	}
	~FContextMenuSuppressor()
	{
		--TimeSliderController->ContextMenuSuppression;
	}

private:
	FContextMenuSuppressor(const FContextMenuSuppressor&);
	FContextMenuSuppressor& operator=(const FContextMenuSuppressor&);

	TSharedRef<FSequencerTimeSliderController> TimeSliderController;
};
