// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Layout/Geometry.h"
#include "Input/Events.h"
#include "SequencerToolMenuContext.generated.h"

class FSequencerTimeSliderController;
class ISequencer;
class SSequencerPlayRateCombo;

UCLASS(MinimalAPI)
class USequencerToolMenuContext : public UObject
{
	GENERATED_BODY()
public:
	TWeakPtr<ISequencer> WeakSequencer;
};

UCLASS(MinimalAPI)
class USequencerClockSourceMenuContext : public USequencerToolMenuContext
{
	GENERATED_BODY()
public:
	TWeakPtr<SSequencerPlayRateCombo> WeakSequencerCombo;
};

UCLASS(MinimalAPI)
class USequencerTimeSliderControllerMenuContext : public UObject
{
	GENERATED_BODY()
	
public:

	// The time slider controller associated with this context
	TWeakPtr<FSequencerTimeSliderController> WeakTimeSliderController;
	
	FGeometry Geometry;

	FPointerEvent PointerEvent;

	FFrameNumber FrameNumber;
};