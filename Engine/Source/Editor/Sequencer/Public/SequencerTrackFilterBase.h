// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Deprecated
/**
 * DEPRECATED SequencerTrackFilterBase.h Use Filters/SequencerTrackFilterBase.h instead
 */
 
#pragma once

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.5, "Use Filters/SequencerTrackFilterBase.h instead of SequencerTrackFilterBase.h.")
#include "Filters/SequencerTrackFilterBase.h"
