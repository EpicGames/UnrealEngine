// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Deprecated
/**
 * DEPRECATED SequencerTrackFilterExtension.h Use Filters/SequencerTrackFilterExtension.h instead
 */
 
#pragma once

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.5, "Use Filters/SequencerTrackFilterExtension.h instead of SequencerTrackFilterExtension.h.")
#include "Filters/SequencerTrackFilterExtension.h"
