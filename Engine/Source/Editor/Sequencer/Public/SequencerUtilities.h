// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Framework/SlateDelegates.h"
#include "Misc/FrameTime.h"
#include "MovieSceneBindingProxy.h"
#include "SequencerCoreFwd.h"
#include "SequencerUtilities.generated.h"

#define UE_API SEQUENCER_API

class ACineCameraActor;
template <typename> class TAttribute;
template <typename> class TSubclassOf;
class UActorFactory;
class UMovieSceneDecorationContainerObject;
class UMovieSceneFolder;
class UMovieSceneTrack;
class UMovieSceneSection;
class UMovieSceneSequence;
class UMovieSceneTimeWarpGetter;
class ISequencer;
class FMenuBuilder;
struct FBakingAnimationKeySettings;
struct FMovieSceneBinding;
struct FMovieScenePossessable;
struct FMovieSceneSequenceID;
struct FMovieSceneSpawnable;
struct FNotificationInfo;
class ULevelSequence;
class UMovieSceneCustomBinding;
enum class EMovieSceneBlendType : uint8;

namespace UE::Sequencer
{

class ITrackExtension;
class ISectionOwnerExtension;

struct FCreateBindingParams
{
	UE_DEPRECATED(5.4, "Please use FSequencerUtilitiesCreateBindingParams directly.")
	FCreateBindingParams(const FString& InBindingNameOverride)
		: BindingNameOverride(InBindingNameOverride)
	{}

	FCreateBindingParams()
	{}
	
	FCreateBindingParams& Name(FString&& InName)
	{
		BindingNameOverride = MoveTemp(InName);
		return *this;
	}
	FCreateBindingParams& Folder(const FName& InFolder)
	{
		DesiredFolder = InFolder;
		return *this;
	}

	FString BindingNameOverride;
	
	FName DesiredFolder;
	
	/* If true, will prefer the creation of a custom or regular Spawnable binding, unless such is incompatible with the passed in object.*/
	bool bSpawnable = false;
	/* If true, will prefer the creation of a custom Replaceable binding, unless such is incompatible with the passed in object.*/
	bool bReplaceable = false;
	/* If true, will allow the creation of custom bindings if they support the object type.*/
	bool bAllowCustomBinding = true;
	/* If true, will allow the creation of an empty binding if a UObject* passed in is nullptr*/
	bool bAllowEmptyBinding = false;
	/* If set, will attempt to replace any existing possessable binding at the provided guid and binding index */
	FGuid ReplacementGuid;
	/* Optional BindingIndex used if bReplace is true to replace a specific possessable binding*/
	int32 BindingIndex = 0;
	/* Optional pre-created custom binding to use when creating the binding.*/
	TObjectPtr<UMovieSceneCustomBinding> CustomBinding = nullptr;
	/* May be used depending on options if an asset UObject is passed in to create a custom or regular spawnable actor binding*/
	TObjectPtr<UActorFactory> ActorFactory = nullptr;
	/* Whether to set up default tracks and child components for a new binding. May be set false for example by a copy/paste which won't want that.*/
	bool bSetupDefaults = true;
};

SEQUENCER_API bool GetPathToObject(UObject* InObject, TArray<UObject*>& OutObjectPath);

} // namespace UE::Sequencer

/* Paste folders params */
USTRUCT(BlueprintType)
struct FMovieScenePasteFoldersParams
{
	GENERATED_BODY()

	FMovieScenePasteFoldersParams() {}
	FMovieScenePasteFoldersParams(UMovieSceneSequence* InSequence, UMovieSceneFolder* InParentFolder = nullptr)
		: Sequence(InSequence)
		, ParentFolder(InParentFolder) {}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TObjectPtr<UMovieSceneSequence> Sequence;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TObjectPtr<UMovieSceneFolder> ParentFolder;
};

/** Decoration paste destination supplied alongside FMovieScenePasteSectionsParams::Tracks. */
USTRUCT(BlueprintType)
struct FMovieScenePasteDecorationTarget
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TObjectPtr<UMovieSceneDecorationContainerObject> Container;

	/** Decoration the user selected directly. Hosted sections matching this decoration's GetHostedSectionClass() are routed here. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TObjectPtr<UObject> PreferredDecoration;
};

/* Paste sections params */
USTRUCT(BlueprintType)
struct FMovieScenePasteSectionsParams
{
	GENERATED_BODY()

	FMovieScenePasteSectionsParams() {}
	FMovieScenePasteSectionsParams(const TArray<UMovieSceneTrack*>& InTracks, const TArray<int32>& InTrackRowIndices, FFrameTime InTime)
		: Tracks(InTracks)
		, TrackRowIndices(InTrackRowIndices)
		, Time(InTime) {}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TArray<TObjectPtr<UMovieSceneTrack>> Tracks;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TArray<int32> TrackRowIndices;

	/** Decoration paste destinations. Each entry has at least Container or PreferredDecoration set. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TArray<FMovieScenePasteDecorationTarget> DecorationTargets;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	FFrameTime Time;

	/** Append a paste target. Decoration info is recorded in DecorationTargets only when at least one of Container / PreferredDecoration is supplied. */
	SEQUENCER_API void AddTarget(
		UMovieSceneTrack* Track,
		int32 RowIndex = INDEX_NONE,
		UMovieSceneDecorationContainerObject* DecorationContainer = nullptr,
		UObject* PreferredDecoration = nullptr);
};

/* Paste decorations params */
USTRUCT(BlueprintType)
struct FMovieScenePasteDecorationsParams
{
	GENERATED_BODY()

	FMovieScenePasteDecorationsParams() {}
	FMovieScenePasteDecorationsParams(const TArray<UMovieSceneDecorationContainerObject*>& InContainers)
		: Containers(InContainers) {}

	/** Targets to receive a clipboard decoration. Each target must be registered-compatible (via IMovieSceneModule::RegisterCompatibleDecoration) and must not already host a decoration of the same class. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TArray<TObjectPtr<UMovieSceneDecorationContainerObject>> Containers;
};

/* Paste tracks params */
USTRUCT(BlueprintType)
struct FMovieScenePasteTracksParams
{
	GENERATED_BODY()

	FMovieScenePasteTracksParams() {}
	FMovieScenePasteTracksParams(UMovieSceneSequence* InSequence, const TArray<FMovieSceneBindingProxy>& InBindings = TArray<FMovieSceneBindingProxy>(), UMovieSceneFolder* InParentFolder = nullptr, const TArray<UMovieSceneFolder*>& InFolders = TArray<UMovieSceneFolder*>())
		: Sequence(InSequence)
		, Bindings(InBindings)
		, ParentFolder(InParentFolder)
		, Folders(InFolders) {}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TObjectPtr<UMovieSceneSequence> Sequence;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TArray<FMovieSceneBindingProxy> Bindings;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TObjectPtr<UMovieSceneFolder> ParentFolder;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TArray<TObjectPtr<UMovieSceneFolder>> Folders;
};

/* Paste bindings params */
USTRUCT(BlueprintType)
struct FMovieScenePasteBindingsParams
{
	GENERATED_BODY()

	FMovieScenePasteBindingsParams(const TArray<FMovieSceneBindingProxy>& InBindings = TArray<FMovieSceneBindingProxy>(), UMovieSceneFolder* InParentFolder = nullptr, const TArray<UMovieSceneFolder*>& InFolders = TArray<UMovieSceneFolder*>(), bool bInDuplicateExistingActors = false)
		: Bindings(InBindings)
		, ParentFolder(InParentFolder)
		, Folders(InFolders)
		, bDuplicateExistingActors(bInDuplicateExistingActors) {}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TArray<FMovieSceneBindingProxy> Bindings;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TObjectPtr<UMovieSceneFolder> ParentFolder;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TArray<TObjectPtr<UMovieSceneFolder>> Folders;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	bool bDuplicateExistingActors;

	/** Original (template) actor path name to world to the pasted actor */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TMap<FName, TObjectPtr<AActor>> PastedActors;
};

USTRUCT()
struct FSequencerChangeBindingInfo
{
	GENERATED_BODY()

	FSequencerChangeBindingInfo() {}
	FSequencerChangeBindingInfo(FGuid InBindingID, int32 InBindingIndex)
		: BindingID(InBindingID)
		, BindingIndex(InBindingIndex) {}

	UPROPERTY()
	FGuid BindingID;

	UPROPERTY()
	int32 BindingIndex = -1;
};

/**
 * Helper structure to track when an sequencer is opened and closed.  Note, it will only track from when DoStartup is invoked and will not enumerate existing open sequencers.
 */
struct FOpenSequencerWatcher
{
	/**
	 * Begin watching the sequencer.
	 *
	 * @param OnStartup - will be called when we start listening for sequencer events. This will always be after engine startup if called during module load; otherwise it will be immediately called.
	 */
	UE_API void DoStartup(TFunction<void ()> StartupComplete);

	/** Invoked when a new sequencer is created. */
	UE_API void OnSequencerCreated(TSharedRef<ISequencer> InSequencer);

	/** Registered delegate when a sequencer is closed. */
	UE_API void OnSequencerClosed(TSharedRef<ISequencer> InSequencer);

	/** Internal structure for tracking sequencers. */
	struct FOpenSequencerData
	{
		/** Weak pointer to the sequencer itself, if locally opened. */
		TWeakPtr<ISequencer> WeakSequencer;

		/** Delegate handle to the Close event for the sequencer, if locally opened. */
		FDelegateHandle OnCloseEventHandle;
	};

	/** List of open sequencers currently known by the watcher. */
	TArray<FOpenSequencerData> OpenSequencers;
};

struct FSequencerUtilities
{
	/* Creates a button (used for +Section) that opens a ComboButton with a user-defined sub-menu content. */
	static UE_API TSharedRef<SWidget> MakeAddButton(FText HoverText, FOnGetContent MenuContent, const TAttribute<bool>& HoverState, TWeakPtr<ISequencer> InSequencer);
	
	/* Creates a button (used for +Section) that fires a user-defined OnClick response with no sub-menu. */
	static UE_API TSharedRef<SWidget> MakeAddButton(FText HoverText, FOnClicked OnClicked, const TAttribute<bool>& HoverState, TWeakPtr<ISequencer> InSequencer);

	static UE_API void MakeTimeWarpMenuEntry(FMenuBuilder& MenuBuilder, UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::FViewModel> WeakViewModel);

	static UE_API void PopulateTimeWarpSubMenu(FMenuBuilder& MenuBuilder, TFunction<void(TSubclassOf<UMovieSceneTimeWarpGetter>)> OnTimeWarpPicked);
	
	UE_DEPRECATED(5.8, "Please use the version that takes an ISectionOwnerExtension")
	static UE_API void PopulateTimeWarpChannelSubMenu(FMenuBuilder& MenuBuilder, UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::ITrackExtension> TrackModel);

	static UE_API void PopulateTimeWarpChannelSubMenu(FMenuBuilder& MenuBuilder, UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::ISectionOwnerExtension> SectionOwner);

	static UE_API void CreateNewSection(UMovieSceneTrack* InTrack, TWeakPtr<ISequencer> InSequencer, int32 InRowIndex, EMovieSceneBlendType InBlendType);

	static UE_API void PopulateMenu_CreateNewSection(FMenuBuilder& MenuBuilder, int32 RowIndex, UMovieSceneTrack* Track, TWeakPtr<ISequencer> InSequencer);

	static UE_API void PopulateMenu_BlenderSubMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track, TWeakPtr<ISequencer> InSequencer);

	static UE_API void PopulateMenu_SetBlendType(FMenuBuilder& MenuBuilder, UMovieSceneSection* Section, TWeakPtr<ISequencer> InSequencer);

	static UE_API void PopulateMenu_SetBlendType(FMenuBuilder& MenuBuilder, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InSections, TWeakPtr<ISequencer> InSequencer);

	static UE_API TArray<FString> GetAssociatedLevelSequenceMapPackages(const ULevelSequence* InSequence);
	static UE_API TArray<FString> GetAssociatedLevelSequenceMapPackages(FName LevelSequencePackageName);

	/** 
	 * Generates a unique FName from a candidate name given a set of already existing names.  
	 * The name is made unique by appending a number to the end.
	 */
	static UE_API FName GetUniqueName(FName CandidateName, const TArray<FName>& ExistingNames);

	/** Add existing actors to Sequencer */
	static UE_API TArray<FGuid> AddActors(TSharedRef<ISequencer> Sequencer, const TArray<TWeakObjectPtr<AActor> >& InActors);

	/** Create a new camera actor and add it to Sequencer */
	static UE_API FGuid CreateCamera(TSharedRef<ISequencer> Sequencer, const bool bSpawnable, ACineCameraActor*& OutActor);

	/** Create a new camera from a rig and add it to Sequencer */
	static UE_API FGuid CreateCameraWithRig(TSharedRef<ISequencer> Sequencer, AActor* Actor, const bool bSpawnable, ACineCameraActor*& OutActor);

	static UE_API FGuid MakeNewSpawnable(TSharedRef<ISequencer> Sequencer, UObject& SourceObject, UActorFactory* ActorFactory = nullptr, bool bSetupDefaults = true, FName SpawnableName = NAME_None);

	/** Convert the requested object binding to old-style spawnable. If there are multiple objects assigned to the possessable, multiple spawnables will be created */
	static UE_API TArray<FMovieSceneSpawnable*> ConvertToSpawnable(TSharedRef<ISequencer> Sequencer, FGuid PossessableGuid);
	
	/** Convert the requested object binding and object binding index to a possessable */
	static UE_API bool CanConvertToPossessable(TSharedRef<ISequencer> Sequencer, FGuid BindingGuid, int32 BindingIndex = 0);
	static UE_API FMovieScenePossessable* ConvertToPossessable(TSharedRef<ISequencer> Sequencer, FGuid BindingGuid, int32 BindingIndex=0);

	/** Convert the selected object binding and object binding index to a custom binding of the chosen type.*/
	static UE_API bool CanConvertToCustomBinding(TSharedRef<ISequencer> Sequencer, FGuid BindingGuid, TSubclassOf<UMovieSceneCustomBinding> CustomBindingType, int32 BindingIndex = 0);
	static UE_API FMovieScenePossessable* ConvertToCustomBinding(TSharedRef<ISequencer> Sequencer, FGuid BindingGuid, TSubclassOf<UMovieSceneCustomBinding> CustomBindingType, int32 BindingIndex = 0);

	/** Copy/paste folders */
	static UE_API void CopyFolders(TSharedRef<ISequencer> Sequencer, const TArray<UMovieSceneFolder*>& Folders, FString& FoldersExportedText, FString& TracksExportedText, FString& ObjectsExportedText);
	static UE_API bool PasteFolders(const FString& TextToImport, FMovieScenePasteFoldersParams PasteFoldersParams, TArray<UMovieSceneFolder*>& OutFolders, TArray<FNotificationInfo>& OutErrors);
	static UE_API bool CanPasteFolders(const FString& TextToImport);

	UE_DEPRECATED(5.5, "CopyFolders now gathers objects and tracks within the folders. Please use CopyFolders that outputs ObjectsExportedText and TracksExportedText")
	static UE_API void CopyFolders(const TArray<UMovieSceneFolder*>& Folders, FString& ExportedText);

	/** Copy/paste tracks */
	static UE_API void CopyTracks(const TArray<UMovieSceneTrack*>& Tracks, const TArray<UMovieSceneFolder*>& InFolders, FString& ExportedText);
	static UE_API bool PasteTracks(const FString& TextToImport, FMovieScenePasteTracksParams PasteTracksParams, TArray<UMovieSceneTrack*>& OutTracks, TArray<FNotificationInfo>& OutErrors);
	static UE_API bool CanPasteTracks(const FString& TextToImport);

	/** Copy/paste sections */
	static UE_API void CopySections(const TArray<UMovieSceneSection*>& Sections, FString& ExportedText);
	static UE_API bool PasteSections(const FString& TextToImport, FMovieScenePasteSectionsParams PasteSectionsParams, TArray<UMovieSceneSection*>& OutSections, TArray<FNotificationInfo>& OutErrors);
	static UE_API bool CanPasteSections(const FString& TextToImport);

	/** Copy/paste decorations */
	static UE_API void CopyDecorations(const TArray<UObject*>& Decorations, FString& ExportedText);
	static UE_API bool PasteDecorations(const FString& TextToImport, FMovieScenePasteDecorationsParams PasteDecorationsParams, TArray<UObject*>& OutDecorations, TArray<FNotificationInfo>& OutErrors);
	static UE_API bool CanPasteDecorations(const FString& TextToImport);

	/** Copy/paste object bindings */
	static UE_API void CopyBindings(TSharedRef<ISequencer> Sequencer, const TArray<FMovieSceneBindingProxy>& Bindings, const TArray<UMovieSceneFolder*>& InFolders, FString& ExportedText);
	static UE_API void CopyBindings(TSharedRef<ISequencer> Sequencer, const TArray<FMovieSceneBindingProxy>& Bindings, const TArray<UMovieSceneFolder*>& InFolders, FOutputDevice& Ar);
	static UE_API bool PasteBindings(const FString& TextToImport, TSharedRef<ISequencer> Sequencer, FMovieScenePasteBindingsParams PasteBindingsParams, TArray<FMovieSceneBindingProxy>& OutBindings, TArray<FNotificationInfo>& OutErrors);
	static UE_API bool CanPasteBindings(TSharedRef<ISequencer> Sequencer, const FString& TextToImport);
	static UE_API TArray<FString> GetPasteBindingsObjectNames(TSharedRef<ISequencer> Sequencer, const FString& TextToImport);
	/**
	 * Recursively finds the most appropriate Resolution Context for a given Parent Guid of a Possessable
	 */

	static UE_API UObject* FindResolutionContext(TSharedRef<ISequencer> Sequencer
		, UMovieSceneSequence& InSequence
		, UMovieScene& InMovieScene
		, const FGuid& InParentGuid
		, UObject* InPlaybackContext);

	/** Utility functions for managing bindings */
	static UE_API FGuid CreateBinding(TSharedRef<ISequencer> Sequencer, UObject& InObject, const UE::Sequencer::FCreateBindingParams& Params = UE::Sequencer::FCreateBindingParams());
	static UE_API FGuid CreateOrReplaceBinding(TSharedRef<ISequencer> Sequencer, UObject* Object, const UE::Sequencer::FCreateBindingParams& Params = UE::Sequencer::FCreateBindingParams());
	static UE_API FGuid CreateOrReplaceBinding(TSharedPtr<ISequencer> Sequencer, UMovieSceneSequence* Sequence, UObject* Object, const UE::Sequencer::FCreateBindingParams& Params = UE::Sequencer::FCreateBindingParams());
	static UE_API void UpdateBindingIDs(TSharedRef<ISequencer> Sequencer, FGuid OldGuid, FGuid NewGuid);
	static UE_API FGuid AssignActor(TSharedRef<ISequencer> Sequencer, AActor* Actor, FGuid InObjectBinding);
	static UE_API void AddActorsToBinding(TSharedRef<ISequencer> Sequencer, const TArray<AActor*>& Actors, const FMovieSceneBindingProxy& ObjectBinding);
	static UE_API void AddObjectsToBinding(TSharedRef<ISequencer> Sequencer, const TArray<UObject*>& Objects, const FMovieSceneBindingProxy& ObjectBinding, UObject* ResolutionContext);
	static UE_API void ReplaceBindingWithActors(TSharedRef<ISequencer> Sequencer, const TArray<AActor*>& Actors, const FMovieSceneBindingProxy& ObjectBinding);
	static UE_API void RemoveActorsFromBinding(TSharedRef<ISequencer> Sequencer, const TArray<AActor*>& Actors, const FMovieSceneBindingProxy& ObjectBinding);
	static UE_API void RemoveAllBindings(TSharedRef<ISequencer> Sequencer, const FMovieSceneBindingProxy& ObjectBinding);
	static UE_API void RemoveInvalidBindings(TSharedRef<ISequencer> Sequencer, const FMovieSceneBindingProxy& ObjectBinding);

	static UE_API bool BakeTransform(TSharedRef<ISequencer> Sequencer, const TArray<FMovieSceneBindingProxy>& ObjectBindings, const FBakingAnimationKeySettings& InSettings);

	/** Show a read only error if the movie scene is locked */
	static UE_API void ShowReadOnlyError();
	
	/** Show an error if spawnable is not allowed in a movie scene*/
	static UE_API void ShowSpawnableNotAllowedError();

	// Methods exposing FSequencer functionality outside of the Sequencer module.
	// This is needed while things are moving into view-models here and there, step by step.
	static UE_API void SaveCurrentMovieSceneAs(TSharedRef<ISequencer> Sequencer);
	static UE_API void SynchronizeExternalSelectionWithSequencerSelection (TSharedRef<ISequencer> Sequencer);
	static UE_API TRange<FFrameNumber> GetTimeBounds(TSharedRef<ISequencer> Sequencer);

	// Functions allowing menus to be built for modifying bindings
	static UE_API void AddChangeClassMenu(FMenuBuilder& MenuBuilder, TSharedRef<ISequencer> Sequencer, const TArray<FSequencerChangeBindingInfo>& BindingsToConvert, TFunction<void()> OnBindingChanged);
	static UE_API void HandleTemplateActorClassPicked(UClass* ChosenClass, TSharedRef<ISequencer> Sequencer, const TArray<FSequencerChangeBindingInfo>& BindingsToConvert, TFunction<void()> OnBindingChanged);

	/** Get a movie scene sequence from a FMovideSceneSequenceID */
	static UE_API UMovieSceneSequence* GetMovieSceneSequence(TSharedPtr<ISequencer>& InSequencer, const FMovieSceneSequenceID& SequenceID);

	/** Snap sections to timeline using source timecode */
	static UE_API void SnapSectionsToTimelineUsingSourceTimecode(TSharedRef<ISequencer> Sequencer, const TArray<UMovieSceneSection*>& Sections);

	/** Sync section using source timecode */
	static UE_API void SyncSectionsUsingSourceTimecode(TSharedRef<ISequencer> Sequencer, const TArray<UMovieSceneSection*>& Sections);

	/** Attempts to automatically fix up broken actor references in the current scene */
	static UE_API void FixActorReferences(TSharedRef<ISequencer> Sequencer);
};

#undef UE_API
