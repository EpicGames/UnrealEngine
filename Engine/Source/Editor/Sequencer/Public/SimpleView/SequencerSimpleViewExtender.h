// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/IDelegateInstance.h"
#include "MVVM/ViewModelPtr.h"
#include "ToolableTimeline/ToolableTimelineChannelFilter.h"

#define UE_API SEQUENCER_API

class ISequencer;

namespace UE::Sequencer
{
	class FChannelModel;
}

namespace UE::Sequencer::ToolableTimeline
{
	class FToolableTimelineChannelCache;
}

/** Enumerates additional view models that a Simple View timeline should treat as selected. Return true to continue enumerating, or false to stop. */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnEnumerateAdditionalSimpleViewSelectedModels, const ISequencer&, TFunctionRef<bool(const UE::Sequencer::FViewModelPtr&)>);

namespace UE::Sequencer::SimpleView
{

/** Extends the Sequencer Simple View */
class FSequencerSimpleViewExtender
{
public:
	/**
	 * Registers a new channel filter function to be applied in the Sequencer's Simple View and triggers
	 * a notification to signal the change in filters.
	 *
	 * @param InFilter The channel filter function to be registered. This function determines
	 *                 whether a channel will be included based on custom filtering logic.
	 * @return A handle representing the registered channel filter, which can be used to
	 *         identify and unregister the filter later if needed.
	 */
	UE_API static FDelegateHandle RegisterChannelFilter(ToolableTimeline::FChannelFilterFunction&& InFilter);

	/**
	 * Unregisters a channel filter from the Sequencer Simple View using the specified delegate handle
	 * and triggers a notification to signal the change in filters.
	 *
	 * @param InDelegateHandle The handle of the channel filter to be unregistered.
	 */
	UE_API static void UnregisterChannelFilter(const FDelegateHandle& InDelegateHandle);

	/**
	 * Retrieves the collection of registered channel filters.
	 *
	 * @return A reference to the static array containing all registered channel filters.
	 */
	static TArray<ToolableTimeline::FRegisteredChannelFilter>& GetRegisteredChannelFilters();

	/**
	 * @return A reference to the multicast delegate that is broadcast whenever channel filters are updated.
	 */
	UE_API static FSimpleMulticastDelegate& OnChannelFiltersChanged();

	/**
	 * Registers a delegate that contributes additional selected view models to the Sequencer Simple View.
	 *
	 * @param InDelegate The delegate to register.
	 * @return A handle representing the registered delegate, which can be used to
	 *         identify and unregister the delegate later if needed.
	 */
	UE_API static FDelegateHandle RegisterAdditionalSelectedModels(FOnEnumerateAdditionalSimpleViewSelectedModels::FDelegate&& InDelegate);

	/**
	 * Unregisters an additional selected view model delegate from the Sequencer Simple View.
	 *
	 * @param InDelegateHandle The handle of the delegate to be unregistered.
	 */
	UE_API static void UnregisterAdditionalSelectedModels(const FDelegateHandle& InDelegateHandle);

	/**
	 * @return A reference to the multicast delegate that is broadcast whenever additional selected view models are updated.
	 */
	UE_API static FSimpleMulticastDelegate& OnAdditionalSelectedModelsChanged();

private:
	friend class ToolableTimeline::FToolableTimelineChannelCache;

	FSequencerSimpleViewExtender() = delete;
	~FSequencerSimpleViewExtender() = delete;
	FSequencerSimpleViewExtender(const FSequencerSimpleViewExtender&) = delete;
	FSequencerSimpleViewExtender(FSequencerSimpleViewExtender&&) = delete;
	FSequencerSimpleViewExtender& operator=(const FSequencerSimpleViewExtender&) = delete;
	FSequencerSimpleViewExtender& operator=(FSequencerSimpleViewExtender&&) = delete;

	/** Retrieves the collection of registered additional selected view model delegates. */
	UE_API static TMap<FDelegateHandle, FOnEnumerateAdditionalSimpleViewSelectedModels::FDelegate>& GetRegisteredAdditionalSelectedModels();
};

} // namespace UE::Sequencer::SimpleView

#undef UE_API
