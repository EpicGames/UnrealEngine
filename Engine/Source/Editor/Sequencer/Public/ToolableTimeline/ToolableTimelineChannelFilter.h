// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Templates/Function.h"

class ISequencer;

namespace UE::Sequencer
{
	class FChannelModel;
}

namespace UE::Sequencer::ToolableTimeline
{

/** @return True to filter out the FChannelModel from display keys in the timeline */
using FChannelFilterFunction = TFunction<bool(ISequencer&, const FChannelModel&)>;

/** Channel filter function instance */
struct FRegisteredChannelFilter
{
	FDelegateHandle Handle;
	FChannelFilterFunction FilterFunction;
};

} // namespace UE::Sequencer::ToolableTimeline
