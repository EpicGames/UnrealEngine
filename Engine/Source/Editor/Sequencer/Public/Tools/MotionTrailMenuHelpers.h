// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolMenus.h"
#include "Tools/MotionTrailOptions.h"
#include "Tools/TrailCategory.h"
#include "MovieSceneCommonHelpers.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBorder.h"

namespace UE::Sequencer::MotionTrailMenu
{

// Populates a UToolMenu with the full motion trail settings submenu content:
// path mode radio buttons, pinning management, trail style, and advanced options.
// Category scopes the pin/unpin actions to the relevant trail type.
SEQUENCER_API void PopulateMotionTrailMenu(UToolMenu* InMenu, ETrailCategory Category = ETrailCategory::All);

// Creates the toggle action for a trail category toolbar button.
// Toggles bShowTrails on/off and per-category visibility.
// Also handles modifier keys (Shift=pin, Ctrl=replace pin, Alt=unpin).
SEQUENCER_API FToolUIAction MakeMotionTrailToggleAction(ETrailCategory Category);

// Registers a motion trails submenu entry on the given toolbar menu.
// Each entry controls visibility for the specified trail category.
// EntryName must be unique per toolbar (e.g., "ControlRigTrails", "MixerTrails").
SEQUENCER_API void RegisterMotionPathsToolbarEntry(
	const FName& InMenuName,
	const FName& OwnerName,
	const FName& EntryName,
	const FText& Label,
	const FText& Tooltip,
	ETrailCategory Category);

// Unregisters toolbar entries previously registered with RegisterMotionPathsToolbarEntry.
SEQUENCER_API void UnregisterMotionPathsToolbarEntry(const FName& OwnerName);

// Creates a spin-box widget bound to a UObject numeric property via FTrackInstancePropertyBindings.
// Usable for any numeric UObject property that should appear in motion trail menus.
template<typename NumericType>
TSharedRef<SWidget> CreatePropertyWidget(UObject* Settings, FName PropertyName)
{
	// clang-format off
	return
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
						.Padding(FMargin(1.0f))
						[
							SNew(SSpinBox<NumericType>)
								.MinValue(0)
								.MaxValue(100)
								.MinDesiredWidth(50)
								.ToolTipText_Lambda(
									[Settings, PropertyName]() -> FText
									{
										FTrackInstancePropertyBindings Binding(PropertyName, PropertyName.ToString());
										NumericType Value = Binding.GetCurrentValue<NumericType>(*Settings);
										return FText::AsNumber(Value);
									}
								)
								.Value_Lambda(
									[Settings, PropertyName]() -> NumericType
									{
										FTrackInstancePropertyBindings Binding(PropertyName, PropertyName.ToString());
										NumericType Value = Binding.GetCurrentValue<NumericType>(*Settings);
										return Value;
									}
								)
								.OnValueChanged_Lambda(
									[Settings, PropertyName](NumericType InValue)
									{
										FTrackInstancePropertyBindings Binding(PropertyName, PropertyName.ToString());
										Binding.CallFunction<NumericType>(*Settings, InValue);
										FPropertyChangedEvent Event(Binding.GetProperty(*Settings));
										Settings->PostEditChangeProperty(Event);
									}
								)
						]
				]
		];
	// clang-format on
}

template<typename NumericType>
FToolMenuEntry CreateProperty(UObject* Settings, FName PropertyName, FName Label)
{
	FText Text = FText::FromString(Label.ToString());
	return FToolMenuEntry::InitWidget(PropertyName, CreatePropertyWidget<NumericType>(Settings, PropertyName), Text);
}

} // namespace UE::Sequencer::MotionTrailMenu
