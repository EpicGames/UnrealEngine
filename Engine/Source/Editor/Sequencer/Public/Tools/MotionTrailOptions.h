// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Delegates/Delegate.h"
#include "Tools/TrailCategory.h"
#include "MotionTrailOptions.generated.h"

#define UE_API SEQUENCER_API

//if true still use old motion trails for sequencer objects.
extern SEQUENCER_API TAutoConsoleVariable<bool> CVarUseOldSequencerMotionTrails;


class USceneComponent;
class AActor;

UENUM(BlueprintType)
enum class EMotionTrailTrailStyle : uint8
{
	Default = 0,
	Dashed = 1,
	Time = 2,
	HeatMap = 3
};

UENUM(BlueprintType)
enum class EMotionTrailRange : uint8
{
	PlaybackRange = 0,
	SelectionRange = 1,
	SpecifiedRange = 2,
	//to do TimeBookmark = 3
};


// TODO: option to make tick size proportional to distance from camera to get a sense of perspective and scale
UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UMotionTrailToolOptions : public UObject
{
	GENERATED_BODY()

public:
	UMotionTrailToolOptions()
		: bShowTrails(false)
		, bShowSelectedTrails(false)
		, TrailStyle(EMotionTrailTrailStyle::Default)
		, DefaultColor(.22f, 0.15f, 1.0f)
		, TimePreColor(.22f, 0.35f, 0.8f)
		, TimePostColor(.85f, 0.25f, 0.1f)
		, DashPreColor(0.2f, 0.9f, 0.3f)
		, DashPostColor(0.7f, 0.2f, 0.7f)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, bShowFullTrail(false) //deprecrated 5.8
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		, MotionTrailRange(EMotionTrailRange::PlaybackRange)
		, TrailThickness(0.0f)
		, FramesBefore(10)
		, FramesAfter(10)
		, EvalsPerFrame(1)
		, bShowKeys(true)
		, bShowFrameNumber_DEPRECATED(false)
		, KeyColor(1.0, 1.0, 1.0)
		, SelectedKeyColor(0.8f, 0.8f, 0.0f)
		, KeySize(3.0f)
		, bShowMarks(false)
		, MarkColor(0.25f, 1.0f, 0.15f)
		, CurrentFrameMarkColor(0.4f, 0.9f, 0.1f)
		, MarkSize(5.0)
		, MaxNumberPinned(10)
		, bLockMarksToFrames_DEPRECATED(true)
		, SecondsPerMark_DEPRECATED(0.1)
	{}

	/** Whether or not to show motion trails */
	UPROPERTY(EditAnywhere, Category = Trail);
	bool bShowTrails;

	/** Whether or not to show selected motion trails */
	UPROPERTY(EditAnywhere, Config, Category = Trail);
	bool bShowSelectedTrails;

	/** How To Show Color*/
	UPROPERTY(EditAnywhere, Config, Category = Trail);
	EMotionTrailTrailStyle TrailStyle;

	/** The color of the motion trail */
	UPROPERTY(EditAnywhere, Config, Category = AdvancedDisplay)
	FLinearColor DefaultColor;

	/** The color of the motion trail before current time if show alternating time colors */
	UPROPERTY(EditAnywhere, Config, Category = AdvancedDisplay)
	FLinearColor TimePreColor;

	/** The color of the motion trail after current time if show alternating time colors */
	UPROPERTY(EditAnywhere, Config, Category = AdvancedDisplay)
	FLinearColor TimePostColor;

	/** The color of the first motion trail color when alternating between frames */
	UPROPERTY(EditAnywhere, Config, Category = AdvancedDisplay)
	FLinearColor DashPreColor;

	/** The color of the next motion trail color when alternating between frames */
	UPROPERTY(EditAnywhere, Config, Category = AdvancedDisplay)
	FLinearColor DashPostColor;

	UE_DEPRECATED(5.8, "Use Motion Trail Range")
	bool bShowFullTrail;

	/** The time range to draw the trail */
	UPROPERTY(EditAnywhere, Config, Category = Trail);
	EMotionTrailRange MotionTrailRange;

	/* The thickness of the motion trail */
	UPROPERTY(EditAnywhere, Config, Category = Trail, Meta = (ClampMin = "0.0"))
	double TrailThickness;

	/** The number of frames to draw before the start of the trail. Requires not showing the full trail */
	UPROPERTY(EditAnywhere, Config, Category = Trail, Meta = (EditCondition = "MotionTrailRange == EMotionTrailRange::SpecifiedRange", ClampMin = "0"))
	int32 FramesBefore;

	/** The number of frames to draw after the end of the trail. Requires not showing the full trail */
	UPROPERTY(EditAnywhere, Config, Category = Trail, Meta = (EditCondition = "MotionTrailRange == EMotionTrailRange::SpecifiedRange", ClampMin = "0"))
	int32 FramesAfter;

	/** No longer exposed and clamped to 1 The number of evaluations per frame */
	int32 EvalsPerFrame;
	
	/** Whether or not to show keys on the motion trail */
	UPROPERTY(EditAnywhere, Config, Category = AdvancedDisplay)
	bool bShowKeys;

	/** Deprecated in 5.6 */
	UPROPERTY()
	bool bShowFrameNumber_DEPRECATED;

	/** The color of the keys */
	UPROPERTY(EditAnywhere, Category = Keys)
	FLinearColor KeyColor;

	/** The color of the selected keys */
	UPROPERTY(EditAnywhere, Config, Category = AdvancedDisplay)
	FLinearColor SelectedKeyColor;

	/** The size of the keys */
	UPROPERTY(EditAnywhere, Config, Category = Keys, Meta = (ClampMin = "0.0"))
	double KeySize;

	/** Whether or not to show marks along the motion trail */
	UPROPERTY(EditAnywhere, Config, Category = Marks)
	bool bShowMarks;

	/** The color of the marks */
	UPROPERTY(EditAnywhere, Config, Category = Marks)
	FLinearColor MarkColor;

	/** The color of the Current Frame mark color */
	UPROPERTY(EditAnywhere, Config, Category = Marks)
	FLinearColor CurrentFrameMarkColor;

	/** The size of the marks */
	UPROPERTY(EditAnywhere, Config, Category = Marks, Meta = (ClampMin = "0.0"))
	double MarkSize;

	/** Max number of pinned trails*/
	UPROPERTY(EditAnywhere, Config, Category = AdvancedDisplay)
	int32 MaxNumberPinned;

	/** Deprecated in 5.6 */
	UPROPERTY()
	bool bLockMarksToFrames_DEPRECATED;

	/** Deprecated in 5.6 */
	UPROPERTY()
	double SecondsPerMark_DEPRECATED;

	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override
	{
		FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
		OnDisplayPropertyChanged.Broadcast(PropertyName);
		SaveConfig();
	}

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDisplayPropertyChanged, FName);
	FOnDisplayPropertyChanged OnDisplayPropertyChanged;

	static UMotionTrailToolOptions* GetTrailOptions()  { return GetMutableDefault<UMotionTrailToolOptions>(); }

	//We put Pinned motion trail information as part of this object so it can be shared by SequencerAnimTools and Control Rig module
	struct FPinnedTrail
	{
		FText TrailName;
		FLinearColor TrailColor;
		bool bHasOffset = false;
		TOptional<FText> SpaceName;
		FGuid TrailGuid;
		ETrailCategory Category = ETrailCategory::All;

		bool operator==(const FGuid& InId) const { return TrailGuid == InId; }

	};
	UE_API void ResetPinnedItems();
	UE_API int32 GetNumPinned() const;
	UE_API FPinnedTrail* GetPinnedTrail(int32 Index);

	UE_API int32 GetIndexFromGuid(FGuid InGuid) const;
	UE_API void PinSelection(ETrailCategory Category = ETrailCategory::All) const;
	UE_API void UnPinSelection(ETrailCategory Category = ETrailCategory::All);
	UE_API void AddPinned(const FPinnedTrail& InPinnedTrail);
	UE_API void PinComponent(USceneComponent* InSceneComponent, const FName& InSocketName) const;
	UE_API void DeletePinned(int32 Index);
	UE_API void DeleteAllPinned(ETrailCategory Category = ETrailCategory::All);
	UE_API void PutPinnnedInSpace(int32 Index, AActor* InActor, const FName& InComponentName);
	UE_API void SetLinearColor(int32 Index, const FLinearColor& Color);
	UE_API void SetHasOffset(int32 Index, bool bHasOffset);
	//get name and tooltip name for the color types
	UE_API TArray<TPair<FText, FText>>& GetTrailStyles();
	UE_API void SetTrailStyle(int32 Index);
	UE_API int32 GetTrailStyleIndex() const;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPinSelection, ETrailCategory);
	FOnPinSelection OnPinSelection;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnUnPinSelection, ETrailCategory);
	FOnUnPinSelection OnUnPinSelection;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAddPinned, FGuid);
	FOnAddPinned OnAddPinned;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FPinComponent, USceneComponent*, FName);
	FPinComponent OnPinComponent;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDeletePinned, FGuid);
	FOnDeletePinned OnDeletePinned;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDeleteAllPinned, ETrailCategory);
	FOnDeleteAllPinned OnDeleteAllPinned;

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnPutPinnedInSpace, FGuid, AActor*, FName);
	FOnPutPinnedInSpace OnPutPinnedInSpace;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSetLinearColor, FGuid, FLinearColor);
	FOnSetLinearColor OnSetLinearColor;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSetHasOffset, FGuid, bool);
	FOnSetHasOffset OnSetHasOffset;

protected:
	TArray<FPinnedTrail> PinnedTrails;
	TArray<TPair<FText, FText>> TrailStylesText;

};



#undef UE_API
