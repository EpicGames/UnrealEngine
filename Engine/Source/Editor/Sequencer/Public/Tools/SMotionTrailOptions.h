// Copyright Epic Games, Inc. All Rights Reserved.
/**
* Hold the View for the Motion Trail Options Widget
*/
#pragma once

#include "CoreMinimal.h"
#include "IDetailsView.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

#define UE_API SEQUENCER_API

class IDetailsView;

class SMotionTrailOptions : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SMotionTrailOptions) {}

	SLATE_END_ARGS()
	~SMotionTrailOptions() = default;

	UE_API void Construct(const FArguments& InArgs);

	TSharedPtr<IDetailsView> DetailsView;
};

#undef UE_API
