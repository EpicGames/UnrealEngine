// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class ISequencer;
struct FQualifiedFrameTime;

class FSequencerSelectionAlignmentUtils
{
public:
	/** Returns true if there is a valid layer bar or key selection. False if there are both layer bars and keys selected. */
	SEQUENCER_API static bool CanAlignSelection(const ISequencer& InSequencer);

	/** Aligns the Sequencer selection to a specified time, optionally transacting. */
	SEQUENCER_API static void AlignSelectionToTime(const ISequencer& InSequencer, const FQualifiedFrameTime& InFrameTime, const bool bInTransact = true);

	/** Aligns the Sequencer selection to the current playhead location, optionally transacting. */
	SEQUENCER_API static void AlignSelectionToPlayhead(const ISequencer& InSequencer, const bool bInTransact = true);
};
