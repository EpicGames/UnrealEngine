// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Misc/EnumClassFlags.h"

// Bitfield identifying which category of motion trail a hierarchy provides.
// Each trail hierarchy sets its category; the trail tool only updates/renders
// hierarchies whose category intersects the user-visible mask.
enum class ETrailCategory : uint8
{
	None       = 0,
	Transform  = 1 << 0,   // Component/actor 3D transform trails
	ControlRig = 1 << 1,   // Control rig bone/control trails
	AnimMixer  = 1 << 2,   // Animation mixer root motion trails
	All        = Transform | ControlRig | AnimMixer
};
ENUM_CLASS_FLAGS(ETrailCategory);

// Per-category visibility state for motion trails. Each toolbar button
// toggles its category via SetCategoryVisible; the trail tool checks
// GetVisibleCategories to decide which hierarchies to update and render.
// Categories default to None (all off) until toggled on by the user.
class FTrailCategoryRegistry
{
public:
	SEQUENCER_API static void SetCategoryVisible(ETrailCategory Category, bool bVisible);
	SEQUENCER_API static bool IsCategoryVisible(ETrailCategory Category);
	SEQUENCER_API static ETrailCategory GetVisibleCategories();

private:
	static ETrailCategory VisibleCategories;
};
