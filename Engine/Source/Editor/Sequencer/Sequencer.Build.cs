// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class Sequencer : ModuleRules
{
	public Sequencer(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"TimeManagement",
				}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ActorPickerMode",
				"AppFramework", 
				"ApplicationCore",
				"AssetRegistry",
				"CinematicCamera",
				"Constraints",
				"ContentBrowser",
				"Core", 
				"CoreUObject", 
				"CurveEditor",
				"DeveloperSettings",
				"InputCore",
				"Engine", 
				"EditorInteractiveToolsFramework",
				"Slate", 
				"SlateCore",
				"SceneOutliner",
				"SequencerCore",
				"EditorStyle",
				"EditorFramework",
				"UnrealEd", 
				"MovieScene", 
				"MovieSceneTracks", 
				"MovieSceneTools", 
				"MovieSceneCapture", 
				"MovieSceneCaptureDialog", 
				"EditorWidgets", 
				"SequencerWidgets",
				"BlueprintGraph",
				"LevelSequence",
				"GraphEditor",
				"PropertyEditor",
				"ViewportInteraction",
				"SerializedRecorderInterface",
				"SubobjectDataInterface",
				"ToolMenus",
				"ToolWidgets",
				"TypedElementFramework",
				"TypedElementRuntime",
				"UniversalObjectLocator",
				"UniversalObjectLocatorEditor",
				"RenderCore",
				"WorkspaceMenuStructure"
			}
			);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.Add("WorkspaceMenuStructure");
		}

		CircularlyReferencedDependentModules.AddRange(
			new string[] {
				"ViewportInteraction",
				"MovieSceneCaptureDialog",
				"UniversalObjectLocatorEditor",
				}
			);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"LevelEditor",
				"DesktopPlatform",
				}
			);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"PropertyEditor",
				"SceneOutliner",
				"CurveEditor",
				"Analytics",
				"SequencerWidgets"
				}
			);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"LevelEditor",
				"MainFrame",
				}
			);

		CircularlyReferencedDependentModules.Add("MovieSceneTools");
	}
}
